// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class Persona : ModuleRules
{
    public Persona(ReadOnlyTargetRules Target) : base(Target)
    {
        PublicIncludePathModuleNames.AddRange(
            new string[] {
				"Engine",
				"SkeletonEditor",
                "AnimationEditor",
                "AdvancedPreviewScene",
            }
        );

        PublicDependencyModuleNames.AddRange(
            new string[] {
                "AdvancedPreviewScene",
                "AnimationEditMode",
				"EditorInteractiveToolsFramework"
            }
        );

        PrivateIncludePathModuleNames.AddRange(
            new string[] {
                "AssetRegistry", 
                "MainFrame",
                "DesktopPlatform",
                "AssetTools",
                "MeshReductionInterface",
                "SequenceRecorder",
                "AnimationBlueprintEditor",
                "EditorInteractiveToolsFramework",
			}
		);

        PrivateDependencyModuleNames.AddRange(
            new string[] {
                "AppFramework",
                "AnimationModifiers",
                "AnimationBlueprintLibrary",
				"AudioEditor",
				"Core", 
                "CoreUObject", 
				"ApplicationCore",
                "Slate", 
                "SlateCore",
				"ContentBrowserData",
                "EditorStyle",
                "Engine",
				"EditorFramework",
				"UnrealEd", 
                "GraphEditor", 
                "InputCore",
                "Kismet", 
                "KismetWidgets",
                "AnimGraph",
                "PropertyEditor",
                "EditorWidgets",
				"DesktopWidgets",
                "BlueprintGraph",
                "RHI",
				"InterchangeEngine",
                "Json",
                "JsonUtilities",
                "ClothingSystemEditorInterface",
                "ClothingSystemRuntimeInterface",
                "ClothingSystemRuntimeCommon",
                "AnimGraphRuntime",
                "CommonMenuExtensions",
                "PinnedCommandList",
                "RenderCore",
				"SkeletalMeshUtilitiesCommon",
				"NaniteUtilities",
				"ToolMenus",
                "CurveEditor",
                "MirrorDataTableEditor",
				"SequencerWidgets",
				"TimeManagement",
                "Sequencer",
				"StatusBar",
				"ToolWidgets",
				"InteractiveToolsFramework",
				"ContentBrowser",
	            "SkeletalMeshDescription",
	            "TypedElementRuntime"
            }
        );

        DynamicallyLoadedModuleNames.AddRange(
            new string[] {
	            "Documentation",
                "MainFrame",
                "DesktopPlatform",
                "SkeletonEditor",
                "AssetTools",
                "AnimationEditor",
                "MeshReductionInterface",
                "SequenceRecorder",
			}
        );
    }
}
