// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimAssetFindReplace.h"

#include "PropertyCustomizationHelpers.h"
#include "SAnimAssetFindReplace.h"
#include "Animation/PoseAsset.h"
#include "Animation/Skeleton.h"
#include "Animation/AnimationAsset.h"
#include "Animation/AnimSequence.h"
#include "Engine/SkeletalMesh.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Input/SSearchBox.h"
#include "Framework/Application/SlateApplication.h"
#include "ToolMenuSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimAssetFindReplace)

#define LOCTEXT_NAMESPACE "AnimAssetFindReplace"

class SAutoCompleteSearchBox : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAutoCompleteSearchBox) {}

	/** The text displayed in the SearchBox when no text has been entered */
	SLATE_ATTRIBUTE(FText, HintText)

	/** The text displayed in the SearchBox when it's created */
	SLATE_ATTRIBUTE(FText, InitialText)

	/** Invoked whenever the text changes */
	SLATE_EVENT(FOnTextChanged, OnTextChanged)

	/** Items to show in the autocomplete popup */
	SLATE_ARGUMENT(TSharedPtr<TArray<TSharedPtr<FString>>>, AutoCompleteItems)
	
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		AutoCompleteItems = InArgs._AutoCompleteItems;

		ChildSlot
		[
			SAssignNew(MenuAnchor, SMenuAnchor)
			.Method(EPopupMethod::CreateNewWindow)
			.Placement(EMenuPlacement::MenuPlacement_BelowAnchor)
			.MenuContent
			(
				SNew(SBox)
				.MaxDesiredHeight(200.0f)
				.MinDesiredWidth(200.0f)
				[
					SAssignNew(AutoCompleteList, SListView<TSharedPtr<FString>>)
					.SelectionMode(ESelectionMode::Single)
					.ListItemsSource(&FilteredAutoCompleteItems)
					.OnSelectionChanged_Lambda([this](TSharedPtr<FString> InString, ESelectInfo::Type InSelectInfo)
					{
						if(InString.IsValid() && InSelectInfo == ESelectInfo::OnMouseClick)
						{
							TGuardValue<bool> GuardValue(bSettingTextFromSearchItem, true);
							SearchBox->SetText(FText::FromString(*InString.Get()));
							MenuAnchor->SetIsOpen(false);
						}
					})
					.OnGenerateRow_Lambda([this](TSharedPtr<FString> InString, const TSharedRef<STableViewBase>& InTableView)
					{
						if(InString.IsValid())
						{
							return
								SNew(STableRow<TSharedPtr<FString>>, InTableView)
								.Content()
								[
									SNew(STextBlock)
									.Text(FText::FromString(*InString.Get()))
									.HighlightText_Lambda([this]()
									{
										return SearchBox->GetText();
									})
								];
						}
						return SNew(STableRow<TSharedPtr<FString>>, InTableView);
					})
					.OnKeyDownHandler_Lambda([this](const FGeometry& InGeometry, const FKeyEvent& InKeyEvent)
					{
						if(InKeyEvent.GetKey() == EKeys::Enter)
						{
							TArray<TSharedPtr<FString>> SelectedItems;
							AutoCompleteList->GetSelectedItems(SelectedItems);
							if(SelectedItems.Num() > 0 && SelectedItems[0].IsValid())
							{
								TGuardValue<bool> GuardValue(bSettingTextFromSearchItem, true);
								SearchBox->SetText(FText::FromString(*SelectedItems[0].Get()));
								MenuAnchor->SetIsOpen(false);
								return FReply::Handled();
							}
						}
						return FReply::Unhandled();
					})
				]
			)
			[
				SAssignNew(SearchBox, SSearchBox)
				.HintText(InArgs._HintText)
				.InitialText(InArgs._InitialText)
				.OnTextChanged_Lambda([this, OnTextChanged = InArgs._OnTextChanged](const FText& InText)
				{
					RefreshAutoCompleteItems();

					if(!bSettingTextFromSearchItem)
					{
						MenuAnchor->SetIsOpen(FilteredAutoCompleteItems.Num() > 0, false);
					}

					OnTextChanged.ExecuteIfBound(InText);
				})
				.OnVerifyTextChanged_Lambda([](const FText& InText, FText& OutErrorMessage)
				{
					return FName::IsValidXName(InText.ToString(), FString(INVALID_NAME_CHARACTERS), &OutErrorMessage);
				})
			]
		];
		
		RefreshAutoCompleteItems();
	}

	void FilterItems(const FText& InText)
	{
		FilteredAutoCompleteItems.Empty();

		for(TSharedPtr<FString> String : *AutoCompleteItems.Get())
		{
			if(String.Get()->Contains(InText.ToString()))
			{
				FilteredAutoCompleteItems.Add(String);
			}
		}
	}

	void RefreshAutoCompleteItems()
	{
		FilterItems(SearchBox->GetText());
		AutoCompleteList->RequestListRefresh();
	}

	TSharedRef<SSearchBox> GetSearchBox() const
	{
		return SearchBox.ToSharedRef();
	}
	
	virtual void OnFocusChanging(const FWeakWidgetPath& PreviousFocusPath, const FWidgetPath& NewWidgetPath, const FFocusEvent& InFocusEvent) override
	{
		if(PreviousFocusPath.ContainsWidget(SearchBox.Get()) && !NewWidgetPath.ContainsWidget(MenuAnchor->GetMenuWindow().Get()))
		{
			MenuAnchor->SetIsOpen(false);
		}
		SCompoundWidget::OnFocusChanging(PreviousFocusPath, NewWidgetPath, InFocusEvent);
	}

	virtual FReply OnPreviewKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override
	{
		if(InKeyEvent.GetKey() == EKeys::Down && MenuAnchor->IsOpen())
		{
			// switch focus to the drop-down autocomplete list
			return FReply::Handled().SetUserFocus(AutoCompleteList.ToSharedRef(), EFocusCause::Navigation);
		}
		return SCompoundWidget::OnPreviewKeyDown(MyGeometry, InKeyEvent); 
	}
	
private:
	TArray<TSharedPtr<FString>> FilteredAutoCompleteItems;
	TSharedPtr<TArray<TSharedPtr<FString>>> AutoCompleteItems;
	TSharedPtr<SListView<TSharedPtr<FString>>> AutoCompleteList;
	TSharedPtr<SMenuAnchor> MenuAnchor;
	TSharedPtr<SSearchBox> SearchBox;
	bool bSettingTextFromSearchItem = false;
};

void UAnimAssetFindReplaceProcessor::Initialize(TSharedRef<SAnimAssetFindReplace> InWidget)
{
	Widget = InWidget;
}

void UAnimAssetFindReplaceProcessor::RequestRefreshUI()
{
	if(TSharedPtr<SAnimAssetFindReplace> PinnedWidget = Widget.Pin())
	{
		PinnedWidget->RequestRefreshUI();
	}
}

void UAnimAssetFindReplaceProcessor::RequestRefreshCachedData()
{
	if(TSharedPtr<SAnimAssetFindReplace> PinnedWidget = Widget.Pin())
	{
		PinnedWidget->RequestRefreshCachedData();
	}
}

void UAnimAssetFindReplaceProcessor::RequestRefreshSearchResults()
{
	if(TSharedPtr<SAnimAssetFindReplace> PinnedWidget = Widget.Pin())
	{
		PinnedWidget->RequestRefreshSearchResults();
	}
}

void UAnimAssetFindReplaceProcessor_StringBase::SetFindString(const FString& InString)
{
	FindString = InString;
	RequestRefreshSearchResults();
}

void UAnimAssetFindReplaceProcessor_StringBase::SetReplaceString(const FString& InString)
{
	ReplaceString = InString;
	RequestRefreshSearchResults();
}

void UAnimAssetFindReplaceProcessor_StringBase::SetSkeletonFilter(const FAssetData& InSkeletonFilter)
{
	SkeletonFilter = InSkeletonFilter;
	RequestRefreshSearchResults();
}

void UAnimAssetFindReplaceProcessor_StringBase::SetFindWholeWord(bool bInFindWholeWord)
{
	bFindWholeWord = bInFindWholeWord;
	RequestRefreshSearchResults();
}

void UAnimAssetFindReplaceProcessor_StringBase::SetSearchCase(ESearchCase::Type InSearchCase)
{
	SearchCase = InSearchCase;
	RequestRefreshSearchResults();
}

void UAnimAssetFindReplaceProcessor_StringBase::ExtendToolbar(FToolMenuSection& InSection)
{
	FToolUIAction MatchCaseCheckbox;
	MatchCaseCheckbox.ExecuteAction = FToolMenuExecuteAction::CreateLambda([this](const FToolMenuContext& InContext)
	{
		SetSearchCase(GetSearchCase() == ESearchCase::CaseSensitive ? ESearchCase::IgnoreCase : ESearchCase::CaseSensitive);
	});

	MatchCaseCheckbox.GetActionCheckState = FToolMenuGetActionCheckState::CreateLambda([this](const FToolMenuContext& InContext)
	{
		return GetSearchCase() == ESearchCase::CaseSensitive ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	});
	
	InSection.AddEntry(
		FToolMenuEntry::InitToolBarButton(
			"MatchCase",
			MatchCaseCheckbox,
			LOCTEXT("MatchCaseCheckboxLabel", "Match Case"),
			LOCTEXT("MatchCaseCheckboxTooltip", "Whether to match case when searching."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Persona.FindReplace.MatchCase"),
			EUserInterfaceActionType::ToggleButton));

	FToolUIAction MatchWholeWordCheckbox;
	MatchWholeWordCheckbox.ExecuteAction = FToolMenuExecuteAction::CreateLambda([this](const FToolMenuContext& InContext)
	{
		SetFindWholeWord(!GetFindWholeWord());
	});

	MatchWholeWordCheckbox.GetActionCheckState = FToolMenuGetActionCheckState::CreateLambda([this](const FToolMenuContext& InContext)
	{
		return GetFindWholeWord() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	});
	
	InSection.AddEntry(
		FToolMenuEntry::InitToolBarButton(
			"MatchWholeWord",
			MatchWholeWordCheckbox,
			LOCTEXT("MatchWholeWordCheckboxLabel", "Match Whole Word"),
			LOCTEXT("MatchWholeWordCheckboxTooltip", "Whether to match the whole word or just part of the word when searching."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Persona.FindReplace.MatchWholeWord"),
			EUserInterfaceActionType::ToggleButton));

	InSection.AddSeparator(NAME_None);

	InSection.AddDynamicEntry("SkeletonFilter", FNewToolMenuSectionDelegate::CreateLambda([this](FToolMenuSection& InSection)
	{
		if(TSharedPtr<SAnimAssetFindReplace> Widget = AnimAssetFindReplacePrivate::GetWidgetFromContext(InSection.Context))
		{
			InSection.AddEntry(
			FToolMenuEntry::InitWidget(
				"SkeletonFilterWidget",
				SNew(SHorizontalBox)
				.ToolTipText(LOCTEXT("SkeletonFilterTooltip", "Choose a Skeleton asset to filter results by."))
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(5.0f, 0.0f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("SkeletonFilterLabel", "Skeleton"))
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SObjectPropertyEntryBox)
					.ObjectPath_Lambda([this]()
					{
						return GetSkeletonFilter().GetObjectPathString();
					})
					.OnObjectChanged_Lambda([this](const FAssetData& InAssetData)
					{
						SetSkeletonFilter(InAssetData);
					})
					.AllowedClass(USkeleton::StaticClass())
				],
			FText::GetEmpty(),
			true, true, true));
		}
	}));
}

TSharedRef<SWidget> UAnimAssetFindReplaceProcessor_StringBase::MakeFindReplaceWidget()
{
	return SAssignNew(FindReplaceWidget, SVerticalBox)
		+SVerticalBox::Slot()
		.Padding(6.0f, 2.0f)
		[
			SAssignNew(FindSearchBox, SAutoCompleteSearchBox)
			.AutoCompleteItems(AutoCompleteItems)
			.HintText(LOCTEXT("FindLabel", "Find"))
			.ToolTipText(LOCTEXT("FindLabel", "Find"))
			.InitialText_Lambda([this](){ return FText::FromString(FindString); })
			.OnTextChanged_Lambda([this](const FText& InText)
			{
				FindString = InText.ToString();
				RequestRefreshSearchResults();
			})
		]
		+SVerticalBox::Slot()
		.Padding(6.0f, 2.0f)
		[
			SAssignNew(ReplaceSearchBox, SAutoCompleteSearchBox)
			.AutoCompleteItems(AutoCompleteItems)
			.HintText(LOCTEXT("ReplaceLabel", "Replace With"))
			.ToolTipText(LOCTEXT("ReplaceLabel", "Replace With"))
			.InitialText_Lambda([this](){ return FText::FromString(ReplaceString); })
			.OnTextChanged_Lambda([this](const FText& InText)
			{
				ReplaceString = InText.ToString();
			})
		];
}

void UAnimAssetFindReplaceProcessor_StringBase::FocusInitialWidget()
{
	if (FindSearchBox.IsValid())
	{
		FWidgetPath WidgetToFocusPath;
		FSlateApplication::Get().GeneratePathToWidgetUnchecked(FindSearchBox.Pin()->GetSearchBox(), WidgetToFocusPath);
		FSlateApplication::Get().SetKeyboardFocus(WidgetToFocusPath, EFocusCause::SetDirectly);
		WidgetToFocusPath.GetWindow()->SetWidgetToFocusOnActivate(FindSearchBox.Pin()->GetSearchBox());
	}
}

bool UAnimAssetFindReplaceProcessor_StringBase::CanCurrentlyReplace() const
{
	return !ReplaceString.IsEmpty();
}

bool UAnimAssetFindReplaceProcessor_StringBase::CanCurrentlyRemove() const
{
	return true;
}

void UAnimAssetFindReplaceProcessor_StringBase::RefreshCachedData()
{
	AutoCompleteItems->Empty();

	// We use the asset registry to query all assets and accumulate their curve names
	const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");

	FARFilter Filter;
	for(const UClass* AssetClass : GetSupportedAssetTypes())
	{
		Filter.ClassPaths.Add(AssetClass->GetClassPathName());
	}
	Filter.bRecursiveClasses = true;

	TArray<FAssetData> FoundAssetData;
	AssetRegistryModule.Get().GetAssets(Filter, FoundAssetData);

	TSet<FString> UniqueNames;

	GetAutoCompleteNames(FoundAssetData, UniqueNames);

	for(FString& UniqueName : UniqueNames)
	{
		AutoCompleteItems->Add(MakeShared<FString>(MoveTemp(UniqueName)));
	}

	AutoCompleteItems->Sort([](TSharedPtr<FString> InLHS, TSharedPtr<FString> InRHS)
	{
		return *InLHS.Get() < *InRHS.Get();
	});

	if(TSharedPtr<SAutoCompleteSearchBox> FindWidget = FindSearchBox.Pin())
	{
		FindWidget->RefreshAutoCompleteItems();
	}

	if(TSharedPtr<SAutoCompleteSearchBox> ReplaceWidget = ReplaceSearchBox.Pin())
	{
		ReplaceWidget->RefreshAutoCompleteItems();
	}
}

bool UAnimAssetFindReplaceProcessor_StringBase::NameMatches(FStringView InNameStringView) const
{
	if(!FindString.IsEmpty())
	{
		if(bFindWholeWord)
		{
			if(InNameStringView.Compare(FindString, SearchCase) == 0)
			{
				return true;
			}
		}
		else
		{
			if(UE::String::FindFirst(InNameStringView, FindString, SearchCase) != INDEX_NONE)
			{
				return true;
			}
		}
	}

	return false;
}

#undef LOCTEXT_NAMESPACE
