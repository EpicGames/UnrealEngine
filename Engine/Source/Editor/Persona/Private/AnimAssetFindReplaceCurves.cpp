// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimAssetFindReplaceCurves.h"

#include "ISkeletonEditorModule.h"
#include "SAnimAssetFindReplace.h"
#include "Animation/PoseAsset.h"
#include "Animation/Skeleton.h"
#include "Animation/AnimationAsset.h"
#include "Animation/AnimSequence.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkinnedAssetCommon.h"
#include "Materials/Material.h"
#include "Materials/MaterialExpressionParameter.h"
#include "Materials/MaterialInterface.h"
#include "String/ParseTokens.h"
#include "ToolMenuSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimAssetFindReplaceCurves)

#define LOCTEXT_NAMESPACE "AnimAssetFindReplaceCurves"

namespace UE::AnimAssetFindReplaceCurves::Private
{

// Helper function used to rename material parameters in skeletal meshes
static void RenameMaterialParameter(USkeletalMesh* InSkeletalMesh, FName InOldName, FName InNewName)
{
	for(const FSkeletalMaterial& SkeletalMaterial : InSkeletalMesh->GetMaterials())
	{
		UMaterial* Material = (SkeletalMaterial.MaterialInterface != nullptr) ? SkeletalMaterial.MaterialInterface->GetMaterial() : nullptr;
		if (Material == nullptr)
		{
			continue;
		}

		// Search all expressions for matching scalar parameters
		for(UMaterialExpression* Expression : Material->GetExpressions())
		{
			UMaterialExpressionParameter* Parameter = Cast<UMaterialExpressionParameter>(Expression);
			if(Parameter == nullptr)
			{
				continue;
			}

			FMaterialParameterMetadata ParameterMeta;
			if (!Parameter->GetParameterValue(ParameterMeta))
			{
				continue;
			}

			if(ParameterMeta.Value.Type != EMaterialParameterType::Scalar)
			{
				continue;
			}

			if(Parameter->GetParameterName() == InOldName)
			{
				FProperty* ParameterNameProperty = UMaterialExpressionParameter::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UMaterialExpressionParameter, ParameterName));
				check(ParameterNameProperty);
				Parameter->PreEditChange(ParameterNameProperty);

				Parameter->ParameterName = InNewName;

				FPropertyChangedEvent ChangedEvent(ParameterNameProperty);
				Parameter->PostEditChangeProperty(ChangedEvent);
			}
		}
	}
}

// Helper function used to remove material parameters in skeletal meshes
static void RemoveMaterialParameters(USkeletalMesh* InSkeletalMesh, TConstArrayView<FName> InMaterialParameterNames)
{
	for(const FSkeletalMaterial& SkeletalMaterial : InSkeletalMesh->GetMaterials())
	{
		bool bMaterialModified = false;
		UMaterial* Material = (SkeletalMaterial.MaterialInterface != nullptr) ? SkeletalMaterial.MaterialInterface->GetMaterial() : nullptr;
		if (Material == nullptr)
		{
			continue;
		}

		// Search all expressions for matching scalar parameters
		for(UMaterialExpression* Expression : Material->GetExpressions())
		{
			UMaterialExpressionParameter* Parameter = Cast<UMaterialExpressionParameter>(Expression);
			if(Parameter == nullptr)
			{
				continue;
			}

			FMaterialParameterMetadata ParameterMeta;
			if (!Parameter->GetParameterValue(ParameterMeta))
			{
				continue;
			}

			if(ParameterMeta.Value.Type != EMaterialParameterType::Scalar)
			{
				continue;
			}

			for(FName NameToRemove : InMaterialParameterNames)
			{
				if(Parameter->GetParameterName() == NameToRemove)
				{
					Material->PreEditChange(nullptr);

					Material->GetExpressionCollection().RemoveExpression(Parameter);
					Material->RemoveExpressionParameter(Parameter);
					Parameter->MarkAsGarbage();

					Material->PostEditChange();
				}
			}
		}
	}
}

}

FString UAnimAssetFindReplaceCurves::GetFindResultStringFromAssetData(const FAssetData& InAssetData) const
{
	if(GetFindString().IsEmpty())
	{
		return FString();
	}

	auto GetMatchingCurveNamesForAsset = [this](const FAssetData& InAssetData, TArray<FString>& OutCurveNames, FName InTag, const FString& InDelimiter)
	{
		const FString TagValue = InAssetData.GetTagValueRef<FString>(InTag);
		if (!TagValue.IsEmpty())
		{
			UE::String::ParseTokens(TagValue, *InDelimiter, [this, &OutCurveNames](FStringView InToken)
			{
				if(GetFindWholeWord())
				{
					if(InToken.Compare(GetFindString(), GetSearchCase()) == 0)
					{
						OutCurveNames.Add(FString(InToken));
					}
				}
				else
				{
					if(UE::String::FindFirst(InToken, GetFindString(), GetSearchCase()) != INDEX_NONE)
					{
						OutCurveNames.Add(FString(InToken));
					}
				}
			}, UE::String::EParseTokensOptions::SkipEmpty);
		}
	};

	TStringBuilder<128> Builder;
	TArray<FString> CurveNames;
	GetMatchingCurveNamesForAsset(InAssetData, CurveNames, USkeleton::CurveNameTag, USkeleton::CurveTagDelimiter);
	if(CurveNames.Num() > 0)
	{
		for(int32 NameIndex = 0; NameIndex < CurveNames.Num(); ++NameIndex)
		{
			Builder.Append(CurveNames[NameIndex]);
			if(NameIndex != CurveNames.Num() - 1)
			{
				Builder.Append(TEXT(", "));
			}
		}
	}

	if(GetSearchMorphTargets())
	{
		UClass* AssetClass = InAssetData.GetClass();
		if(AssetClass->IsChildOf(USkeletalMesh::StaticClass()))
		{
			TArray<FString> MorphNames;
			GetMatchingCurveNamesForAsset(InAssetData, MorphNames, USkeletalMesh::MorphNamesTag, USkeletalMesh::MorphNamesTagDelimiter);
			if(MorphNames.Num() > 0)
			{
				if(Builder.Len() > 0)
				{
					Builder.Append(TEXT(" "));
				}
				Builder.Append(LOCTEXT("MorphTargets", "Morph targets: ").ToString());
				for(int32 NameIndex = 0; NameIndex < MorphNames.Num(); ++NameIndex)
				{
					Builder.Append(MorphNames[NameIndex]);
					if(NameIndex != MorphNames.Num() - 1)
					{
						Builder.Append(TEXT(", "));
					}
				}
			}
		}
	}

	if(GetSearchMaterials())
	{
		UClass* AssetClass = InAssetData.GetClass();
		if(AssetClass->IsChildOf(USkeletalMesh::StaticClass()))
		{
			TArray<FString> MaterialNames;
			GetMatchingCurveNamesForAsset(InAssetData, MaterialNames, USkeletalMesh::MaterialParamNamesTag, USkeletalMesh::MaterialParamNamesTagDelimiter);
			if(MaterialNames.Num() > 0)
			{
				if(Builder.Len() > 0)
				{
					Builder.Append(TEXT(" "));
				}
				Builder.Append(LOCTEXT("MaterialParameters", "Material parameters: ").ToString());
				for(int32 NameIndex = 0; NameIndex < MaterialNames.Num(); ++NameIndex)
				{
					Builder.Append(MaterialNames[NameIndex]);
					if(NameIndex != MaterialNames.Num() - 1)
					{
						Builder.Append(TEXT(", "));
					}
				}
			}
		}
	}

	return FString(Builder.ToString());
}

TConstArrayView<UClass*> UAnimAssetFindReplaceCurves::GetSupportedAssetTypes() const
{
	static UClass* Types[] = { UPoseAsset::StaticClass(), UAnimSequenceBase::StaticClass(), USkeleton::StaticClass(), USkeletalMesh::StaticClass() };
	return Types;
}

bool UAnimAssetFindReplaceCurves::ShouldFilterOutAsset(const FAssetData& InAssetData, bool& bOutIsOldAsset) const
{
	UClass* AssetClass = InAssetData.GetClass();

	FString TagValue;
	if(InAssetData.GetTagValue<FString>(USkeleton::CurveNameTag, TagValue))
	{
		bOutIsOldAsset = false;
	}
	else
	{
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
		if(AssetClass->IsChildOf(UAnimSequenceBase::StaticClass()))
		{
			// Check the package object version - the asset was saving empty tags for curves, so the absence of curves is
			// not the same as an empty value
			FAssetPackageData PackageData;
			const UE::AssetRegistry::EExists PackageExists = AssetRegistryModule.Get().TryGetAssetPackageData(InAssetData.PackageName, PackageData);
			if (PackageExists == UE::AssetRegistry::EExists::Exists)
			{
				bOutIsOldAsset = PackageData.FileVersionUE < VER_UE4_SKELETON_ADD_SMARTNAMES;
			}
			else
			{
				// Does not exist or unknown - treat it as 'old'
				bOutIsOldAsset = true;
			}
		}
		else if(AssetClass->IsChildOf(UPoseAsset::StaticClass()))
		{
			// Check the package custom version - the asset was saving empty tags for curves, so the absence of curves is
			// not the same as an empty value
			FAssetPackageData PackageData;
			const UE::AssetRegistry::EExists PackageExists = AssetRegistryModule.Get().TryGetAssetPackageData(InAssetData.PackageName, PackageData);
			if (PackageExists == UE::AssetRegistry::EExists::Exists)
			{
				for(const UE::AssetRegistry::FPackageCustomVersion& CustomVersion : PackageData.GetCustomVersions())
				{
					if(CustomVersion.Key == FAnimPhysObjectVersion::GUID)
					{
						bOutIsOldAsset = CustomVersion.Version < FAnimPhysObjectVersion::SmartNameRefactorForDeterministicCooking;
					}
				}

				// No FAnimPhysObjectVersion, treat as old
				bOutIsOldAsset = true;
			}
			else
			{
				// Does not exist or unknown - treat it as 'old'
				bOutIsOldAsset = true;
			}
		}
		else if(AssetClass->IsChildOf(USkeleton::StaticClass()))
		{
			bOutIsOldAsset = true;
		}
		else if(AssetClass->IsChildOf(USkeletalMesh::StaticClass()))
		{
			// Skeletal meshes didn't have curves before, so cant be 'old' 
			bOutIsOldAsset = false;
		}
		else
		{
			// Assume unknown assets are not 'old'
			bOutIsOldAsset = false;
		}
	}
	
	if(GetSearchMorphTargets())
	{
		if(AssetClass->IsChildOf(USkeletalMesh::StaticClass()))
		{
			FString MorphTagValue;
			if(!InAssetData.GetTagValue<FString>(USkeletalMesh::MorphNamesTag, MorphTagValue))
			{
				bOutIsOldAsset = true;
			}
		}
	}

	if(GetSearchMaterials())
	{
		if(AssetClass->IsChildOf(USkeletalMesh::StaticClass()))
		{
			FString MaterialParamTagValue;
			if(!InAssetData.GetTagValue<FString>(USkeletalMesh::MaterialParamNamesTag, MaterialParamTagValue))
			{
				bOutIsOldAsset = true;
			}
		}
	}

	if(GetFindString().IsEmpty())
	{
		return true;
	}

	if(GetSkeletonFilter().IsValid() )
	{
		if(InAssetData.GetClass() != USkeleton::StaticClass())
		{
			FString SkeletonPath;
			if(InAssetData.GetTagValue<FString>(TEXT("Skeleton"), SkeletonPath))
			{
				if(SkeletonPath != GetSkeletonFilter().GetExportTextName())
				{
					return true;
				}
			}
		}
		else
		{
			if(InAssetData.ToSoftObjectPath() != GetSkeletonFilter().ToSoftObjectPath())
			{
				return true;
			}
		}
	}

	bool bFoundMatch = false;
	UE::String::ParseTokens(TagValue, *USkeleton::CurveTagDelimiter, [this, &bFoundMatch](FStringView InToken)
	{
		if(NameMatches(InToken))
		{
			bFoundMatch = true;
		}
	}, UE::String::EParseTokensOptions::SkipEmpty);
	
	if(GetSearchMorphTargets())
	{
		if(AssetClass->IsChildOf(USkeletalMesh::StaticClass()))
		{
			FString MorphTagValue;
			if(InAssetData.GetTagValue<FString>(USkeletalMesh::MorphNamesTag, MorphTagValue))
			{
				UE::String::ParseTokens(MorphTagValue, *USkeletalMesh::MorphNamesTagDelimiter, [this, &bFoundMatch](FStringView InToken)
				{
					if(NameMatches(InToken))
					{
						bFoundMatch = true;
					}
				}, UE::String::EParseTokensOptions::SkipEmpty);
			}
		}
	}

	if(GetSearchMaterials())
	{
		if(AssetClass->IsChildOf(USkeletalMesh::StaticClass()))
		{
			FString MaterialParamTagValue;
			if(InAssetData.GetTagValue<FString>(USkeletalMesh::MaterialParamNamesTag, MaterialParamTagValue))
			{
				UE::String::ParseTokens(MaterialParamTagValue, *USkeletalMesh::MaterialParamNamesTagDelimiter, [this, &bFoundMatch](FStringView InToken)
				{
					if(NameMatches(InToken))
					{
						bFoundMatch = true;
					}
				}, UE::String::EParseTokensOptions::SkipEmpty);
			}
		}
	}

	return !bFoundMatch;
}

void UAnimAssetFindReplaceCurves::ReplaceInAsset(const FAssetData& InAssetData) const
{
	if(UObject* Asset = InAssetData.GetAsset())
	{
		if(UAnimSequenceBase* AnimSequenceBase = Cast<UAnimSequenceBase>(Asset))
		{
			Asset->MarkPackageDirty();

			if(GetFindWholeWord())
			{
				const FAnimationCurveIdentifier FindCurveId(FName(GetFindString()), ERawCurveTrackTypes::RCT_Float);
				const FAnimationCurveIdentifier ReplaceCurveId(FName(GetReplaceString()), ERawCurveTrackTypes::RCT_Float);
				IAnimationDataController::FScopedBracket ScopedBracket(AnimSequenceBase->GetController(), LOCTEXT("ReplaceCurves", "Replace Curves"));
				AnimSequenceBase->GetController().RenameCurve(FindCurveId, ReplaceCurveId);
			}
			else
			{
				TArray<TPair<FAnimationCurveIdentifier, FAnimationCurveIdentifier>> FindReplacePairs;
				const TArray<FFloatCurve>& Curves = AnimSequenceBase->GetDataModel()->GetFloatCurves();
				for(const FFloatCurve& Curve : Curves)
				{
					FString CurveName = Curve.GetName().ToString();
					if(NameMatches(CurveName))
					{
						const FAnimationCurveIdentifier FindCurveId(*CurveName, ERawCurveTrackTypes::RCT_Float);
						const FString NewName = CurveName.Replace(*GetFindStringRef(), *GetReplaceStringRef(), GetSearchCase());
						const FAnimationCurveIdentifier ReplaceCurveId(*NewName, ERawCurveTrackTypes::RCT_Float);

						FindReplacePairs.Emplace(FindCurveId, ReplaceCurveId);
					}
				}

				if(FindReplacePairs.Num() > 0)
				{
					IAnimationDataController::FScopedBracket ScopedBracket(AnimSequenceBase->GetController(), LOCTEXT("ReplaceCurves", "Replace Curves"));

					for(const TPair<FAnimationCurveIdentifier, FAnimationCurveIdentifier>& FindReplacePair : FindReplacePairs)
					{
						AnimSequenceBase->GetController().RenameCurve(FindReplacePair.Key, FindReplacePair.Value);
					}
				}
			}
		}
		else if(UPoseAsset* PoseAsset = Cast<UPoseAsset>(Asset))
		{
			if(GetFindWholeWord())
			{
				Asset->Modify();
				
				const FName FindCurveName(GetFindString());
				const FName ReplaceCurveName(GetReplaceString());
				PoseAsset->RenamePoseOrCurveName(FindCurveName, ReplaceCurveName);
			}
			else
			{
				TArray<TPair<FName, FName>> FindReplacePairs;

				for(const FName& PoseName : PoseAsset->GetPoseFNames())
				{
					FString CurveName = PoseName.ToString();
					if(NameMatches(CurveName))
					{
						const FName FindCurveName(*CurveName);
						const FString NewName = CurveName.Replace(*GetFindStringRef(), *GetReplaceStringRef(), GetSearchCase());
						const FName ReplaceCurveName(*NewName);

						FindReplacePairs.Emplace(FindCurveName, ReplaceCurveName);
					}
				}

				if(FindReplacePairs.Num() > 0)
				{
					Asset->Modify();

					for(const TPair<FName, FName>& FindReplacePair : FindReplacePairs)
					{
						PoseAsset->RenamePoseOrCurveName(FindReplacePair.Key, FindReplacePair.Value);
					}
				}
			}
		}
		else if(USkeleton* Skeleton = Cast<USkeleton>(Asset))
		{
			if(GetFindWholeWord())
			{
				Asset->Modify();

				const FName FindCurveName(GetFindString());
				const FName ReplaceCurveName(GetReplaceString());
				Skeleton->RenameCurveMetaData(FindCurveName, ReplaceCurveName);
			}
			else
			{
				TArray<TPair<FName, FName>> FindReplacePairs;

				Skeleton->ForEachCurveMetaData([this, &FindReplacePairs](FName InCurveName, const FCurveMetaData& InMetaData)
				{
					const FString CurveNameString = InCurveName.ToString();
					if(NameMatches(CurveNameString))
					{
						const FName FindCurveName(InCurveName);
						const FString NewName = CurveNameString.Replace(*GetFindStringRef(), *GetReplaceStringRef(), GetSearchCase());
						const FName ReplaceCurveName(*NewName);

						FindReplacePairs.Emplace(FindCurveName, ReplaceCurveName);
					}
				});

				if(FindReplacePairs.Num() > 0)
				{
					Asset->Modify();

					for(const TPair<FName, FName>& FindReplacePair : FindReplacePairs)
					{
						Skeleton->RenameCurveMetaData(FindReplacePair.Key, FindReplacePair.Value);
					}
				}
			}
		}
		else if(USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(Asset))
		{
			if(UAnimCurveMetaData* AnimCurveMetaData = SkeletalMesh->GetAssetUserData<UAnimCurveMetaData>())
			{
				if(GetFindWholeWord())
				{
					Asset->Modify();

					const FName FindCurveName(GetFindString());
					const FName ReplaceCurveName(GetReplaceString());
					AnimCurveMetaData->RenameCurveMetaData(FindCurveName, ReplaceCurveName);
				}
				else
				{
					TArray<TPair<FName, FName>> FindReplacePairs;

					AnimCurveMetaData->ForEachCurveMetaData([this, &FindReplacePairs](FName InCurveName, const FCurveMetaData& InMetaData)
					{
						const FString CurveNameString = InCurveName.ToString();
						if(NameMatches(CurveNameString))
						{
							const FName FindCurveName(InCurveName);
							const FString NewName = CurveNameString.Replace(*GetFindStringRef(), *GetReplaceStringRef(), GetSearchCase());
							const FName ReplaceCurveName(*NewName);

							FindReplacePairs.Emplace(FindCurveName, ReplaceCurveName);
						}
					});

					if(FindReplacePairs.Num() > 0)
					{
						Asset->Modify();

						for(const TPair<FName, FName>& FindReplacePair : FindReplacePairs)
						{
							AnimCurveMetaData->RenameCurveMetaData(FindReplacePair.Key, FindReplacePair.Value);
						}
					}
				}
			}

			if(GetSearchMorphTargets())
			{
				if(GetFindWholeWord())
				{
					Asset->Modify();

					const FName FindMorphName(GetFindString());
					const FName ReplaceMorphName(GetReplaceString());
					SkeletalMesh->RenameMorphTarget(FindMorphName, ReplaceMorphName);
				}
				else
				{
					TArray<TPair<FName, FName>> FindReplacePairs;

					for(UMorphTarget* MorphTarget : SkeletalMesh->GetMorphTargets())
					{
						const FString MorphNameString = MorphTarget->GetName();
						if(NameMatches(MorphNameString))
						{
							const FName FindMorphName(MorphTarget->GetFName());
							const FString NewName = MorphNameString.Replace(*GetFindStringRef(), *GetReplaceStringRef(), GetSearchCase());
							const FName ReplaceMorphName(*NewName);

							FindReplacePairs.Emplace(FindMorphName, ReplaceMorphName);
						}
					}

					if(FindReplacePairs.Num() > 0)
					{
						Asset->Modify();

						for(const TPair<FName, FName>& FindReplacePair : FindReplacePairs)
						{
							SkeletalMesh->RenameMorphTarget(FindReplacePair.Key, FindReplacePair.Value);
						}
					}
				}
			}

			if(GetSearchMaterials())
			{
				if(GetFindWholeWord())
				{
					Asset->Modify();

					const FName FindMorphName(GetFindString());
					const FName ReplaceMorphName(GetReplaceString());
					UE::AnimAssetFindReplaceCurves::Private::RenameMaterialParameter(SkeletalMesh, FindMorphName, ReplaceMorphName);
				}
				else
				{
					TArray<TPair<FName, FName>> FindReplacePairs;

					for (const FSkeletalMaterial& SkeletalMaterial : SkeletalMesh->GetMaterials())
					{
						UMaterial* Material = (SkeletalMaterial.MaterialInterface != nullptr) ? SkeletalMaterial.MaterialInterface->GetMaterial() : nullptr;
						if (Material)
						{
							TArray<FMaterialParameterInfo> OutParameterInfo;
							TArray<FGuid> OutParameterIds;
							SkeletalMaterial.MaterialInterface->GetAllScalarParameterInfo(OutParameterInfo, OutParameterIds);

							for (const FMaterialParameterInfo& MaterialParameterInfo : OutParameterInfo)
							{
								const FString ParameterNameString = MaterialParameterInfo.Name.ToString();
								if(NameMatches(ParameterNameString))
								{
									const FName FindMorphName(MaterialParameterInfo.Name);
									const FString NewName = ParameterNameString.Replace(*GetFindStringRef(), *GetReplaceStringRef(), GetSearchCase());
									const FName ReplaceMorphName(*NewName);

									FindReplacePairs.Emplace(FindMorphName, ReplaceMorphName);
								}
							}
						}
					}

					if(FindReplacePairs.Num() > 0)
					{
						Asset->Modify();

						for(const TPair<FName, FName>& FindReplacePair : FindReplacePairs)
						{
							UE::AnimAssetFindReplaceCurves::Private::RenameMaterialParameter(SkeletalMesh, FindReplacePair.Key, FindReplacePair.Value);
						}
					}
				}
			}
		}
	}
}

void UAnimAssetFindReplaceCurves::RemoveInAsset(const FAssetData& InAssetData) const
{
	if(UObject* Asset = InAssetData.GetAsset())
	{
		if(UAnimSequenceBase* AnimSequenceBase = Cast<UAnimSequenceBase>(Asset))
		{
			Asset->MarkPackageDirty();

			if(GetFindWholeWord())
			{
				const FAnimationCurveIdentifier CurveId(FName(GetFindString()), ERawCurveTrackTypes::RCT_Float);
				IAnimationDataController::FScopedBracket ScopedBracket(AnimSequenceBase->GetController(), LOCTEXT("RemoveCurves", "Remove Curves"));
				AnimSequenceBase->GetController().RemoveCurve(CurveId);
			}
			else
			{
				TSet<FAnimationCurveIdentifier> CurveIdsToRemove;
				const TArray<FFloatCurve>& Curves = AnimSequenceBase->GetDataModel()->GetFloatCurves();
				for(const FFloatCurve& Curve : Curves)
				{
					FString CurveName = Curve.GetName().ToString();
					if(NameMatches(CurveName))
					{
						const FAnimationCurveIdentifier CurveId(*CurveName, ERawCurveTrackTypes::RCT_Float);
						CurveIdsToRemove.Add(CurveId);
					}
				}

				if(CurveIdsToRemove.Num() > 0)
				{
					IAnimationDataController::FScopedBracket ScopedBracket(AnimSequenceBase->GetController(), LOCTEXT("RemoveCurves", "Remove Curves"));
					for(const FAnimationCurveIdentifier& CurveIdToRemove : CurveIdsToRemove)
					{
						AnimSequenceBase->GetController().RemoveCurve(CurveIdToRemove);
					}
				}
			}
		}
		else if(UPoseAsset* PoseAsset = Cast<UPoseAsset>(Asset))
		{
			if(GetFindWholeWord())
			{
				Asset->Modify();
				PoseAsset->RemovePoseOrCurveNames({ FName(GetFindString()) });
			}
			else
			{
				TArray<FName> CurveIdsToRemove;

				for(const FName& PoseName : PoseAsset->GetPoseFNames())
				{
					if(NameMatches(PoseName.ToString()))
					{
						CurveIdsToRemove.AddUnique(PoseName);
					}
				}

				for(const FName& CurveName : PoseAsset->GetCurveFNames())
				{
					if(NameMatches(CurveName.ToString()))
					{
						CurveIdsToRemove.AddUnique(CurveName);
					}
				}
				
				if(CurveIdsToRemove.Num() > 0)
				{
					Asset->Modify();
					PoseAsset->RemovePoseOrCurveNames(CurveIdsToRemove);
				}
			}
		}
		else if(USkeleton* Skeleton = Cast<USkeleton>(Asset))
		{
			if(GetFindWholeWord())
			{
				Asset->Modify();
				Skeleton->RemoveCurveMetaData(FName(GetFindString()));
			}
			else
			{
				TArray<FName> CurvesToRemove;
				Skeleton->ForEachCurveMetaData([this, &CurvesToRemove](FName InCurveName, const FCurveMetaData& InCurveMetaData)
				{
				 	if(NameMatches(InCurveName.ToString()))
				 	{
						CurvesToRemove.AddUnique(InCurveName);
				 	}
				});

				if(CurvesToRemove.Num() > 0)
				{
					Asset->Modify();
					Skeleton->RemoveCurveMetaData(CurvesToRemove);
				}
			}
		}
		else if(USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(Asset))
		{
			if(UAnimCurveMetaData* AnimCurveMetaData = SkeletalMesh->GetAssetUserData<UAnimCurveMetaData>())
			{
				if(GetFindWholeWord())
				{
					Asset->Modify();
					AnimCurveMetaData->RemoveCurveMetaData(FName(GetFindString()));
				}
				else
				{
					TArray<FName> CurvesToRemove;
					AnimCurveMetaData->ForEachCurveMetaData([this, &CurvesToRemove](FName InCurveName, const FCurveMetaData& InCurveMetaData)
					{
						if(NameMatches(InCurveName.ToString()))
						{
							CurvesToRemove.AddUnique(InCurveName);
						}
					});

					if(CurvesToRemove.Num() > 0)
					{
						Asset->Modify();
						AnimCurveMetaData->RemoveCurveMetaData(CurvesToRemove);
					}
				}
			}

			if(GetSearchMorphTargets())
			{
				if(GetFindWholeWord())
				{
					Asset->Modify();
					SkeletalMesh->RemoveMorphTargets({FName(GetFindString())});
				}
				else
				{
					TArray<FName> MorphsToRemove;
					for(UMorphTarget* MorphTarget : SkeletalMesh->GetMorphTargets())
					{
						if(NameMatches(MorphTarget->GetFName().ToString()))
						{
							MorphsToRemove.AddUnique(MorphTarget->GetFName());
						}
					}

					if(MorphsToRemove.Num() > 0)
					{
						Asset->Modify();
						SkeletalMesh->RemoveMorphTargets(MorphsToRemove);
					}
				}
			}

			if(GetSearchMaterials())
			{
				if(GetFindWholeWord())
				{
					Asset->Modify();
					UE::AnimAssetFindReplaceCurves::Private::RemoveMaterialParameters(SkeletalMesh, {FName(GetFindString())});
				}
				else
				{
					TArray<FName> MaterialParametersToRemove;
					for(UMorphTarget* MorphTarget : SkeletalMesh->GetMorphTargets())
					{
						if(NameMatches(MorphTarget->GetFName().ToString()))
						{
							MaterialParametersToRemove.AddUnique(MorphTarget->GetFName());
						}
					}

					if(MaterialParametersToRemove.Num() > 0)
					{
						Asset->Modify();
						UE::AnimAssetFindReplaceCurves::Private::RemoveMaterialParameters(SkeletalMesh, MaterialParametersToRemove);
					}
				}
			}
		}
	}
}

void UAnimAssetFindReplaceCurves::ExtendToolbar(FToolMenuSection& InSection)
{
	Super::ExtendToolbar(InSection);

	InSection.AddDynamicEntry("CurveSearchOptions", FNewToolMenuSectionDelegate::CreateLambda([this](FToolMenuSection& InSection)
	{
		FToolUIAction SearchMorphTargetsCheckbox;
		SearchMorphTargetsCheckbox.ExecuteAction = FToolMenuExecuteAction::CreateLambda([this](const FToolMenuContext& InContext)
		{
			SetSearchMorphTargets(!GetSearchMorphTargets());
		});

		SearchMorphTargetsCheckbox.GetActionCheckState = FToolMenuGetActionCheckState::CreateLambda([this](const FToolMenuContext& InContext)
		{
			return GetSearchMorphTargets() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		});
		
		InSection.AddEntry(
		FToolMenuEntry::InitToolBarButton(
			"SearchMorphTargets",
			SearchMorphTargetsCheckbox,
			LOCTEXT("SearchMorphTargetsCheckboxLabel", "Morph Targets"),
			LOCTEXT("SearchMorphTargetsCheckboxTooltip", "Whether to search morph targets in skeletal meshes while searching for curves."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Persona.Tabs.MorphTargetPreviewer"),
			EUserInterfaceActionType::ToggleButton));

		FToolUIAction SearchMaterialsCheckbox;
		SearchMaterialsCheckbox.ExecuteAction = FToolMenuExecuteAction::CreateLambda([this](const FToolMenuContext& InContext)
		{
			SetSearchMaterials(!GetSearchMaterials());
		});

		SearchMaterialsCheckbox.GetActionCheckState = FToolMenuGetActionCheckState::CreateLambda([this](const FToolMenuContext& InContext)
		{
			return GetSearchMaterials() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		});
		
		InSection.AddEntry(
		FToolMenuEntry::InitToolBarButton(
			"SearchMaterials",
			SearchMaterialsCheckbox,
			LOCTEXT("SearchMaterialsCheckboxLabel", "Materials"),
			LOCTEXT("SearchMaterialsCheckboxTooltip", "Whether to search material parameters in skeletal meshes while searching for curves."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.Material"),
			EUserInterfaceActionType::ToggleButton));
	}));
}

void UAnimAssetFindReplaceCurves::GetAutoCompleteNames(TArrayView<FAssetData> InAssetDatas, TSet<FString>& OutUniqueNames) const
{
	for (const FAssetData& AssetData : InAssetDatas)
	{
		auto GetNamesForAsset = [this, &OutUniqueNames](const FAssetData& InAssetData, FName InTag, const FString& InDelimiter)
		{
			const FString TagValue = InAssetData.GetTagValueRef<FString>(InTag);
			if (!TagValue.IsEmpty())
			{
				UE::String::ParseTokens(TagValue, *USkeleton::CurveTagDelimiter, [&OutUniqueNames](FStringView InToken)
				{
					OutUniqueNames.Add(FString(InToken));
				}, UE::String::EParseTokensOptions::SkipEmpty);
			}
		};

		GetNamesForAsset(AssetData, USkeleton::CurveNameTag, USkeleton::CurveTagDelimiter);

		UClass* AssetClass = AssetData.GetClass();
		if(AssetClass->IsChildOf(USkeletalMesh::StaticClass()))
		{
			GetNamesForAsset(AssetData, USkeletalMesh::MorphNamesTag, USkeletalMesh::MorphNamesTagDelimiter);
			GetNamesForAsset(AssetData, USkeletalMesh::MaterialParamNamesTag, USkeletalMesh::MaterialParamNamesTagDelimiter);
		}
	}
}

void UAnimAssetFindReplaceCurves::SetSearchMorphTargets(bool bInSearchMorphTargets)
{
	bSearchMorphTargets = bInSearchMorphTargets;
	RequestRefreshSearchResults();
}

void UAnimAssetFindReplaceCurves::SetSearchMaterials(bool bInSearchMaterials)
{
	bSearchMaterials = bInSearchMaterials;
	RequestRefreshSearchResults();
}

#undef LOCTEXT_NAMESPACE
