// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimAssetFindReplaceNotifies.h"

#include "IEditableSkeleton.h"
#include "ISkeletonEditorModule.h"
#include "PersonaModule.h"
#include "SAnimAssetFindReplace.h"
#include "Animation/Skeleton.h"
#include "Animation/AnimationAsset.h"
#include "Animation/AnimSequence.h"
#include "Engine/SkeletalMesh.h"
#include "String/ParseTokens.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimAssetFindReplaceNotifies)

#define LOCTEXT_NAMESPACE "AnimAssetFindReplaceNotifies"

FString UAnimAssetFindReplaceNotifies::GetFindResultStringFromAssetData(const FAssetData& InAssetData) const
{
	if(GetFindString().IsEmpty())
	{
		return FString();
	}

	auto GetMatchingNotifyNamesForAsset = [this](const FAssetData& InAssetData, TArray<FString>& OutNotifyNames)
	{
		const FString TagValue = InAssetData.GetTagValueRef<FString>(USkeleton::AnimNotifyTag);
		if (!TagValue.IsEmpty())
		{
			UE::String::ParseTokens(TagValue, *USkeleton::AnimNotifyTagDelimiter, [this, &OutNotifyNames](FStringView InToken)
			{
				if(GetFindWholeWord())
				{
					if(InToken.Compare(GetFindString(), GetSearchCase()) == 0)
					{
						OutNotifyNames.Add(FString(InToken));
					}
				}
				else
				{
					if(UE::String::FindFirst(InToken, GetFindString(), GetSearchCase()) != INDEX_NONE)
					{
						OutNotifyNames.Add(FString(InToken));
					}
				}
			}, UE::String::EParseTokensOptions::SkipEmpty);
		}
	};

	TStringBuilder<128> Builder;
	TArray<FString> NotifyNames;
	GetMatchingNotifyNamesForAsset(InAssetData, NotifyNames);
	if(NotifyNames.Num() > 0)
	{
		for(int32 NameIndex = 0; NameIndex < NotifyNames.Num(); ++NameIndex)
		{
			Builder.Append(NotifyNames[NameIndex]);
			if(NameIndex != NotifyNames.Num() - 1)
			{
				Builder.Append(TEXT(", "));
			}
		}
	}

	return FString(Builder.ToString());
}

TConstArrayView<UClass*> UAnimAssetFindReplaceNotifies::GetSupportedAssetTypes() const
{
	FPersonaModule& PersonaModule = FModuleManager::GetModuleChecked<FPersonaModule>("Persona");
	
	static TArray<UClass*> Types;
	static uint32 Version = 0;
	if(Types.Num() == 0 || Version != PersonaModule.GetNotifyHostAssetVersion())
	{
		Types.Reset();
		Types.Append({ UAnimSequenceBase::StaticClass(), USkeleton::StaticClass() });

		for(const FTopLevelAssetPath& TopLevelAssetPath : PersonaModule.GetAllNotifyHostAssetClassPaths())
		{
			FSoftClassPath SoftClassPath(TopLevelAssetPath.ToString());
			if(UClass* ResolvedClass = SoftClassPath.ResolveClass())
			{
				Types.Add(ResolvedClass);
			}
		}

		Version = PersonaModule.GetNotifyHostAssetVersion();
	}

	return Types;
}

bool UAnimAssetFindReplaceNotifies::ShouldFilterOutAsset(const FAssetData& InAssetData, bool& bOutIsOldAsset) const
{
	FString TagValue;
	if(InAssetData.GetTagValue<FString>(USkeleton::AnimNotifyTag, TagValue))
	{
		bOutIsOldAsset = false;

		if(GetFindString().IsEmpty())
		{
			return true;
		}

		if(GetSkeletonFilter().IsValid() )
		{
			if(InAssetData.GetClass() != USkeleton::StaticClass())
			{
				FString SkeletonPath;
				if(InAssetData.GetTagValue<FString>(TEXT("Skeleton"), SkeletonPath))
				{
					if(SkeletonPath != GetSkeletonFilter().GetExportTextName())
					{
						return true;
					}
				}
			}
			else
			{
				if(InAssetData.ToSoftObjectPath() != GetSkeletonFilter().ToSoftObjectPath())
				{
					return true;
				}
			}
		}

		bool bFoundMatch = false;
		UE::String::ParseTokens(TagValue, *USkeleton::AnimNotifyTagDelimiter, [this, &bFoundMatch](FStringView InToken)
		{
			if(NameMatches(InToken))
			{
				bFoundMatch = true;
			}
		}, UE::String::EParseTokensOptions::SkipEmpty);

		if(bFoundMatch)
		{
			return false;
		}
	}
	else
	{
		const UClass* AssetClass = InAssetData.GetClass();
		if(AssetClass->IsChildOf(UAnimSequenceBase::StaticClass()))
		{
			bOutIsOldAsset = true;
		}
		else if(AssetClass->IsChildOf(USkeleton::StaticClass()))
		{
			bOutIsOldAsset = true;
		}
		else
		{
			// Assume unknown assets are not 'old'
			bOutIsOldAsset = false;
		}
	}

	return true;
}

void UAnimAssetFindReplaceNotifies::ReplaceInAsset(const FAssetData& InAssetData) const
{
	if(UObject* Asset = InAssetData.GetAsset())
	{
		if(UAnimSequenceBase* AnimSequenceBase = Cast<UAnimSequenceBase>(Asset))
		{
			if(GetFindWholeWord())
			{
				Asset->Modify();
				AnimSequenceBase->RenameNotifies(FName(GetFindString()), FName(GetReplaceString()));
			}
			else
			{
				TArray<TPair<FName, FName>> FindReplacePairs;

				for(const FAnimNotifyEvent& Notify : AnimSequenceBase->Notifies)
				{
					// Only handle named notifiesif(!Notify.IsBlueprintNotify())
					{
						FString NotifyName = Notify.NotifyName.ToString();
						if(NameMatches(NotifyName))
						{
							const FName FindNotifyName(*NotifyName);
							const FString NewName = NotifyName.Replace(*GetFindStringRef(), *GetReplaceStringRef(), GetSearchCase());
							const FName ReplaceNotifyName(*NewName);
							FindReplacePairs.AddUnique(TPair<FName, FName>(FindNotifyName, ReplaceNotifyName));
						}
					}
				}

				if(FindReplacePairs.Num() > 0)
				{
					Asset->Modify();

					for(const TPair<FName, FName>& FindReplacePair : FindReplacePairs)
					{
						AnimSequenceBase->RenameNotifies(FindReplacePair.Key, FindReplacePair.Value);
					}
				}
			}
		}
		else if(USkeleton* Skeleton = Cast<USkeleton>(Asset))
		{
			ISkeletonEditorModule& SkeletonEditorModule = FModuleManager::GetModuleChecked<ISkeletonEditorModule>("SkeletonEditor");
			TSharedRef<IEditableSkeleton> EditableSkeleton = SkeletonEditorModule.CreateEditableSkeleton(Skeleton);
			if(GetFindWholeWord())
			{
				Asset->Modify();
				EditableSkeleton->RenameNotify(FName(GetReplaceString()), FName(GetFindString()), false);
			}
			else
			{
				TArray<TPair<FName, FName>> FindReplacePairs;

				for(const FName& NotifyName : Skeleton->AnimationNotifies)
				{
					FString NotifyString = NotifyName.ToString();
					if(NameMatches(NotifyString))
					{
						const FString NewName = NotifyString.Replace(*GetFindStringRef(), *GetReplaceStringRef(), GetSearchCase());
						const FName ReplaceNotifyName(*NewName);
						FindReplacePairs.AddUnique(TPair<FName, FName>(NotifyName, ReplaceNotifyName));
					}
				}

				if(FindReplacePairs.Num() > 0)
				{
					Asset->Modify();
					for(const TPair<FName, FName>& FindReplacePair : FindReplacePairs)
					{
						EditableSkeleton->RenameNotify(FindReplacePair.Value, FindReplacePair.Key, false);
					}
				}
			}
		}
		else
		{
			FPersonaModule& PersonaModule = FModuleManager::GetModuleChecked<FPersonaModule>("Persona");
			if(const FPersonaModule::FNotifyHostAssetParameters* NotifyHostParameters = PersonaModule.FindNotifyHostAsset(Asset))
			{
				NotifyHostParameters->OnReplaceNotify.ExecuteIfBound(Asset, GetFindStringRef(), GetReplaceStringRef(), GetFindWholeWord(), GetSearchCase());
			}
		}
	}
}

void UAnimAssetFindReplaceNotifies::RemoveInAsset(const FAssetData& InAssetData) const
{
	if(UObject* Asset = InAssetData.GetAsset())
	{
		if(UAnimSequenceBase* AnimSequenceBase = Cast<UAnimSequenceBase>(Asset))
		{
			if(GetFindWholeWord())
			{
				Asset->Modify();
				AnimSequenceBase->RemoveNotifies( { FName(GetFindString()) } );
			}
			else
			{
				TArray<FName> NotifiesToRemove;
				for(const FAnimNotifyEvent& Notify : AnimSequenceBase->Notifies)
				{
					FString NotifyNameString = Notify.NotifyName.ToString();
					if(NameMatches(NotifyNameString))
					{
						NotifiesToRemove.AddUnique(Notify.NotifyName);
					}
				}

				if(NotifiesToRemove.Num() > 0)
				{
					Asset->Modify();
					AnimSequenceBase->RemoveNotifies(NotifiesToRemove);
				}
			}
		}
		else if(USkeleton* Skeleton = Cast<USkeleton>(Asset))
		{
			ISkeletonEditorModule& SkeletonEditorModule = FModuleManager::GetModuleChecked<ISkeletonEditorModule>("SkeletonEditor");
			TSharedRef<IEditableSkeleton> EditableSkeleton = SkeletonEditorModule.CreateEditableSkeleton(Skeleton);
			if(GetFindWholeWord())
			{
				EditableSkeleton->DeleteAnimNotifies( { FName(GetFindString()) }, false);
			}
			else
			{
				TArray<FName> NotifiesToRemove;

				for(const FName& NotifyName : Skeleton->AnimationNotifies)
				{
					if(NameMatches(NotifyName.ToString()))
					{
						NotifiesToRemove.AddUnique(NotifyName);
					}
				}

				if(NotifiesToRemove.Num() > 0)
				{
					EditableSkeleton->DeleteAnimNotifies(NotifiesToRemove, false);
				}
			}
		}
	}
	else
	{
		FPersonaModule& PersonaModule = FModuleManager::GetModuleChecked<FPersonaModule>("Persona");
		if(const FPersonaModule::FNotifyHostAssetParameters* NotifyHostParameters = PersonaModule.FindNotifyHostAsset(Asset))
		{
			NotifyHostParameters->OnRemoveNotify.ExecuteIfBound(Asset, GetFindStringRef(), GetFindWholeWord(), GetSearchCase());
		}
	}
}

void UAnimAssetFindReplaceNotifies::GetAutoCompleteNames(TArrayView<FAssetData> InAssetDatas, TSet<FString>& OutUniqueNames) const
{
	for (const FAssetData& AssetData : InAssetDatas)
	{
		const FString TagValue = AssetData.GetTagValueRef<FString>(USkeleton::AnimNotifyTag);
		if (!TagValue.IsEmpty())
		{
			UE::String::ParseTokens(TagValue, *USkeleton::AnimNotifyTagDelimiter, [&OutUniqueNames](FStringView InToken)
			{
				OutUniqueNames.Add(FString(InToken));
			}, UE::String::EParseTokensOptions::SkipEmpty);
		}
	}
}

#undef LOCTEXT_NAMESPACE
