// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimAssetFindReplaceSyncMarkers.h"

#include "IEditableSkeleton.h"
#include "ISkeletonEditorModule.h"
#include "SAnimAssetFindReplace.h"
#include "Animation/Skeleton.h"
#include "Animation/AnimationAsset.h"
#include "Animation/AnimSequence.h"
#include "Engine/SkeletalMesh.h"
#include "String/ParseTokens.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimAssetFindReplaceSyncMarkers)

#define LOCTEXT_NAMESPACE "AnimAssetFindReplaceSyncMarkers"

FString UAnimAssetFindReplaceSyncMarkers::GetFindResultStringFromAssetData(const FAssetData& InAssetData) const
{
	if(GetFindString().IsEmpty())
	{
		return FString();
	}

	auto GetMatchingSyncMarkerNamesForAsset = [this](const FAssetData& InAssetData, TArray<FString>& OutSyncMarkerNames)
	{
		const FString TagValue = InAssetData.GetTagValueRef<FString>(USkeleton::AnimSyncMarkerTag);
		if (!TagValue.IsEmpty())
		{
			UE::String::ParseTokens(TagValue, *USkeleton::AnimSyncMarkerTagDelimiter, [this, &OutSyncMarkerNames](FStringView InToken)
			{
				if(GetFindWholeWord())
				{
					if(InToken.Compare(GetFindString(), GetSearchCase()) == 0)
					{
						OutSyncMarkerNames.Add(FString(InToken));
					}
				}
				else
				{
					if(UE::String::FindFirst(InToken, GetFindString(), GetSearchCase()) != INDEX_NONE)
					{
						OutSyncMarkerNames.Add(FString(InToken));
					}
				}
			}, UE::String::EParseTokensOptions::SkipEmpty);
		}
	};

	TStringBuilder<128> Builder;
	TArray<FString> SyncMarkerNames;
	GetMatchingSyncMarkerNamesForAsset(InAssetData, SyncMarkerNames);
	if(SyncMarkerNames.Num() > 0)
	{
		for(int32 NameIndex = 0; NameIndex < SyncMarkerNames.Num(); ++NameIndex)
		{
			Builder.Append(SyncMarkerNames[NameIndex]);
			if(NameIndex != SyncMarkerNames.Num() - 1)
			{
				Builder.Append(TEXT(", "));
			}
		}
	}

	return FString(Builder.ToString());
}

TConstArrayView<UClass*> UAnimAssetFindReplaceSyncMarkers::GetSupportedAssetTypes() const
{
	static UClass* Types[] = { UAnimSequence::StaticClass(), USkeleton::StaticClass() };
	return Types;
}

bool UAnimAssetFindReplaceSyncMarkers::ShouldFilterOutAsset(const FAssetData& InAssetData, bool& bOutIsOldAsset) const
{
	FString TagValue;
	if(InAssetData.GetTagValue<FString>(USkeleton::AnimSyncMarkerTag, TagValue))
	{
		bOutIsOldAsset = false;

		if(GetFindString().IsEmpty())
		{
			return true;
		}

		if(GetSkeletonFilter().IsValid() )
		{
			if(InAssetData.GetClass() != USkeleton::StaticClass())
			{
				FString SkeletonPath;
				if(InAssetData.GetTagValue<FString>(TEXT("Skeleton"), SkeletonPath))
				{
					if(SkeletonPath != GetSkeletonFilter().GetExportTextName())
					{
						return true;
					}
				}
			}
			else
			{
				if(InAssetData.ToSoftObjectPath() != GetSkeletonFilter().ToSoftObjectPath())
				{
					return true;
				}
			}
		}

		bool bFoundMatch = false;
		UE::String::ParseTokens(TagValue, *USkeleton::AnimSyncMarkerTagDelimiter, [this, &bFoundMatch](FStringView InToken)
		{
			if(NameMatches(InToken))
			{
				bFoundMatch = true;
			}
		}, UE::String::EParseTokensOptions::SkipEmpty);

		if(bFoundMatch)
		{
			return false;
		}
	}
	else
	{
		const UClass* AssetClass = InAssetData.GetClass();
		if(AssetClass->IsChildOf(UAnimSequence::StaticClass()))
		{
			bOutIsOldAsset = true;
		}
		else if(AssetClass->IsChildOf(USkeleton::StaticClass()))
		{
			bOutIsOldAsset = true;
		}
		else
		{
			// Assume unknown assets are not 'old'
			bOutIsOldAsset = false;
		}
	}

	return true;
}

void UAnimAssetFindReplaceSyncMarkers::ReplaceInAsset(const FAssetData& InAssetData) const
{
	if(UObject* Asset = InAssetData.GetAsset())
	{
		if(UAnimSequence* AnimSequence = Cast<UAnimSequence>(Asset))
		{
			if(GetFindWholeWord())
			{
				Asset->Modify();
				AnimSequence->RenameSyncMarkers(FName(GetFindString()), FName(GetReplaceString()));
			}
			else
			{
				TArray<TPair<FName, FName>> FindReplacePairs;

				for(const FAnimSyncMarker& SyncMarker : AnimSequence->AuthoredSyncMarkers)
				{
					FString SyncMarkerName = SyncMarker.MarkerName.ToString();
					if(NameMatches(SyncMarkerName))
					{
						const FName FindSyncMarkerName(*SyncMarkerName);
						const FString NewName = SyncMarkerName.Replace(*GetFindStringRef(), *GetReplaceStringRef(), GetSearchCase());
						const FName ReplaceSyncMarkerName(*NewName);
						FindReplacePairs.AddUnique(TPair<FName, FName>(FindSyncMarkerName, ReplaceSyncMarkerName));
					}
				}

				if(FindReplacePairs.Num() > 0)
				{
					Asset->Modify();

					for(const TPair<FName, FName>& FindReplacePair : FindReplacePairs)
					{
						AnimSequence->RenameSyncMarkers(FindReplacePair.Key, FindReplacePair.Value);
					}
				}
			}
		}
		else if(USkeleton* Skeleton = Cast<USkeleton>(Asset))
		{
			ISkeletonEditorModule& SkeletonEditorModule = FModuleManager::GetModuleChecked<ISkeletonEditorModule>("SkeletonEditor");
			TSharedRef<IEditableSkeleton> EditableSkeleton = SkeletonEditorModule.CreateEditableSkeleton(Skeleton);
			if(GetFindWholeWord())
			{
				Asset->Modify();
				EditableSkeleton->RenameSyncMarker(FName(GetReplaceString()), FName(GetFindString()));
			}
			else
			{
				TArray<TPair<FName, FName>> FindReplacePairs;

				for(const FName& SyncMarkerName : Skeleton->GetExistingMarkerNames())
				{
					FString SyncMarkerString = SyncMarkerName.ToString();
					if(NameMatches(SyncMarkerString))
					{
						const FString NewName = SyncMarkerString.Replace(*GetFindStringRef(), *GetReplaceStringRef(), GetSearchCase());
						const FName ReplaceSyncMarkerName(*NewName);
						FindReplacePairs.AddUnique(TPair<FName, FName>(SyncMarkerName, ReplaceSyncMarkerName));
					}
				}

				if(FindReplacePairs.Num() > 0)
				{
					Asset->Modify();
					for(const TPair<FName, FName>& FindReplacePair : FindReplacePairs)
					{
						EditableSkeleton->RenameSyncMarker(FindReplacePair.Value, FindReplacePair.Key);
					}
				}
			}
		}
	}
}

void UAnimAssetFindReplaceSyncMarkers::RemoveInAsset(const FAssetData& InAssetData) const
{
	if(UObject* Asset = InAssetData.GetAsset())
	{
		if(UAnimSequence* AnimSequence = Cast<UAnimSequence>(Asset))
		{
			if(GetFindWholeWord())
			{
				Asset->Modify();
				AnimSequence->RemoveSyncMarkers( { FName(GetFindString()) } );
			}
			else
			{
				TArray<FName> SyncMarkerToRemove;
				for(const FAnimSyncMarker& SyncMarker : AnimSequence->AuthoredSyncMarkers)
				{
					FString SyncMarkerNameString = SyncMarker.MarkerName.ToString();
					if(NameMatches(SyncMarkerNameString))
					{
						SyncMarkerToRemove.AddUnique(SyncMarker.MarkerName);
					}
				}

				if(SyncMarkerToRemove.Num() > 0)
				{
					Asset->Modify();
					AnimSequence->RemoveSyncMarkers(SyncMarkerToRemove);
				}
			}
		}
		else if(USkeleton* Skeleton = Cast<USkeleton>(Asset))
		{
			ISkeletonEditorModule& SkeletonEditorModule = FModuleManager::GetModuleChecked<ISkeletonEditorModule>("SkeletonEditor");
			TSharedRef<IEditableSkeleton> EditableSkeleton = SkeletonEditorModule.CreateEditableSkeleton(Skeleton);
			if(GetFindWholeWord())
			{
				EditableSkeleton->DeleteSyncMarkers( { FName(GetFindString()) } );
			}
			else
			{
				TArray<FName> SyncMarkersToRemove;

				for(const FName& SyncMarkerName : Skeleton->GetExistingMarkerNames())
				{
					if(NameMatches(SyncMarkerName.ToString()))
					{
						SyncMarkersToRemove.AddUnique(SyncMarkerName);
					}
				}

				if(SyncMarkersToRemove.Num() > 0)
				{
					EditableSkeleton->DeleteSyncMarkers(SyncMarkersToRemove);
				}
			}
		}
	}
}

void UAnimAssetFindReplaceSyncMarkers::GetAutoCompleteNames(TArrayView<FAssetData> InAssetDatas, TSet<FString>& OutUniqueNames) const
{
	for (const FAssetData& AssetData : InAssetDatas)
	{
		const FString TagValue = AssetData.GetTagValueRef<FString>(USkeleton::AnimSyncMarkerTag);
		if (!TagValue.IsEmpty())
		{
			UE::String::ParseTokens(TagValue, *USkeleton::AnimSyncMarkerTagDelimiter, [&OutUniqueNames](FStringView InToken)
			{
				OutUniqueNames.Add(FString(InToken));
			}, UE::String::EParseTokensOptions::SkipEmpty);
		}
	}
}

#undef LOCTEXT_NAMESPACE
