// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNodeDetails.h"
#include "AnimationGraphSchema.h"
#include "Modules/ModuleManager.h"
#include "UObject/UnrealType.h"
#include "Widgets/Text/STextBlock.h"
#include "BoneContainer.h"
#include "Engine/SkeletalMesh.h"
#include "Animation/AnimationAsset.h"
#include "DetailWidgetRow.h"
#include "IDetailPropertyRow.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "PropertyCustomizationHelpers.h"
#include "SlateOptMacros.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Animation/AnimInstance.h"
#include "Animation/EditorParentPlayerListObj.h"
#include "Animation/AnimBlueprintGeneratedClass.h"
#include "Widgets/SToolTip.h"
#include "IDocumentation.h"
#include "ObjectEditorUtils.h"
#include "AnimGraphNode_Base.h"
#include "Widgets/Views/STreeView.h"
#include "BoneSelectionWidget.h"
#include "Animation/BlendProfile.h"
#include "AnimGraphNode_AssetPlayerBase.h"
#include "ISkeletonEditorModule.h"
#include "EdGraph/EdGraph.h"
#include "BlueprintEditor.h"
#include "Animation/EditorAnimCurveBoneLinks.h"
#include "IEditableSkeleton.h"
#include "IDetailChildrenBuilder.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "BoneControllers/AnimNode_SkeletalControlBase.h"
#include "AnimGraphNode_StateMachine.h"
#include "AnimGraphNode_SequencePlayer.h"
#include "Engine/SkeletalMeshSocket.h"
#include "Styling/CoreStyle.h"
#include "LODInfoUILayout.h"
#include "IPersonaToolkit.h"
#include "Interfaces/Interface_BoneReferenceSkeletonProvider.h"
#include "IPropertyAccessEditor.h"
#include "IPropertyUtilities.h"
#include "Algo/Accumulate.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SWidgetSwitcher.h"

#define LOCTEXT_NAMESPACE "KismetNodeWithOptionalPinsDetails"

/////////////////////////////////////////////////////
// FAnimGraphNodeDetails 

TSharedRef<IDetailCustomization> FAnimGraphNodeDetails::MakeInstance()
{
	return MakeShareable(new FAnimGraphNodeDetails());
}

void FAnimGraphNodeDetails::CustomizeDetails(class IDetailLayoutBuilder& DetailBuilder)
{
	DetailLayoutBuilder = &DetailBuilder;
	
	TArray< TWeakObjectPtr<UObject> > SelectedObjectsList;
	DetailBuilder.GetObjectsBeingCustomized(SelectedObjectsList);

	// Hide the pin options property; it's represented inline per-property instead
	IDetailCategoryBuilder& PinOptionsCategory = DetailBuilder.EditCategory("PinOptions");
	TSharedRef<IPropertyHandle> AvailablePins = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UAnimGraphNode_Base, ShowPinForProperties));
	DetailBuilder.HideProperty(AvailablePins);

	// get first animgraph nodes
	UAnimGraphNode_Base* AnimGraphNode = Cast<UAnimGraphNode_Base>(SelectedObjectsList[0].Get());
	if (AnimGraphNode == nullptr)
	{
		return;
	}

	// Ensure that switching the binding type will refresh the details panel
	IDetailCategoryBuilder& BindingCategory = DetailBuilder.EditCategory("Bindings");
	BindingCategory.SetSortOrder(MAX_int32);
	
	TSharedRef<IPropertyHandle> BindingProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UAnimGraphNode_Base, Binding));
	BindingProperty->SetOnPropertyValueChanged(FSimpleDelegate::CreateLambda([&DetailBuilder]()
	{
		DetailBuilder.ForceRefreshDetails();
	}));

	AnimGraphNode->OnPinVisibilityChanged().AddSP(this, &FAnimGraphNodeDetails::OnPinVisibilityChanged);

	// make sure type matches with all the nodes. 
	const UAnimGraphNode_Base* FirstNodeType = AnimGraphNode;
	for (int32 Index = 1; Index < SelectedObjectsList.Num(); ++Index)
	{
		UAnimGraphNode_Base* CurrentNode = Cast<UAnimGraphNode_Base>(SelectedObjectsList[Index].Get());
		if (!CurrentNode || CurrentNode->GetClass() != FirstNodeType->GetClass())
		{
			// if type mismatches, multi selection doesn't work, just return
			return;
		}
		else
		{
			CurrentNode->OnPinVisibilityChanged().AddSP(this, &FAnimGraphNodeDetails::OnPinVisibilityChanged);
		}
	}

	TargetSkeleton = AnimGraphNode->HasValidBlueprint() ? AnimGraphNode->GetAnimBlueprint()->TargetSkeleton.Get() : nullptr;
	TargetSkeletonName = TargetSkeleton ? FObjectPropertyBase::GetExportPath(TargetSkeleton) : FString(TEXT(""));
	bIsAnimBPTemplate = AnimGraphNode->HasValidBlueprint() ? AnimGraphNode->GetAnimBlueprint()->bIsTemplate : false;

	// Get the node property
	const FStructProperty* NodeProperty = AnimGraphNode->GetFNodeProperty();
	if (NodeProperty == nullptr)
	{
		return;
	}

	// In the case of property row generators, we dont have a details view, so we can skip this work
	if(DetailBuilder.GetDetailsViewSharedPtr() == nullptr)
	{
		return;
	}
	
	// customize anim graph node's own details if needed
	AnimGraphNode->CustomizeDetails(DetailBuilder);

	// Hide categories not relevant for interface BPs
	if(AnimGraphNode->HasValidBlueprint() && AnimGraphNode->GetBlueprint()->BlueprintType == BPTYPE_Interface)
	{
		DetailBuilder.HideCategory("Functions");
		DetailBuilder.HideCategory("Tag");
	}
	
	// Hide the Node property as we are going to be adding its inner properties below
	TSharedRef<IPropertyHandle> NodePropertyHandle = DetailBuilder.GetProperty(NodeProperty->GetFName(), AnimGraphNode->GetClass());
	DetailBuilder.HideProperty(NodePropertyHandle);

	uint32 NumChildHandles = 0;
	FPropertyAccess::Result Result = NodePropertyHandle->GetNumChildren(NumChildHandles);
	if (Result != FPropertyAccess::Fail)
	{
		for (uint32 ChildHandleIndex = 0; ChildHandleIndex < NumChildHandles; ++ChildHandleIndex)
		{
			TSharedPtr<IPropertyHandle> TargetPropertyHandle = NodePropertyHandle->GetChildHandle(ChildHandleIndex);
			if (TargetPropertyHandle.IsValid())
			{
				FProperty* TargetProperty = TargetPropertyHandle->GetProperty();
				IDetailCategoryBuilder& CurrentCategory = DetailBuilder.EditCategory(FObjectEditorUtils::GetCategoryFName(TargetProperty));

				int32 CustomPinIndex = AnimGraphNode->ShowPinForProperties.IndexOfByPredicate([TargetProperty](const FOptionalPinFromProperty& InOptionalPin)
				{
					return TargetProperty->GetFName() == InOptionalPin.PropertyName;
				});

				if (CustomPinIndex != INDEX_NONE)
				{
					const FOptionalPinFromProperty& OptionalPin = AnimGraphNode->ShowPinForProperties[CustomPinIndex];

					// Not optional
					if (!OptionalPin.bCanToggleVisibility && OptionalPin.bShowPin)
					{
						// Always displayed as a pin, so hide the property
						DetailBuilder.HideProperty(TargetPropertyHandle);
						continue;
					}

					if (!TargetPropertyHandle->GetProperty())
					{
						continue;
					}

					// if customized, do not do anything
					if (TargetPropertyHandle->IsCustomized())
					{
						continue;
					}

					// sometimes because of order of customization
					// this gets called first for the node you'd like to customize
					// then the above statement won't work
					// so you can mark certain property to have meta data "CustomizeProperty"
					// which will trigger below statement
					if (OptionalPin.bPropertyIsCustomized)
					{
						continue;
					}

					TSharedRef<SWidget> InternalCustomWidget = CreatePropertyWidget(TargetProperty, TargetPropertyHandle.ToSharedRef(), AnimGraphNode->GetClass());

					if (OptionalPin.bCanToggleVisibility)
					{
						IDetailPropertyRow& PropertyRow = CurrentCategory.AddProperty(TargetPropertyHandle);

						TSharedPtr<SWidget> NameWidget;
						TSharedPtr<SWidget> ValueWidget;
						FDetailWidgetRow Row;
						PropertyRow.GetDefaultWidgets(NameWidget, ValueWidget, Row);

						ValueWidget = (InternalCustomWidget == SNullWidget::NullWidget) ? ValueWidget : InternalCustomWidget;

						const FName OptionalPinArrayEntryName(*FString::Printf(TEXT("ShowPinForProperties[%d].bShowPin"), CustomPinIndex));
						TSharedRef<IPropertyHandle> ShowHidePropertyHandle = DetailBuilder.GetProperty(OptionalPinArrayEntryName);

						ShowHidePropertyHandle->MarkHiddenByCustomization();

						ValueWidget->SetVisibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FAnimGraphNodeDetails::GetVisibilityOfProperty, ShowHidePropertyHandle)));

						// If we have an edit condition, that comes as part of the default name widget, so just use a text block to avoid duplicate checkboxes
						TSharedPtr<SWidget> PropertyNameWidget;
						if (TargetProperty->HasMetaData(TEXT("EditCondition")))
						{
							PropertyNameWidget = SNew(STextBlock)
							.Text(TargetProperty->GetDisplayNameText())
							.Font(IDetailLayoutBuilder::GetDetailFont())
							.ToolTipText(TargetProperty->GetToolTipText());
						}
						else
						{
							PropertyNameWidget = NameWidget;
						}

						NameWidget = PropertyNameWidget;

						const bool bShowChildren = GetVisibilityOfProperty(ShowHidePropertyHandle) == EVisibility::Visible;
						PropertyRow.CustomWidget(bShowChildren)
						.NameContent()
						.MinDesiredWidth(Row.NameWidget.MinWidth)
						.MaxDesiredWidth(Row.NameWidget.MaxWidth)
						[
							NameWidget.ToSharedRef()
						]
						.ValueContent()
						.MinDesiredWidth(Row.ValueWidget.MinWidth)
						.MaxDesiredWidth(Row.ValueWidget.MaxWidth)
						[
							ValueWidget.ToSharedRef()
						];
					}
					else if (InternalCustomWidget != SNullWidget::NullWidget)
					{
						// A few properties are internally customized within this customization. Here we
						// catch instances of these that don't have an optional pin flag.
						IDetailPropertyRow& PropertyRow = CurrentCategory.AddProperty(TargetPropertyHandle);
						PropertyRow.CustomWidget()
						.NameContent()
						[
							TargetPropertyHandle->CreatePropertyNameWidget()
						]
						.ValueContent()
						[
							InternalCustomWidget
						];
					}
					else
					{
						CurrentCategory.AddProperty(TargetPropertyHandle);
					}
				}
			}
		}
	}
}


TSharedRef<SWidget> FAnimGraphNodeDetails::CreatePropertyWidget(FProperty* TargetProperty, TSharedRef<IPropertyHandle> TargetPropertyHandle, UClass* NodeClass)
{
	if(const FObjectPropertyBase* ObjectProperty = CastField<const FObjectPropertyBase>( TargetProperty ))
	{
		if(ObjectProperty->PropertyClass->IsChildOf(UAnimationAsset::StaticClass()))
		{
			bool bAllowClear = !(ObjectProperty->PropertyFlags & CPF_NoClear);

			return SNew(SObjectPropertyEntryBox)
				.PropertyHandle(TargetPropertyHandle)
				.AllowedClass(ObjectProperty->PropertyClass)
				.AllowClear(bAllowClear)
				.OnShouldFilterAsset(FOnShouldFilterAsset::CreateSP(this, &FAnimGraphNodeDetails::OnShouldFilterAnimAsset, NodeClass));
		}
	}

	return SNullWidget::NullWidget;
}

bool FAnimGraphNodeDetails::OnShouldFilterAnimAsset( const FAssetData& AssetData, UClass* NodeToFilterFor ) const
{
	FAssetDataTagMapSharedView::FFindTagResult Result = AssetData.TagsAndValues.FindTag("Skeleton");
	if (Result.IsSet())
	{
		bool bIsAssetCompatible;
		if (bIsAnimBPTemplate)
		{
			// If we are a template, we are always compatible
			bIsAssetCompatible = true;
		}
		else
		{
			bIsAssetCompatible = TargetSkeleton && TargetSkeleton->IsCompatibleForEditor(AssetData);
		}

		if (bIsAssetCompatible)
		{
			const UClass* AssetClass = AssetData.GetClass();
			// If node is an 'asset player', only let you select the right kind of asset for it
			if (!NodeToFilterFor->IsChildOf(UAnimGraphNode_AssetPlayerBase::StaticClass()) || (AssetClass && SupportNodeClassForAsset(AssetClass, NodeToFilterFor)))
			{
				return false;
			}
		}
	}
	return true;
}

EVisibility FAnimGraphNodeDetails::GetVisibilityOfProperty(TSharedRef<IPropertyHandle> Handle) const
{
	bool bShowAsPin;
	if (FPropertyAccess::Success == Handle->GetValue(/*out*/ bShowAsPin))
	{
		return bShowAsPin ? EVisibility::Hidden : EVisibility::Visible;
	}
	else
	{
		return EVisibility::Visible;
	}
}

void FAnimGraphNodeDetails::OnBlendProfileChanged(UBlendProfile* NewProfile, TSharedPtr<IPropertyHandle> PropertyHandle)
{
	if(PropertyHandle.IsValid())
	{
		PropertyHandle->SetValue(NewProfile);
	}
}

void FAnimGraphNodeDetails::OnPinVisibilityChanged(bool bInIsVisible, int32 InOptionalPinIndex)
{
	DetailLayoutBuilder->ForceRefreshDetails();
}

TSharedRef<IPropertyTypeCustomization> FInputScaleBiasCustomization::MakeInstance() 
{
	return MakeShareable(new FInputScaleBiasCustomization());
}

float GetMinValue(float Scale, float Bias)
{
	return Scale != 0.0f ? (FMath::Abs(Bias) < SMALL_NUMBER ? 0.0f : -Bias) / Scale : 0.0f; // to avoid displaying of - in front of 0
}

float GetMaxValue(float Scale, float Bias)
{
	return Scale != 0.0f ? (1.0f - Bias) / Scale : 0.0f;
}

void UpdateInputScaleBiasWithMinValue(float MinValue, TSharedRef<class IPropertyHandle> InputBiasScaleStructPropertyHandle)
{
	InputBiasScaleStructPropertyHandle->NotifyPreChange();

	TSharedRef<class IPropertyHandle> BiasProperty = InputBiasScaleStructPropertyHandle->GetChildHandle("Bias").ToSharedRef();
	TSharedRef<class IPropertyHandle> ScaleProperty = InputBiasScaleStructPropertyHandle->GetChildHandle("Scale").ToSharedRef();
	TArray<void*> BiasDataArray;
	TArray<void*> ScaleDataArray;
	BiasProperty->AccessRawData(BiasDataArray);
	ScaleProperty->AccessRawData(ScaleDataArray);
	check(BiasDataArray.Num() == ScaleDataArray.Num());
	for(int32 DataIndex = 0; DataIndex < BiasDataArray.Num(); ++DataIndex)
	{
		float* BiasPtr = (float*)BiasDataArray[DataIndex];
		float* ScalePtr = (float*)ScaleDataArray[DataIndex];
		check(BiasPtr);
		check(ScalePtr);

		const float MaxValue = GetMaxValue(*ScalePtr, *BiasPtr);
		const float Difference = MaxValue - MinValue;
		*ScalePtr = Difference != 0.0f? 1.0f / Difference : 0.0f;
		*BiasPtr = -MinValue * *ScalePtr;
	}

	InputBiasScaleStructPropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
}

void UpdateInputScaleBiasWithMaxValue(float MaxValue, TSharedRef<class IPropertyHandle> InputBiasScaleStructPropertyHandle)
{
	InputBiasScaleStructPropertyHandle->NotifyPreChange();

	TSharedRef<class IPropertyHandle> BiasProperty = InputBiasScaleStructPropertyHandle->GetChildHandle("Bias").ToSharedRef();
	TSharedRef<class IPropertyHandle> ScaleProperty = InputBiasScaleStructPropertyHandle->GetChildHandle("Scale").ToSharedRef();
	TArray<void*> BiasDataArray;
	TArray<void*> ScaleDataArray;
	BiasProperty->AccessRawData(BiasDataArray);
	ScaleProperty->AccessRawData(ScaleDataArray);
	check(BiasDataArray.Num() == ScaleDataArray.Num());
	for(int32 DataIndex = 0; DataIndex < BiasDataArray.Num(); ++DataIndex)
	{
		float* BiasPtr = (float*)BiasDataArray[DataIndex];
		float* ScalePtr = (float*)ScaleDataArray[DataIndex];
		check(BiasPtr);
		check(ScalePtr);

		const float MinValue = GetMinValue(*ScalePtr, *BiasPtr);
		const float Difference = MaxValue - MinValue;
		*ScalePtr = Difference != 0.0f ? 1.0f / Difference : 0.0f;
		*BiasPtr = -MinValue * *ScalePtr;
	}

	InputBiasScaleStructPropertyHandle->NotifyPostChange(EPropertyChangeType::ValueSet);
}

TOptional<float> GetMinValueInputScaleBias(TSharedRef<class IPropertyHandle> InputBiasScaleStructPropertyHandle)
{
	TSharedRef<class IPropertyHandle> BiasProperty = InputBiasScaleStructPropertyHandle->GetChildHandle("Bias").ToSharedRef();
	TSharedRef<class IPropertyHandle> ScaleProperty = InputBiasScaleStructPropertyHandle->GetChildHandle("Scale").ToSharedRef();
	float Scale = 1.0f;
	float Bias = 0.0f;
	if(ScaleProperty->GetValue(Scale) == FPropertyAccess::Success && BiasProperty->GetValue(Bias) == FPropertyAccess::Success)
	{
		return GetMinValue(Scale, Bias);
	}

	return TOptional<float>();
}

TOptional<float> GetMaxValueInputScaleBias(TSharedRef<class IPropertyHandle> InputBiasScaleStructPropertyHandle)
{
	TSharedRef<class IPropertyHandle> BiasProperty = InputBiasScaleStructPropertyHandle->GetChildHandle("Bias").ToSharedRef();
	TSharedRef<class IPropertyHandle> ScaleProperty = InputBiasScaleStructPropertyHandle->GetChildHandle("Scale").ToSharedRef();
	float Scale = 1.0f;
	float Bias = 0.0f;
	if(ScaleProperty->GetValue(Scale) == FPropertyAccess::Success && BiasProperty->GetValue(Bias) == FPropertyAccess::Success)
	{
		return GetMaxValue(Scale, Bias);
	}

	return TOptional<float>();
}


void FInputScaleBiasCustomization::CustomizeHeader(TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{

}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void FInputScaleBiasCustomization::CustomizeChildren( TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils )
{
	TWeakPtr<IPropertyHandle> WeakStructPropertyHandle = StructPropertyHandle;

	StructBuilder
	.AddProperty(StructPropertyHandle)
	.CustomWidget()
	.NameContent()
	[
		StructPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MinDesiredWidth(250.0f)
	.MaxDesiredWidth(250.0f)
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.Padding(FMargin(0.0f, 2.0f, 3.0f, 2.0f))
		[
			SNew(SNumericEntryBox<float>)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.ToolTipText(LOCTEXT("MinInputScaleBias", "Minimum input value"))
			.AllowSpin(true)
			.MinSliderValue(0.0f)
			.MaxSliderValue(2.0f)
			.Value_Lambda([WeakStructPropertyHandle]()
			{
				return GetMinValueInputScaleBias(WeakStructPropertyHandle.Pin().ToSharedRef());
			})
			.OnValueChanged_Lambda([WeakStructPropertyHandle](float InValue)
			{
				UpdateInputScaleBiasWithMinValue(InValue, WeakStructPropertyHandle.Pin().ToSharedRef());
			})
		]
		+SHorizontalBox::Slot()
		.Padding(FMargin(0.0f, 2.0f, 0.0f, 2.0f))
		[
			SNew(SNumericEntryBox<float>)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.ToolTipText(LOCTEXT("MaxInputScaleBias", "Maximum input value"))
			.AllowSpin(true)
			.MinSliderValue(0.0f)
			.MaxSliderValue(2.0f)
			.Value_Lambda([WeakStructPropertyHandle]()
			{
				return GetMaxValueInputScaleBias(WeakStructPropertyHandle.Pin().ToSharedRef());
			})
			.OnValueChanged_Lambda([WeakStructPropertyHandle](float InValue)
			{
				UpdateInputScaleBiasWithMaxValue(InValue, WeakStructPropertyHandle.Pin().ToSharedRef());
			})
		]
	];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

/////////////////////////////////////////////////////////////////////////////////////////////
//  FBoneReferenceCustomization
/////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<IPropertyTypeCustomization> FBoneReferenceCustomization::MakeInstance()
{
	return MakeShareable(new FBoneReferenceCustomization());
}

void FBoneReferenceCustomization::CustomizeHeader( TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils )
{
	StructProperty = StructPropertyHandle;
	
	BoneNameProperty = FindStructMemberProperty(StructProperty, GET_MEMBER_NAME_CHECKED(FBoneReference, BoneName));
	check(BoneNameProperty->IsValidHandle());

	// record selected objects in case they are needed to find the skeleton
	IPropertyUtilities* PropertyUtility = StructCustomizationUtils.GetPropertyUtilities().Get();
	SelectedObjects = PropertyUtility->GetSelectedObjects();
	
	HeaderRow
	.NameContent()
	[
		StructPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MaxDesiredWidth(0.0f)
	[
		SNew(SWidgetSwitcher)
		.WidgetIndex_Lambda([this]()
		{
			// If we have a skeleton, show the picker widget, otherwise default to the text box.
			return GetSkeleton() != nullptr ? 0 : 1;
		})
		+ SWidgetSwitcher::Slot()
		[
			SNew(SBoneSelectionWidget)
			.bShowNone(true)
			.ToolTipText(StructPropertyHandle->GetToolTipText())
			.OnBoneSelectionChanged(this, &FBoneReferenceCustomization::OnBoneSelectionChanged)
			.OnGetSelectedBone(this, &FBoneReferenceCustomization::GetSelectedBone)
			.OnGetReferenceSkeleton(this, &FBoneReferenceCustomization::GetReferenceSkeleton)
		]
		+ SWidgetSwitcher::Slot()
		[
			SNew(SEditableTextBox)
			.Font(StructCustomizationUtils.GetRegularFont())
			.ToolTipText(StructPropertyHandle->GetToolTipText())
			.OnTextCommitted_Lambda([this](const FText& InText, ETextCommit::Type InTextCommit)
			{
				BoneNameProperty->SetValue(FName(InText.ToString()));
			})
			.Text_Lambda([this]()
			{
				FName Name;
				BoneNameProperty->GetValue(Name);
				return FText::FromName(Name);
			})
		]
	];
}


USkeleton* FBoneReferenceCustomization::GetSkeleton() const 
{
	TArray<UObject*> Objects;
	StructProperty->GetOuterObjects(Objects);

	// optionally add selected objects which may manage the skeleton provider
	if (Objects.IsEmpty())
	{
		for (const TWeakObjectPtr<UObject>& SelectedObject : SelectedObjects)
		{
			Objects.Add(SelectedObject.Get());
		}
	}

	TArray<UPackage*> CandidatePackages;
	StructProperty->GetOuterPackages(CandidatePackages);

	// Scan each candidate Package for a provider Object
	for (const UPackage* Package : CandidatePackages)
	{
		if (!Package)
		{
			continue; // skip invalid packages
		}
		ForEachObjectWithPackage(Package, [&Objects](UObject* Object)
		{
			if (Object->Implements<UBoneReferenceSkeletonProvider>())
			{
				Objects.Add(Object);
				return false; // stop scanning this package because the object with the skeleton provider has been found
			}
			return true;
		},
		EGetObjectsFlags::None, RF_ClassDefaultObject | RF_Transient);
	}

	auto FindSkeletonForObject = [this](UObject* InObject) -> USkeleton* 
	{
		for( ; InObject; InObject = InObject->GetOuter())
		{
			if (const UAnimGraphNode_Base* AnimGraphNode = Cast<UAnimGraphNode_Base>(InObject))
			{
				return AnimGraphNode->GetAnimBlueprint()->TargetSkeleton;
			}

			if (USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(InObject))
			{
				return SkeletalMesh->GetSkeleton();
			}

			if (const ULODInfoUILayout* LODInfoUILayout = Cast<ULODInfoUILayout>(InObject))
			{
				USkeletalMesh* SkeletalMesh = LODInfoUILayout->GetPersonaToolkit()->GetPreviewMesh();
				if (ensure(SkeletalMesh))
				{
					return SkeletalMesh->GetSkeleton();
				}

				return nullptr;
			}

			if (const UAnimationAsset* AnimationAsset = Cast<UAnimationAsset>(InObject))
			{
				if(AnimationAsset->IsAsset())
				{
					return AnimationAsset->GetSkeleton();
				}
			}

			if (UAnimInstance* AnimInstance = Cast<UAnimInstance>(InObject))
			{
				if (AnimInstance->CurrentSkeleton)
				{
					return AnimInstance->CurrentSkeleton;
				}
				if (UAnimBlueprintGeneratedClass* AnimBPClass = Cast<UAnimBlueprintGeneratedClass>(AnimInstance->GetClass()))
				{
					return AnimBPClass->TargetSkeleton;
				}
			}

			// editor animation curve bone links are responsible for linking joints to curve
			// this is editor object that only exists for editor
			if (const UEditorAnimCurveBoneLinks* AnimCurveObj = Cast<UEditorAnimCurveBoneLinks>(InObject))
			{
				if(USkeleton* Skeleton = Cast<USkeleton>(AnimCurveObj->AnimCurveMetaData->GetOuter()))
				{
					return Skeleton;
				}
				if(USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(AnimCurveObj->AnimCurveMetaData->GetOuter()))
				{
					return SkeletalMesh->GetSkeleton();
				}
			}

			if (IBoneReferenceSkeletonProvider* SkeletonProvider = Cast<IBoneReferenceSkeletonProvider>(InObject))
			{
				bool bInvalidSkeletonIsError = false;
				return SkeletonProvider->GetSkeleton(bInvalidSkeletonIsError, StructProperty.Get());
			}
		}

		return nullptr;
	};
	
	for (UObject* Object : Objects)
	{
		if(USkeleton* Skeleton = FindSkeletonForObject(Object))
		{
			return Skeleton;
		}
	}

	return nullptr;
}

TSharedPtr<IPropertyHandle> FBoneReferenceCustomization::FindStructMemberProperty(TSharedPtr<IPropertyHandle> PropertyHandle, const FName& PropertyName)
{
	uint32 NumChildren = 0;
	PropertyHandle->GetNumChildren(NumChildren);
	for (uint32 ChildIdx = 0; ChildIdx < NumChildren; ++ChildIdx)
	{
		TSharedPtr<IPropertyHandle> ChildHandle = PropertyHandle->GetChildHandle(ChildIdx);
		if (ChildHandle->GetProperty()->GetFName() == PropertyName)
		{
			return ChildHandle;
		}
	}

	return TSharedPtr<IPropertyHandle>();
}

void FBoneReferenceCustomization::OnBoneSelectionChanged(FName Name)
{
	BoneNameProperty->SetValue(Name);
}

FName FBoneReferenceCustomization::GetSelectedBone(bool& bMultipleValues) const
{
	FString OutText;
	
	const FPropertyAccess::Result Result = BoneNameProperty->GetValueAsFormattedString(OutText);
	bMultipleValues = (Result == FPropertyAccess::MultipleValues);

	return FName(*OutText);
}

const struct FReferenceSkeleton&  FBoneReferenceCustomization::GetReferenceSkeleton() const
{
	static FReferenceSkeleton EmptySkeleton;
	
	const USkeleton* Skeleton = GetSkeleton();
	return Skeleton ? Skeleton->GetReferenceSkeleton() : EmptySkeleton; 
}

/////////////////////////////////////////////////////////////////////////////////////////////
//  FBoneSocketTargetCustomization
/////////////////////////////////////////////////////////////////////////////////////////////
TSharedRef<IPropertyTypeCustomization> FBoneSocketTargetCustomization::MakeInstance()
{
	return MakeShareable(new FBoneSocketTargetCustomization());
}

void FBoneSocketTargetCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	// set property handle
	StructProperty = StructPropertyHandle;
	
	ResolveChildProperties();
	
	HeaderRow
	.NameContent()
	[
		StructPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		SNew(SWidgetSwitcher)
		.WidgetIndex_Lambda([this]()
		{
			// If we have a skeleton, show the picker widget, otherwise default to a text label indicating that
			// none is available.
			return GetSkeleton() != nullptr ? 0 : 1;
		})
		+ SWidgetSwitcher::Slot()
		[
			SNew(SBoneSelectionWidget)
			.ToolTipText(StructPropertyHandle->GetToolTipText())
			.bShowSocket(true)
			.bShowNone(true)
			.OnBoneSelectionChanged(this, &FBoneSocketTargetCustomization::OnBoneSelectionChanged)
			.OnGetSelectedBone(this, &FBoneSocketTargetCustomization::GetSelectedBone)
			.OnGetReferenceSkeleton(this, &FBoneReferenceCustomization::GetReferenceSkeleton)
			.OnGetSocketList(this, &FBoneSocketTargetCustomization::GetSocketList)
		]
		+ SWidgetSwitcher::Slot()
		[
			SNew(STextBlock)
			.Font(StructCustomizationUtils.GetRegularFont())
			.Text(LOCTEXT("BoneSelectionNoSkeleton", "No Skeleton Available"))
		]
	];
}

void FBoneSocketTargetCustomization::ResolveChildProperties()
{
	const TSharedPtr<IPropertyHandle> BoneReferenceProperty = FindStructMemberProperty(StructProperty, GET_MEMBER_NAME_CHECKED(FBoneSocketTarget, BoneReference));
	check(BoneReferenceProperty->IsValidHandle());
	BoneNameProperty = FindStructMemberProperty(BoneReferenceProperty.ToSharedRef(), GET_MEMBER_NAME_CHECKED(FBoneReference, BoneName));
	
	const TSharedPtr<IPropertyHandle> SocketReferenceProperty = FindStructMemberProperty(StructProperty, GET_MEMBER_NAME_CHECKED(FBoneSocketTarget, SocketReference));
	check(SocketReferenceProperty->IsValidHandle());
	SocketNameProperty = FindStructMemberProperty(SocketReferenceProperty.ToSharedRef(), GET_MEMBER_NAME_CHECKED(FSocketReference, SocketName));
	
	UseSocketProperty = FindStructMemberProperty(StructProperty, GET_MEMBER_NAME_CHECKED(FBoneSocketTarget, bUseSocket));

	check(BoneNameProperty->IsValidHandle() && SocketNameProperty->IsValidHandle() && UseSocketProperty->IsValidHandle());
}


TSharedPtr<IPropertyHandle> FBoneSocketTargetCustomization::GetNameProperty() const
{
	bool bUseSocket = false;
	if (UseSocketProperty->GetValue(bUseSocket) == FPropertyAccess::Success)
	{
		if (bUseSocket)
		{
			return SocketNameProperty;
		}

		return BoneNameProperty;
	}

	return TSharedPtr<IPropertyHandle>();
}
void FBoneSocketTargetCustomization::OnBoneSelectionChanged(FName Name)
{
	// figure out if the name is BoneName or socket name
	bool bUseSocket = false;
	if (GetReferenceSkeleton().FindBoneIndex(Name) == INDEX_NONE)
	{
		// make sure socket exists
		const TArray<class USkeletalMeshSocket*>& Sockets = GetSocketList();
		for (int32 Idx = 0; Idx < Sockets.Num(); ++Idx)
		{
			if (Sockets[Idx]->SocketName == Name)
			{
				bUseSocket = true;
				break;
			}
		}

		// we should find one
		ensure(bUseSocket);
	}

	// set correct value
	UseSocketProperty->SetValue(bUseSocket);

	const TSharedPtr<IPropertyHandle> NameProperty = GetNameProperty();
	if (ensureAlways(NameProperty.IsValid()))
	{
		NameProperty->SetValue(Name);
	}
}

FName FBoneSocketTargetCustomization::GetSelectedBone(bool& bMultipleValues) const
{
	FString OutText;

	const TSharedPtr<IPropertyHandle> NameProperty = GetNameProperty();
	if (NameProperty.IsValid())
	{
		const FPropertyAccess::Result Result = NameProperty->GetValueAsFormattedString(OutText);
		bMultipleValues = (Result == FPropertyAccess::MultipleValues);
	}
	else
	{
		// there is no single value
		bMultipleValues = true;
		return NAME_None;
	}

	return FName(*OutText);
}

const TArray<USkeletalMeshSocket*>& FBoneSocketTargetCustomization::GetSocketList() const
{
	static TArray<USkeletalMeshSocket*> EmptySocketList;
	
	if (const USkeleton* Skeleton = GetSkeleton())
	{
		return Skeleton->Sockets;
	}
	return EmptySocketList;
}

/////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<IDetailCustomization> FAnimGraphParentPlayerDetails::MakeInstance(TSharedRef<FBlueprintEditor> InBlueprintEditor)
{
	return MakeShareable(new FAnimGraphParentPlayerDetails(InBlueprintEditor));
}


void FAnimGraphParentPlayerDetails::CustomizeDetails(class IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> SelectedObjects;
	DetailBuilder.GetObjectsBeingCustomized(SelectedObjects);
	check(SelectedObjects.Num() == 1);

	EditorObject = Cast<UEditorParentPlayerListObj>(SelectedObjects[0].Get());
	check(EditorObject);
	
	IDetailCategoryBuilder& Category = DetailBuilder.EditCategory("AnimGraphOverrides");
	DetailBuilder.HideProperty("Overrides");

	struct FObjectToEntryBuilder
	{
	private:
		TMap<UObject*, TSharedPtr<FPlayerTreeViewEntry>> ObjectToEntryMap;
		TArray<TSharedPtr<FPlayerTreeViewEntry>>& ListEntries;

	private:
		TSharedPtr<FPlayerTreeViewEntry> AddObject(UObject* Object)
		{
			TSharedPtr<FPlayerTreeViewEntry> Result = ObjectToEntryMap.FindRef(Object);
			if (!Result.IsValid() && (Object != nullptr))
			{
				bool bTopLevel = false;
				TSharedPtr<FPlayerTreeViewEntry> ThisNode;

				if (UBlueprint* Blueprint = Cast<UBlueprint>(Object))
				{
					ThisNode = MakeShareable(new FPlayerTreeViewEntry(Blueprint->GetName(), EPlayerTreeViewEntryType::Blueprint));
					bTopLevel = true;
				}
				else if (UAnimGraphNode_StateMachine* StateMachineNode = Cast<UAnimGraphNode_StateMachine>(Object))
				{
					// Don't create a node for these, the graph speaks for it
				}
				else if (UAnimGraphNode_AssetPlayerBase * AssetPlayerBase = Cast<UAnimGraphNode_AssetPlayerBase>(Object))
				{
					FString Title = AssetPlayerBase->GetNodeTitle(ENodeTitleType::FullTitle).ToString();
					ThisNode = MakeShareable(new FPlayerTreeViewEntry(Title, EPlayerTreeViewEntryType::Node));
				}
				else if (UAnimGraphNode_Base* Node = Cast<UAnimGraphNode_Base>(Object))
				{
					ThisNode = MakeShareable(new FPlayerTreeViewEntry(Node->GetName(), EPlayerTreeViewEntryType::Node));
				}
				else if (UEdGraph* Graph = Cast<UEdGraph>(Object))
				{
					ThisNode = MakeShareable(new FPlayerTreeViewEntry(Graph->GetName(), EPlayerTreeViewEntryType::Graph));
				}

				if (ThisNode.IsValid())
				{
					ObjectToEntryMap.Add(Object, ThisNode);
				}

				if (bTopLevel)
				{
					ListEntries.Add(ThisNode);
					Result = ThisNode;
				}
				else
				{
					TSharedPtr<FPlayerTreeViewEntry> Outer = AddObject(Object->GetOuter());
					Result = Outer;

					if (ThisNode.IsValid())
					{
						Result = ThisNode;
						check(Outer.IsValid())
						Outer->Children.Add(Result);
					}
				}
			}

			return Result;
		}

		void SortInternal(TArray<TSharedPtr<FPlayerTreeViewEntry>>& ListToSort)
		{
			ListToSort.Sort([](TSharedPtr<FPlayerTreeViewEntry> A, TSharedPtr<FPlayerTreeViewEntry> B) { return A->EntryName < B->EntryName; });

			for (TSharedPtr<FPlayerTreeViewEntry>& Entry : ListToSort)
			{
				SortInternal(Entry->Children);
			}
		}

	public:
		FObjectToEntryBuilder(TArray<TSharedPtr<FPlayerTreeViewEntry>>& InListEntries)
			: ListEntries(InListEntries)
		{
		}

		void AddNode(UAnimGraphNode_Base* Node, FAnimParentNodeAssetOverride& Override)
		{
			TSharedPtr<FPlayerTreeViewEntry> Result = AddObject(Node);
			if (Result.IsValid())
			{
				Result->Override = &Override;
			}
		}

		void Sort()
		{
			SortInternal(ListEntries);
		}
	};

	FObjectToEntryBuilder EntryBuilder(ListEntries);

	// Build a hierarchy of entires for a tree view in the form of Blueprint->Graph->Node
	for (FAnimParentNodeAssetOverride& Override : EditorObject->Overrides)
	{
		UAnimGraphNode_Base* Node = EditorObject->GetVisualNodeFromGuid(Override.ParentNodeGuid);
		EntryBuilder.AddNode(Node, Override);
	}

	// Sort the nodes
	EntryBuilder.Sort();

	FDetailWidgetRow& Row = Category.AddCustomRow(FText::GetEmpty());
	TSharedRef<STreeView<TSharedPtr<FPlayerTreeViewEntry>>> TreeView = SNew(STreeView<TSharedPtr<FPlayerTreeViewEntry>>)
		.SelectionMode(ESelectionMode::None)
		.OnGenerateRow(this, &FAnimGraphParentPlayerDetails::OnGenerateRow)
		.OnGetChildren(this, &FAnimGraphParentPlayerDetails::OnGetChildren)
		.TreeItemsSource(&ListEntries)
		.HeaderRow
		(
			SNew(SHeaderRow)
			+SHeaderRow::Column(FName("Name"))
			.FillWidth(0.5f)
			.DefaultLabel(LOCTEXT("ParentPlayer_NameCol", "Name"))

			+SHeaderRow::Column(FName("Asset"))
			.FillWidth(0.5f)
			.DefaultLabel(LOCTEXT("ParentPlayer_AssetCol", "Asset"))
		);

	// Expand top level (blueprint) entries so the panel seems less empty
	for (TSharedPtr<FPlayerTreeViewEntry> Entry : ListEntries)
	{
		TreeView->SetItemExpansion(Entry, true);
	}

	Row
	[
		TreeView->AsShared()
	];
}

TSharedRef<ITableRow> FAnimGraphParentPlayerDetails::OnGenerateRow(TSharedPtr<FPlayerTreeViewEntry> EntryPtr, const TSharedRef< STableViewBase >& OwnerTable)
{
	return SNew(SParentPlayerTreeRow, OwnerTable).Item(EntryPtr).OverrideObject(EditorObject).BlueprintEditor(BlueprintEditorPtr);
}

void FAnimGraphParentPlayerDetails::OnGetChildren(TSharedPtr<FPlayerTreeViewEntry> InParent, TArray< TSharedPtr<FPlayerTreeViewEntry> >& OutChildren)
{
	OutChildren.Append(InParent->Children);
}

void SParentPlayerTreeRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
{
	Item = InArgs._Item;
	EditorObject = InArgs._OverrideObject;
	BlueprintEditor = InArgs._BlueprintEditor;

	if(Item->Override)
	{
		GraphNode = EditorObject->GetVisualNodeFromGuid(Item->Override->ParentNodeGuid);
	}
	else
	{
		GraphNode = NULL;
	}

	SMultiColumnTableRow<TSharedPtr<FAnimGraphParentPlayerDetails>>::Construct(FSuperRowType::FArguments(), InOwnerTableView);
}

TSharedRef<SWidget> SParentPlayerTreeRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	TSharedPtr<SHorizontalBox> HorizBox;
	SAssignNew(HorizBox, SHorizontalBox);

	if(ColumnName == "Name")
	{
		HorizBox->AddSlot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SExpanderArrow, SharedThis(this))
			];

		Item->GenerateNameWidget(HorizBox);
	}
	else if(Item->Override)
	{
		HorizBox->AddSlot()
			.Padding(2)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "ToggleButton")
				.ToolTip(IDocumentation::Get()->CreateToolTip(LOCTEXT("FocusNodeButtonTip", "Open the graph that contains this node in read-only mode and focus on the node"), NULL, "Shared/Editors/Persona", "FocusNodeButton"))
				.OnClicked(FOnClicked::CreateSP(this, &SParentPlayerTreeRow::OnFocusNodeButtonClicked))
				.Content()
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("GenericViewButton"))
				]
				
			];

		if(GraphNode)
		{
			TArray<const UClass*> AllowedClasses;
			AllowedClasses.Add(UAnimationAsset::StaticClass());
			HorizBox->AddSlot()
				.VAlign(VAlign_Center)
				.FillWidth(1.f)
				[
					SNew(SObjectPropertyEntryBox)
					.ObjectPath(this, &SParentPlayerTreeRow::GetCurrentAssetPath)
					.OnShouldFilterAsset(this, &SParentPlayerTreeRow::OnShouldFilterAsset)
					.OnObjectChanged(this, &SParentPlayerTreeRow::OnAssetSelected)
					.AllowedClass(GraphNode->GetAnimationAssetClass())
				];

			HorizBox->AddSlot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), "NoBorder")
					.Visibility(this, &SParentPlayerTreeRow::GetResetToDefaultVisibility)
					.OnClicked(this, &SParentPlayerTreeRow::OnResetButtonClicked)
					.ToolTip(IDocumentation::Get()->CreateToolTip(LOCTEXT("ResetToParentButtonTip", "Undo the override, returning to the default asset for this node"), NULL, "Shared/Editors/Persona", "ResetToParentButton"))
					.Content()
					[
						SNew(SImage)
						.Image(FAppStyle::GetBrush("PropertyWindow.DiffersFromDefault"))
					]
				];
		}
	}

	return HorizBox.ToSharedRef();
}

bool SParentPlayerTreeRow::OnShouldFilterAsset(const FAssetData& AssetData)
{
	const USkeleton* CurrentSkeleton = CastChecked<UAnimBlueprint>(BlueprintEditor.Pin()->GetBlueprintObj())->TargetSkeleton;
	return CurrentSkeleton != nullptr && !CurrentSkeleton->IsCompatibleForEditor(AssetData);
}

void SParentPlayerTreeRow::OnAssetSelected(const FAssetData& AssetData)
{
	Item->Override->NewAsset = Cast<UAnimationAsset>(AssetData.GetAsset());
	EditorObject->ApplyOverrideToBlueprint(*Item->Override);
}

FReply SParentPlayerTreeRow::OnFocusNodeButtonClicked()
{
	TSharedPtr<FBlueprintEditor> SharedBlueprintEditor = BlueprintEditor.Pin();
	if(SharedBlueprintEditor.IsValid())
	{
		if(GraphNode)
		{
			UEdGraph* EdGraph = GraphNode->GetGraph();
			TSharedPtr<SGraphEditor> GraphEditor = SharedBlueprintEditor->OpenGraphAndBringToFront(EdGraph);
			if (GraphEditor.IsValid())
			{
				GraphEditor->JumpToNode(GraphNode, false);
			}
		}
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

const UAnimationAsset* SParentPlayerTreeRow::GetCurrentAssetToUse() const
{
	if(Item->Override->NewAsset)
	{
		return Item->Override->NewAsset;
	}
	
	if(GraphNode)
	{
		return GraphNode->GetAnimationAsset();
	}

	return nullptr;
}

EVisibility SParentPlayerTreeRow::GetResetToDefaultVisibility() const
{
	FAnimParentNodeAssetOverride* HierarchyOverride = EditorObject->GetBlueprint()->GetAssetOverrideForNode(Item->Override->ParentNodeGuid, true);

	if(HierarchyOverride)
	{
		return Item->Override->NewAsset != HierarchyOverride->NewAsset ? EVisibility::Visible : EVisibility::Hidden;
	}

	return Item->Override->NewAsset != GraphNode->GetAnimationAsset() ? EVisibility::Visible : EVisibility::Hidden;
}

FReply SParentPlayerTreeRow::OnResetButtonClicked()
{
	FAnimParentNodeAssetOverride* HierarchyOverride = EditorObject->GetBlueprint()->GetAssetOverrideForNode(Item->Override->ParentNodeGuid, true);
	
	Item->Override->NewAsset = HierarchyOverride ? ToRawPtr(HierarchyOverride->NewAsset) : GraphNode->GetAnimationAsset();

	// Apply will remove the override from the object
	EditorObject->ApplyOverrideToBlueprint(*Item->Override);
	return FReply::Handled();
}

FString SParentPlayerTreeRow::GetCurrentAssetPath() const
{
	const UAnimationAsset* Asset = GetCurrentAssetToUse();
	return Asset ? Asset->GetPathName() : FString("");
}

FORCENOINLINE bool FPlayerTreeViewEntry::operator==(const FPlayerTreeViewEntry& Other)
{
	return EntryName == Other.EntryName;
}

void FPlayerTreeViewEntry::GenerateNameWidget(TSharedPtr<SHorizontalBox> Box)
{
	// Get an appropriate image icon for the row
	const FSlateBrush* EntryImageBrush = NULL;
	switch(EntryType)
	{
		case EPlayerTreeViewEntryType::Blueprint:
			EntryImageBrush = FAppStyle::GetBrush("ClassIcon.Blueprint");
			break;
		case EPlayerTreeViewEntryType::Graph:
			EntryImageBrush = FAppStyle::GetBrush("GraphEditor.EventGraph_16x");
			break;
		case EPlayerTreeViewEntryType::Node:
			EntryImageBrush = FAppStyle::GetBrush("GraphEditor.Default_16x");
			break;
		default:
			break;
	}

	Box->AddSlot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SImage)
			.Image(EntryImageBrush)
		];

	Box->AddSlot()
		.VAlign(VAlign_Center)
		.Padding(FMargin(5.0f, 0.0f, 0.0f, 0.0f))
		.AutoWidth()
		[
			SNew(STextBlock)
			.Font(FCoreStyle::GetDefaultFontStyle("Bold", 10))
			.Text(FText::FromString(EntryName))
		];
}

void FAnimGraphNodeBindingExtension::GetOptionalPinData(const IPropertyHandle& PropertyHandle, int32& OutOptionalPinIndex, UAnimGraphNode_Base*& OutAnimGraphNode) const
{
	OutOptionalPinIndex = INDEX_NONE;

	TArray<UObject*> Objects;
	PropertyHandle.GetOuterObjects(Objects);

	FProperty* Property = PropertyHandle.GetProperty();
	if(Property && Objects.Num())
	{
		OutAnimGraphNode = Cast<UAnimGraphNode_Base>(Objects[0]);
		if (OutAnimGraphNode != nullptr)
		{
			OutOptionalPinIndex = OutAnimGraphNode->ShowPinForProperties.IndexOfByPredicate([Property](const FOptionalPinFromProperty& InOptionalPin)
			{
				UStruct* OwnerStruct = Property->GetOwnerStruct();
				if(OwnerStruct)
				{
					// Checking the owner struct here avoids placing binding widgets on inner structs that have properties
					// that share the same name as the anim node we are customizing
					if(OwnerStruct->IsChildOf(FAnimNode_Base::StaticStruct()))
					{
						return Property->GetFName() == InOptionalPin.PropertyName;
					}
				}
				
				return false;
			});
		}
	}
}

bool FAnimGraphNodeBindingExtension::IsPropertyExtendable(const UClass* InObjectClass, const IPropertyHandle& PropertyHandle) const
{
	int32 OptionalPinIndex;
	UAnimGraphNode_Base* AnimGraphNode;
	GetOptionalPinData(PropertyHandle, OptionalPinIndex, AnimGraphNode);

	if(OptionalPinIndex != INDEX_NONE)
	{
		const FOptionalPinFromProperty& OptionalPin = AnimGraphNode->ShowPinForProperties[OptionalPinIndex];

		// Not optional
		if (!OptionalPin.bCanToggleVisibility && OptionalPin.bShowPin)
		{
			return false;
		}

		if(!PropertyHandle.GetProperty())
		{
			return false;
		}

		return OptionalPin.bCanToggleVisibility;
	}

	return false;
}

void FAnimGraphNodeBindingExtension::ExtendWidgetRow(FDetailWidgetRow& InWidgetRow, const IDetailLayoutBuilder& InDetailBuilder, const UClass* InObjectClass, TSharedPtr<IPropertyHandle> InPropertyHandle)
{
	int32 OptionalPinIndex;
	UAnimGraphNode_Base* AnimGraphNode;
	GetOptionalPinData(*InPropertyHandle.Get(), OptionalPinIndex, AnimGraphNode);
	check(OptionalPinIndex != INDEX_NONE);

	TArray<UObject*> OuterObjects;
	InPropertyHandle->GetOuterObjects(OuterObjects);

	TArray<UAnimGraphNode_Base*> AnimGraphNodes;
	Algo::Transform(OuterObjects, AnimGraphNodes, [](UObject* InObject){ return Cast<UAnimGraphNode_Base>(InObject); });

	FProperty* AnimNodeProperty = InPropertyHandle->GetProperty();
	TSharedPtr<IPropertyHandle> ParentHandle = InPropertyHandle->GetParentHandle();
	FProperty* ParentProperty = ParentHandle.IsValid() ? ParentHandle->GetProperty() : nullptr;
	
	FName PropertyName;
	if(FArrayProperty* ParentArrayProperty = CastField<FArrayProperty>(ParentProperty))
	{
		int32 ArrayIndex = InPropertyHandle->GetIndexInArray();
		check(ArrayIndex != INDEX_NONE);
		PropertyName = FName(ParentArrayProperty->GetFName(), InPropertyHandle->GetIndexInArray() + 1);
		AnimNodeProperty = ParentArrayProperty;
	}
	else
	{
		PropertyName = AnimNodeProperty->GetFName();
	}

	const FName OptionalPinArrayEntryName(*FString::Printf(TEXT("ShowPinForProperties[%d].bShowPin"), OptionalPinIndex));
	TSharedRef<IPropertyHandle> ShowPinPropertyHandle = InDetailBuilder.GetProperty(OptionalPinArrayEntryName, UAnimGraphNode_Base::StaticClass());
	ShowPinPropertyHandle->MarkHiddenByCustomization();

	TSharedPtr<SWidget> BindingWidget = UAnimationGraphSchema::MakeBindingWidgetForPin(AnimGraphNodes, PropertyName, false, true);
	if(BindingWidget.IsValid())
	{
		InWidgetRow.ExtensionContent()
		[
			BindingWidget.ToSharedRef()
		];
	}
}

#undef LOCTEXT_NAMESPACE

