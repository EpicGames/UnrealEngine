// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "IPropertyTypeCustomization.h"
#include "PropertyHandle.h"
#include "IDetailCustomization.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "IDetailPropertyExtensionHandler.h"

struct FAssetData;
class FBlueprintEditor;
class UAnimationAsset;
class UAnimGraphNode_Base;
class UBlendProfile;
class UEditorParentPlayerListObj;
class USkeleton;
class IEditableSkeleton;
struct FAnimParentNodeAssetOverride;

class FAnimGraphNodeBindingExtension : public IDetailPropertyExtensionHandler
{
public:
	// IDetailPropertyExtensionHandler interface
	virtual bool IsPropertyExtendable(const UClass* InObjectClass, const IPropertyHandle& PropertyHandle) const override;
	virtual void ExtendWidgetRow(FDetailWidgetRow& InWidgetRow, const IDetailLayoutBuilder& InDetailBuilder, const UClass* InObjectClass, TSharedPtr<IPropertyHandle> PropertyHandle) override;

private:
	// Helper function
	void GetOptionalPinData(const IPropertyHandle& PropertyHandle, int32& OutOptionalPinIndex, UAnimGraphNode_Base*& OutAnimGraphNode) const;
};

/////////////////////////////////////////////////////
// FAnimGraphNodeDetails 

class FAnimGraphNodeDetails : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();

	// IDetailCustomization interface
	virtual void CustomizeDetails(class IDetailLayoutBuilder& DetailBuilder) override;
	// End of IDetailCustomization interface

protected:
	// Creates a widget for the supplied property
	TSharedRef<SWidget> CreatePropertyWidget(FProperty* TargetProperty, TSharedRef<IPropertyHandle> TargetPropertyHandle, UClass* NodeClass);

	// Creates the 'as pin' toggle widget for a property
	TSharedRef<SWidget> CreateAsPinWidget(TSharedRef<IPropertyHandle> InPropertyHandle);

	EVisibility GetVisibilityOfProperty(TSharedRef<IPropertyHandle> Handle) const;

	/** Delegate to handle filtering of asset pickers */
	bool OnShouldFilterAnimAsset( const FAssetData& AssetData, UClass* NodeToFilterFor ) const;

	/** Called when a blend profile is selected */
	void OnBlendProfileChanged(UBlendProfile* NewProfile, TSharedPtr<IPropertyHandle> PropertyHandle);

	/** Called when pin visibility changes */
	void OnPinVisibilityChanged(bool bInIsVisible, int32 InOptionalPinIndex);
	
	/** The skeleton we're operating on */
	USkeleton* TargetSkeleton = nullptr;

	/** Path to the current blueprints skeleton to allow us to filter asset pickers */
	FString TargetSkeletonName;

	/** Builder that built this customization (used for refresh) */
	IDetailLayoutBuilder* DetailLayoutBuilder = nullptr;

	/** Whether or not the anim blueprint is a template */
	bool bIsAnimBPTemplate = false;
};

/////////////////////////////////////////////////////
// FInputScaleBiasCustomization

class FInputScaleBiasCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	// IPropertyTypeCustomization interface
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	// End of IPropertyTypeCustomization interface

	FText GetMinValue(TSharedRef<class IPropertyHandle> StructPropertyHandle) const;
	FText GetMaxValue(TSharedRef<class IPropertyHandle> StructPropertyHandle) const;
	void OnMinValueCommitted(const FText& NewText, ETextCommit::Type CommitInfo, TSharedRef<class IPropertyHandle> StructPropertyHandle);
	void OnMaxValueCommitted(const FText& NewText, ETextCommit::Type CommitInfo, TSharedRef<class IPropertyHandle> StructPropertyHandle);
};

//////////////////////////////////////////////////////////////////////////
// FBoneReferenceCustomization

class FBoneReferenceCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	// IPropertyTypeCustomization interface
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override {}
	const struct FReferenceSkeleton&  GetReferenceSkeleton() const;

protected:
	USkeleton* GetSkeleton() const;
	
	static TSharedPtr<IPropertyHandle> FindStructMemberProperty(TSharedPtr<IPropertyHandle> PropertyHandle, const FName& PropertyName);
	
	TSharedPtr<IPropertyHandle> StructProperty;
	
	// Property to change after bone has been picked
	TSharedPtr<IPropertyHandle> BoneNameProperty;

	TArray<TWeakObjectPtr<UObject>> SelectedObjects;

private:

	// Bone tree widget delegates
	virtual void OnBoneSelectionChanged(FName Name);
	virtual FName GetSelectedBone(bool& bMultipleValues) const;
};

//////////////////////////////////////////////////////////////////////////
// FBoneSocketTargetCustomization

class FBoneSocketTargetCustomization : public FBoneReferenceCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

private:
	void ResolveChildProperties();	
	
	// Property to change after bone has been picked
	TSharedPtr<IPropertyHandle> SocketNameProperty;
	TSharedPtr<IPropertyHandle> UseSocketProperty;

	// Bone tree widget delegates
	virtual void OnBoneSelectionChanged(FName Name) override;
	virtual FName GetSelectedBone(bool& bMultipleValues) const override;
	const TArray<class USkeletalMeshSocket*>& GetSocketList() const;

	TSharedPtr<IPropertyHandle> GetNameProperty() const;
};

//////////////////////////////////////////////////////////////////////////
// 

// Type used to identify rows in a parent player tree list
namespace EPlayerTreeViewEntryType
{
	enum Type
	{
		Blueprint,
		Graph,
		Node
	};
}

// Describes a single row entry in a player tree view
struct FPlayerTreeViewEntry : public TSharedFromThis<FPlayerTreeViewEntry>
{
	FPlayerTreeViewEntry(FString Name, EPlayerTreeViewEntryType::Type InEntryType, FAnimParentNodeAssetOverride* InOverride = NULL)
	: EntryName(Name)
	, EntryType(InEntryType)
	, Override(InOverride)
	{}

	FORCENOINLINE bool operator==(const FPlayerTreeViewEntry& Other);

	void GenerateNameWidget(TSharedPtr<SHorizontalBox> Box);

	// Name for the row
	FString EntryName;

	// What the row represents 
	EPlayerTreeViewEntryType::Type EntryType;

	// Node asset override for rows that represent nodes
	FAnimParentNodeAssetOverride* Override;

	// Children array for rows that represent blueprints and graphs.
	TArray<TSharedPtr<FPlayerTreeViewEntry>> Children;
};

class FAnimGraphParentPlayerDetails : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance(TSharedRef<class FBlueprintEditor> InBlueprintEditor);

	virtual void CustomizeDetails(class IDetailLayoutBuilder& DetailBuilder);

private:

	FAnimGraphParentPlayerDetails(const TSharedRef<class FBlueprintEditor>& InBlueprintEditor) : BlueprintEditorPtr(InBlueprintEditor)
	{}

	TSharedRef<ITableRow> OnGenerateRow(TSharedPtr<FPlayerTreeViewEntry> EventPtr, const TSharedRef< STableViewBase >& OwnerTable);
	void OnGetChildren(TSharedPtr<FPlayerTreeViewEntry> InParent, TArray< TSharedPtr<FPlayerTreeViewEntry> >& OutChildren);

	// Entries in the tree view
	TArray<TSharedPtr<FPlayerTreeViewEntry>> ListEntries;
	
	// Hosting Blueprint Editor instance
	TWeakPtr<class FBlueprintEditor> BlueprintEditorPtr;

	// Editor meta-object containing override information
	UEditorParentPlayerListObj* EditorObject;
};

class SParentPlayerTreeRow : public SMultiColumnTableRow<TSharedPtr<FAnimGraphParentPlayerDetails>>
{
public:
	SLATE_BEGIN_ARGS(SParentPlayerTreeRow){}
		SLATE_ARGUMENT(TSharedPtr<FPlayerTreeViewEntry>, Item);
		SLATE_ARGUMENT(UEditorParentPlayerListObj*, OverrideObject);
		SLATE_ARGUMENT(TWeakPtr<class FBlueprintEditor>, BlueprintEditor);
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView);

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName);

private:
	
	// Should an asset be filtered, ensures we only approve assets with matching skeletons
	bool OnShouldFilterAsset(const FAssetData& AssetData);

	// Sets the override asset when selected from the asset picker
	void OnAssetSelected(const FAssetData& AssetData);

	void OnCloseMenu(){}

	// Called when the user clicks the focus button, opens a graph panel if necessary in
	// read only mode and focusses on the node.
	FReply OnFocusNodeButtonClicked();

	// Gets the current asset, either an override if one is selected or the original from the node.
	const UAnimationAsset* GetCurrentAssetToUse() const;

	// Whether or not we should show the reset to default button next to the asset picker
	EVisibility GetResetToDefaultVisibility() const;

	// Resets the selected asset override back to the original node's asset
	FReply OnResetButtonClicked();

	// Gets the full path to the current asset
	FString GetCurrentAssetPath() const;

	// Editor object containing all possible overrides
	UEditorParentPlayerListObj* EditorObject;

	// Tree item we are representing
	TSharedPtr<FPlayerTreeViewEntry> Item;

	// Graphnode this row represents, if any
	UAnimGraphNode_Base* GraphNode;

	// Blueprint editor pointer
	TWeakPtr<FBlueprintEditor> BlueprintEditor;
};
