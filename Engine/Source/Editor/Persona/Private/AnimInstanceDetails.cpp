// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimInstanceDetails.h"
#include "UObject/UnrealType.h"
#include "Animation/AnimationAsset.h"
#include "DetailWidgetRow.h"
#include "IDetailPropertyRow.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "PropertyCustomizationHelpers.h"
#include "Animation/AnimInstance.h"
#include "Animation/Skeleton.h"
#include "ObjectEditorUtils.h"

/////////////////////////////////////////////////////
// FAnimInstanceDetails 

TSharedRef<IDetailCustomization> FAnimInstanceDetails::MakeInstance()
{
	return MakeShareable(new FAnimInstanceDetails());
}

void FAnimInstanceDetails::CustomizeDetails(class IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> Objects;
	DetailBuilder.GetObjectsBeingCustomized(Objects);

	TArray<UClass*> ObjectClasses;
	USkeleton* TargetSkeleton = NULL;

	// Grab the skeleton we are displaying for filtering
	for (TWeakObjectPtr<UObject> Object : Objects)
	{
		if (UObject* ObjectPtr = Object.Get())
		{
			ObjectClasses.AddUnique(Object->GetClass());
			if (UAnimInstance* AnimInst = Cast<UAnimInstance>(ObjectPtr))
			{
				if (TargetSkeleton && TargetSkeleton != AnimInst->CurrentSkeleton)
				{
					TargetSkeleton = NULL;
					break;
				}
				else
				{
					TargetSkeleton = AnimInst->CurrentSkeleton;
				}
			}
		}
	}

	// Grab common base class for property population
	UClass* CommonBaseClass = UClass::FindCommonBase(ObjectClasses);

	// If everything uses the same skeleton we can filter based on that skeleton
	if (TargetSkeleton)
	{
		for (TFieldIterator<FProperty> It(CommonBaseClass, EFieldIteratorFlags::IncludeSuper); It; ++It)
		{
			FProperty* TargetProperty = *It;

			TSharedRef<IPropertyHandle> TargetPropertyHandle = DetailBuilder.GetProperty(*TargetProperty->GetName(), CommonBaseClass);
			if ((*TargetPropertyHandle).GetProperty() )
			{
				IDetailCategoryBuilder& CurrentCategory = DetailBuilder.EditCategory(FObjectEditorUtils::GetCategoryFName(TargetProperty));
				
				IDetailPropertyRow& PropertyRow = CurrentCategory.AddProperty(TargetPropertyHandle);

				TSharedPtr<SWidget> NameWidget;
				TSharedPtr<SWidget> ValueWidget;
				FDetailWidgetRow Row;
				PropertyRow.GetDefaultWidgets(NameWidget, ValueWidget, Row);

				TSharedRef<SWidget> TempWidget = CreateFilteredObjectPropertyWidget(TargetProperty, TargetPropertyHandle, TargetSkeleton);
				ValueWidget = (TempWidget == SNullWidget::NullWidget) ? ValueWidget : TempWidget;

				const bool bShowChildren = true;
				PropertyRow.CustomWidget(bShowChildren)
					.NameContent()
					.MinDesiredWidth(Row.NameWidget.MinWidth)
					.MaxDesiredWidth(Row.NameWidget.MaxWidth)
					[
						NameWidget.ToSharedRef()
					]
				.ValueContent()
					.MinDesiredWidth(Row.ValueWidget.MinWidth)
					.MaxDesiredWidth(Row.ValueWidget.MaxWidth)
					[
						ValueWidget.ToSharedRef()
					];
			}
		}
	}
}

TSharedRef<SWidget> FAnimInstanceDetails::CreateFilteredObjectPropertyWidget(FProperty* TargetProperty, TSharedRef<IPropertyHandle> TargetPropertyHandle, const USkeleton* TargetSkeleton)
{
	if(const FObjectPropertyBase* ObjectProperty = CastField<const FObjectPropertyBase>( TargetProperty ))
	{
		if(ObjectProperty->PropertyClass->IsChildOf(UAnimationAsset::StaticClass()))
		{
			bool bAllowClear = !(ObjectProperty->PropertyFlags & CPF_NoClear);

			return SNew(SObjectPropertyEntryBox)
				.PropertyHandle(TargetPropertyHandle)
				.AllowedClass(ObjectProperty->PropertyClass)
				.AllowClear(bAllowClear)
				.OnShouldFilterAsset(this, &FAnimInstanceDetails::OnShouldFilterAnimAsset, TargetSkeleton);
		}
	}

	return SNullWidget::NullWidget;
}

bool FAnimInstanceDetails::OnShouldFilterAnimAsset(const FAssetData& AssetData, const USkeleton* TargetSkeleton) const
{
	return !(TargetSkeleton && TargetSkeleton->IsCompatibleForEditor(AssetData));
}
