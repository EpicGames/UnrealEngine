// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "PropertyHandle.h"
#include "IDetailCustomization.h"

struct FAssetData;
class USkeleton;

/////////////////////////////////////////////////////
// FAnimInstanceDetails 

class FAnimInstanceDetails : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();

	// IDetailCustomization interface
	virtual void CustomizeDetails(class IDetailLayoutBuilder& DetailBuilder) override;
	// End of IDetailCustomization interface

protected:

	// Creates a filtered object widget if the supplied property is an object property
	TSharedRef<SWidget> CreateFilteredObjectPropertyWidget(FProperty* TargetProperty, TSharedRef<IPropertyHandle> TargetPropertyHandle, const USkeleton* TargetSkeleton);

	/** Delegate to handle filtering of asset pickers */
	bool OnShouldFilterAnimAsset(const FAssetData& AssetData, const USkeleton* TargetSkeleton) const;
};
