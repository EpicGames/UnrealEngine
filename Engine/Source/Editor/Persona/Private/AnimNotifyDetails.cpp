// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNotifyDetails.h"
#include "Animation/Skeleton.h"
#include "Widgets/Text/STextBlock.h"
#include "Animation/AnimTypes.h"
#include "Animation/AnimationAsset.h"
#include "DetailWidgetRow.h"
#include "IDetailPropertyRow.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "Animation/AnimMontage.h"
#include "Animation/EditorNotifyObject.h"
#include "AssetSearchBoxUtilPersona.h"
#include "IDetailGroup.h"
#include "ObjectEditorUtils.h"
#include "Animation/AnimNotifies/AnimNotify.h"
#include "Widgets/Input/STextComboBox.h"
#include "Animation/AnimNotifies/AnimNotifyState.h"

namespace UE::AnimNotifyDetails::Private
{
	constexpr TCHAR CategoryDelimiter = TEXT('|');
	constexpr FStringView AnimNotifyCategory = TEXTVIEW("AnimNotify");
	static const FName AnimNotifyCategoryName = FName(AnimNotifyCategory.Len(), AnimNotifyCategory.GetData());
	static const FName AdvancedCategoryName = TEXT("Advanced");
	
	/**
	 * Splits a category name into its parent and leaf category names. Returns a pair of (Parent, Leaf) category names.
	 * If there is no parent category, then the parent name will be empty
	 */
	static TPair<FName, FName> SplitCategory(const FName CategoryName)
	{
		const FString CategoryString = CategoryName.ToString();
		const FStringView CategoryView = CategoryString;

		int32 DelimiterLocation = 0;
		if (CategoryString.FindLastChar(CategoryDelimiter, DelimiterLocation))
		{
			const FStringView ParentCategoryView = CategoryView.Left(DelimiterLocation);
			const FStringView LeafCategoryView = CategoryView.RightChop(DelimiterLocation + 1);
			const FName ParentCategoryName = FName(ParentCategoryView.Len(), ParentCategoryView.GetData());
			const FName LeafCategoryName = FName(LeafCategoryView.Len(), LeafCategoryView.GetData());
			
			return MakeTuple(ParentCategoryName, LeafCategoryName);
		}
		
		return MakeTuple(NAME_None, CategoryName);
	}

	/**
	 * Strips the leading "Anim Notify" category from the category name, if there is any.
	 * 
	 * Helps as a number of anim notifies were authored with the "Anim Notify" category but did not previously display
	 * it correctly. This prevents such notifies from showing an extra category level
	 */
	static FName StripAnimNotifyPrefix(const FName CategoryName)
	{
		if (CategoryName.IsNone())
		{
			return CategoryName;
		}

		const FString CategoryString = CategoryName.ToString();
		const FStringView CategoryView = CategoryString;

		int32 DelimiterLocation = 0;
		if (CategoryString.StartsWith(AnimNotifyCategory))
		{
			FStringView StrippedCategoryView = CategoryView.RightChop(AnimNotifyCategory.Len() + 1);
			if (!StrippedCategoryView.IsEmpty() && StrippedCategoryView[0] == CategoryDelimiter)
			{
				StrippedCategoryView = StrippedCategoryView.RightChop(1);
			}

			if (StrippedCategoryView.IsEmpty())
			{
				return NAME_None;
			}

			return FName(StrippedCategoryView.Len(), StrippedCategoryView.GetData());
		}

		return CategoryName;
	}

	/**
	 * Adds a series of subgroups for the specified category name, with a new subgroup for each category separated by
	 * a |. Appends the category to the subgroup map to avoid creating categories multiple times
	 */
	static IDetailGroup& FindOrAddSubgroup(IDetailCategoryBuilder& Category, const FName CategoryName,
		TMap<FName, IDetailGroup*>& SubgroupMap)
	{
		if (IDetailGroup** ExistingGroup = SubgroupMap.Find(CategoryName))
		{
			check(*ExistingGroup);
			return **ExistingGroup;
		}
		
		const TPair<FName, FName> SplitCategoryName = SplitCategory(CategoryName);
		const FName ParentCategoryName = SplitCategoryName.Key;
		const FName LeafCategoryName = SplitCategoryName.Value;
		const FText DisplayName = FObjectEditorUtils::GetCategoryText(LeafCategoryName);

		IDetailGroup* Subgroup = nullptr;

		if (ParentCategoryName.IsNone())
		{
			Subgroup = &Category.AddGroup(LeafCategoryName, DisplayName);
		}
		else
		{
			IDetailGroup& ParentGroup = FindOrAddSubgroup(Category, ParentCategoryName, SubgroupMap);
			Subgroup = &ParentGroup.AddGroup(LeafCategoryName, DisplayName);
		}
		check(Subgroup);

		SubgroupMap.Add(CategoryName, Subgroup);
		return *Subgroup;
	}
	
	/**
	 * Adds subgroups for the specified property, but not the property itself
	 */
	static void AddSubgroupForProperty(IDetailCategoryBuilder& Category, const FProperty* Property,
		TMap<FName, IDetailGroup*>& SubgroupMap)
	{
		if (Property)
		{
			const FName CategoryName = StripAnimNotifyPrefix(FObjectEditorUtils::GetCategoryFName(Property));
			if (!CategoryName.IsNone())
			{
				FindOrAddSubgroup(Category, CategoryName, SubgroupMap);
			}
		}
	}

	/**
	 * Adds a subcategory to the specified category with the "Advanced" name. Uses a subgroup map and advanced subgroup
	 * map to avoid creating duplicate categories
	 */
	static IDetailGroup& FindOrAddAdvancedCategory(FName CategoryName, TMap<FName, IDetailGroup*>& SubgroupMap,
		TMap<FName, IDetailGroup*>& AdvancedSubgroupMap)
	{
		if (IDetailGroup** ExistingAdvancedGroup = AdvancedSubgroupMap.Find(CategoryName))
		{
			check(*ExistingAdvancedGroup);
			return **ExistingAdvancedGroup;
		}
		else
		{
			IDetailGroup** PropertyGroup = SubgroupMap.Find(CategoryName);
			check(PropertyGroup);
			check(*PropertyGroup);

			static FText AdvancedCategoryText;
			if (AdvancedCategoryText.IsEmpty())
			{
				AdvancedCategoryText = FObjectEditorUtils::GetCategoryText(AdvancedCategoryName);
			}

			IDetailGroup& NewAdvancedGroup = (*PropertyGroup)->AddGroup(AdvancedCategoryName, AdvancedCategoryText);

			AdvancedSubgroupMap.Add(CategoryName, &NewAdvancedGroup);
			return NewAdvancedGroup;
		}
	}
}

TSharedRef<IDetailCustomization> FAnimNotifyDetails::MakeInstance()
{
	return MakeShareable( new FAnimNotifyDetails );
}

void FAnimNotifyDetails::CustomizeDetails( IDetailLayoutBuilder& DetailBuilder )
{
	const UClass* DetailObjectClass = nullptr;
	const UClass* BaseClass = DetailBuilder.GetBaseClass();
	TArray<TWeakObjectPtr<UObject>> SelectedObjects;
	DetailBuilder.GetObjectsBeingCustomized(SelectedObjects);

	check(SelectedObjects.Num() > 0);

	UObject* CommonOuter = nullptr;
	for (const TWeakObjectPtr<UObject>& WeakObject : SelectedObjects)
	{
		if (const UEditorNotifyObject* EditorObject = Cast<UEditorNotifyObject>(WeakObject.Get()))
		{
			UpdateSlotNames(EditorObject->AnimObject);

			// Find common outer object
			if (CommonOuter == nullptr)
			{
				CommonOuter = EditorObject->AnimObject;
			}
			else if (EditorObject->AnimObject && CommonOuter != EditorObject->AnimObject)
			{
				CommonOuter = nullptr;
			}
		}		
	}
	

	TSharedRef<IPropertyHandle> EventHandle = DetailBuilder.GetProperty(TEXT("Event"));
	IDetailCategoryBuilder& EventCategory = DetailBuilder.EditCategory(TEXT("Category"));
	EventCategory.AddProperty(EventHandle).OverrideResetToDefault(FResetToDefaultOverride::Hide());

	// Hide notify objects that aren't set
	UObject* NotifyObject = nullptr;
	UObject* NotifyStateObject = nullptr;
	FString NotifyClassName;
	TSharedPtr<IPropertyHandle> PropertyHandleToUse = nullptr;
	TSharedRef<IPropertyHandle> NotifyPropHandle = DetailBuilder.GetProperty(TEXT("Event.Notify"));
	TSharedRef<IPropertyHandle> NotifyStatePropHandle = DetailBuilder.GetProperty(TEXT("Event.NotifyStateClass"));

	const FPropertyAccess::Result NotifyPropertyAccessResult = NotifyPropHandle->GetValue(NotifyObject);
	const FPropertyAccess::Result NotifyStatePropertyAccessResult = NotifyStatePropHandle->GetValue(NotifyStateObject);
	
	// Don't want to edit the notify name here.
	DetailBuilder.HideProperty(TEXT("Event.NotifyName"));

	IDetailCategoryBuilder& AnimNotifyCategory = DetailBuilder.EditCategory(
		UE::AnimNotifyDetails::Private::AnimNotifyCategoryName, FText::GetEmpty(), ECategoryPriority::TypeSpecific);

	const bool bValidNotifyObjects = NotifyPropertyAccessResult != FPropertyAccess::Fail;
	const bool bValidNotifyStateClasses = NotifyStatePropertyAccessResult != FPropertyAccess::Fail;

	UObject* NotifyPtr = nullptr;
	DetailBuilder.HideProperty(TEXT("Event.Notify"));
	if (bValidNotifyStateClasses)
	{
		DetailBuilder.HideProperty(TEXT("Event.NotifyStateClass"));
		DetailBuilder.HideProperty(TEXT("Event.EndLink"));

		UClass* NotifyStateClass = nullptr;
		UAnimNotifyState* NotifyState = nullptr;

		// Find common notify state class, only valid if all of the selected objects are of the same notify type
		for (const TWeakObjectPtr<UObject>& WeakObject : SelectedObjects)
		{
			if (const UEditorNotifyObject* EditorObject = Cast<UEditorNotifyObject>(WeakObject.Get()))
			{
				NotifyState = EditorObject->Event.NotifyStateClass.Get();
				if (NotifyStateClass == nullptr)
				{
					if (NotifyState)
					{
						NotifyStateClass = NotifyState->GetClass();
					}
					else
					{
						break;
					}
				}
				else if (NotifyState && NotifyState->GetClass() != NotifyStateClass)
				{
					NotifyStateClass = nullptr;
					break;
				}
			}
		}

		NotifyPtr = NotifyState;
		PropertyHandleToUse = NotifyStatePropHandle;
		DetailObjectClass = NotifyStateClass;
	}

	// In case there wasn't a valid AnimNotifyState class try to find an AnimNotify class instead
	if (bValidNotifyObjects && NotifyPtr == nullptr)
	{
		UClass* NotifyClass = nullptr;
		UAnimNotify* Notify = nullptr;
		
		// Find common notify class, only valid if all of the selected objects are of the same notify type
		for (const TWeakObjectPtr<UObject>& WeakObject : SelectedObjects)
		{
			if (const UEditorNotifyObject* EditorObject = Cast<UEditorNotifyObject>(WeakObject.Get()))
			{
				Notify = EditorObject->Event.Notify.Get();
				if (NotifyClass == nullptr)
				{
					if (Notify)
					{
						NotifyClass = Notify->GetClass();
					}
					else
					{
						break;
					}
				}
				else if (Notify && Notify->GetClass() != NotifyClass)
				{
					NotifyClass = nullptr;
					break;
				}
			}
		}

		NotifyPtr = Notify;
		PropertyHandleToUse = NotifyPropHandle;
		DetailObjectClass = NotifyClass;
		
	}
	// This means we ended up in a situation where there is no valid notify or notify-state class - so why are we trying to customize a notify
	//check(NotifyPtr != nullptr);

	UAnimMontage* CurrentMontage = Cast<UAnimMontage>(CommonOuter);

	// If we have a montage, and it has slots (which it should have) generate custom link properties
	if(CurrentMontage && CurrentMontage->SlotAnimTracks.Num() > 0)
	{
		CustomizeLinkProperties(DetailBuilder, EventHandle, CurrentMontage);
	}
	else
	{
		// No montage, hide link properties
		HideLinkProperties(DetailBuilder, EventHandle);
	}

	TMap<FName, IDetailGroup*> SubgroupMap;
	TMap<FName, IDetailGroup*> AdvancedSubgroupMap;
	TArray<TSharedPtr<IPropertyHandle>> PropertyHandles;
	TArray<TSharedPtr<IPropertyHandle>> AdvancedPropertyHandles;

	// Customizations do not run for instanced properties, so we have to resolve the properties and then
	// customize them here instead.
	if(PropertyHandleToUse->IsValidHandle())
	{
		uint32 NumChildren = 0;
		PropertyHandleToUse->GetNumChildren(NumChildren);
		if(NumChildren > 0)
		{
			TSharedPtr<IPropertyHandle> BaseHandle = PropertyHandleToUse->GetChildHandle(0);
			DetailBuilder.HideProperty(PropertyHandleToUse);

			BaseHandle->GetNumChildren(NumChildren);
			DetailBuilder.HideProperty(BaseHandle);

			for(uint32 ChildIdx = 0 ; ChildIdx < NumChildren ; ++ChildIdx)
			{
				TSharedPtr<IPropertyHandle> NotifyProperty = BaseHandle->GetChildHandle(ChildIdx);
				FProperty* Prop = NotifyProperty->GetProperty();
				
				if(Prop && !Prop->HasAnyPropertyFlags(CPF_DisableEditOnInstance))
				{
					if(!CustomizeProperty(AnimNotifyCategory, NotifyPtr, NotifyProperty))
					{
						// Add our subgroups first, so we can make sure they are sorted before the normal properties
						UE::AnimNotifyDetails::Private::AddSubgroupForProperty(AnimNotifyCategory, Prop, SubgroupMap);

						if (Prop->HasAnyPropertyFlags(CPF_AdvancedDisplay))
						{
							AdvancedPropertyHandles.Add(NotifyProperty);
						}
						else
						{
							PropertyHandles.Add(NotifyProperty);
						}
					}
				}
			}
		}
	}

	for (TSharedPtr<IPropertyHandle> PropertyHandle : PropertyHandles)
	{
		check(PropertyHandle);
		FProperty* Property = PropertyHandle->GetProperty();
		check(Property);
		
		const FName PropertyGroupName =
			UE::AnimNotifyDetails::Private::StripAnimNotifyPrefix(FObjectEditorUtils::GetCategoryFName(Property));

		if (!PropertyGroupName.IsNone())
		{
			IDetailGroup** PropertyGroup = SubgroupMap.Find(PropertyGroupName);
			check(PropertyGroup);
			check(*PropertyGroup);

			(*PropertyGroup)->AddPropertyRow(PropertyHandle.ToSharedRef());
		}
		else
		{
			AnimNotifyCategory.AddProperty(PropertyHandle);
		}
	}

	// Iterate over all of the advanced properties last so we can their advanced categories as needed, to sort after the
	// normal properties
	for (TSharedPtr<IPropertyHandle> PropertyHandle : AdvancedPropertyHandles)
	{
		check(PropertyHandle);
		FProperty* Property = PropertyHandle->GetProperty();
		check(Property);

		const FName PropertyGroupName = 
			UE::AnimNotifyDetails::Private::StripAnimNotifyPrefix(FObjectEditorUtils::GetCategoryFName(Property));

		if (!PropertyGroupName.IsNone())
		{
			IDetailGroup& AdvancedCategory = UE::AnimNotifyDetails::Private::FindOrAddAdvancedCategory(
				PropertyGroupName, SubgroupMap, AdvancedSubgroupMap);
			AdvancedCategory.AddPropertyRow(PropertyHandle.ToSharedRef());
		}
		else
		{
			// If we're just adding the property to the top level category, then AddProperty will automatically handle
			// setting whether it's advanced or not
			AnimNotifyCategory.AddProperty(PropertyHandle);
		}
	}

	struct FPropVisPair
	{
		const TCHAR* NotifyName;
		TAttribute<EVisibility> Visibility;
	};

	TriggerFilterModeHandle = DetailBuilder.GetProperty(TEXT("Event.NotifyFilterType"));

	FPropVisPair TriggerSettingNames[] = { { TEXT("Event.NotifyTriggerChance"), Cast<UAnimNotifyState>(NotifyPtr) == nullptr ? EVisibility::Visible : EVisibility::Hidden }
										 , { TEXT("Event.bTriggerOnDedicatedServer"), TAttribute<EVisibility>(EVisibility::Visible) }
										 , { TEXT("Event.bTriggerOnFollower"), TAttribute<EVisibility>(EVisibility::Visible) }
										 , { TEXT("Event.NotifyFilterType"), TAttribute<EVisibility>(EVisibility::Visible) }
										 , { TEXT("Event.NotifyFilterLOD"), TAttribute<EVisibility>(this, &FAnimNotifyDetails::VisibilityForLODFilterMode) } };


	IDetailCategoryBuilder& TriggerSettingCategory = DetailBuilder.EditCategory(TEXT("Trigger Settings"), FText::GetEmpty(), ECategoryPriority::TypeSpecific);

	for (FPropVisPair& NotifyPair : TriggerSettingNames)
	{
		TSharedRef<IPropertyHandle> NotifyPropertyHandle = DetailBuilder.GetProperty(NotifyPair.NotifyName);
		DetailBuilder.HideProperty(NotifyPropertyHandle);
		TriggerSettingCategory.AddProperty(NotifyPropertyHandle).Visibility(NotifyPair.Visibility);
	}
}

EVisibility FAnimNotifyDetails::VisibilityForLODFilterMode() const
{
	uint8 FilterModeValue = 0;
	FPropertyAccess::Result Ret = TriggerFilterModeHandle.Get()->GetValue(FilterModeValue);
	if (Ret == FPropertyAccess::Result::Success)
	{
		return (FilterModeValue == ENotifyFilterType::LOD) ? EVisibility::Visible : EVisibility::Hidden;
	}

	return EVisibility::Hidden; //Hidden if we get fail or MultipleValues from the property
}

void FAnimNotifyDetails::AddBoneNameProperty(IDetailCategoryBuilder& CategoryBuilder, UObject* Notify,  TSharedPtr<IPropertyHandle> Property)
{
	int32 PropIndex = NameProperties.Num();

	if(Notify && Property->IsValidHandle())
	{
		NameProperties.Add(Property);
		// get all the possible suggestions for the bones and sockets.
		if(const UAnimationAsset* AnimAsset = Cast<const UAnimationAsset>(Notify->GetOuter()))
		{
			if(const USkeleton* Skeleton = AnimAsset->GetSkeleton())
			{
				CategoryBuilder.AddProperty(Property.ToSharedRef())
					.CustomWidget()
					.NameContent()
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.Padding(FMargin(0, 1, 0, 1))
						[
							SNew(STextBlock)
							.Text(Property->GetPropertyDisplayName())
							.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
						]
					]
				.ValueContent()
					[
						SNew(SAssetSearchBoxForBones, Skeleton, Property)
						.IncludeSocketsForSuggestions(true)
						.MustMatchPossibleSuggestions(false)
						.HintText(NSLOCTEXT("AnimNotifyDetails", "Hint Text", "Bone Name..."))
						.OnTextCommitted(this, &FAnimNotifyDetails::OnSearchBoxCommitted, PropIndex)
					];
			}
		}
	}
}

void FAnimNotifyDetails::AddCurveNameProperty(IDetailCategoryBuilder& CategoryBuilder, UObject* Notify, TSharedPtr<IPropertyHandle> Property)
{
	int32 PropIndex = NameProperties.Num();
	
	if(Notify && Property->IsValidHandle())
	{
		NameProperties.Add(Property);

		if(const UAnimationAsset* AnimAsset = Cast<const UAnimationAsset>(Notify->GetOuter()))
		{
			if(const USkeleton* Skeleton = AnimAsset->GetSkeleton())
			{
				CategoryBuilder.AddProperty(Property.ToSharedRef())
					.CustomWidget()
					.NameContent()
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.Padding(FMargin(2, 1, 0, 1))
						[
							SNew(STextBlock)
							.Text(Property->GetPropertyDisplayName())
							.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
						]
					]
				.ValueContent()
					[
						SNew(SAssetSearchBoxForCurves, Skeleton, Property)
						.IncludeSocketsForSuggestions(true)
						.MustMatchPossibleSuggestions(true)
						.HintText(NSLOCTEXT("AnimNotifyDetails", "Curve Name Hint Text", "Curve Name..."))
						.OnTextCommitted(this, &FAnimNotifyDetails::OnSearchBoxCommitted, PropIndex)
					];
			}
		}
	}
}

void FAnimNotifyDetails::OnSearchBoxCommitted(const FText& InSearchText, ETextCommit::Type CommitInfo, int32 PropertyIndex )
{
	NameProperties[PropertyIndex]->SetValue( InSearchText.ToString() );
}

void FAnimNotifyDetails::ClearInstancedSelectionDropDown(IDetailCategoryBuilder& CategoryBuilder, TSharedRef<IPropertyHandle> PropHandle, bool bShowChildren /*= true*/)
{
	IDetailPropertyRow& PropRow = CategoryBuilder.AddProperty(PropHandle);
	
	PropRow
	.OverrideResetToDefault(FResetToDefaultOverride::Hide())
	.CustomWidget(bShowChildren)
	.NameContent()
	[
		PropHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		SNullWidget::NullWidget
	];
}

void FAnimNotifyDetails::CustomizeLinkProperties(IDetailLayoutBuilder& Builder, TSharedRef<IPropertyHandle> NotifyProperty, UAnimSequenceBase* AnimSequenceBase)
{
	uint32 NumChildProperties = 0;
	NotifyProperty->GetNumChildren(NumChildProperties);

	if(NumChildProperties > 0)
	{
		IDetailCategoryBuilder& LinkCategory = Builder.EditCategory(TEXT("AnimLink"));
		for(uint32 ChildIdx = 0 ; ChildIdx < NumChildProperties ; ++ChildIdx)
		{
			TSharedPtr<IPropertyHandle> ChildHandle = NotifyProperty->GetChildHandle(ChildIdx);
			FString OuterFieldType = ChildHandle->GetProperty()->GetOwnerVariant().GetName();

			if(ChildHandle->GetProperty()->GetName() == GET_MEMBER_NAME_CHECKED(FAnimNotifyEvent, EndLink).ToString()
			   || OuterFieldType == FString(TEXT("AnimLinkableElement")))
			{
				// If we get a slot index property replace it with a dropdown showing the names of the 
				// slots, as the indices are hidden from the user.
				if(ChildHandle->GetProperty()->GetName() == TEXT("SlotIndex"))
				{
					int32 SlotIdx = INDEX_NONE;
					ChildHandle->GetValue(SlotIdx);
					
					TSharedPtr<FString> InitialItem = nullptr;
					if (SlotNameItems.IsValidIndex(SlotIdx))
					{
						InitialItem = SlotNameItems[SlotIdx];
					}
					else
					{
						InitialItem = MakeShared<FString>(TEXT("Multiple Values"));
					}

					LinkCategory.AddProperty(ChildHandle)
						.CustomWidget()
						.NameContent()
						[
							ChildHandle->CreatePropertyNameWidget(NSLOCTEXT("NotifyDetails", "SlotIndexName", "Slot"))
						]
						.ValueContent()
						[
							SNew(STextComboBox)
							.OptionsSource(&SlotNameItems)
							.OnSelectionChanged(this, &FAnimNotifyDetails::OnSlotSelected, ChildHandle)
							.OnComboBoxOpening(this, &FAnimNotifyDetails::UpdateSlotNames, AnimSequenceBase)
							.InitiallySelectedItem(InitialItem)
							.Font(Builder.GetDetailFont())
						];
				}
				else
				{
					LinkCategory.AddProperty(ChildHandle);
				}
			}
		}
	}
}

void FAnimNotifyDetails::HideLinkProperties(IDetailLayoutBuilder& Builder, TSharedRef<IPropertyHandle> NotifyProperty)
{
	uint32 NumChildProperties = 0;
	NotifyProperty->GetNumChildren(NumChildProperties);

	if(NumChildProperties > 0)
	{
		for(uint32 ChildIdx = 0 ; ChildIdx < NumChildProperties ; ++ChildIdx)
		{
			TSharedPtr<IPropertyHandle> ChildHandle = NotifyProperty->GetChildHandle(ChildIdx);
			FString OuterFieldType = ChildHandle->GetProperty()->GetOwnerVariant().GetName();
			if(ChildHandle->GetProperty()->GetName() == GET_MEMBER_NAME_CHECKED(FAnimNotifyEvent, EndLink).ToString()
			   || OuterFieldType == FString(TEXT("AnimLinkableElement")))
			{
				Builder.HideProperty(ChildHandle);
			}
		}
	}
}

bool FAnimNotifyDetails::CustomizeProperty(IDetailCategoryBuilder& CategoryBuilder, UObject* Notify, TSharedPtr<IPropertyHandle> Property)
{
	TFunction<void(const TSharedPtr<IPropertyHandle>)> FixBoneNamePropertyRecurse;
	FixBoneNamePropertyRecurse = [this, &FixBoneNamePropertyRecurse, &CategoryBuilder, &Notify](const TSharedPtr<IPropertyHandle>& InPropertyHandle)
	{
		const bool bHasExpandMeta = InPropertyHandle->GetBoolMetaData(TEXT("AnimNotifyExpand"));
		bool bParentIsObjectPtr = false;

		TSharedPtr<IPropertyHandle> ParentProp = InPropertyHandle->GetParentHandle();
		if (ParentProp.IsValid() && ParentProp->IsValidHandle())
		{
			bParentIsObjectPtr = (CastField<FObjectPropertyBase>(ParentProp->GetProperty()) != nullptr);
		}

		// Recurse into Object Ptrs or properties with AnimNotifyExpand
		if (bParentIsObjectPtr || bHasExpandMeta)
		{
			IDetailLayoutBuilder& LayoutBuilder = CategoryBuilder.GetParentLayout();
			LayoutBuilder.HideProperty(InPropertyHandle);

			uint32 NumChildren = 0;
			InPropertyHandle->GetNumChildren(NumChildren);
			for (uint32 i = 0; i < NumChildren; ++i)
			{
				TSharedPtr<IPropertyHandle> ChildHandle = InPropertyHandle->GetChildHandle(i);
				FixBoneNamePropertyRecurse(ChildHandle);
			}
		}
		else if (InPropertyHandle->GetBoolMetaData(TEXT("AnimNotifyBoneName")))
		{
			// Convert this property to a bone name property
			AddBoneNameProperty(CategoryBuilder, Notify, InPropertyHandle);
		}
		else
		{
			CategoryBuilder.AddProperty(InPropertyHandle);
		}
	};

	if(Notify && Notify->GetClass() && Property->IsValidHandle())
	{
		FString ClassName = Notify->GetClass()->GetName();
		FString PropertyName = Property->GetProperty()->GetName();
		bool bIsBoneName = Property->GetBoolMetaData(TEXT("AnimNotifyBoneName"));

		if(ClassName.Find(TEXT("AnimNotify_PlayParticleEffect")) != INDEX_NONE && PropertyName == TEXT("SocketName"))
		{
			AddBoneNameProperty(CategoryBuilder, Notify, Property);
			return true;
		}
		else if(ClassName.Find(TEXT("AnimNotifyState_TimedParticleEffect")) != INDEX_NONE && PropertyName == TEXT("SocketName"))
		{
			AddBoneNameProperty(CategoryBuilder, Notify, Property);
			return true;
		}
		else if(ClassName.Find(TEXT("AnimNotify_PlaySound")) != INDEX_NONE && PropertyName == TEXT("AttachName"))
		{
			AddBoneNameProperty(CategoryBuilder, Notify, Property);
			return true;
		}
		else if (ClassName.Find(TEXT("_SoundLibrary")) != INDEX_NONE && PropertyName == TEXT("SoundContext"))
		{
			CategoryBuilder.AddProperty(Property);
			FixBoneNamePropertyRecurse(Property);
			return true;
		}
		else if (ClassName.Find(TEXT("AnimNotifyState_Trail")) != INDEX_NONE)
		{
			if(PropertyName == TEXT("FirstSocketName") || PropertyName == TEXT("SecondSocketName"))
			{
				AddBoneNameProperty(CategoryBuilder, Notify, Property);
				return true;
			}
			else if(PropertyName == TEXT("WidthScaleCurve"))
			{
				AddCurveNameProperty(CategoryBuilder, Notify, Property);
				return true;
			}
		}
		else if (bIsBoneName)
		{
			AddBoneNameProperty(CategoryBuilder, Notify, Property);
			return true;
		}
	}
	return false;
}

void FAnimNotifyDetails::UpdateSlotNames(UAnimSequenceBase* AnimObject)
{
	if(UAnimMontage* MontageObj = Cast<UAnimMontage>(AnimObject))
	{
		for(FSlotAnimationTrack& Slot : MontageObj->SlotAnimTracks)
		{
			if(!SlotNameItems.ContainsByPredicate([&Slot](TSharedPtr<FString>& Item){return Slot.SlotName.ToString() == *Item;}))
			{
				SlotNameItems.Add(MakeShareable(new FString(*Slot.SlotName.ToString()))); 
			}
		}
	}
}

void FAnimNotifyDetails::OnSlotSelected(TSharedPtr<FString> SlotName, ESelectInfo::Type SelectInfo, TSharedPtr<IPropertyHandle> Property)
{
	if(SelectInfo != ESelectInfo::Direct && Property->IsValidHandle())
	{
		int32 NewIndex = SlotNameItems.Find(SlotName);
		if(NewIndex != INDEX_NONE)
		{
			Property->SetValue(NewIndex);
		}
	}
}
