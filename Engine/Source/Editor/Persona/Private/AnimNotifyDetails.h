// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "PropertyHandle.h"
#include "IDetailCustomization.h"

class IDetailCategoryBuilder;
class IDetailLayoutBuilder;
class UAnimSequenceBase;
class UEditorNotifyObject;

class FAnimNotifyDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override;

private:

	/** Array of pointers to the name properties */
	TArray<TSharedPtr<IPropertyHandle>> NameProperties;
	/** List of slot names for selection dropdown, in same order as slots in the montage */
	TArray<TSharedPtr<FString>> SlotNameItems;

	/** Customize a notify property that is inside an instanced property
	 *	@param CategoryBuilder The category to place the property into
	 *	@param Notify the UAnimNotify or UAnimNotifyState object for the property
	 *	@param Property Handle to the property to customize
	 */
	bool CustomizeProperty(IDetailCategoryBuilder& CategoryBuilder, UObject* Notify, TSharedPtr<IPropertyHandle> Property);

	/** Adds a Bone Name property to the details layout
	 *	@param CategoryBuilder The category to add the property to
	 *	@param Notify the UAnimNotify or UAnimNotifyState object for the property
	 *	@param Property Handle to the property to customize
	 */
	void AddBoneNameProperty(IDetailCategoryBuilder& CategoryBuilder, UObject* Notify,  TSharedPtr<IPropertyHandle> Property);

	/** Adds a Curve Name property to the details layout
	 *	@param CategoryBuilder The category to add the property to
	 *	@param Notify the UAnimNotify or UAnimNotifyState object for the property
	 *	@param Property Handle to the property to customize
	 */
	void AddCurveNameProperty(IDetailCategoryBuilder& CategoryBuilder, UObject* Notify,  TSharedPtr<IPropertyHandle> Property);

	/** Handles search box commit for name properties
	 *	@param InSearchText Text that was committed
	 *	@param CommitInfo Commit method
	 *	@param PropertyIndex Index of the name properties in internal array
	 */
	void OnSearchBoxCommitted(const FText& InSearchText, ETextCommit::Type CommitInfo, int32 PropertyIndex);

	/** Get the search suggestions */
	TArray<FString> GetSearchSuggestions() const;

	/** Removes the dropdown selection of instanced objects from the header for the property
	 *	@param CategoryBuilder The category builder for the property
	 *	@param PropHandle Handle to the property to modify
	 *	@param bShowChildren Whether or not to show the instanced object children
	 */
	void ClearInstancedSelectionDropDown(IDetailCategoryBuilder& CategoryBuilder, TSharedRef<IPropertyHandle> PropHandle, bool bShowChildren = true);

	/** Move properties representing notify linking information into their own category
	 *	@param Builder Layout builder for this object
	 *	@param NotifyProperty The property that contains the linking properties
	 */
	void CustomizeLinkProperties(IDetailLayoutBuilder& Builder, TSharedRef<IPropertyHandle> NotifyProperty, UAnimSequenceBase* AnimSequenceBase);

	/** Hide any properties relating to notify linking
	 *	@param Builder Layout builder for this object
	 *	@param NotifyProperty The notify property that contains the linking properties
	 */
	void HideLinkProperties(IDetailLayoutBuilder& Builder, TSharedRef<IPropertyHandle> NotifyProperty);

	/** Updates the list of slot names used for combo box
	 *	@param AnimObject Object being edited, where we will search for slots
	 */
	void UpdateSlotNames(UAnimSequenceBase* AnimObject);

	/** Called when the user selects a slot
	 *	@param Index Index of the new slot
	 */
	void OnSlotSelected(TSharedPtr<FString> SlotName, ESelectInfo::Type SelectInfo, TSharedPtr<IPropertyHandle> Property);

	/** Return whether or not the LOD filter mode should be visible */
	EVisibility VisibilityForLODFilterMode() const;

	/** Caches the Filter Mode handle so we can look up its value after customization has finished */
	TSharedPtr<IPropertyHandle> TriggerFilterModeHandle;
};
