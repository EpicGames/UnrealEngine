// Copyright Epic Games, Inc. All Rights Reserved.


#include "AnimNotifyNodeFactory.h"


TArray<FAnimNotifyNodeFactoryFunction> FAnimNotifyNodeFactory::RegisteredFactories;
TArray<FDelegateHandle> FAnimNotifyNodeFactory::RegisteredFactoryHandles;

FDelegateHandle FAnimNotifyNodeFactory::RegisterFactory(FAnimNotifyNodeFactoryFunction InFactory)
{
	FDelegateHandle NewHandle(FDelegateHandle::GenerateNewHandle);
	RegisteredFactories.Add(InFactory);
	RegisteredFactoryHandles.Add(NewHandle);
	return NewHandle;
}

void FAnimNotifyNodeFactory::UnregisterFactory(FDelegateHandle InHandle)
{
	int32 Index;
	if (RegisteredFactoryHandles.Find(InHandle, Index))
	{
		RegisteredFactoryHandles.RemoveAt(Index);
		RegisteredFactories.RemoveAt(Index);
	}
}
