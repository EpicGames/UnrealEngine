// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/Events.h"
#include "AnimNotifyPanelContextMenuContext.generated.h"

class SAnimNotifyTrack;
struct INotifyNodeObjectInterface;
struct FAnimNotifyEvent; 

UCLASS()
class UAnimNotifyPanelContextMenuContext : public UObject
{
	GENERATED_BODY()

public:
	TWeakPtr<SAnimNotifyTrack> NotifyTrack = nullptr;
	INotifyNodeObjectInterface* NodeObject = nullptr;
	FAnimNotifyEvent* NotifyEvent = nullptr;
	int32 NotifyIndex = INDEX_NONE;
	int32 NodeIndex = INDEX_NONE;
	FPointerEvent MouseEvent; 
};
