// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimSequenceCurveEditorCommands.h"

#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"

#define LOCTEXT_NAMESPACE "AnimSequenceCurveEditorCommands"

void FAnimSequenceCurveEditorCommands::RegisterCommands()
{
	UI_COMMAND(EditSelectedCurves, "Edit Selected Curves", "Edit the selected curves in the curve editor tab", EUserInterfaceActionType::Button, FInputChord());
}

#undef LOCTEXT_NAMESPACE
