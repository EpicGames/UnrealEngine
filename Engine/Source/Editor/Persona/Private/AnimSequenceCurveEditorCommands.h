// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Framework/Commands/Commands.h"
#include "Internationalization/Internationalization.h"
#include "Styling/AppStyle.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

class FUICommandInfo;

/**
 * Defines commands for the anim sequence curve editor
 */
class FAnimSequenceCurveEditorCommands : public TCommands<FAnimSequenceCurveEditorCommands>
{
public:
	FAnimSequenceCurveEditorCommands()
		: TCommands<FAnimSequenceCurveEditorCommands>
		(
			TEXT("AnimSequenceCurveEditor"),
			NSLOCTEXT("Contexts", "AnimSequenceCurveEditor", "Anim Sequence Curve Editor"),
			NAME_None,
			FAppStyle::GetAppStyleSetName()
		)
	{
	}

	TSharedPtr<FUICommandInfo> EditSelectedCurves;

public:
	virtual void RegisterCommands() override;
};
