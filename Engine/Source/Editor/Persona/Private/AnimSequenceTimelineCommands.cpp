// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimSequenceTimelineCommands.h"

#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"

#define LOCTEXT_NAMESPACE "AnimSequenceTimelineCommands"

void FAnimSequenceTimelineCommands::RegisterCommands()
{
	UI_COMMAND(PasteDataIntoCurve, "Paste Clipboard Data Into Curve", "Paste curve data from the clipboard into the selected curve", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control | EModifierKey::Shift, EKeys::V));
	UI_COMMAND(EditSelectedCurves, "Edit Selected Curves", "Edit the selected curves in the curve editor tab", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(AddNotifyTrack, "Add Notify Track", "Add a new notify track", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(InsertNotifyTrack, "Insert Notify Track", "Insert a new notify track above here", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(RemoveNotifyTrack, "Remove Notify Track", "Remove this notify track", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(AddCurve, "Add Curve...", "Add a new variable float curve", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(EditCurve, "Edit Curve", "Edit this curve in the curve editor tab", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ShowCurveKeys, "Show Curve Keys", "Show keys for all curves in the timeline", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(UseTreeView, "Use Tree View", "Whether to use tree view for animation curves. This will treat period-seperated curves as seperate tree entries.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(AddMetadata, "Add Metadata...", "Add a new constant (metadata) float curve", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ConvertCurveToMetaData, "Convert To Metadata", "Convert this curve to a constant (metadata) curve.\nThis is a destructive operation and will remove all keys from this curve.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ConvertMetaDataToCurve, "Convert To Curve", "Convert this metadata curve to a full curve", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(RemoveCurve, "Remove Curve", "Remove this curve", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(RemoveAllCurves, "Remove All Curves", "Remove all the curves in this animation", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(CopySelectedCurveNames, "Copy Selected Curve Names", "Copy the name of the selected curves to the clipboard", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(DisplaySeconds, "Seconds", "Display the time in seconds", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(DisplayFrames, "Frames", "Display the time in frames", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(DisplayPercentage, "Percentage", "Display the percentage along with the time with the scrubber", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(DisplaySecondaryFormat, "Secondary", "Display the time or frame count as a secondary format along with the scrubber", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(SnapToFrames, "Frames", "Snap to frame boundaries", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(SnapToNotifies, "Notifies", "Snap to notifies and notify states", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(SnapToMontageSections, "Montage Sections", "Snap to montage sections", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(SnapToCompositeSegments, "Composite Segments", "Snap to composite segments", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(AddComment, "Add Comment", "Add a comment to this curve", EUserInterfaceActionType::ToggleButton, FInputChord());
}

#undef LOCTEXT_NAMESPACE
