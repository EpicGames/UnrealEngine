// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Framework/Commands/Commands.h"
#include "Internationalization/Internationalization.h"
#include "Styling/AppStyle.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

class FUICommandInfo;

/**
 * Defines commands for the anim sequence timeline editor
 */
class FAnimSequenceTimelineCommands : public TCommands<FAnimSequenceTimelineCommands>
{
public:
	FAnimSequenceTimelineCommands()
		: TCommands<FAnimSequenceTimelineCommands>
		(
			TEXT("AnimSequenceCurveEditor"),
			NSLOCTEXT("Contexts", "AnimSequenceTimelineEditor", "Anim Sequence Timeline Editor"),
			NAME_None,
			FAppStyle::GetAppStyleSetName()
		)
	{
	}

	TSharedPtr<FUICommandInfo> EditSelectedCurves;
	
	TSharedPtr<FUICommandInfo> AddNotifyTrack;
	
	TSharedPtr<FUICommandInfo> PasteDataIntoCurve;

	TSharedPtr<FUICommandInfo> InsertNotifyTrack;

	TSharedPtr<FUICommandInfo> RemoveNotifyTrack;

	TSharedPtr<FUICommandInfo> AddCurve;

	TSharedPtr<FUICommandInfo> EditCurve;

	TSharedPtr<FUICommandInfo> ShowCurveKeys;

	TSharedPtr<FUICommandInfo> UseTreeView;

	TSharedPtr<FUICommandInfo> AddMetadata;

	TSharedPtr<FUICommandInfo> ConvertCurveToMetaData;

	TSharedPtr<FUICommandInfo> ConvertMetaDataToCurve;

	TSharedPtr<FUICommandInfo> RemoveCurve;

	TSharedPtr<FUICommandInfo> RemoveAllCurves;

	TSharedPtr<FUICommandInfo> CopySelectedCurveNames;
	
	TSharedPtr<FUICommandInfo> DisplaySeconds;

	TSharedPtr<FUICommandInfo> DisplayFrames;

	TSharedPtr<FUICommandInfo> DisplayPercentage;

	TSharedPtr<FUICommandInfo> DisplaySecondaryFormat;

	TSharedPtr<FUICommandInfo> SnapToFrames;

	TSharedPtr<FUICommandInfo> SnapToNotifies;

	TSharedPtr<FUICommandInfo> SnapToMontageSections;

	TSharedPtr<FUICommandInfo> SnapToCompositeSegments;

	TSharedPtr<FUICommandInfo> AddComment;
public:
	virtual void RegisterCommands() override;
};
