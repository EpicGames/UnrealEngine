// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimTimeline/AnimModel.h"
#include "IPersonaPreviewScene.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "AnimPreviewInstance.h"
#include "Preferences/PersonaOptions.h"
#include "Animation/EditorAnimBaseObj.h"
#include "AnimTimeline/AnimTimelineTrack.h"
#include "Animation/AnimSequence.h"

#define LOCTEXT_NAMESPACE "FAnimModel"

const FAnimModel::FSnapType FAnimModel::FSnapType::Frames("Frames", LOCTEXT("FramesSnapName", "Frames"), [](const FAnimModel& InModel, double InTime)
{
	// Round to nearest frame
	FFrameRate FrameRate = InModel.GetFrameRate();
	if(FrameRate.IsValid())
	{
		return FrameRate.AsSeconds(FrameRate.AsFrameTime(InTime).RoundToFrame());
	}
	
	return InTime;
});

const FAnimModel::FSnapType FAnimModel::FSnapType::Notifies("Notifies", LOCTEXT("NotifiesSnapName", "Notifies"));

const FAnimModel::FSnapType FAnimModel::FSnapType::CompositeSegment("CompositeSegment", LOCTEXT("CompositeSegmentSnapName", "Composite Segments"));

const FAnimModel::FSnapType FAnimModel::FSnapType::MontageSection("MontageSection", LOCTEXT("MontageSectionSnapName", "Montage Sections"));

FAnimModel::FAnimModel(const TSharedRef<IPersonaPreviewScene>& InPreviewScene, const TSharedRef<IEditableSkeleton>& InEditableSkeleton, const TSharedRef<FUICommandList>& InCommandList)
	: WeakPreviewScene(InPreviewScene)
	, WeakEditableSkeleton(InEditableSkeleton)
	, WeakCommandList(InCommandList)
	, bIsSelecting(false)
{
}

void FAnimModel::Initialize()
{

}

FAnimatedRange FAnimModel::GetViewRange() const
{
	return ViewRange;
}

FAnimatedRange FAnimModel::GetWorkingRange() const
{
	return WorkingRange;
}

FFrameRate FAnimModel::GetFrameRate() const
{
	return GetAnimSequenceBase()->GetSamplingFrameRate();
}

int32 FAnimModel::GetTickResolution() const
{
	return FMath::RoundToInt32((double)GetDefault<UPersonaOptions>()->TimelineScrubSnapValue * GetFrameRate().AsDecimal());
}

TRange<FFrameNumber> FAnimModel::GetPlaybackRange() const
{
	const int32 Resolution = GetTickResolution();
	return TRange<FFrameNumber>(FFrameNumber(FMath::RoundToInt32(PlaybackRange.GetLowerBoundValue() * (double)Resolution)), FFrameNumber(FMath::RoundToInt32(PlaybackRange.GetUpperBoundValue() * (double)Resolution)));
}

FFrameNumber FAnimModel::GetScrubPosition() const
{
	if(WeakPreviewScene.IsValid())
	{
		UDebugSkelMeshComponent* PreviewMeshComponent = WeakPreviewScene.Pin()->GetPreviewMeshComponent();
		if(PreviewMeshComponent && PreviewMeshComponent->IsPreviewOn())
		{
			return FFrameNumber(FMath::RoundToInt32(PreviewMeshComponent->PreviewInstance->GetCurrentTime() * (double)GetTickResolution()));
		}
	}

	return FFrameNumber(0);
}

float FAnimModel::GetScrubTime() const
{
	if(WeakPreviewScene.IsValid())
	{
		UDebugSkelMeshComponent* PreviewMeshComponent = WeakPreviewScene.Pin()->GetPreviewMeshComponent();
		if(PreviewMeshComponent && PreviewMeshComponent->IsPreviewOn())
		{
			return PreviewMeshComponent->PreviewInstance->GetCurrentTime();
		}
	}

	return 0.0f;
}

void FAnimModel::SetScrubPosition(FFrameTime NewScrubPostion) const
{
	if(WeakPreviewScene.IsValid())
	{
		UDebugSkelMeshComponent* PreviewMeshComponent = WeakPreviewScene.Pin()->GetPreviewMeshComponent();
		if(PreviewMeshComponent && PreviewMeshComponent->IsPreviewOn())
		{
			if(PreviewMeshComponent->PreviewInstance->IsPlaying())
			{
				PreviewMeshComponent->PreviewInstance->SetPlaying(false);
			}
			
			PreviewMeshComponent->PreviewInstance->SetPosition(static_cast<float>(NewScrubPostion.AsDecimal() / static_cast<double>(GetTickResolution())));
		}
	}
}

void FAnimModel::HandleViewRangeChanged(TRange<double> InRange, EViewRangeInterpolation InInterpolation)
{
	SetViewRange(InRange);
}

void FAnimModel::SetViewRange(TRange<double> InRange)
{
	ViewRange = InRange;

	if(WorkingRange.HasLowerBound() && WorkingRange.HasUpperBound())
	{
		WorkingRange = TRange<double>::Hull(WorkingRange, ViewRange);
	}
	else
	{
		WorkingRange = ViewRange;
	}
}

void FAnimModel::HandleWorkingRangeChanged(TRange<double> InRange)
{
	WorkingRange = InRange;
}

bool FAnimModel::IsTrackSelected(const TSharedRef<const FAnimTimelineTrack>& InTrack) const
{ 
	return SelectedTracks.Find(InTrack) != nullptr;
}

void FAnimModel::ClearTrackSelection()
{
	SelectedTracks.Empty();
}

void FAnimModel::SetTrackSelected(const TSharedRef<FAnimTimelineTrack>& InTrack, bool bIsSelected)
{
	if(bIsSelected)
	{
		SelectedTracks.Add(InTrack);
	}
	else
	{
		SelectedTracks.Remove(InTrack);
	}
}

void FAnimModel::AddReferencedObjects(FReferenceCollector& Collector)
{
	EditorObjectTracker.AddReferencedObjects(Collector);
}

void FAnimModel::SelectObjects(const TArray<UObject*>& Objects)
{
	if(!bIsSelecting)
	{
		TGuardValue<bool> GuardValue(bIsSelecting, true);
		OnSelectObjects.ExecuteIfBound(Objects);

		OnHandleObjectsSelectedDelegate.Broadcast(Objects);
	}
}

UObject* FAnimModel::ShowInDetailsView(UClass* EdClass)
{
	UObject* Obj = EditorObjectTracker.GetEditorObjectForClass(EdClass);
	if(Obj != nullptr)
	{
		if(Obj->IsA(UEditorAnimBaseObj::StaticClass()))
		{
			if(!bIsSelecting)
			{
				TGuardValue<bool> GuardValue(bIsSelecting, true);

				ClearTrackSelection();

				UEditorAnimBaseObj *EdObj = Cast<UEditorAnimBaseObj>(Obj);
				InitDetailsViewEditorObject(EdObj);

				TArray<UObject*> Objects;
				Objects.Add(EdObj);
				OnSelectObjects.ExecuteIfBound(Objects);

				OnHandleObjectsSelectedDelegate.Broadcast(Objects);
			}
		}
	}
	return Obj;
}

void FAnimModel::ClearDetailsView()
{
	if(!bIsSelecting)
	{
		TGuardValue<bool> GuardValue(bIsSelecting, true);

		TArray<UObject*> Objects;
		OnSelectObjects.ExecuteIfBound(Objects);
		OnHandleObjectsSelectedDelegate.Broadcast(Objects);
	}
}

float FAnimModel::CalculateSequenceLengthOfEditorObject() const
{
	if(UAnimSequenceBase* AnimSequenceBase = GetAnimSequenceBase())
	{
		return AnimSequenceBase->GetPlayLength();
	}

	return 0.0f;
}

void FAnimModel::RecalculateSequenceLength()
{
	if(UAnimSequenceBase* AnimSequenceBase = GetAnimSequenceBase())
	{
		AnimSequenceBase->ClampNotifiesAtEndOfSequence();
	}
}

void FAnimModel::SetEditableTime(int32 TimeIndex, double Time, bool bIsDragging)
{
	EditableTimes[TimeIndex] = FMath::Clamp(Time, 0.0, (double)CalculateSequenceLengthOfEditorObject());

	OnSetEditableTime(TimeIndex, EditableTimes[TimeIndex], bIsDragging);
}

bool FAnimModel::Snap(float& InOutTime, float InSnapMargin, TArrayView<const FName> InSkippedSnapTypes, FName& OutSnapType) const
{
	double DoubleTime = InOutTime;
	bool bResult = Snap(DoubleTime, (double)InSnapMargin, InSkippedSnapTypes, OutSnapType);
	InOutTime = static_cast<float>(DoubleTime);
	return bResult;
}

bool FAnimModel::Snap(double& InOutTime, double InSnapMargin, TArrayView<const FName> InSkippedSnapTypes, FName& OutSnapType) const
{
	InSnapMargin = FMath::Max(InSnapMargin, (double)KINDA_SMALL_NUMBER);

	double ClosestDelta = DBL_MAX;
	double ClosestSnapTime = DBL_MAX;
	FName ClosestSnapType = NAME_None;

	// Check for enabled snap functions first
	for(const TPair<FName, FSnapType>& SnapTypePair : SnapTypes)
	{
		if(SnapTypePair.Value.SnapFunction != nullptr)
		{
			if(IsSnapChecked(SnapTypePair.Value.Type))
			{
				if(!InSkippedSnapTypes.Contains(SnapTypePair.Value.Type))
				{
					double SnappedTime = SnapTypePair.Value.SnapFunction(*this, InOutTime);
					if(SnappedTime != InOutTime)
					{
						double Delta = FMath::Abs(SnappedTime - InOutTime);
						if(Delta < InSnapMargin && Delta < ClosestDelta)
						{
							ClosestSnapType = SnapTypePair.Value.Type;
							ClosestDelta = Delta;
							ClosestSnapTime = SnappedTime;
						}
					}
				}
			}
		}
	}

	// Find the closest in-range enabled snap time
	for(const FSnapTime& SnapTime : SnapTimes)
	{
		double Delta = FMath::Abs(SnapTime.Time - InOutTime);
		if(Delta < InSnapMargin && Delta < ClosestDelta)
		{
			if(!InSkippedSnapTypes.Contains(SnapTime.Type))
			{
				if(const FSnapType* SnapType = SnapTypes.Find(SnapTime.Type))
				{
					if(IsSnapChecked(SnapTime.Type))
					{
						ClosestSnapType = SnapTime.Type;
						ClosestDelta = Delta;
						ClosestSnapTime = SnapTime.Time;
					}
				}
			}
		}
	}

	if(ClosestDelta != DBL_MAX)
	{
		OutSnapType = ClosestSnapType;
		InOutTime = ClosestSnapTime;
		return true;
	}

	return false;
}

void FAnimModel::ToggleSnap(FName InSnapName)
{
	if(IsSnapChecked(InSnapName))
	{
		GetMutableDefault<UPersonaOptions>()->TimelineEnabledSnaps.Remove(InSnapName);
	}
	else
	{
		GetMutableDefault<UPersonaOptions>()->TimelineEnabledSnaps.AddUnique(InSnapName);
	}
}

bool FAnimModel::IsSnapChecked(FName InSnapName) const
{
	return GetDefault<UPersonaOptions>()->TimelineEnabledSnaps.Contains(InSnapName);
}

bool FAnimModel::IsSnapAvailable(FName InSnapName) const
{
	return SnapTypes.Find(InSnapName) != nullptr;
}

void FAnimModel::BuildContextMenu(FMenuBuilder& InMenuBuilder)
{
	// Let each selected item contribute to the context menu
	TSet<FName> ExistingMenuTypes;
	for(const TSharedRef<FAnimTimelineTrack>& SelectedItem : SelectedTracks)
	{
		SelectedItem->AddToContextMenu(InMenuBuilder, ExistingMenuTypes);
	}
}

void FAnimModel::AddRootTrack(TSharedRef<FAnimTimelineTrack> InTrack)
{
	if (GetMutableDefault<UPersonaOptions>()->GetAllowedAnimationEditorTracks().PassesFilter(InTrack->GetTypeName()))
	{
		RootTracks.Add(InTrack);
	}
}

void FAnimModel::ClearRootTracks()
{
	RootTracks.Empty();
}

void FAnimModel::ForEachRootTrack(TFunctionRef<void(FAnimTimelineTrack&)> InFunction)
{
	for (TSharedRef<FAnimTimelineTrack>& Track : RootTracks)
	{
		InFunction(Track.Get());
	}
}

#undef LOCTEXT_NAMESPACE
