// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimatedRange.h"
#include "Templates/SharedPointer.h"
#include "ITimeSlider.h"
#include "PersonaDelegates.h"
#include "UObject/GCObject.h"
#include "EditorObjectsTracker.h"
#include "Containers/ArrayView.h"

class FAnimTimelineTrack;
enum class EViewRangeInterpolation;
class UAnimSequenceBase;
class IPersonaPreviewScene;
class IEditableSkeleton;
class FUICommandList;
class UEditorAnimBaseObj;

/** Model of the anim timeline's data */
class FAnimModel : public TSharedFromThis<FAnimModel>, public FGCObject
{
public:
	FAnimModel(const TSharedRef<IPersonaPreviewScene>& InPreviewScene, const TSharedRef<IEditableSkeleton>& InEditableSkeleton, const TSharedRef<FUICommandList>& InCommandList);

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	virtual ~FAnimModel() = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** Bind commands and perform any one-time initialization */
	virtual void Initialize();

	/** Get the root tracks representing the tree */
	UE_DEPRECATED(5.8, "Please use GetAllRootTracks instead.")
	const TArray<TSharedRef<FAnimTimelineTrack>>& GetRootTracks() const
	{
		return RootTracks;
	}

	/** Get the root tracks representing the tree - alias to aid deprecation */
	const TArray<TSharedRef<FAnimTimelineTrack>>& GetAllRootTracks() const
	{
		return RootTracks;
	}

	/** Get the current view range */
	FAnimatedRange GetViewRange() const;

	/** Set the current view range */
	void SetViewRange(TRange<double> InRange);

	/** Get the working range of the model's data */
	FAnimatedRange GetWorkingRange() const;

	/** Get the playback range of the model's data */
	TRange<FFrameNumber> GetPlaybackRange() const;

	/** Get the current scrub position */
	FFrameNumber GetScrubPosition() const;

	/** Get the current scrub time */
	float GetScrubTime() const;

	/** Set the current scrub position */
	void SetScrubPosition(FFrameTime NewScrubPostion) const;

	/** Handle the view range being changed */
	void HandleViewRangeChanged(TRange<double> InRange, EViewRangeInterpolation InInterpolation);

	/** Handle the working range being changed */
	void HandleWorkingRangeChanged(TRange<double> InRange);

	/** Delegate fired when tracks have changed */
	DECLARE_EVENT(FAnimModel, FOnTracksChanged)
	FOnTracksChanged& OnTracksChanged() { return OnTracksChangedDelegate; }

	/** Delegate fired when objects have been selected */
	DECLARE_EVENT_OneParam(FAnimModel, FOnHandleObjectsSelected, const TArray<UObject*>& /*InObjects*/)
	FOnHandleObjectsSelected& OnHandleObjectsSelected() { return OnHandleObjectsSelectedDelegate; }

	/** Refresh the tracks we have using our underlying asset */
	virtual void RefreshTracks() = 0;

	/** Update the displayed range if the length of the sequence could have changed */
	virtual void UpdateRange() = 0;

	/** Get the underlying anim sequence */
	virtual UAnimSequenceBase* GetAnimSequenceBase() const = 0;

	/** Get the underlying anim sequence, cast to the specified type */
	template<typename AssetType>
	AssetType* GetAsset() const
	{
		return Cast<AssetType>(GetAnimSequenceBase());
	}

	/** Get the preview scene */
	TSharedRef<IPersonaPreviewScene> GetPreviewScene() const { return WeakPreviewScene.Pin().ToSharedRef(); }

	/** Get the editable skeleton */
	TSharedRef<IEditableSkeleton> GetEditableSkeleton() const { return WeakEditableSkeleton.Pin().ToSharedRef(); }

	/** Get the command list */
	TSharedRef<FUICommandList> GetCommandList() const { return WeakCommandList.Pin().ToSharedRef(); }

	/** Get whether a track is selected */
	bool IsTrackSelected(const TSharedRef<const FAnimTimelineTrack>& InTrack) const;

	/** Clear all track selection */
	void ClearTrackSelection();

	/** Set whether a track is selected */
	void SetTrackSelected(const TSharedRef<FAnimTimelineTrack>& InTrack, bool bIsSelected);
	
	/** Creates an editor object from the given type to be used in a details panel */
	UObject* ShowInDetailsView(UClass* EdClass);

	/** 'Selects' objects and shows them in the details view */
	void SelectObjects(const TArray<UObject*>& Objects);

	/** Clears the detail view of whatever we displayed last */
	void ClearDetailsView();

	/** FGCObject interface */
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FAnimModel");
	}

	/** Recalculate sequence length after modifying */
	virtual void RecalculateSequenceLength();

	/**
	 * Calculates the sequence length of the object 
	 *  @return		New sequence length
	 */
	virtual float CalculateSequenceLengthOfEditorObject() const;

	/** Allows derived classes to init newly created editor objects */
	virtual void InitDetailsViewEditorObject(UEditorAnimBaseObj* EdObj) {}

	/** Access editable times */
	const TArray<double>& GetEditableTimes() const { return EditableTimes; }
	void SetEditableTimes(const TArray<double>& InTimes) { EditableTimes = InTimes; }

	/** Set an editable time */
	void SetEditableTime(int32 TimeIndex, double Time, bool bIsDragging);

	/** Override point for derived types to handle setting a time */
	virtual void OnSetEditableTime(int32 TimeIndex, double Time, bool bIsDragging) {}

	/** Get the framerate specified by the anim sequence */
	FFrameRate GetFrameRate() const;

	/** Get the tick resolution we are displaying at */
	int32 GetTickResolution() const;

	/** Delegate used to edit object details */
	FOnObjectsSelected OnSelectObjects;

	/** Delegate used to invoke a tab in a Persona asset editor */
	FOnInvokeTab OnInvokeTab;

	/** 
	 * Snaps a time to the currently set snap times
	 * @param	InOutTime				The time to snap
	 * @param	InSnapMargin			The margin (in seconds) to limit the snap to
	 * @param	InSkippedSnapTypes		Array of snap types that should be ignored when checking for snap
	 * @param	OutSnapType				The type of snap that was snapped to
	 * @return true if a snap occurred
	 */
	bool Snap(float& InOutTime, float InSnapMargin, TArrayView<const FName> InSkippedSnapTypes, FName& OutSnapType) const;
	bool Snap(double& InOutTime, double InSnapMargin, TArrayView<const FName> InSkippedSnapTypes, FName& OutSnapType) const;

	/** Toggle the specified snap type */
	void ToggleSnap(FName InSnapName);

	/** Check whether the specified snap is enabled */
	bool IsSnapChecked(FName InSnapName) const;

	/** Check whether the specified snap is available */
	bool IsSnapAvailable(FName InSnapName) const;

	/** Build a context menu for selected items */
	virtual void BuildContextMenu(FMenuBuilder& InMenuBuilder);

	/** Run a predicate for each root track */
	void ForEachRootTrack(TFunctionRef<void(FAnimTimelineTrack&)> InFunction);

protected:
	/** Adds a track to the root tracks collection */
	void AddRootTrack(TSharedRef<FAnimTimelineTrack> InTrack);

	/** Clears all root tracks */
	void ClearRootTracks();

protected:	
	/** Tracks used to generate a tree */
	TArray<TSharedRef<FAnimTimelineTrack>> RootTracks;

	/** Tracks that are selected */
	TSet<TSharedRef<FAnimTimelineTrack>> SelectedTracks;

	/** The range we are currently viewing */
	FAnimatedRange ViewRange;

	/** The working range of this model, encompassing the view range */
	FAnimatedRange WorkingRange;

	/** The playback range of this model for each timeframe */
	FAnimatedRange PlaybackRange;

	/** Delegate fired when tracks change */
	FOnTracksChanged OnTracksChangedDelegate;

	/** Delegate fired when selection changes */
	FOnHandleObjectsSelected OnHandleObjectsSelectedDelegate;

	/** The preview scene that this model is linked to */
	TWeakPtr<IPersonaPreviewScene> WeakPreviewScene;

	/** The editable skeleton that this model is linked to */
	TWeakPtr<IEditableSkeleton> WeakEditableSkeleton;

	/** The command list we bind to */
	TWeakPtr<FUICommandList> WeakCommandList;

	/** Tracks objects created for the details panel */
	FEditorObjectTracker EditorObjectTracker;

	/** Times that can be edited by dragging bars in the UI */
	TArray<double> EditableTimes;

	/** Struct describing the type of a snap */
	struct FSnapType
	{
		FSnapType(const FName& InType, const FText& InDisplayName, TFunction<double(const FAnimModel&, double)> InSnapFunction = nullptr)
			: Type(InType)
			, DisplayName(InDisplayName)
			, SnapFunction(InSnapFunction)
		{}

		/** Identifier for this snap type */
		FName Type;

		/** Display name for this snap type */
		FText DisplayName;

		/** Optional function to snap with */
		TFunction<double(const FAnimModel&, double)> SnapFunction;

		/** Built-in snap types */
		static const FSnapType Frames;
		static const FSnapType Notifies;
		static const FSnapType CompositeSegment;
		static const FSnapType MontageSection;
	};

	/** Struct describing a time that can be snapped to */
	struct FSnapTime
	{
		FSnapTime(const FName& InType, double InTime)
			: Type(InType)
			, Time(InTime)
		{}

		/** Type corresponding to a FSnapType */
		FName Type;

		/** The time to snap to */
		double Time;
	};

	/** Snap types for this model */
	TMap<FName, FSnapType> SnapTypes;

	/** Times that can be snapped to when dragging */
	TArray<FSnapTime> SnapTimes;

	/** Recursion guard for selection */
	bool bIsSelecting;
};
