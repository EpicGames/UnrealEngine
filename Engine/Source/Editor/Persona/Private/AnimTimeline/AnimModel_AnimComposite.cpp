// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimTimeline/AnimModel_AnimComposite.h"
#include "Animation/AnimComposite.h"
#include "AnimTimeline/AnimTimelineTrack.h"
#include "AnimTimeline/AnimTimelineTrack_CompositePanel.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "AnimPreviewInstance.h"
#include "IPersonaPreviewScene.h"

#define LOCTEXT_NAMESPACE "FAnimModel_AnimComposite"

FAnimModel_AnimComposite::FAnimModel_AnimComposite(const TSharedRef<IPersonaPreviewScene>& InPreviewScene, const TSharedRef<IEditableSkeleton>& InEditableSkeleton, const TSharedRef<FUICommandList>& InCommandList, UAnimComposite* InAnimComposite)
	: FAnimModel_AnimSequenceBase(InPreviewScene, InEditableSkeleton, InCommandList, InAnimComposite)
	, AnimComposite(InAnimComposite)
{
	SnapTypes.Add(FAnimModel::FSnapType::CompositeSegment.Type, FAnimModel::FSnapType::CompositeSegment);
}

void FAnimModel_AnimComposite::RefreshTracks()
{
	ClearTrackSelection();

	// Clear all tracks
	ClearRootTracks();

	// Add the composite root track
	if(!CompositeRoot.IsValid())
	{
		CompositeRoot = MakeShared<FAnimTimelineTrack_CompositeRoot>(SharedThis(this));
	}

	CompositeRoot->ClearChildren();
	AddRootTrack(CompositeRoot.ToSharedRef());

	TSharedRef<FAnimTimelineTrack_CompositePanel> CompositePanel = MakeShared<FAnimTimelineTrack_CompositePanel>(SharedThis(this));
	CompositeRoot->AddChild(CompositePanel);

	// Add notifies
	RefreshNotifyTracks();

	// Add curves
	RefreshCurveTracks();

	// Snaps
	RefreshSnapTimes();

	// Tell the UI to refresh
	OnTracksChangedDelegate.Broadcast();

	UpdateRange();
}

void FAnimModel_AnimComposite::RefreshSnapTimes()
{
	FAnimModel_AnimSequenceBase::RefreshSnapTimes();
	
	for(const FAnimSegment& Segment : AnimComposite->AnimationTrack.AnimSegments)
	{
		SnapTimes.Add(FSnapTime(FSnapType::CompositeSegment.Type, (double)Segment.StartPos));
		SnapTimes.Add(FSnapTime(FSnapType::CompositeSegment.Type, (double)(Segment.StartPos + Segment.AnimEndTime)));
	}
}

UAnimSequenceBase* FAnimModel_AnimComposite::GetAnimSequenceBase() const 
{
	return AnimComposite;
}

void FAnimModel_AnimComposite::RecalculateSequenceLength()
{
	// Remove Gaps and update Montage Sequence Length
	if(AnimComposite)
	{
		AnimComposite->InvalidateRecursiveAsset();

		float NewSequenceLength = CalculateSequenceLengthOfEditorObject();
		if (NewSequenceLength != AnimComposite->GetPlayLength())
		{
			ClampToEndTime(NewSequenceLength);

			AnimComposite->SetCompositeLength(NewSequenceLength);

			// Reset view if we changed length (note: has to be done after ->SetCompositeLength)!
			UpdateRange();

			UAnimPreviewInstance* PreviewInstance = (GetPreviewScene()->GetPreviewMeshComponent()) ? ToRawPtr(GetPreviewScene()->GetPreviewMeshComponent()->PreviewInstance) : nullptr;
			if (PreviewInstance)
			{
				// Re-set the position, so instance is clamped properly
				PreviewInstance->SetPosition(PreviewInstance->GetCurrentTime(), false); 
			}
		}
	}

	FAnimModel::RecalculateSequenceLength();
}

float FAnimModel_AnimComposite::CalculateSequenceLengthOfEditorObject() const
{
	return AnimComposite->AnimationTrack.GetLength();
}

#undef LOCTEXT_NAMESPACE
