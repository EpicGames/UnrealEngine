// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimTimeline/AnimModel_AnimSequenceBase.h"

class UAnimComposite;

/** Anim model for an anim composite */
class FAnimModel_AnimComposite : public FAnimModel_AnimSequenceBase
{
public:
	FAnimModel_AnimComposite(const TSharedRef<IPersonaPreviewScene>& InPreviewScene, const TSharedRef<IEditableSkeleton>& InEditableSkeleton, const TSharedRef<FUICommandList>& InCommandList, UAnimComposite* InAnimComposite);

	/** FAnimModel interface */
	virtual void RefreshTracks() override;
	virtual UAnimSequenceBase* GetAnimSequenceBase() const override;
	virtual void RecalculateSequenceLength() override;
	virtual float CalculateSequenceLengthOfEditorObject() const override;
	virtual void RefreshSnapTimes() override;

private:
	/** The anim composite we wrap */
	UAnimComposite* AnimComposite;

	/** Root track for the composite */
	TSharedPtr<FAnimTimelineTrack> CompositeRoot;
};
