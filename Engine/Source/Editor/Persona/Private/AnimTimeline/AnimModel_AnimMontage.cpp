// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimTimeline/AnimModel_AnimMontage.h"
#include "Animation/AnimMontage.h"
#include "AnimTimeline/AnimTimelineTrack.h"
#include "AnimTimeline/AnimTimelineTrack_Notifies.h"
#include "AnimTimeline/AnimTimelineTrack_TimingPanel.h"
#include "AnimTimeline/AnimTimelineTrack_MontagePanel.h"
#include "ScopedTransaction.h"
#include "Factories/AnimMontageFactory.h"
#include "Animation/EditorCompositeSection.h"
#include "IPersonaPreviewScene.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "AnimPreviewInstance.h"
#include "AnimTimeline/AnimTimelineTrack_Montage.h"
#include "SAnimMontagePanel.h"
#include "IEditableSkeleton.h"

#define LOCTEXT_NAMESPACE "FAnimModel_AnimMontage"

FAnimModel_AnimMontage::FAnimModel_AnimMontage(const TSharedRef<IPersonaPreviewScene>& InPreviewScene, const TSharedRef<IEditableSkeleton>& InEditableSkeleton, const TSharedRef<FUICommandList>& InCommandList, UAnimMontage* InAnimMontage)
	: FAnimModel_AnimSequenceBase(InPreviewScene, InEditableSkeleton, InCommandList, InAnimMontage)
	, AnimMontage(InAnimMontage)
	, bSectionTimingEnabled(false)
{
	SnapTypes.Add(FAnimModel::FSnapType::CompositeSegment.Type, FAnimModel::FSnapType::CompositeSegment);
	SnapTypes.Add(FAnimModel::FSnapType::MontageSection.Type, FAnimModel::FSnapType::MontageSection);

	// Clear display flags
	for(bool& bElementNodeDisplayFlag : TimingElementNodeDisplayFlags)
	{
		bElementNodeDisplayFlag = true;
	}
}

void FAnimModel_AnimMontage::Initialize()
{
	FAnimModel_AnimSequenceBase::Initialize();

	GetEditableSkeleton()->RegisterOnSlotsChanged(FSimpleMulticastDelegate::FDelegate::CreateSP(this, &FAnimModel_AnimMontage::RefreshTracks));
}

void FAnimModel_AnimMontage::RefreshTracks()
{
	ClearTrackSelection();

	// Clear all tracks
	ClearRootTracks();

	bool bIsChildAnimMontage = AnimMontage->HasParentAsset();

	// Add the montage root track
	if(!MontageRoot.IsValid())
	{
		MontageRoot = MakeShared<FAnimTimelineTrack_Montage>(SharedThis(this));
	}

	MontageRoot->ClearChildren();
	AddRootTrack(MontageRoot.ToSharedRef());

	// Create & add the montage panel
	MontagePanel = MakeShared<FAnimTimelineTrack_MontagePanel>(SharedThis(this));
	MontageRoot->SetMontagePanel(MontagePanel.ToSharedRef());
	MontageRoot->AddChild(MontagePanel.ToSharedRef());

	// Add the timing panel
	TimingPanel = MakeShared<FAnimTimelineTrack_TimingPanel>(SharedThis(this));
	MontageRoot->AddChild(TimingPanel.ToSharedRef());

	// Add notifies
	RefreshNotifyTracks();

	// Add curves
	RefreshCurveTracks();

	// Refresh snaps
	RefreshSnapTimes();

	// Refresh section times
	RefreshSectionTimes();

	// Tell the UI to refresh
	OnTracksChangedDelegate.Broadcast();

	UpdateRange();
}

void FAnimModel_AnimMontage::RefreshSnapTimes()
{
	FAnimModel_AnimSequenceBase::RefreshSnapTimes();

	for(const FCompositeSection& Section : AnimMontage->CompositeSections)
	{
		SnapTimes.Add(FSnapTime(FSnapType::MontageSection.Type, (double)Section.GetTime()));
	}

	for(const FSlotAnimationTrack& Slot : AnimMontage->SlotAnimTracks)
	{
		for(const FAnimSegment& Segment : Slot.AnimTrack.AnimSegments)
		{
			SnapTimes.Add(FSnapTime(FSnapType::CompositeSegment.Type, (double)Segment.StartPos));
			SnapTimes.Add(FSnapTime(FSnapType::CompositeSegment.Type, (double)(Segment.StartPos + Segment.AnimEndTime)));
		}
	}
}

void FAnimModel_AnimMontage::RefreshSectionTimes()
{
	EditableTimes.Empty();
	for(const FCompositeSection& Section : AnimMontage->CompositeSections)
	{
		EditableTimes.Add((double)Section.GetTime());
	}
}

UAnimSequenceBase* FAnimModel_AnimMontage::GetAnimSequenceBase() const 
{
	return AnimMontage;
}

float FAnimModel_AnimMontage::CalculateSequenceLengthOfEditorObject() const
{
	return AnimMontage->CalculateSequenceLength();
}

void FAnimModel_AnimMontage::OnSetEditableTime(int32 TimeIndex, double Time, bool bIsDragging)
{
	if(!bIsDragging)
	{
		if(AnimMontage && AnimMontage->CompositeSections.IsValidIndex(TimeIndex))
		{
			const FScopedTransaction Transaction(LOCTEXT("EditSection", "Edit Section Start Time"));
			AnimMontage->Modify();
	
			FCompositeSection& Section = AnimMontage->CompositeSections[TimeIndex];
			Section.SetTime(static_cast<float>(Time));
			Section.Link(AnimMontage, static_cast<float>(Time));

			SortSections();
			RefreshNotifyTriggerOffsets();
			OnMontageModified();			

			// Tell the UI to refresh
			OnTracksChangedDelegate.Broadcast();
		}
	}

	OnSectionTimeDragged.ExecuteIfBound(TimeIndex, static_cast<float>(Time), bIsDragging);
}

void FAnimModel_AnimMontage::OnMontageModified()
{
	AnimMontage->PostEditChange();
	AnimMontage->MarkPackageDirty();
}

void FAnimModel_AnimMontage::SortSections()
{
	struct FCompareSections
	{
		bool operator()( const FCompositeSection &A, const FCompositeSection &B ) const
		{
			return A.GetTime() < B.GetTime();
		}
	};
	if (AnimMontage != nullptr)
	{
		AnimMontage->CompositeSections.Sort(FCompareSections());
	}

	EnsureStartingSection();

	RefreshSectionTimes();

	OnSectionsChanged.ExecuteIfBound();
}

void FAnimModel_AnimMontage::EnsureStartingSection()
{
	if (UAnimMontageFactory::EnsureStartingSection(AnimMontage))
	{
		OnMontageModified();
	}
}

void FAnimModel_AnimMontage::RefreshNotifyTriggerOffsets()
{
	for(auto Iter = AnimMontage->Notifies.CreateIterator(); Iter; ++Iter)
	{
		FAnimNotifyEvent& Notify = (*Iter);

		// Offset for the beginning of a notify
		EAnimEventTriggerOffsets::Type PredictedOffset = AnimMontage->CalculateOffsetForNotify(Notify.GetTime());
		Notify.RefreshTriggerOffset(PredictedOffset);

		// Offset for the end of a notify state if necessary
		if(Notify.GetDuration() > 0.0f)
		{
			PredictedOffset = AnimMontage->CalculateOffsetForNotify(Notify.GetTime() + Notify.GetDuration());
			Notify.RefreshEndTriggerOffset(PredictedOffset);
		}
		else
		{
			Notify.EndTriggerTimeOffset = 0.0f;
		}
	}
}

void FAnimModel_AnimMontage::ShowSectionInDetailsView(int32 SectionIndex)
{
	UEditorCompositeSection *Obj = Cast<UEditorCompositeSection>(ShowInDetailsView(UEditorCompositeSection::StaticClass()));
	if ( Obj != nullptr )
	{
		Obj->InitFromAnim(AnimMontage, FOnAnimObjectChange::CreateSP(&MontagePanel->GetAnimMontagePanel().Get(), &SAnimMontagePanel::OnMontageChange));
		Obj->InitSection(SectionIndex);
	}
	RestartPreviewFromSection(SectionIndex);
}

void FAnimModel_AnimMontage::RecalculateSequenceLength()
{
	// Remove Gaps and update Montage Sequence Length
	if(AnimMontage)
	{
		AnimMontage->InvalidateRecursiveAsset();

		const float CurrentCalculatedLength = CalculateSequenceLengthOfEditorObject();
		if(!FMath::IsNearlyEqual(CurrentCalculatedLength, AnimMontage->GetPlayLength(), UE_KINDA_SMALL_NUMBER))
		{
			ClampToEndTime(CurrentCalculatedLength);

			RefreshSectionTimes();

			AnimMontage->SetCompositeLength(CurrentCalculatedLength);

			// Reset view if we changed length (note: has to be done after ->SetCompositeLength)!
			UpdateRange();

			UAnimPreviewInstance* PreviewInstance = (GetPreviewScene()->GetPreviewMeshComponent()) ? ToRawPtr(GetPreviewScene()->GetPreviewMeshComponent()->PreviewInstance) : nullptr;
			if (PreviewInstance)
			{
				// Re-set the position, so instance is clamped properly
				PreviewInstance->SetPosition(PreviewInstance->GetCurrentTime(), false); 
			}
		}
	}

	FAnimModel_AnimSequenceBase::RecalculateSequenceLength();
}

bool FAnimModel_AnimMontage::ClampToEndTime(float NewEndTime)
{
	float SequenceLength = AnimMontage->GetPlayLength();

	bool bClampingNeeded = (SequenceLength > 0.f && NewEndTime < SequenceLength);
	if(bClampingNeeded)
	{
		for(int32 i=0; i < AnimMontage->CompositeSections.Num(); i++)
		{
			if(AnimMontage->CompositeSections[i].GetTime() > NewEndTime)
			{
				float CurrentTime = AnimMontage->CompositeSections[i].GetTime();
				AnimMontage->CompositeSections[i].SetTime(NewEndTime);
			}
		}

		for(int32 i=0; i < AnimMontage->Notifies.Num(); i++)
		{
			FAnimNotifyEvent& Notify = AnimMontage->Notifies[i];
			float NotifyTime = Notify.GetTime();

			if(NotifyTime >= NewEndTime)
			{
				Notify.SetTime(NewEndTime);
				Notify.TriggerTimeOffset = GetTriggerTimeOffsetForType(AnimMontage->CalculateOffsetForNotify(Notify.GetTime()));
			}
		}
	}

	return bClampingNeeded;
}

void FAnimModel_AnimMontage::RestartPreviewFromSection(int32 FromSectionIdx)
{
	if (UDebugSkelMeshComponent* MeshComponent = GetPreviewScene()->GetPreviewMeshComponent())
	{
		if (UAnimPreviewInstance* Preview = MeshComponent->PreviewInstance)
		{
			Preview->MontagePreview_PreviewNormal(FromSectionIdx, Preview->IsPlaying());
		}
	}
}

bool FAnimModel_AnimMontage::IsTimingElementDisplayEnabled(ENotifyTimingElementType::Type ElementType) const
{
	return TimingElementNodeDisplayFlags[ElementType];
}

void FAnimModel_AnimMontage::ToggleTimingElementDisplayEnabled(ENotifyTimingElementType::Type ElementType)
{
	TimingElementNodeDisplayFlags[ElementType] = !TimingElementNodeDisplayFlags[ElementType];
}

bool FAnimModel_AnimMontage::IsSectionTimingDisplayEnabled() const
{
	return bSectionTimingEnabled;
}

void FAnimModel_AnimMontage::ToggleSectionTimingDisplay()
{
	bSectionTimingEnabled = !bSectionTimingEnabled;
}

void FAnimModel_AnimMontage::OnDataModelChanged(const EAnimDataModelNotifyType& NotifyType, IAnimationDataModel* Model, const FAnimDataModelNotifPayload& PayLoad)
{
	NotifyCollector.Handle(NotifyType);

	switch(NotifyType)
	{ 
	case EAnimDataModelNotifyType::CurveAdded:
	case EAnimDataModelNotifyType::CurveRemoved:
		{
			if (NotifyCollector.IsNotWithinBracket())
			{
				RefreshTracks();
			}
			break;
		}
	case EAnimDataModelNotifyType::BracketClosed:
		{
			if (NotifyCollector.IsNotWithinBracket())
			{
				RefreshTracks();
			}
			break;
		}
	}
}

#undef LOCTEXT_NAMESPACE
