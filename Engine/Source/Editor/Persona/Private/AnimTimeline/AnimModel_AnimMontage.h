// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimTimeline/AnimModel_AnimSequenceBase.h"
#include "SAnimTimingPanel.h"

class UAnimMontage;
class FAnimTimelineTrack_Montage;
class FAnimTimelineTrack_MontagePanel;
class FAnimTimelineTrack_TimingPanel;

/** Delegate fired when a section is dragged/dropped (only on drop does the time actually get set) */
DECLARE_DELEGATE_ThreeParams(FOnSectionTimeDragged, int32 /*SectionIndex*/, float /*Time*/, bool /*bIsDragging*/);

/** Anim model for an anim montage */
class FAnimModel_AnimMontage : public FAnimModel_AnimSequenceBase
{
public:
	FAnimModel_AnimMontage(const TSharedRef<IPersonaPreviewScene>& InPreviewScene, const TSharedRef<IEditableSkeleton>& InEditableSkeleton, const TSharedRef<FUICommandList>& InCommandList, UAnimMontage* InAnimMontage);

	/** FAnimModel interface */
	virtual void Initialize() override;
	virtual void RefreshTracks() override;
	virtual UAnimSequenceBase* GetAnimSequenceBase() const override;
	virtual float CalculateSequenceLengthOfEditorObject() const override;
	virtual void RecalculateSequenceLength() override;
	virtual void OnSetEditableTime(int32 TimeIndex, double Time, bool bIsDragging) override;
	virtual bool ClampToEndTime(float NewEndTime) override;
	virtual void RefreshSnapTimes() override;

	/** Refresh the UI representation of the sections */
	void RefreshSectionTimes();

	void OnMontageModified();

	void SortSections();

	void EnsureStartingSection();

	void RefreshNotifyTriggerOffsets();

	void ShowSectionInDetailsView(int32 SectionIndex);

	void RestartPreviewFromSection(int32 FromSectionIdx);

	/** Delegate fired when montage sections have changed */
	FSimpleDelegate	OnSectionsChanged;

	/** Delegate fired when a section is dragged/dropped (only on drop does the time actually get set) */
	FOnSectionTimeDragged OnSectionTimeDragged;

	/** Access the montage panel track */
	TSharedPtr<FAnimTimelineTrack_MontagePanel> GetMontagePanel() const { return MontagePanel; }

	/** Timing panel display options */
	bool IsTimingElementDisplayEnabled(ENotifyTimingElementType::Type ElementType) const;
	void ToggleTimingElementDisplayEnabled(ENotifyTimingElementType::Type ElementType);

	/** Montage section track timing options */
	bool IsSectionTimingDisplayEnabled() const;
	void ToggleSectionTimingDisplay();

protected:	
	virtual void OnDataModelChanged(const EAnimDataModelNotifyType& NotifyType, IAnimationDataModel* Model, const FAnimDataModelNotifPayload& PayLoad) override;

private:
	/** The anim montage we wrap */
	UAnimMontage* AnimMontage;

	/** Root track for the montage */
	TSharedPtr<FAnimTimelineTrack_Montage> MontageRoot;

	/** Montage panel */
	TSharedPtr<FAnimTimelineTrack_MontagePanel> MontagePanel;

	/** Timing panel */
	TSharedPtr<FAnimTimelineTrack_TimingPanel> TimingPanel;

	/** Display flags for timing panel */
	bool TimingElementNodeDisplayFlags[ENotifyTimingElementType::Max];

	/** Whether montage section timing is enabled on the section track */
	bool bSectionTimingEnabled;
};
