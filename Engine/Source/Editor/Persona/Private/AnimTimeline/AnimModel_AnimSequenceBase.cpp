// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimTimeline/AnimModel_AnimSequenceBase.h"
#include "Animation/AnimSequence.h"
#include "AnimTimeline/AnimTimelineTrack.h"
#include "AnimTimeline/AnimTimelineTrack_Notifies.h"
#include "AnimTimeline/AnimTimelineTrack_NotifiesPanel.h"
#include "AnimTimeline/AnimTimelineTrack_Curves.h"
#include "AnimTimeline/AnimTimelineTrack_Curve.h"
#include "AnimTimeline/AnimTimelineTrack_CompoundCurve.h"
#include "AnimTimeline/AnimTimelineTrack_FloatCurve.h"
#include "AnimTimeline/AnimTimelineTrack_VectorCurve.h"
#include "AnimTimeline/AnimTimelineTrack_TransformCurve.h"
#include "AnimTimeline/AnimTimelineTrack_Attributes.h"
#include "AnimTimeline/AnimTimelineTrack_PerBoneAttributes.h"
#include "AnimTimeline/AnimTimelineTrack_Attribute.h"
#include "AnimSequenceTimelineCommands.h"
#include "Framework/Commands/UICommandList.h"
#include "IAnimationEditor.h"
#include "Preferences/PersonaOptions.h"
#include "FrameNumberDisplayFormat.h"
#include "Framework/Commands/GenericCommands.h"
#include "IPersonaPreviewScene.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "AnimPreviewInstance.h"
#include "AnimTimelineClipboard.h"
#include "HAL/PlatformApplicationMisc.h"
#include "ScopedTransaction.h"
#include "Animation/AnimCurveTypes.h"

#define LOCTEXT_NAMESPACE "FAnimModel_AnimSequence"

FAnimModel_AnimSequenceBase::FAnimModel_AnimSequenceBase(const TSharedRef<IPersonaPreviewScene>& InPreviewScene, const TSharedRef<IEditableSkeleton>& InEditableSkeleton, const TSharedRef<FUICommandList>& InCommandList, UAnimSequenceBase* InAnimSequenceBase)
	: FAnimModel(InPreviewScene, InEditableSkeleton, InCommandList)
	, AnimSequenceBase(InAnimSequenceBase)
{
	SnapTypes.Add(FAnimModel::FSnapType::Frames.Type, FAnimModel::FSnapType::Frames);
	SnapTypes.Add(FAnimModel::FSnapType::Notifies.Type, FAnimModel::FSnapType::Notifies);

	UpdateRange();

	// Clear display flags
	for(bool& bElementNodeDisplayFlag : NotifiesTimingElementNodeDisplayFlags)
	{
		bElementNodeDisplayFlag = false;
	}

	AnimSequenceBase->RegisterOnNotifyChanged(UAnimSequenceBase::FOnNotifyChanged::CreateRaw(this, &FAnimModel_AnimSequenceBase::RefreshSnapTimes));	

	AnimSequenceBase->GetDataModel()->GetModifiedEvent().AddRaw(this, &FAnimModel_AnimSequenceBase::OnDataModelChanged);
}

FAnimModel_AnimSequenceBase::~FAnimModel_AnimSequenceBase()
{
	AnimSequenceBase->UnregisterOnNotifyChanged(this);
	AnimSequenceBase->GetDataModel()->GetModifiedEvent().RemoveAll(this);
}

void FAnimModel_AnimSequenceBase::Initialize()
{
	TSharedRef<FUICommandList> CommandList = WeakCommandList.Pin().ToSharedRef();

	const FAnimSequenceTimelineCommands& Commands = FAnimSequenceTimelineCommands::Get();

	CommandList->MapAction(
		FGenericCommands::Get().Rename,
		FExecuteAction::CreateLambda([this]
		{
			SelectedTracks.Array()[0]->RequestRename();
		}),
		FCanExecuteAction::CreateLambda([this]
		{
			return (SelectedTracks.Num() > 0) && (SelectedTracks.Array()[0]->CanRename());
		})
	);

	CommandList->MapAction(
		Commands.EditSelectedCurves,
		FExecuteAction::CreateSP(this, &FAnimModel_AnimSequenceBase::EditSelectedCurves),
		FCanExecuteAction::CreateSP(this, &FAnimModel_AnimSequenceBase::CanEditSelectedCurves));

	CommandList->MapAction(
		Commands.CopySelectedCurveNames,
		FExecuteAction::CreateSP(this, &FAnimModel_AnimSequenceBase::CopySelectedCurveNamesToClipboard));
	
	CommandList->MapAction(
		Commands.DisplayFrames,
		FExecuteAction::CreateSP(this, &FAnimModel_AnimSequenceBase::SetDisplayFormat, EFrameNumberDisplayFormats::Frames),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FAnimModel_AnimSequenceBase::IsDisplayFormatChecked, EFrameNumberDisplayFormats::Frames));

	CommandList->MapAction(
		Commands.DisplaySeconds,
		FExecuteAction::CreateSP(this, &FAnimModel_AnimSequenceBase::SetDisplayFormat, EFrameNumberDisplayFormats::Seconds),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FAnimModel_AnimSequenceBase::IsDisplayFormatChecked, EFrameNumberDisplayFormats::Seconds));

	CommandList->MapAction(
		Commands.DisplayPercentage,
		FExecuteAction::CreateSP(this, &FAnimModel_AnimSequenceBase::ToggleDisplayPercentage),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FAnimModel_AnimSequenceBase::IsDisplayPercentageChecked));

	CommandList->MapAction(
		Commands.DisplaySecondaryFormat,
		FExecuteAction::CreateSP(this, &FAnimModel_AnimSequenceBase::ToggleDisplaySecondary),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FAnimModel_AnimSequenceBase::IsDisplaySecondaryChecked));

	CommandList->MapAction(
		Commands.SnapToFrames,
		FExecuteAction::CreateSP(this, &FAnimModel_AnimSequenceBase::ToggleSnap, FAnimModel::FSnapType::Frames.Type),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FAnimModel_AnimSequenceBase::IsSnapChecked, FAnimModel::FSnapType::Frames.Type), 
		FIsActionButtonVisible::CreateSP(this, &FAnimModel_AnimSequenceBase::IsSnapAvailable, FAnimModel::FSnapType::Frames.Type));

	CommandList->MapAction(
		Commands.SnapToNotifies,
		FExecuteAction::CreateSP(this, &FAnimModel_AnimSequenceBase::ToggleSnap, FAnimModel::FSnapType::Notifies.Type),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FAnimModel_AnimSequenceBase::IsSnapChecked, FAnimModel::FSnapType::Notifies.Type), 
		FIsActionButtonVisible::CreateSP(this, &FAnimModel_AnimSequenceBase::IsSnapAvailable, FAnimModel::FSnapType::Notifies.Type));

	CommandList->MapAction(
		Commands.SnapToCompositeSegments,
		FExecuteAction::CreateSP(this, &FAnimModel_AnimSequenceBase::ToggleSnap, FAnimModel::FSnapType::CompositeSegment.Type),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FAnimModel_AnimSequenceBase::IsSnapChecked, FAnimModel::FSnapType::CompositeSegment.Type),
		FIsActionButtonVisible::CreateSP(this, &FAnimModel_AnimSequenceBase::IsSnapAvailable, FAnimModel::FSnapType::CompositeSegment.Type));

	CommandList->MapAction(
		Commands.SnapToMontageSections,
		FExecuteAction::CreateSP(this, &FAnimModel_AnimSequenceBase::ToggleSnap, FAnimModel::FSnapType::MontageSection.Type),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FAnimModel_AnimSequenceBase::IsSnapChecked, FAnimModel::FSnapType::MontageSection.Type),
		FIsActionButtonVisible::CreateSP(this, &FAnimModel_AnimSequenceBase::IsSnapAvailable, FAnimModel::FSnapType::MontageSection.Type));

	CommandList->MapAction(
		FGenericCommands::Get().Copy,
		FExecuteAction::CreateSP(this, &FAnimModel_AnimSequenceBase::CopyToClipboard),
		FCanExecuteAction::CreateSP(this, &FAnimModel_AnimSequenceBase::CanCopyToClipboard));

	CommandList->MapAction(
		Commands.PasteDataIntoCurve,
		FExecuteAction::CreateSP(this, &FAnimModel_AnimSequenceBase::PasteDataFromClipboardToSelectedCurve),
		FCanExecuteAction::CreateSP(this, &FAnimModel_AnimSequenceBase::CanPasteDataFromClipboardToSelectedCurve));

	CommandList->MapAction(
		FGenericCommands::Get().Paste,
		FExecuteAction::CreateSP(this, &FAnimModel_AnimSequenceBase::PasteFromClipboard),
		FCanExecuteAction::CreateSP(this, &FAnimModel_AnimSequenceBase::CanPasteFromClipboard));
	
	CommandList->MapAction(
		FGenericCommands::Get().Cut,
		FExecuteAction::CreateSP(this, &FAnimModel_AnimSequenceBase::CutToClipboard),
		FCanExecuteAction::CreateSP(this, &FAnimModel_AnimSequenceBase::CanCutToClipboard));

	CommandList->MapAction(
		FGenericCommands::Get().Delete,
		FExecuteAction::CreateSP(this, &FAnimModel_AnimSequenceBase::RemoveSelectedCurves),
		FCanExecuteAction::CreateSP(this, &FAnimModel_AnimSequenceBase::AreAnyCurvesSelected));
}


void FAnimModel_AnimSequenceBase::RefreshTracks()
{
	ClearTrackSelection();

	// Clear all tracks
	ClearRootTracks();

	// Add notifies
	RefreshNotifyTracks();

	// Add curves
	RefreshCurveTracks();

	// Add attributes
	RefreshAttributeTracks();

	// Snaps
	RefreshSnapTimes();

	// Tell the UI to refresh
	OnTracksChangedDelegate.Broadcast();

	UpdateRange();
}

UAnimSequenceBase* FAnimModel_AnimSequenceBase::GetAnimSequenceBase() const 
{
	return AnimSequenceBase;
}

void FAnimModel_AnimSequenceBase::RefreshNotifyTracks()
{
	if(!NotifyRoot.IsValid())
	{
		// Add a root track for notifies & then the main 'panel' legacy track
		NotifyRoot = MakeShared<FAnimTimelineTrack_Notifies>(SharedThis(this));
	}

	NotifyRoot->ClearChildren();
	AddRootTrack(NotifyRoot.ToSharedRef());

	if(!NotifyPanel.IsValid())
	{
		NotifyPanel = MakeShared<FAnimTimelineTrack_NotifiesPanel>(SharedThis(this));
		NotifyRoot->SetAnimNotifyPanel(NotifyPanel.ToSharedRef());
	}

	NotifyRoot->AddChild(NotifyPanel.ToSharedRef());
}

void FAnimModel_AnimSequenceBase::RefreshCurveTracks()
{
	if(!CurveRoot.IsValid())
	{
		// Add a root track for curves
		CurveRoot = MakeShared<FAnimTimelineTrack_Curves>(SharedThis(this));
	}

	CurveRoot->ClearChildren();
	AddRootTrack(CurveRoot.ToSharedRef());

	// Next add a track for each float curve
	const FAnimationCurveData& AnimationModelCurveData = AnimSequenceBase->GetDataModel()->GetCurveData();
	const UPersonaOptions* PersonaOptions = GetDefault<UPersonaOptions>();
	if (!PersonaOptions || !PersonaOptions->bUseTreeViewForAnimationCurves)
	{
		for (const FFloatCurve& FloatCurve : AnimationModelCurveData.FloatCurves)
		{
			CurveRoot->AddChild(MakeShared<FAnimTimelineTrack_FloatCurve>(&FloatCurve, SharedThis(this)));
		}
	}
	else
	{
		FAnimTimelineTrack_CompoundCurve::AddGroupedCurveTracks(AnimationModelCurveData.FloatCurves, *CurveRoot, SharedThis(this), PersonaOptions->AnimationCurveGroupingDelimiters);
	}

	UAnimSequence* AnimSequence = Cast<UAnimSequence>(AnimSequenceBase);
	if (AnimSequence)
	{
		if(!AdditiveRoot.IsValid())
		{
			// Add a root track for additive layers
			AdditiveRoot = MakeShared<FAnimTimelineTrack>(LOCTEXT("AdditiveLayerTrackList_Title", "Additive Layer Tracks"), LOCTEXT("AdditiveLayerTrackList_Tooltip", "Additive modifications to bone transforms"), SharedThis(this), true);
		}

		AdditiveRoot->ClearChildren();
		AddRootTrack(AdditiveRoot.ToSharedRef());

		// Next add a track for each transform curve
		for(const FTransformCurve& TransformCurve : AnimationModelCurveData.TransformCurves)
		{
			TSharedRef<FAnimTimelineTrack_TransformCurve> TransformCurveTrack = MakeShared<FAnimTimelineTrack_TransformCurve>(&TransformCurve, SharedThis(this));
			TransformCurveTrack->SetExpanded(false);
			AdditiveRoot->AddChild(TransformCurveTrack);

			FText TransformName = FText::FromName(TransformCurve.GetName());
			FLinearColor TransformColor = TransformCurve.GetColor();
			FLinearColor XColor = FLinearColor::Red;
			FLinearColor YColor = FLinearColor::Green;
			FLinearColor ZColor = FLinearColor::Blue;
			FText XName = LOCTEXT("VectorXTrackName", "X");
			FText YName = LOCTEXT("VectorYTrackName", "Y");
			FText ZName = LOCTEXT("VectorZTrackName", "Z");
			
			FText VectorFormat = LOCTEXT("TransformVectorFormat", "{0}.{1}");
			FText TranslationName = LOCTEXT("TransformTranslationTrackName", "Translation");
			TSharedRef<FAnimTimelineTrack_VectorCurve> TranslationCurveTrack = MakeShared<FAnimTimelineTrack_VectorCurve>(&TransformCurve.TranslationCurve, TransformCurve.GetName(), 0, ERawCurveTrackTypes::RCT_Transform, TranslationName, FText::Format(VectorFormat, TransformName, TranslationName), TransformColor, SharedThis(this));
			TranslationCurveTrack->SetExpanded(false);
			TransformCurveTrack->AddChild(TranslationCurveTrack);			

			FText ComponentFormat = LOCTEXT("TransformComponentFormat", "{0}.{1}.{2}");
			TranslationCurveTrack->AddChild(MakeShared<FAnimTimelineTrack_Curve>(&TransformCurve.TranslationCurve.FloatCurves[0], TransformCurve.GetName(), 0, ERawCurveTrackTypes::RCT_Transform, XName, FText::Format(ComponentFormat, TransformName, TranslationName, XName), XColor, XColor, SharedThis(this)));
			TranslationCurveTrack->AddChild(MakeShared<FAnimTimelineTrack_Curve>(&TransformCurve.TranslationCurve.FloatCurves[1], TransformCurve.GetName(), 1, ERawCurveTrackTypes::RCT_Transform, YName, FText::Format(ComponentFormat, TransformName, TranslationName, YName), YColor, YColor, SharedThis(this)));
			TranslationCurveTrack->AddChild(MakeShared<FAnimTimelineTrack_Curve>(&TransformCurve.TranslationCurve.FloatCurves[2], TransformCurve.GetName(), 2, ERawCurveTrackTypes::RCT_Transform, ZName, FText::Format(ComponentFormat, TransformName, TranslationName, ZName), ZColor, ZColor, SharedThis(this)));

			FText RollName = LOCTEXT("RotationRollTrackName", "Roll");
			FText PitchName = LOCTEXT("RotationPitchTrackName", "Pitch");
			FText YawName = LOCTEXT("RotationYawTrackName", "Yaw");
			FText RotationName = LOCTEXT("TransformRotationTrackName", "Rotation");
			TSharedRef<FAnimTimelineTrack_VectorCurve> RotationCurveTrack = MakeShared<FAnimTimelineTrack_VectorCurve>(&TransformCurve.RotationCurve, TransformCurve.GetName(), 3, ERawCurveTrackTypes::RCT_Transform, RotationName, FText::Format(VectorFormat, TransformName, RotationName), TransformColor, SharedThis(this));
			RotationCurveTrack->SetExpanded(false);
			TransformCurveTrack->AddChild(RotationCurveTrack);
			RotationCurveTrack->AddChild(MakeShared<FAnimTimelineTrack_Curve>(&TransformCurve.RotationCurve.FloatCurves[0], TransformCurve.GetName(), 3, ERawCurveTrackTypes::RCT_Transform, RollName, FText::Format(ComponentFormat, TransformName, RotationName, RollName), XColor, XColor, SharedThis(this)));
			RotationCurveTrack->AddChild(MakeShared<FAnimTimelineTrack_Curve>(&TransformCurve.RotationCurve.FloatCurves[1], TransformCurve.GetName(), 4, ERawCurveTrackTypes::RCT_Transform, PitchName, FText::Format(ComponentFormat, TransformName, RotationName, PitchName), YColor, YColor, SharedThis(this)));
			RotationCurveTrack->AddChild(MakeShared<FAnimTimelineTrack_Curve>(&TransformCurve.RotationCurve.FloatCurves[2], TransformCurve.GetName(), 5, ERawCurveTrackTypes::RCT_Transform, YawName, FText::Format(ComponentFormat, TransformName, RotationName, YawName), ZColor, ZColor, SharedThis(this)));

			FText ScaleName = LOCTEXT("TransformScaleTrackName", "Scale");
			TSharedRef<FAnimTimelineTrack_VectorCurve> ScaleCurveTrack = MakeShared<FAnimTimelineTrack_VectorCurve>(&TransformCurve.ScaleCurve, TransformCurve.GetName(), 6, ERawCurveTrackTypes::RCT_Transform, ScaleName, FText::Format(VectorFormat, TransformName, ScaleName), TransformColor, SharedThis(this));
			ScaleCurveTrack->SetExpanded(false);
			TransformCurveTrack->AddChild(ScaleCurveTrack);
			ScaleCurveTrack->AddChild(MakeShared<FAnimTimelineTrack_Curve>(&TransformCurve.ScaleCurve.FloatCurves[0], TransformCurve.GetName(), 6, ERawCurveTrackTypes::RCT_Transform, XName, FText::Format(ComponentFormat, TransformName, ScaleName, XName), XColor, XColor, SharedThis(this)));
			ScaleCurveTrack->AddChild(MakeShared<FAnimTimelineTrack_Curve>(&TransformCurve.ScaleCurve.FloatCurves[1], TransformCurve.GetName(), 7, ERawCurveTrackTypes::RCT_Transform, YName, FText::Format(ComponentFormat, TransformName, ScaleName, YName), YColor, YColor, SharedThis(this)));
			ScaleCurveTrack->AddChild(MakeShared<FAnimTimelineTrack_Curve>(&TransformCurve.ScaleCurve.FloatCurves[2], TransformCurve.GetName(), 8, ERawCurveTrackTypes::RCT_Transform, ZName, FText::Format(ComponentFormat, TransformName, ScaleName, ZName), ZColor, ZColor, SharedThis(this)));
		}		
	}
}

void FAnimModel_AnimSequenceBase::RefreshAttributeTracks()
{
	if (UAnimSequence* AnimSequence = Cast<UAnimSequence>(AnimSequenceBase))
	{
		if (!AttributesRoot.IsValid())
		{
			// Add a root track for attributes
			AttributesRoot = MakeShared<FAnimTimelineTrack_Attributes>(SharedThis(this));
		}

		AttributesRoot->ClearChildren();
		AddRootTrack(AttributesRoot.ToSharedRef());
			   
		TMap<FName, TSharedPtr<FAnimTimelineTrack_PerBoneAttributes>> BoneTracks;


		// Next add a track for each attribute curve
		for (const FAnimatedBoneAttribute& BoneAttribute : AnimSequence->GetDataModel()->GetAttributes())
		{
			TSharedPtr<FAnimTimelineTrack_PerBoneAttributes> BoneTrack = nullptr;

			if (BoneTracks.Contains(BoneAttribute.Identifier.GetBoneName()))
			{
				BoneTrack = BoneTracks.FindChecked(BoneAttribute.Identifier.GetBoneName());
			}
			else
			{
				TSharedRef<FAnimTimelineTrack_PerBoneAttributes> BoneTrackRef = MakeShared<FAnimTimelineTrack_PerBoneAttributes>(BoneAttribute.Identifier.GetBoneName(), SharedThis(this));
				AttributesRoot->AddChild(BoneTrackRef);

				BoneTracks.Add(BoneAttribute.Identifier.GetBoneName(), BoneTrackRef);

				BoneTrack = BoneTrackRef;

				BoneTrackRef->SetExpanded(false);
			}

			TSharedRef<FAnimTimelineTrack_Attribute> AttributeTrack = MakeShared<FAnimTimelineTrack_Attribute>(BoneAttribute, SharedThis(this));
			BoneTrack->AddChild(AttributeTrack);
		}		
	}
}

void FAnimModel_AnimSequenceBase::OnDataModelChanged(const EAnimDataModelNotifyType& NotifyType, IAnimationDataModel* Model, const FAnimDataModelNotifPayload& PayLoad)
{
	NotifyCollector.Handle(NotifyType);

	switch(NotifyType)
	{ 
		case EAnimDataModelNotifyType::CurveAdded:
		case EAnimDataModelNotifyType::CurveChanged:
		case EAnimDataModelNotifyType::CurveRemoved:
		case EAnimDataModelNotifyType::TrackAdded:
		case EAnimDataModelNotifyType::TrackChanged:
		case EAnimDataModelNotifyType::TrackRemoved:
		case EAnimDataModelNotifyType::AttributeAdded:
		case EAnimDataModelNotifyType::AttributeChanged:
		case EAnimDataModelNotifyType::AttributeRemoved:
		case EAnimDataModelNotifyType::SequenceLengthChanged:
		case EAnimDataModelNotifyType::FrameRateChanged:
		{
			if (NotifyCollector.IsNotWithinBracket())
			{
				RefreshTracks();
			}
			break;
		}
		case EAnimDataModelNotifyType::BracketClosed:
		{
			if (NotifyCollector.IsNotWithinBracket())
			{
				RefreshTracks();
			}
			break;
		}
	}
}

void FAnimModel_AnimSequenceBase::EditSelectedCurves()
{
	TArray<IAnimationEditor::FCurveEditInfo> EditCurveInfo;
	for(TSharedRef<FAnimTimelineTrack>& SelectedTrack : SelectedTracks)
	{
		if(SelectedTrack->IsA<FAnimTimelineTrack_Curve>())
		{
			TSharedRef<FAnimTimelineTrack_Curve> CurveTrack = StaticCastSharedRef<FAnimTimelineTrack_Curve>(SelectedTrack);
			const TArray<const FRichCurve*>& Curves = CurveTrack->GetCurves();
			int32 NumCurves = Curves.Num();
			for(int32 CurveIndex = 0; CurveIndex < NumCurves; ++CurveIndex)
			{
				if(CurveTrack->CanEditCurve(CurveIndex))
				{
					FText FullName = CurveTrack->GetFullCurveName(CurveIndex);
					FLinearColor Color = CurveTrack->GetCurveColor(CurveIndex);
					FName Name;
					ERawCurveTrackTypes Type;
					int32 EditCurveIndex;
					CurveTrack->GetCurveEditInfo(CurveIndex, Name, Type, EditCurveIndex);
					FSimpleDelegate OnCurveChanged = FSimpleDelegate::CreateSP(&CurveTrack.Get(), &FAnimTimelineTrack_Curve::HandleCurveChanged);
					EditCurveInfo.AddUnique(IAnimationEditor::FCurveEditInfo(FullName, Color, Name, Type, EditCurveIndex, OnCurveChanged));
				}
			}
		}
	}

	if(EditCurveInfo.Num() > 0)
	{
		OnEditCurves.ExecuteIfBound(AnimSequenceBase, EditCurveInfo, nullptr);
	}
}

bool FAnimModel_AnimSequenceBase::CanEditSelectedCurves() const
{
	for(const TSharedRef<FAnimTimelineTrack>& SelectedTrack : SelectedTracks)
	{
		if(SelectedTrack->IsA<FAnimTimelineTrack_Curve>())
		{
			TSharedRef<FAnimTimelineTrack_Curve> CurveTrack = StaticCastSharedRef<FAnimTimelineTrack_Curve>(SelectedTrack);
			const TArray<const FRichCurve*>& Curves = CurveTrack->GetCurves();
			for(int32 CurveIndex = 0; CurveIndex < Curves.Num(); ++CurveIndex)
			{
				if(CurveTrack->CanEditCurve(CurveIndex))
				{
					return true;
				}
			}
		}
	}

	return false;
}

void FAnimModel_AnimSequenceBase::RemoveSelectedCurves()
{
	IAnimationDataController& Controller = AnimSequenceBase->GetController();
	Controller.OpenBracket(LOCTEXT("CurvePanel_RemoveCurves", "Remove Curves"));

	bool bDeletedCurve = false;

	for(TSharedRef<FAnimTimelineTrack>& SelectedTrack : SelectedTracks)
	{
		if(SelectedTrack->IsA<FAnimTimelineTrack_FloatCurve>())
		{
			TSharedRef<FAnimTimelineTrack_FloatCurve> FloatCurveTrack = StaticCastSharedRef<FAnimTimelineTrack_FloatCurve>(SelectedTrack);
			FName CurveName = FloatCurveTrack->GetFName();
			
			const FAnimationCurveIdentifier FloatCurveId(CurveName, ERawCurveTrackTypes::RCT_Float);
			if (AnimSequenceBase->GetDataModel()->FindCurve(FloatCurveId))
			{
				Controller.RemoveCurve(FloatCurveId);
				bDeletedCurve = true;
			}
		}
		else if(SelectedTrack->IsA<FAnimTimelineTrack_TransformCurve>())
		{
			TSharedRef<FAnimTimelineTrack_TransformCurve> TransformCurveTrack = StaticCastSharedRef<FAnimTimelineTrack_TransformCurve>(SelectedTrack);
			FName CurveName = TransformCurveTrack->GetFName();

			const FAnimationCurveIdentifier TransformCurveId(CurveName, ERawCurveTrackTypes::RCT_Transform);
			if (AnimSequenceBase->GetDataModel()->FindTransformCurve(TransformCurveId))
			{
				Controller.RemoveCurve(TransformCurveId);
				bDeletedCurve = true;
			}
		}
		else if (SelectedTrack->IsA<FAnimTimelineTrack_CompoundCurve>())
		{
			// Lowest priority for base class delete, ERawCurveTrackTypes::RCT_Transform must take priority
			TSharedRef<FAnimTimelineTrack_CompoundCurve> CurveTrack = StaticCastSharedRef<FAnimTimelineTrack_CompoundCurve>(SelectedTrack);

			// Find all editable curves in this track
			for (const FName TrackCurveName : CurveTrack->GetCurveNames())
			{
				const FAnimationCurveIdentifier CurveId(TrackCurveName, ERawCurveTrackTypes::RCT_Float);
				if (AnimSequenceBase->GetDataModel()->FindCurve(CurveId))
				{
					Controller.RemoveCurve(CurveId);
					bDeletedCurve = true;
				}
			}
		}
	}

	Controller.CloseBracket();

	if(bDeletedCurve)
	{
		AnimSequenceBase->PostEditChange();

		if (GetPreviewScene()->GetPreviewMeshComponent()->PreviewInstance != nullptr)
		{
			GetPreviewScene()->GetPreviewMeshComponent()->PreviewInstance->RefreshCurveBoneControllers();
		}
	}
}


void FAnimModel_AnimSequenceBase::CopySelectedCurveNamesToClipboard()
{
	TArray<FString> TrackNames; 
	for(TSharedRef<FAnimTimelineTrack>& SelectedTrack : SelectedTracks)
	{
		if(SelectedTrack->IsA<FAnimTimelineTrack_FloatCurve>())
		{
			TSharedRef<FAnimTimelineTrack_FloatCurve> FloatCurveTrack = StaticCastSharedRef<FAnimTimelineTrack_FloatCurve>(SelectedTrack);
			TrackNames.Add(FloatCurveTrack->GetFName().ToString());
		}
		else if(SelectedTrack->IsA<FAnimTimelineTrack_TransformCurve>())
		{
			TSharedRef<FAnimTimelineTrack_TransformCurve> TransformCurveTrack = StaticCastSharedRef<FAnimTimelineTrack_TransformCurve>(SelectedTrack);
			TrackNames.Add(TransformCurveTrack->GetFName().ToString());
		}

	}
	if (!TrackNames.IsEmpty())
	{
		FPlatformApplicationMisc::ClipboardCopy(*FString::Join(TrackNames, TEXT("\n")));
	}
}


void FAnimModel_AnimSequenceBase::SetDisplayFormat(EFrameNumberDisplayFormats InFormat)
{
	GetMutableDefault<UPersonaOptions>()->TimelineDisplayFormat = InFormat;
}

bool FAnimModel_AnimSequenceBase::IsDisplayFormatChecked(EFrameNumberDisplayFormats InFormat) const
{
	return GetDefault<UPersonaOptions>()->TimelineDisplayFormat == InFormat;
}

void FAnimModel_AnimSequenceBase::ToggleDisplayPercentage()
{
	GetMutableDefault<UPersonaOptions>()->bTimelineDisplayPercentage = !GetDefault<UPersonaOptions>()->bTimelineDisplayPercentage;
}

bool FAnimModel_AnimSequenceBase::IsDisplayPercentageChecked() const
{
	return GetDefault<UPersonaOptions>()->bTimelineDisplayPercentage;
}

void FAnimModel_AnimSequenceBase::ToggleDisplaySecondary()
{
	GetMutableDefault<UPersonaOptions>()->bTimelineDisplayFormatSecondary = !GetDefault<UPersonaOptions>()->bTimelineDisplayFormatSecondary;
}

bool FAnimModel_AnimSequenceBase::IsDisplaySecondaryChecked() const
{
	return GetDefault<UPersonaOptions>()->bTimelineDisplayFormatSecondary;
}

bool FAnimModel_AnimSequenceBase::AreAnyCurvesSelected() const
{
	if (!SelectedTracks.IsEmpty())
	{
		for (const TSharedRef<FAnimTimelineTrack>& SelectedTrack : SelectedTracks)
		{
			if (SelectedTrack->IsA<FAnimTimelineTrack_Curve>())
			{
				return true;
			}
		}
	}

	return false;
}

void FAnimModel_AnimSequenceBase::CopyToClipboard() const
{
	if (!SelectedTracks.IsEmpty())
	{
		if (UAnimTimelineClipboardContent * ClipboardContent = UAnimTimelineClipboardContent::Create())
		{
			FAnimTimelineClipboardUtilities::CopySelectedTracksToClipboard(SelectedTracks, ClipboardContent);
			FAnimTimelineClipboardUtilities::CopyContentToClipboard(ClipboardContent);
		}
		else
		{
			UE_LOGF(LogAnimation, Warning, "Failed to get create valid clipboard object for Animation Timeline while attempting to copy data");
		}
	}
}

bool FAnimModel_AnimSequenceBase::CanCopyToClipboard()
{
	if (!SelectedTracks.IsEmpty())
	{
		for (const TSharedRef<FAnimTimelineTrack> & Track : SelectedTracks)
		{
			if (!Track->SupportsCopy())
			{
				return false;
			}
		}
	}
	else
	{
		return false;
	}
	
	return true;
}

void FAnimModel_AnimSequenceBase::PasteDataFromClipboardToSelectedCurve()
{
	if (const UAnimTimelineClipboardContent* ClipboardContent = FAnimTimelineClipboardUtilities::GetContentFromClipboard())
	{
		const FScopedTransaction Transaction(LOCTEXT("AnimSequenceBase_PasteCurveData", "Paste Data To Selected Curve"));

		// Paste data
		FAnimTimelineClipboardUtilities::OverwriteSelectedCurveDataFromClipboard(ClipboardContent, SelectedTracks, AnimSequenceBase);
	}
	else
	{
		UE_LOGF(LogAnimation, Warning, "Failed to get valid clipboard for Animation Timeline while attempting to paste data");
	}
}

bool FAnimModel_AnimSequenceBase::CanPasteDataFromClipboardToSelectedCurve()
{
	return FAnimTimelineClipboardUtilities::CanOverwriteSelectedCurveDataFromClipboard(SelectedTracks);
}

void FAnimModel_AnimSequenceBase::PasteFromClipboard()
{
	if (const UAnimTimelineClipboardContent* ClipboardContent = FAnimTimelineClipboardUtilities::GetContentFromClipboard())
	{
		const FScopedTransaction Transaction(LOCTEXT("AnimSequenceBase_PasteCurves", "Paste"));
		// Paste curves from clipboard
		FAnimTimelineClipboardUtilities::OverwriteOrAddCurvesFromClipboardContent(ClipboardContent, AnimSequenceBase);
	}
	else
	{
		UE_LOGF(LogAnimation, Warning, "Failed to get valid clipboard for Animation Timeline while attempting to paste data");
	}
}

bool FAnimModel_AnimSequenceBase::CanPasteFromClipboard()
{
	const UAnimTimelineClipboardContent* ClipboardContent = FAnimTimelineClipboardUtilities::GetContentFromClipboard();
	return ClipboardContent && !ClipboardContent->IsEmpty();
}

void FAnimModel_AnimSequenceBase::CutToClipboard()
{
	const FScopedTransaction Transaction(LOCTEXT("AnimSequenceBase_CutCurveSelection", "Cut Selection"));
	
	CopyToClipboard();
	RemoveSelectedCurves();
}

bool FAnimModel_AnimSequenceBase::CanCutToClipboard()
{
	return CanCopyToClipboard();
}

void FAnimModel_AnimSequenceBase::UpdateRange()
{
	FAnimatedRange OldPlaybackRange = PlaybackRange;

	// update playback range
	PlaybackRange = FAnimatedRange(0.0, (double)AnimSequenceBase->GetPlayLength());

	if (OldPlaybackRange != PlaybackRange)
	{
		// Update view/range if playback range changed
		SetViewRange(PlaybackRange);
	}
}

bool FAnimModel_AnimSequenceBase::IsNotifiesTimingElementDisplayEnabled(ENotifyTimingElementType::Type ElementType) const
{
	return NotifiesTimingElementNodeDisplayFlags[ElementType];
}

void FAnimModel_AnimSequenceBase::ToggleNotifiesTimingElementDisplayEnabled(ENotifyTimingElementType::Type ElementType)
{
	NotifiesTimingElementNodeDisplayFlags[ElementType] = !NotifiesTimingElementNodeDisplayFlags[ElementType];
}

bool FAnimModel_AnimSequenceBase::ClampToEndTime(float NewEndTime)
{
	float SequenceLength = AnimSequenceBase->GetPlayLength();

	//if we had a valid sequence length before and our new end time is shorter
	//then we need to clamp.
	return (SequenceLength > 0.f && NewEndTime < SequenceLength);
}

void FAnimModel_AnimSequenceBase::RefreshSnapTimes()
{
	SnapTimes.Reset();
	for(const FAnimNotifyEvent& Notify : AnimSequenceBase->Notifies)
	{
		SnapTimes.Emplace(FSnapType::Notifies.Type, (double)Notify.GetTime());
		if(Notify.NotifyStateClass != nullptr)
		{
			SnapTimes.Emplace(FSnapType::Notifies.Type, (double)(Notify.GetTime() + Notify.GetDuration()));
		}
	}
}

#undef LOCTEXT_NAMESPACE
