// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimTimeline/AnimModel.h"
#include "PersonaDelegates.h"
#include "SAnimTimingPanel.h"
#include "EditorUndoClient.h"
#include "Animation/AnimSequenceHelpers.h"
#include "Animation/AnimData/AnimDataModelNotifyCollector.h"

class UAnimSequenceBase;
class FAnimTimelineTrack_Notifies;
class FAnimTimelineTrack_Curves;
class FAnimTimelineTrack;
class FAnimTimelineTrack_NotifiesPanel;
class FAnimTimelineTrack_Attributes;
enum class EFrameNumberDisplayFormats : uint8;

/** Anim model for an anim sequence base */
class FAnimModel_AnimSequenceBase : public FAnimModel
{
public:
	FAnimModel_AnimSequenceBase(const TSharedRef<IPersonaPreviewScene>& InPreviewScene, const TSharedRef<IEditableSkeleton>& InEditableSkeleton, const TSharedRef<FUICommandList>& InCommandList, UAnimSequenceBase* InAnimSequenceBase);

	~FAnimModel_AnimSequenceBase();

	/** FAnimModel interface */
	virtual void RefreshTracks() override;
	virtual UAnimSequenceBase* GetAnimSequenceBase() const override;
	virtual void Initialize() override;
	virtual void UpdateRange() override;

	const TSharedPtr<FAnimTimelineTrack_Notifies>& GetNotifyRoot() const { return NotifyRoot; }

	/** Delegate used to edit curves */
	FOnEditCurves OnEditCurves;

	/** Notify track timing options */
	bool IsNotifiesTimingElementDisplayEnabled(ENotifyTimingElementType::Type ElementType) const;
	void ToggleNotifiesTimingElementDisplayEnabled(ENotifyTimingElementType::Type ElementType);

	/** 
	 * Clamps the sequence to the specified length 
	 * @return		Whether clamping was/is necessary
	 */
	virtual bool ClampToEndTime(float NewEndTime);
	
	/** Refresh any simple snap times */
	virtual void RefreshSnapTimes();

protected:
	/** Refresh notify tracks */
	void RefreshNotifyTracks();

	/** Refresh curve tracks */
	void RefreshCurveTracks();

	/** Refresh attribute tracks */
	void RefreshAttributeTracks();

	/** Callback for any change made to the IAnimationDataModel embedded in the AnimSequenceBase instance this represents */
	virtual void OnDataModelChanged(const EAnimDataModelNotifyType& NotifyType, IAnimationDataModel* Model, const FAnimDataModelNotifPayload& PayLoad);

private:
	/** UI handlers */
	void EditSelectedCurves();
	bool CanEditSelectedCurves() const;
	void RemoveSelectedCurves();
	void CopySelectedCurveNamesToClipboard();
	void SetDisplayFormat(EFrameNumberDisplayFormats InFormat);
	bool IsDisplayFormatChecked(EFrameNumberDisplayFormats InFormat) const;
	void ToggleDisplayPercentage();
	bool IsDisplayPercentageChecked() const;
	void ToggleDisplaySecondary();
	bool IsDisplaySecondaryChecked() const;
	bool AreAnyCurvesSelected() const;
	
	/** Copy selected curves to clipboard */
	void CopyToClipboard() const;
	bool CanCopyToClipboard();

	/** Paste curve data into selected curve. Only modifies curves, does not add any new curves. */
	void PasteDataFromClipboardToSelectedCurve();
	bool CanPasteDataFromClipboardToSelectedCurve();

	/** Paste curves from clipboard. Adds or overwrites curves (if identifiers collide) */
	void PasteFromClipboard();
	bool CanPasteFromClipboard();

	/** Cut selected curves to clipboard */
	void CutToClipboard();
	bool CanCutToClipboard();

private:
	/** The anim sequence base we wrap */
	UAnimSequenceBase* AnimSequenceBase;

	/** Root track for notifies */
	TSharedPtr<FAnimTimelineTrack_Notifies> NotifyRoot;

	/** Legacy notify panel track */
	TSharedPtr<FAnimTimelineTrack_NotifiesPanel> NotifyPanel;

	/** Root track for curves */
	TSharedPtr<FAnimTimelineTrack_Curves> CurveRoot;

	/** Root track for additive layers */
	TSharedPtr<FAnimTimelineTrack> AdditiveRoot;

	/** Root track for custom attributes */
	TSharedPtr<FAnimTimelineTrack_Attributes> AttributesRoot;

	/** Display flags for notifies track */
	bool NotifiesTimingElementNodeDisplayFlags[ENotifyTimingElementType::Max];
protected:
	UE::Anim::FAnimDataModelNotifyCollector NotifyCollector;
};
