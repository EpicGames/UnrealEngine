// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimTimelineClipboard.h"
#include "AnimTimelineTrack.h"
#include "AnimTimelineTrack_Curve.h"
#include "UnrealExporter.h"
#include "Exporters/Exporter.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Misc/StringOutputDevice.h"
#include "Factories.h"
#include "Animation/AnimData/AnimDataModel.h"
#include "Animation/Skeleton.h"
#include "Animation/AnimSequenceBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimTimelineClipboard)

FAnimationCurveIdentifier UAnimCurveBaseCopyObject::GetAnimationCurveIdentifier() const
{
	FAnimationCurveIdentifier OutAnimationCurveIdentifier;
	
	OutAnimationCurveIdentifier.Axis = Axis;
	OutAnimationCurveIdentifier.Channel = Channel;
	OutAnimationCurveIdentifier.CurveName = CurveName;
	OutAnimationCurveIdentifier.CurveType = CurveType;
	
	return OutAnimationCurveIdentifier;
}

bool UAnimTimelineClipboardContent::AreAllCurvesOfTrackType(const ERawCurveTrackTypes Type) const
{
	for (const TObjectPtr<UAnimCurveBaseCopyObject>& CurveCopyObj : Curves)
	{
		if (CurveCopyObj->CurveType != Type)
		{
			return false;
		}
	}

	return !Curves.IsEmpty();
}

bool UAnimTimelineClipboardContent::IsEmpty() const
{
	return Curves.IsEmpty();
}

UAnimTimelineClipboardContent* UAnimTimelineClipboardContent::Create()
{
	return NewObject<UAnimTimelineClipboardContent>(GetTransientPackage());
}

void FAnimTimelineClipboardUtilities::CopyContentToClipboard(UAnimTimelineClipboardContent* Content)
{
	// Clear the mark state for saving.
	UnMarkAllObjects(EObjectMark(OBJECTMARK_TagExp | OBJECTMARK_TagImp));
	
	// Export the clipboard to text.
	FStringOutputDevice Archive;
	const FExportObjectInnerContext Context;
	UExporter::ExportToOutputDevice(&Context, Content, nullptr, Archive, TEXT("copy"), 0, PPF_ExportsNotFullyQualified | PPF_Copy | PPF_Delimited, false, Content->GetOuter());
	FPlatformApplicationMisc::ClipboardCopy(*Archive);
}

class FAnimCurvesClipboardObjectTextFactory : public FCustomizableTextObjectFactory
{
public:
	
	FAnimCurvesClipboardObjectTextFactory()
		: FCustomizableTextObjectFactory(GWarn)
		, AnimationCurvesClipboard(nullptr) 
	{
	}

	UAnimTimelineClipboardContent* AnimationCurvesClipboard;

protected:

	virtual bool CanCreateClass(UClass* InObjectClass, bool& bOmitSubObjs) const override
	{
		if (InObjectClass->IsChildOf(UAnimTimelineClipboardContent::StaticClass()))
		{
			return true;
		}
		return false;
	}


	virtual void ProcessConstructedObject(UObject* CreatedObject) override
	{
		if (CreatedObject->IsA<UAnimTimelineClipboardContent>())
		{
			AnimationCurvesClipboard = CastChecked<UAnimTimelineClipboardContent>(CreatedObject);
		}
	}
};

bool FAnimTimelineClipboardUtilities::CanPasteContentToClipboard(const FString& TextToImport)
{
	const FAnimCurvesClipboardObjectTextFactory ClipboardContentFactory;
	return ClipboardContentFactory.CanCreateObjectsFromText(TextToImport);
}

const UAnimTimelineClipboardContent* FAnimTimelineClipboardUtilities::GetContentFromClipboard()
{
	// Get the text from the clipboard.
	FString ClipboardText;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardText);

	// Try to create curves clipboard content from text.
	if (CanPasteContentToClipboard(ClipboardText))
	{
		FAnimCurvesClipboardObjectTextFactory ClipboardContentFactory;
		ClipboardContentFactory.ProcessBuffer(GetTransientPackage(), RF_Transactional, ClipboardText);
		return ClipboardContentFactory.AnimationCurvesClipboard;
	}

	return nullptr;
}

UAnimCurveBaseCopyObject::UAnimCurveBaseCopyObject() : CurveName(NAME_None), CurveType(ERawCurveTrackTypes::RCT_MAX), Channel(ETransformCurveChannel::Invalid), Axis(EVectorCurveChannel::Invalid), OriginName(NAME_None)
{
}

bool FAnimTimelineClipboardUtilities::CopySelectedTracksToClipboard(const TSet<TSharedRef<FAnimTimelineTrack>>& CurveTracks, UAnimTimelineClipboardContent* ClipboardContent)
{
	check(ClipboardContent != nullptr);
	
	const int32 StartSize = ClipboardContent->Curves.Num();
	
	for (const TSharedRef<FAnimTimelineTrack>& SelectedTrack : CurveTracks)
	{
		SelectedTrack->Copy(ClipboardContent);
	}

	return StartSize != ClipboardContent->Curves.Num();
}

bool FAnimTimelineClipboardUtilities::CanOverwriteSelectedCurveDataFromClipboard(const TSet<TSharedRef<FAnimTimelineTrack>>& SelectedTracks)
{
	const UAnimTimelineClipboardContent* ClipboardContent = FAnimTimelineClipboardUtilities::GetContentFromClipboard();

	// Validate clipboard
	if (ClipboardContent && !ClipboardContent->IsEmpty())
	{
		// Exit on multiple curves selected or no curves selected
		if (SelectedTracks.Num() != 1)
		{
			return false;
		}

		const TSharedRef<FAnimTimelineTrack> Track = *SelectedTracks.CreateConstIterator();
		
		if (Track->IsA<FAnimTimelineTrack_Curve>())
		{
			const FAnimTimelineTrack_Curve & CurveTrack = Track->As<FAnimTimelineTrack_Curve>();

			// Get current track information
			FName CurveName;
			ERawCurveTrackTypes CurveType;
			int32 CurveIndex;
			CurveTrack.GetCurveEditInfo(0 /* Unused */, CurveName, CurveType, CurveIndex);
			
			// Handle special case of pasting float curve into subtrack
			const bool bIsPastingFloatTrackIntoSubtrack = (!CurveTrack.GetCurves().IsEmpty() && ClipboardContent->AreAllCurvesOfTrackType(ERawCurveTrackTypes::RCT_Float));
			
			// Allow pasting of content into current track
			const bool bDoesSelectedTrackMatchClipboardTrack = ClipboardContent->AreAllCurvesOfTrackType(CurveType);
			
			return (bIsPastingFloatTrackIntoSubtrack || bDoesSelectedTrackMatchClipboardTrack) && ClipboardContent->Curves.Num() == 1;
		}
	}
	
	// Invalid clipboard
	return false;
}

void FAnimTimelineClipboardUtilities::OverwriteSelectedCurveDataFromClipboard(const UAnimTimelineClipboardContent* ClipboardContent, const TSet<TSharedRef<FAnimTimelineTrack>>& CurveTracks, UAnimSequenceBase* InTargetSequence)
{
	check(ClipboardContent != nullptr);
	check(InTargetSequence != nullptr)
	IAnimationDataController& Controller = InTargetSequence->GetController();
	
	// TODO: Allow for multiple curve pasting. Currently only single curve selection is supported. Need to sort selected curves or create array.
	
	if (const TSharedRef<FAnimTimelineTrack>& Track = *CurveTracks.CreateConstIterator(); Track->IsA<FAnimTimelineTrack_Curve>())
	{
		IAnimationDataController::FScopedBracket ScopedBracket(Controller, NSLOCTEXT("AnimTimelineClipboardUtilities", "PasteCurveKeys_Bracket", "Paste Animation Curve Keys"));
		
		const FAnimTimelineTrack_Curve & CurveTrack = Track->As<FAnimTimelineTrack_Curve>();

		// Get selected curve track information
		FName SelectedCurveTrackName;
		ERawCurveTrackTypes SelectedCurveTrackType;	// The value is supposed to always be RCT_TRANSFORM as mentioned in FAnimTimelineTrack_Curve but this is not true.
		int32 SelectedCurveTrackIndex;
		CurveTrack.GetCurveEditInfo(0 /* Unused */, SelectedCurveTrackName, SelectedCurveTrackType, SelectedCurveTrackIndex);
		FAnimationCurveIdentifier SelectedCurveTrackIdentifier(SelectedCurveTrackName, SelectedCurveTrackType);
		
		// Special case for pasting into child tracks for vectors tracks inside a transform track
		if (!CurveTrack.GetCurves().IsEmpty() && SelectedCurveTrackType == ERawCurveTrackTypes::RCT_Transform && ClipboardContent->AreAllCurvesOfTrackType(ERawCurveTrackTypes::RCT_Float))
		{
			const int VectorFloatDataIndex = SelectedCurveTrackIndex % 3;	// 0 = X, 1 = Y, 2 = Z
			const int TransformDataIndex = SelectedCurveTrackIndex / 3;		// 0 = Translation, 1 = Rotation, 2 = Scale

			// Ensure identifier information matches
			SelectedCurveTrackIdentifier.Channel = static_cast<ETransformCurveChannel>(TransformDataIndex);

			// Update animation model data
			if (const FTransformCurve* SelectedTransform = Controller.GetModel()->FindTransformCurve(SelectedCurveTrackIdentifier))
			{
				FTransformCurve TempTransformCurve;

				// Get new transform with incoming data
				{
					TempTransformCurve.CopyCurve(*SelectedTransform);
					FVectorCurve* NewVectorCurve = TempTransformCurve.GetVectorCurveByIndex(TransformDataIndex);
					NewVectorCurve->FloatCurves[VectorFloatDataIndex] = CastChecked<UFloatCurveCopyObject>(ClipboardContent->Curves[0])->Curve.FloatCurve;
				}

				// Update transform 
				{
					TArray<float> TimeKeys;
					TArray<FTransform> TransformValues;
					TempTransformCurve.GetKeys(TimeKeys, TransformValues);
					Controller.SetTransformCurveKeys(SelectedCurveTrackIdentifier, TransformValues, TimeKeys);
				}
			}
			
			return;
		}
		
		// Update curve data for float and transform tracks
		if (ClipboardContent->AreAllCurvesOfTrackType(SelectedCurveTrackType))
		{
			if (SelectedCurveTrackType == ERawCurveTrackTypes::RCT_Float)
			{
				const UFloatCurveCopyObject* FloatCurve = Cast<UFloatCurveCopyObject>(ClipboardContent->Curves[0]);
				Controller.SetCurveKeys(SelectedCurveTrackIdentifier, FloatCurve->Curve.FloatCurve.GetConstRefOfKeys());
				Controller.SetCurveFlags(SelectedCurveTrackIdentifier, FloatCurve->Curve.GetCurveTypeFlags());
				Controller.SetCurveComment(SelectedCurveTrackIdentifier, FloatCurve->Curve.Comment);
			}
			else if (SelectedCurveTrackType == ERawCurveTrackTypes::RCT_Transform)
			{
				const UTransformCurveCopyObject* TransformCurve = Cast<UTransformCurveCopyObject>(ClipboardContent->Curves[0]);

				TArray<float> TimeKeys;
				TArray<FTransform> TransformValues;
				TransformCurve->Curve.GetKeys(TimeKeys, TransformValues); // TODO: Optimize this if possible!
				
				Controller.SetTransformCurveKeys(SelectedCurveTrackIdentifier, TransformValues, TimeKeys);
				Controller.SetCurveFlags(SelectedCurveTrackIdentifier, TransformCurve->Curve.GetCurveTypeFlags());
				Controller.SetCurveComment(SelectedCurveTrackIdentifier, TransformCurve->Curve.Comment);
			}
		}
	}
	
}

void FAnimTimelineClipboardUtilities::OverwriteOrAddCurvesFromClipboardContent(const UAnimTimelineClipboardContent* ClipboardContent, UAnimSequenceBase* InTargetSequence)
{
	check(InTargetSequence != nullptr)
	check(ClipboardContent != nullptr)
	
	USkeleton * Skeleton = InTargetSequence->GetSkeleton();
	check(Skeleton != nullptr)
	IAnimationDataController& Controller = InTargetSequence->GetController();
	
	if (ClipboardContent->Curves.Num())
	{
		IAnimationDataController::FScopedBracket ScopedBracket(Controller, NSLOCTEXT("AnimTimelineClipboardUtilities", "PasteCurves_Bracket", "Paste Animation Curves"));
	    for (const TObjectPtr<UAnimCurveBaseCopyObject>& InCurveCopyObj : ClipboardContent->Curves)
	    {
		    // Exit on types not supported by animation controller
		    if (InCurveCopyObj->CurveType != ERawCurveTrackTypes::RCT_Float && InCurveCopyObj->CurveType != ERawCurveTrackTypes::RCT_Transform)
		    {
			    UE_LOGF(LogAnimation, Warning, "Attempting to paste curve { %ls } of unsupported type { %ls } by animation controller. ", *InCurveCopyObj->CurveName.ToString(), *UEnum::GetValueAsString(InCurveCopyObj->CurveType));
			    continue;
		    }
		    
		    // Ensure we are not pasting data into the same curve we copying data from
		    if (InCurveCopyObj->OriginName.Compare(Controller.GetModelInterface().GetObject()->GetOuter()->GetFName()) == 0 && Controller.GetModel()->FindCurve(InCurveCopyObj->GetAnimationCurveIdentifier()))
		    {
			    // Do nothing and exit.
				UE_LOGF(LogAnimation, Warning, "Attempting to paste already existing curve: %ls", *InCurveCopyObj->CurveName.ToString());
		    }
		    else
		    {
			    FAnimationCurveIdentifier CurveIdentifier = InCurveCopyObj->GetAnimationCurveIdentifier();

		    	// Create curve with clipboard information if needed
		    	if (!Controller.GetModel()->FindCurve(CurveIdentifier))
		    	{
		    		Controller.AddCurve(CurveIdentifier);
		    	}
		    	
			    if (InCurveCopyObj->CurveType == ERawCurveTrackTypes::RCT_Float)
			    {
				    const UFloatCurveCopyObject* FloatCopyObject = Cast<UFloatCurveCopyObject>(InCurveCopyObj);
				    
				    Controller.SetCurveKeys(CurveIdentifier, FloatCopyObject->Curve.FloatCurve.GetConstRefOfKeys());
				    Controller.SetCurveFlags(CurveIdentifier, FloatCopyObject->Curve.GetCurveTypeFlags());
			    	Controller.SetCurveComment(CurveIdentifier, FloatCopyObject->Curve.Comment);
			    }
			    else if (InCurveCopyObj->CurveType == ERawCurveTrackTypes::RCT_Transform)
			    {
				    const UTransformCurveCopyObject* TransformCopyObject = Cast<UTransformCurveCopyObject>(InCurveCopyObj);
				    
				    TArray<float> TimeKeys;
				    TArray<FTransform> TransformValues;
				    TransformCopyObject->Curve.GetKeys(TimeKeys, TransformValues); // TODO: Optimize this if possible!
	    
				    Controller.SetTransformCurveKeys(CurveIdentifier, TransformValues, TimeKeys);
				    Controller.SetCurveFlags(CurveIdentifier, TransformCopyObject->Curve.GetCurveTypeFlags());
			    	Controller.SetCurveComment(CurveIdentifier, TransformCopyObject->Curve.Comment);
			    }
		    }
	    }
    }
}

