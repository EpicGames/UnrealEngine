// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Animation/AnimCurveTypes.h"
#include "Animation/AnimTypes.h"
#include "AnimTimelineTrack.h"
#include "Animation/AnimData/CurveIdentifier.h"
#include "AnimTimelineClipboard.generated.h"

class UAnimSequenceBase;

///////////////////////////////////////////////////////////////////////////////
// Animation Curve Copy Objects
///////////////////////////////////////////////////////////////////////////////

/** Object used to export AnimCurves to clipboard */
UCLASS(Transient)
class UAnimCurveBaseCopyObject : public UObject
{
	GENERATED_BODY()
	
public:

	UAnimCurveBaseCopyObject();

	// Store information for FAnimationCurveIdentifier.

	UPROPERTY()
	FName CurveName;
	
	UPROPERTY()
	ERawCurveTrackTypes CurveType;

	UPROPERTY()
	ETransformCurveChannel Channel;

	UPROPERTY()
	EVectorCurveChannel Axis;

	// Context information

	/** This curve's data owner name (ex. UAnimSequenceBase Name), if any. Used internally to check if the user is attempting to paste curve data into source curve. */
	UPROPERTY()
	FName OriginName;
	
	// Helper methods

	/** Get the information that identifies this curve */
	FAnimationCurveIdentifier GetAnimationCurveIdentifier() const;

	/**
	 * Create a copy object used to export  to clipboard
	 * @tparam T Curve copy object type to create
	 * @return Copy object for curve
	 */
	template<typename T>
	static T* Create()
	{
		return NewObject<T>(GetTransientPackage(), T::StaticClass(), NAME_None, RF_Transient);
	}
};

UCLASS(Transient)
class UFloatCurveCopyObject : public UAnimCurveBaseCopyObject
{
	GENERATED_BODY()
	
public:
	
	UPROPERTY()
	FFloatCurve Curve;
};

UCLASS(Transient)
class UTransformCurveCopyObject : public UAnimCurveBaseCopyObject
{
	GENERATED_BODY()
	
public:
	
	UPROPERTY()
	FTransformCurve Curve;
};

UCLASS(Transient)
class UVectorCurveCopyObject : public UAnimCurveBaseCopyObject
{
	GENERATED_BODY()
	
public:
	
	UPROPERTY()
	FVectorCurve Curve;
};

///////////////////////////////////////////////////////////////////////////////
// Animation Timeline Clipboard
///////////////////////////////////////////////////////////////////////////////

UCLASS(Transient)
class UAnimTimelineClipboardContent : public UObject
{
public:
	GENERATED_BODY()

	/** Animation Timeline Editor Curves */
	UPROPERTY(Export)
	TArray<TObjectPtr<UAnimCurveBaseCopyObject>> Curves;

	// Utility functions
	
	/** Determine if all curve store in clipboard match the given Curve Track Type. */
	bool AreAllCurvesOfTrackType(ERawCurveTrackTypes Type) const;

	/** Determine if clipboard content is empty. */
	bool IsEmpty() const;
	
	/** Create new clipboard content object  */
	static UAnimTimelineClipboardContent* Create();
};

struct FAnimTimelineClipboardUtilities
{
	// Clipboard Content

	/** Copy anim timeline clipboard content to system's clipboard */
	static void CopyContentToClipboard(UAnimTimelineClipboardContent* Content);

	/** Determine if a anim timeline clipboard content can be constructed from system's clipboard */
	static bool CanPasteContentToClipboard(const FString& TextToImport);

	/** Attempt to get the anim timeline clipboard content from system's clipboard. */
	static const UAnimTimelineClipboardContent* GetContentFromClipboard();

	// Tracks

	/**
	 * Copy all the selected tracks into clipboard content.
	 * @param CurveTracks Selected Tracks in the Animation Timeline Editor
	 * @param ClipboardContent Clipboard object where data will be copied to
	 * @return True if any data was copied
	 */
	static bool CopySelectedTracksToClipboard(const TSet<TSharedRef<FAnimTimelineTrack>>& CurveTracks, UAnimTimelineClipboardContent* ClipboardContent);

	// Curve Copying and Pasting

	/**
	 * Determine if curve data can be pasted into selected curve track data.  
	 * @param SelectedTracks Selected Tracks in the Animation Timeline Editor
	 * @return Whether pasting is possible
	 */
	static bool CanOverwriteSelectedCurveDataFromClipboard(const TSet<TSharedRef<FAnimTimelineTrack>>& SelectedTracks);

	/**
	 * Overwrite curve data of the selected curve track with data from the clipboard without changing selected curve identifier.
	 * @param ClipboardContent Clipboard containing the curve(s) to be pasted
	 * @param CurveTracks Selected Tracks in the Animation Timeline Editor
	 * @param InTargetSequence Base Animation Sequence that will be modified
	 */
	static void OverwriteSelectedCurveDataFromClipboard(const UAnimTimelineClipboardContent* ClipboardContent, const TSet<TSharedRef<FAnimTimelineTrack>>& CurveTracks, UAnimSequenceBase* InTargetSequence);

	/**
	 * Add or overwrite, if identifiers collide, all the curves from the clipboard object to the given Skeleton's and Controller's UAnimSequenceBase.
	 * @param ClipboardContent Clipboard containing the curves to be pasted
	 * @param InTargetSequence Base Animation Sequence that will be modified
	 */
	static void OverwriteOrAddCurvesFromClipboardContent(const UAnimTimelineClipboardContent* ClipboardContent, UAnimSequenceBase* InTargetSequence);
};