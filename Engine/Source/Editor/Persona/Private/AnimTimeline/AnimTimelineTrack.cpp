// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimTimeline/AnimTimelineTrack.h"
#include "AnimTimeline/AnimModel.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SCheckBox.h"
#include "Styling/AppStyle.h"
#include "Widgets/SOverlay.h"
#include "Preferences/PersonaOptions.h"
#include "Animation/AnimSequenceBase.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Views/SExpanderArrow.h"
#include "Widgets/Views/STableViewBase.h"
#include "AnimTimeline/SAnimOutlinerItem.h"
 
#define LOCTEXT_NAMESPACE "FAnimTimelineTrack"

const float FAnimTimelineTrack::OutlinerRightPadding = 8.0f;

ANIMTIMELINE_IMPLEMENT_TRACK(FAnimTimelineTrack);

FText FAnimTimelineTrack::GetLabel() const
{
	return DisplayName;
}

FText FAnimTimelineTrack::GetToolTipText() const
{
	return ToolTipText;
}

bool FAnimTimelineTrack::Traverse_ChildFirst(const TFunctionRef<bool(FAnimTimelineTrack&)>& InPredicate, bool bIncludeThisTrack)
{
	for (TSharedRef<FAnimTimelineTrack>& Child : Children)
	{
		if (!Child->Traverse_ChildFirst(InPredicate, true))
		{
			return false;
		}
	}

	return bIncludeThisTrack ? InPredicate(*this) : true;
}

bool FAnimTimelineTrack::Traverse_ParentFirst(const TFunctionRef<bool(FAnimTimelineTrack&)>& InPredicate, bool bIncludeThisTrack)
{
	if (bIncludeThisTrack && !InPredicate(*this))
	{
		return false;
	}

	for (TSharedRef<FAnimTimelineTrack>& Child : Children)
	{
		if (!Child->Traverse_ParentFirst(InPredicate, true))
		{
			return false;
		}
	}

	return true;
}

bool FAnimTimelineTrack::TraverseVisible_ChildFirst(const TFunctionRef<bool(FAnimTimelineTrack&)>& InPredicate, bool bIncludeThisTrack)
{
	// If the item is not expanded, its children ain't visible
	if (IsExpanded())
	{
		for (TSharedRef<FAnimTimelineTrack>& Child : Children)
		{
			if (Child->IsVisible() && !Child->TraverseVisible_ChildFirst(InPredicate, true))
			{
				return false;
			}
		}
	}

	if (bIncludeThisTrack && IsVisible())
	{
		return InPredicate(*this);
	}

	// Continue iterating regardless of visibility
	return true;
}

bool FAnimTimelineTrack::TraverseVisible_ParentFirst(const TFunctionRef<bool(FAnimTimelineTrack&)>& InPredicate, bool bIncludeThisTrack)
{
	if (bIncludeThisTrack && IsVisible() && !InPredicate(*this))
	{
		return false;
	}

	// If the item is not expanded, its children ain't visible
	if (IsExpanded())
	{
		for (TSharedRef<FAnimTimelineTrack>& Child : Children)
		{
			if (Child->IsVisible() && !Child->TraverseVisible_ParentFirst(InPredicate, true))
			{
				return false;
			}
		}
	}

	return true;
}

TSharedRef<SWidget> FAnimTimelineTrack::GenerateContainerWidgetForOutliner(const TSharedRef<SAnimOutlinerItem>& InRow)
{
	TSharedPtr<SBorder> OuterBorder;
	TSharedPtr<SHorizontalBox> InnerHorizontalBox;
	TSharedRef<SWidget> Widget = GenerateStandardOutlinerWidget(InRow, true, OuterBorder, InnerHorizontalBox);

	if(bIsHeaderTrack)
	{
		OuterBorder->SetBorderBackgroundColor(FAppStyle::GetColor("AnimTimeline.Outliner.HeaderColor"));
	}

	return Widget;
}

TSharedRef<SWidget> FAnimTimelineTrack::GenerateStandardOutlinerWidget(const TSharedRef<SAnimOutlinerItem>& InRow, bool bWithLabelText, TSharedPtr<SBorder>& OutOuterBorder, TSharedPtr<SHorizontalBox>& OutInnerHorizontalBox)
{
	TSharedRef<SWidget> Widget =
		SAssignNew(OutOuterBorder, SBorder)
		.ToolTipText(this, &FAnimTimelineTrack::GetToolTipText)
		.BorderImage(FAppStyle::GetBrush("Sequencer.Section.BackgroundTint"))
		.BorderBackgroundColor(FAppStyle::GetColor("AnimTimeline.Outliner.ItemColor"))
		[
			SAssignNew(OutInnerHorizontalBox, SHorizontalBox)
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(4.0f, 1.0f)
			[
				SNew(SExpanderArrow, InRow)
			]
		];

	if(bWithLabelText)
	{
		OutInnerHorizontalBox->AddSlot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			.Padding(2.0f, 1.0f)
			.FillWidth(1.0f)
			[
				SNew(STextBlock)
				.TextStyle(&FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("AnimTimeline.Outliner.Label"))
				.Text(this, &FAnimTimelineTrack::GetLabel)
				.HighlightText(InRow->GetHighlightText())
			];
	}

	return Widget;
}

TSharedRef<SWidget> FAnimTimelineTrack::GenerateContainerWidgetForTimeline()
{
	return SNullWidget::NullWidget;
}

void FAnimTimelineTrack::AddToContextMenu(FMenuBuilder& InMenuBuilder, TSet<FName>& InOutExistingMenuTypes) const
{

}

float FAnimTimelineTrack::GetMaxInput() const
{
	return GetModel()->GetAnimSequenceBase()->GetPlayLength(); 
}

float FAnimTimelineTrack::GetViewMinInput() const
{
	return static_cast<float>(GetModel()->GetViewRange().GetLowerBoundValue());
}

float FAnimTimelineTrack::GetViewMaxInput() const
{
	return static_cast<float>(GetModel()->GetViewRange().GetUpperBoundValue());
}

float FAnimTimelineTrack::GetScrubValue() const
{
	const FFrameRate FrameRate = GetModel()->GetFrameRate();
	const int64 Resolution = FMath::RoundToInt(static_cast<double>(GetDefault<UPersonaOptions>()->TimelineScrubSnapValue) * FrameRate.AsDecimal());
	return static_cast<float>(static_cast<double>(GetModel()->GetScrubPosition().Value) / static_cast<double>(Resolution));
}

void FAnimTimelineTrack::SelectObjects(const TArray<UObject*>& SelectedItems)
{
	GetModel()->SelectObjects(SelectedItems);
}

void FAnimTimelineTrack::OnSetInputViewRange(float ViewMin, float ViewMax)
{
	GetModel()->SetViewRange(TRange<double>(ViewMin, ViewMax));
}

void FAnimTimelineTrack::AddChild(const TSharedRef<FAnimTimelineTrack>& InChild)
{
	if (GetMutableDefault<UPersonaOptions>()->GetAllowedAnimationEditorTracks().PassesFilter(InChild->GetTypeName()))
	{
		Children.Add(InChild); 
	}
}

#undef LOCTEXT_NAMESPACE
