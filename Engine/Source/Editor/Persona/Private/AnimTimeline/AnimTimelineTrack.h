// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnimTimeline/AnimModel.h"

class SAnimOutlinerItem;
class SWidget;
enum class ECheckBoxState : uint8;
struct EVisibility;
class FAnimModel;
class FMenuBuilder;
class SBorder;
class SHorizontalBox;
class UAnimTimelineClipboardContent;

/**
 * Structure used to define padding for a track
 */
struct FAnimTrackPadding
{
	FAnimTrackPadding(float InUniform) 
		: Top(InUniform)
		, Bottom(InUniform) {}

	FAnimTrackPadding(float InTop, float InBottom) 
		: Top(InTop)
		, Bottom(InBottom) {}

	/** @return The sum total of the separate padding values */
	float Combined() const
	{
		return Top + Bottom;
	}

	/** Padding to be applied to the top of the track */
	float Top;

	/** Padding to be applied to the bottom of the track */
	float Bottom;
};

// Simple RTTI implementation for tracks
#define ANIMTIMELINE_DECLARE_BASE_TRACK(BaseClassName) \
	public: \
		static const FName& GetStaticTypeName() { return BaseClassName::TypeName; } \
		virtual const FName& GetTypeName() const { return BaseClassName::GetStaticTypeName(); } \
		virtual bool IsKindOf(const FName& InTypeName) const { return InTypeName == BaseClassName::GetStaticTypeName(); } \
		template<typename Type> bool IsA() const { return IsKindOf(Type::GetStaticTypeName()); } \
		template<typename Type> const Type& As() const { return *static_cast<const Type*>(this); } \
		template<typename Type> Type& As() { return *static_cast<Type*>(this); } \
	private: \
		static const FName TypeName;
 
#define ANIMTIMELINE_DECLARE_TRACK(ClassName, BaseClassName) \
	public: \
		static const FName& GetStaticTypeName() { return ClassName::TypeName; } \
		virtual const FName& GetTypeName() const override { return ClassName::GetStaticTypeName(); } \
		virtual bool IsKindOf(const FName& InTypeName) const override { return InTypeName == ClassName::GetStaticTypeName() || BaseClassName::IsKindOf(InTypeName); } \
	private: \
		static const FName TypeName;
 

#define ANIMTIMELINE_IMPLEMENT_TRACK(ClassName) \
	const FName ClassName::TypeName = TEXT(#ClassName);

/** Track to be displayed by the anim timeline */
class FAnimTimelineTrack : public TSharedFromThis<FAnimTimelineTrack>
{
	ANIMTIMELINE_DECLARE_BASE_TRACK(FAnimTimelineTrack);

public:
	static const float OutlinerRightPadding;

	FAnimTimelineTrack(const FText& InDisplayName, const FText& InToolTipText, const TSharedPtr<FAnimModel>& InModel, bool bInIsHeaderTrack = false)
		: Model(InModel)
		, DisplayName(InDisplayName)
		, ToolTipText(InToolTipText)
		, Padding(0.0f)
		, Height(24.0f)
		, bIsHovered(false)
		, bIsVisible(true)
		, bIsExpanded(true)
		, bIsHeaderTrack(bInIsHeaderTrack)
	{
	}

	virtual ~FAnimTimelineTrack() = default;

	/** Get the children of this object */
	const TArray<TSharedRef<FAnimTimelineTrack>>& GetChildren() { return Children; }

	/** Add a child track */
	void AddChild(const TSharedRef<FAnimTimelineTrack>& InChild);

	/** Clear all child tracks */
	void ClearChildren() { Children.Empty(); }

	/** Get the label to display */
	virtual FText GetLabel() const;

	/** Get the details text to display */
	virtual FText GetToolTipText() const;

	/** Get the parent data model for this track */
	TSharedRef<FAnimModel> GetModel() const { return Model.Pin().ToSharedRef(); }

	/**
	 * Iterate this entire track tree, child first.
	 *
	 * @param 	InPredicate			Predicate to call for each track, returning whether to continue iteration or not
	 * @param 	bIncludeThisTrack	Whether to include this track in the iteration, or just children
	 * @return  true where the client prematurely exited the iteration, false otherwise
	 */
	bool Traverse_ChildFirst(const TFunctionRef<bool(FAnimTimelineTrack&)>& InPredicate, bool bIncludeThisTrack = true);

	/**
	 * Iterate this entire track tree, parent first.
	 *
	 * @param 	InPredicate			Predicate to call for each track, returning whether to continue iteration or not
	 * @param 	bIncludeThisTrack	Whether to include this track in the iteration, or just children
	 * @return  true where the client prematurely exited the iteration, false otherwise
	 */
	bool Traverse_ParentFirst(const TFunctionRef<bool(FAnimTimelineTrack&)>& InPredicate, bool bIncludeThisTrack = true);

	/**
	 * Iterate any visible portions of this track's sub-tree, child first.
	 *
	 * @param 	InPredicate			Predicate to call for each track, returning whether to continue iteration or not
	 * @param 	bIncludeThisTrack	Whether to include this track in the iteration, or just children
	 * @return  true where the client prematurely exited the iteration, false otherwise
	 */
	bool TraverseVisible_ChildFirst(const TFunctionRef<bool(FAnimTimelineTrack&)>& InPredicate, bool bIncludeThisTrack = true);

	/**
	 * Iterate any visible portions of this track's sub-tree, parent first.
	 *
	 * @param 	InPredicate			Predicate to call for each track, returning whether to continue iteration or not
	 * @param 	bIncludeThisTrack	Whether to include this track in the iteration, or just children
	 * @return  true where the client prematurely exited the iteration, false otherwise
	 */
	bool TraverseVisible_ParentFirst(const TFunctionRef<bool(FAnimTimelineTrack&)>& InPredicate, bool bIncludeThisTrack = true);

	/** Generate a widget for the outliner for this track */
	virtual TSharedRef<SWidget> GenerateContainerWidgetForOutliner(const TSharedRef<SAnimOutlinerItem>& InRow);

	/** Generate a widget for the outliner for this track */
	virtual TSharedRef<SWidget> GenerateContainerWidgetForTimeline();

	/** Add items to the context menu */
	virtual void AddToContextMenu(FMenuBuilder& InMenuBuilder, TSet<FName>& InOutExistingMenuTypes) const;

	/** Get the height of this track */
	virtual float GetHeight() const { return Height; }

	/** Set the height of this track */
	void SetHeight(float InHeight) { Height = InHeight; }

	/** Get the padding of this track */
	const FAnimTrackPadding& GetPadding() const { return Padding; }

	/** Get whether this track is hovered */
	bool IsHovered() const { return bIsHovered; }

	/** Set whether this track is hovered */
	void SetHovered(bool bInIsHovered) { bIsHovered = bInIsHovered; }

	/** Get whether this track is visible */
	bool IsVisible() const { return bIsVisible; }

	/** Set whether this track is visible */
	void SetVisible(bool bInIsVisible) { bIsVisible = bInIsVisible; }

	/** Get whether this track is expanded */
	bool IsExpanded() const { return bIsExpanded; }

	/** Set whether this track is expanded */
	void SetExpanded(bool bInIsExpanded) { bIsExpanded = bInIsExpanded; }

	/** Get whether this track supports selection in the outliner */
	virtual bool SupportsSelection() const { return false; }

	/** Get whether this track supports filtering in the outliner */
	virtual bool SupportsFiltering() const { return true; }

	/** Get whether this track supports copying to the clipboard */
	virtual bool SupportsCopy() const { return false; }

	/** Copy track information to clipboard */
	virtual void Copy(UAnimTimelineClipboardContent* InOutClipboard) const {};
	
	/** Get whether this track can be renamed */
	virtual bool CanRename() const { return false; }

	/** Request this track be renamed */
	virtual void RequestRename() {}

protected:
	/** Generate an outliner widget */
	TSharedRef<SWidget> GenerateStandardOutlinerWidget(const TSharedRef<SAnimOutlinerItem>& InRow, bool bWithLabelText, TSharedPtr<SBorder>& OutOuterBorder, TSharedPtr<SHorizontalBox>& OutInnerHorizontalBox);

protected:
	float GetMinInput() const { return 0.0f; }
	float GetMaxInput() const;
	float GetViewMinInput() const;
	float GetViewMaxInput() const;
	float GetScrubValue() const;
	void SelectObjects(const TArray<UObject*>& SelectedItems);
	void OnSetInputViewRange(float ViewMin, float ViewMax);

protected:
	/** Child objects */
	TArray<TSharedRef<FAnimTimelineTrack>> Children;

	/** The parent model */
	TWeakPtr<FAnimModel> Model;

	/** Display name to use */
	FText DisplayName;

	/** Tooltip text to use */
	FText ToolTipText;

	/** The padding of the track */
	FAnimTrackPadding Padding;

	/** The height of the track */
	float Height;

	/** Whether this track is hovered */
	bool bIsHovered : 1;

	/** Whether this track is visible */
	bool bIsVisible : 1;

	/** Whether this track is expanded */
	bool bIsExpanded : 1;

	/** Whether this is a header track */
	bool bIsHeaderTrack : 1;
};
