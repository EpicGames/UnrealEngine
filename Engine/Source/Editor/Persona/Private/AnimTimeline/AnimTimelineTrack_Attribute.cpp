// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimTimeline/AnimTimelineTrack_Attribute.h"
#include "AnimTimeline/SAnimOutlinerItem.h"
#include "Animation/AnimData/IAnimationDataModel.h"

#define LOCTEXT_NAMESPACE "FAnimTimelineTrack_CustomBoneAttributes"

ANIMTIMELINE_IMPLEMENT_TRACK(FAnimTimelineTrack_Attribute);

FAnimTimelineTrack_Attribute::FAnimTimelineTrack_Attribute(const FAnimatedBoneAttribute& InAttribute, const TSharedRef<FAnimModel>& InModel)
	: FAnimTimelineTrack(LOCTEXT("AttributeTrackLabel", "Attribute"), LOCTEXT("AttributeToolTip", "Attribute contained in this asset"), InModel), Attribute(InAttribute)
{
}

TSharedRef<SWidget> FAnimTimelineTrack_Attribute::GenerateContainerWidgetForOutliner(const TSharedRef<SAnimOutlinerItem>& InRow)
{
	TSharedPtr<SBorder> OuterBorder;
	TSharedPtr<SHorizontalBox> InnerHorizontalBox;
	OutlinerWidget = GenerateStandardOutlinerWidget(InRow, false, OuterBorder, InnerHorizontalBox);
	
	InnerHorizontalBox->AddSlot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		.Padding(2.0f, 1.0f)
		.AutoWidth()
		[
			SNew(STextBlock)
			.Text(this, &FAnimTimelineTrack_Attribute::GetLabel)
			.HighlightText(InRow->GetHighlightText())
		];

	return OutlinerWidget.ToSharedRef();
}

FText FAnimTimelineTrack_Attribute::GetLabel() const
{
	return FText::FromName(Attribute.Identifier.GetName());
}

FText FAnimTimelineTrack_Attribute::GetToolTipText() const
{
	return FText::Format(LOCTEXT("AttributeTooltipFormat", "Attribute: {0}\nType: {1}\nNumber of Keys: {2}"), FText::FromName(Attribute.Identifier.GetName()), FText::FromName(Attribute.Identifier.GetType()->GetFName()), FText::AsNumber(Attribute.Curve.GetNumKeys()));
}

#undef LOCTEXT_NAMESPACE
