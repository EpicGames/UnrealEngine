// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimTimeline/AnimTimelineTrack.h"

struct FAnimatedBoneAttribute;

/** Animation timeline track inserted for each animated attribute (inserted as child of FAnimTimelineTrack_PerBoneAttributes) */
class FAnimTimelineTrack_Attribute : public FAnimTimelineTrack
{
	ANIMTIMELINE_DECLARE_TRACK(FAnimTimelineTrack_Attribute, FAnimTimelineTrack);

public:
	FAnimTimelineTrack_Attribute(const FAnimatedBoneAttribute& InAttribute, const TSharedRef<FAnimModel>& InModel);

	/** FAnimTimelineTrack interface */
	virtual TSharedRef<SWidget> GenerateContainerWidgetForOutliner(const TSharedRef<SAnimOutlinerItem>& InRow) override;
	virtual FText GetLabel() const override;
	virtual FText GetToolTipText() const override;
	   	   
private:
	const FAnimatedBoneAttribute& Attribute;
	TSharedPtr<SWidget> OutlinerWidget;
};
