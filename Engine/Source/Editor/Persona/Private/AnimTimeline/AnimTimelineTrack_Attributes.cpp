// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimTimeline/AnimTimelineTrack_Attributes.h"
#include "PersonaUtils.h"
#include "Widgets/SBoxPanel.h"
#include "AnimSequenceTimelineCommands.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Animation/AnimSequence.h"
#include "Widgets/Input/STextEntryPopup.h"
#include "Framework/Application/SlateApplication.h"
#include "ScopedTransaction.h"
#include "AnimTimeline/SAnimOutlinerItem.h"

#define LOCTEXT_NAMESPACE "FAnimTimelineTrack_Attributes"

ANIMTIMELINE_IMPLEMENT_TRACK(FAnimTimelineTrack_Attributes);

FAnimTimelineTrack_Attributes::FAnimTimelineTrack_Attributes(const TSharedRef<FAnimModel>& InModel)
	: FAnimTimelineTrack(LOCTEXT("AttributesRootTrackLabel", "Attributes"), LOCTEXT("AttributesRootToolTip", "Animated (per bone) Attribute data contained in this asset"), InModel)
{
}

TSharedRef<SWidget> FAnimTimelineTrack_Attributes::GenerateContainerWidgetForOutliner(const TSharedRef<SAnimOutlinerItem>& InRow)
{
	TSharedPtr<SBorder> OuterBorder;
	TSharedPtr<SHorizontalBox> InnerHorizontalBox;
	OutlinerWidget = GenerateStandardOutlinerWidget(InRow, false, OuterBorder, InnerHorizontalBox);

	OuterBorder->SetBorderBackgroundColor(FAppStyle::GetColor("AnimTimeline.Outliner.HeaderColor"));

	InnerHorizontalBox->AddSlot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		.Padding(2.0f, 1.0f)
		.AutoWidth()
		[
			SNew(STextBlock)
			.TextStyle(&FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("AnimTimeline.Outliner.Label"))
			.Text(this, &FAnimTimelineTrack_Attributes::GetLabel)
			.HighlightText(InRow->GetHighlightText())
		];

	return OutlinerWidget.ToSharedRef();
}

#undef LOCTEXT_NAMESPACE
