// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimTimeline/AnimTimelineTrack.h"

/** Root-level animation timeline track under which per-bone animated attributes are inserted */
class FAnimTimelineTrack_Attributes : public FAnimTimelineTrack
{
	ANIMTIMELINE_DECLARE_TRACK(FAnimTimelineTrack_Attributes, FAnimTimelineTrack);

public:
	FAnimTimelineTrack_Attributes(const TSharedRef<FAnimModel>& InModel);

	/** FAnimTimelineTrack interface */
	virtual TSharedRef<SWidget> GenerateContainerWidgetForOutliner(const TSharedRef<SAnimOutlinerItem>& InRow) override;

private:
	TSharedPtr<SWidget> OutlinerWidget;
};
