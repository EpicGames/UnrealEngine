// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimTimeline/AnimTimelineTrack_CompositePanel.h"
#include "SAnimCompositePanel.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Layout/SBox.h"
#include "SequencerUtilities.h"
#include "AnimSequenceTimelineCommands.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "AnimTimeline/AnimTimelineTrack_Notifies.h"
#include "ScopedTransaction.h"
#include "Widgets/Views/SExpanderArrow.h"
#include "Widgets/Layout/SBorder.h"
#include "AnimTimeline/SAnimOutlinerItem.h"
#include "Animation/AnimComposite.h"

#define LOCTEXT_NAMESPACE "FAnimTimelineTrack_CompositePanel"

ANIMTIMELINE_IMPLEMENT_TRACK(FAnimTimelineTrack_CompositeRoot);
ANIMTIMELINE_IMPLEMENT_TRACK(FAnimTimelineTrack_CompositePanel);

FAnimTimelineTrack_CompositeRoot::FAnimTimelineTrack_CompositeRoot(const TSharedPtr<FAnimModel>& InModel)
	: FAnimTimelineTrack(LOCTEXT("CompositeTitle", "Composite"), LOCTEXT("CompositeTooltip", "Composite animation track"), InModel, true)
{
}

FAnimTimelineTrack_CompositePanel::FAnimTimelineTrack_CompositePanel(const TSharedRef<FAnimModel>& InModel)
	: FAnimTimelineTrack(LOCTEXT("TrackTitle", "Composite"), LOCTEXT("TrackTooltip", "Composite sections"), InModel)
{
	SetHeight(48.0f);
}

TSharedRef<SWidget> FAnimTimelineTrack_CompositePanel::GenerateContainerWidgetForTimeline()
{
	return SAssignNew(AnimCompositePanel, SAnimCompositePanel, GetModel())
		.Composite(CastChecked<UAnimComposite>(GetModel()->GetAnimSequenceBase()))
		.InputMin(this, &FAnimTimelineTrack_CompositePanel::GetMinInput)
		.InputMax(this, &FAnimTimelineTrack_CompositePanel::GetMaxInput)
		.ViewInputMin(this, &FAnimTimelineTrack_CompositePanel::GetViewMinInput)
		.ViewInputMax(this, &FAnimTimelineTrack_CompositePanel::GetViewMaxInput)
		.OnSetInputViewRange(this, &FAnimTimelineTrack_CompositePanel::OnSetInputViewRange);
}

#undef LOCTEXT_NAMESPACE
