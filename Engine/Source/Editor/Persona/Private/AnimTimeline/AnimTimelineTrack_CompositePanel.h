// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimTimeline/AnimTimelineTrack.h"

class SAnimCompositePanel;
class SVerticalBox;

class FAnimTimelineTrack_CompositeRoot : public FAnimTimelineTrack
{
	ANIMTIMELINE_DECLARE_TRACK(FAnimTimelineTrack_CompositeRoot, FAnimTimelineTrack);

public:
	FAnimTimelineTrack_CompositeRoot(const TSharedPtr<FAnimModel>& InModel);
};

/** A timeline track that re-uses the legacy panel widget to display anim composite sections */
class FAnimTimelineTrack_CompositePanel : public FAnimTimelineTrack
{
	ANIMTIMELINE_DECLARE_TRACK(FAnimTimelineTrack_CompositePanel, FAnimTimelineTrack);

public:
	FAnimTimelineTrack_CompositePanel(const TSharedRef<FAnimModel>& InModel);

	/** FAnimTimelineTrack interface */
	virtual TSharedRef<SWidget> GenerateContainerWidgetForTimeline() override;
	virtual bool SupportsFiltering() const override { return false; }

	TSharedPtr<SAnimCompositePanel> GetAnimCompositePanel() { return AnimCompositePanel; }

private:
	/** The legacy composite panel */
	TSharedPtr<SAnimCompositePanel> AnimCompositePanel;
};
