// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimTimelineTrack_CompoundCurve.h"
#include "AnimTimelineTrack_FloatCurve.h"
#include "Containers/BitArray.h"

ANIMTIMELINE_IMPLEMENT_TRACK(FAnimTimelineTrack_CompoundCurve);
ANIMTIMELINE_IMPLEMENT_TRACK(FAnimTimelineTrack_FloatCurveWithDisplayName);

FAnimTimelineTrack_FloatCurveWithDisplayName::FAnimTimelineTrack_FloatCurveWithDisplayName(const FFloatCurve* InCurve, FText InDisplayName, const TSharedRef<FAnimModel>& InModel)
	: FAnimTimelineTrack_FloatCurve(InCurve, InModel)
{
	DisplayName = InDisplayName;
}

FText FAnimTimelineTrack_FloatCurveWithDisplayName::GetLabel() const
{
	// FAnimTimelineTrack_FloatCurve forces the label to its underlying curve name
	// We want to use the shorter alias for it
	return FAnimTimelineTrack::GetLabel();
}

bool FAnimTimelineTrack_FloatCurveWithDisplayName::ShowCurves() const
{
	// Always show curves even if expanded
	return true;
}

TArray<const FRichCurve*> FAnimTimelineTrack_CompoundCurve::ToRichCurves(TArrayView<FFloatCurve const* const> InCurves)
{
	TArray<const FRichCurve*> RichCurves;
	Algo::Transform(InCurves, RichCurves, [](const FFloatCurve* Curve) { return &Curve->FloatCurve; });
	return RichCurves;
}

FAnimTimelineTrack_CompoundCurve::FAnimTimelineTrack_CompoundCurve(TArray<const FFloatCurve*> InCurves, const FText& InCurveName, const FText& InFullCurveName, const FLinearColor& InBackgroundColor, const TSharedRef<FAnimModel>& InModel)
	: FAnimTimelineTrack_Curve(ToRichCurves(InCurves), InCurveName, InFullCurveName, FLinearColor{}, InBackgroundColor, InModel)
	, Curves(MoveTemp(InCurves))
{
	OuterCurveIndex = 0;
	Algo::Transform(Curves, CurveNames, [](const FFloatCurve* Curve) { return Curve->GetName(); });
}

/** FAnimTimelineTrack_Curve interface */
FLinearColor FAnimTimelineTrack_CompoundCurve::GetCurveColor(int32 InCurveIndex) const
{
	return Curves[InCurveIndex]->Color;
}

FText FAnimTimelineTrack_CompoundCurve::GetFullCurveName(int32 InCurveIndex) const
{
	return FText::FromName(Curves[InCurveIndex]->GetName());
}

bool FAnimTimelineTrack_CompoundCurve::CanEditCurve(int32 InCurveIndex) const
{
	return true;
}

void FAnimTimelineTrack_CompoundCurve::GetCurveEditInfo(int32 InCurveIndex, FName& OutName, ERawCurveTrackTypes& OutType, int32& OutCurveIndex) const
{
	OutName = Curves[InCurveIndex]->GetName();
	OutType = ERawCurveTrackTypes::RCT_Float;
	OutCurveIndex = 0;
}

struct FAnimTimelineTrack_CompoundCurve::FCurveGroup
{
	using CharType = FString::ElementType;

	FString FullName; // Full path name of this curve group, e.g. A.B or A.B.C
	int		NameLength = 0; // Length of the last part of the full name

	const FFloatCurve*	Curve = nullptr;	// Curve of FullName
	TArray<FCurveGroup>	Subgroups;			// Children of this curve group

	// \brief Get the last part of the curve name
	FStringView Name() const
	{
		return FStringView{FullName}.Right(NameLength);
	}

	//! \brief Get the name for lookup i.e. ".{Name}", Delimiter+Name
	FStringView LookupName() const
	{
		return FStringView{FullName}.Right(NameLength + 1);
	}

	FCurveGroup() = default;
	FCurveGroup(FStringView InLookupName, FStringView ParentName)
	: FullName{ParentName, InLookupName.Len()}
	{
		FullName += InLookupName;
		NameLength = InLookupName.Len() - !ParentName.IsEmpty();
	}

	bool operator==(FStringView InLookupName) const
	{
		return LookupName() == InLookupName;
	}

	bool IsRoot() const
	{
		return FullName.IsEmpty();
	}

	FCurveGroup& FindOrEmplace(FStringView SubgroupLookupName)
	{
		FCurveGroup* Subgroup = Subgroups.FindByKey(SubgroupLookupName);
		if (!Subgroup)
		{
			Subgroup = &Subgroups.Emplace_GetRef(SubgroupLookupName, FullName);
		}
		return *Subgroup;
	}

	// Combine non-curve groups only have one child
	void Compress()
	{
		TArray<FCurveGroup> Temp;
		while (Subgroups.Num() == 1 && Curve == nullptr)
		{
			FCurveGroup& Subgroup = Subgroups[0];
			NameLength = NameLength + Subgroup.NameLength + 1;
			FullName = MoveTemp(Subgroup.FullName);
			Curve = Subgroup.Curve;
			// Moving Subgroups must avoid aliasing (this.Subgroup.Subgroups in a data member of this.Subgroup)
			Temp = MoveTemp(Subgroup.Subgroups);
			Subgroups = MoveTemp(Temp);
		}
		for (FCurveGroup& Subgroup : Subgroups)
		{
			Subgroup.Compress();
		}
	}

	static FCurveGroup MakeCurveGroup(TArrayView<const FFloatCurve> FloatCurves, FStringView Delimiters)
	{
		static constexpr int BitSetSize = 256; // Char set for 256 ASCII characters, should be enough to define delimiters
		TBitArray<TInlineAllocator<BitSetSize/FBitSet::BitsPerWord>> DelimiterSet;
		DelimiterSet.Init(false, 256);
		check(DelimiterSet.Num() >= BitSetSize);
		for (CharType Delimiter : Delimiters)
		{
			if (static_cast<int32>(Delimiter) < BitSetSize)
			{
				DelimiterSet[Delimiter] = true;
			}
		}
		auto IsDelimiter = [&](CharType Char) -> bool {
			if (Char < BitSetSize)
			{
				return DelimiterSet[Char];
			}
			else
			{
				// Should be optimized on Windows where TCHAR is one byte
				return UE::String::FindFirstChar(Delimiters, Char) != INDEX_NONE;
			}
		};

		FCurveGroup RootGroup;
		for (const FFloatCurve& FloatCurve : FloatCurves)
		{
			FString CurveNameStr = FloatCurve.GetName().ToString();
			FStringView CurveName = CurveNameStr;

			// We need c++ 20 std::split_view here
			// for (auto GroupName : std::ranges::views::split(CurveName, TEXT(".")))
			int32 GroupBegin = 0;
			FCurveGroup* CurrentGroup = &RootGroup;
			for (int32 I = 0; I < CurveName.Len(); I++)
			{
				const bool bIsLastGroup = I == CurveName.Len() - 1;
				if (bIsLastGroup || IsDelimiter(CurveName[I]))
				{
					const int32 GroupEnd = I + static_cast<int32>(bIsLastGroup);
					const FStringView GroupName = CurveName.SubStr(GroupBegin, GroupEnd - GroupBegin);

					// GroupName maybe empty, we may skip empty groups
					// Which could lead to trouble for this tricky case ["A.B", "A..B", "A...B"]
					CurrentGroup = &CurrentGroup->FindOrEmplace(GroupName);

					GroupBegin = I;
				}
			}

			// Set curve on the leaf group
			ensure(CurrentGroup && !CurrentGroup->Curve);
			CurrentGroup->Curve = &FloatCurve;
		}

		RootGroup.Compress();
		return RootGroup;
	}

	//! \brief Collect descendant curves under this group
	//! \note This recursive function is O(Curves), make the tree view creation O(Curves^2)
	//!       Which should be OK given the small number of curves we have, add cache later when the demand raises
	void GatherCurves(TArray<const FFloatCurve*>& InOutCurves)
	{
		if (Curve)
		{
			InOutCurves.Add(Curve);
		}
		else
		{
			for (FCurveGroup& Subgroup : Subgroups)
			{
				Subgroup.GatherCurves(InOutCurves);
			}
		}
	}

	// Recursive add curve groups to ParentTrack
	void AddCurvesToTrack(const TSharedRef<FAnimModel>& InModel, FAnimTimelineTrack* ParentTrack)
	{
		TSharedPtr<FAnimTimelineTrack> CurveTrack;

		// If this curve group is not root, create a node for it
		if (!IsRoot())
		{
			if (Curve != nullptr)
			{
				// If the current group have a curve, create a single curve track for it
				// Such track can still have child tracks, those child curve will not show up when editing this curve, e.g. Curves ["Left.Foot", "Left.Foot.IK"]
				CurveTrack = MakeShared<FAnimTimelineTrack_FloatCurveWithDisplayName>(Curve, FText::FromStringView(Name()), InModel);
			}
			else
			{
				// Create a compound track for all descend curves in this group
				TArray<const FFloatCurve*> CurvesInGroup;
				GatherCurves(CurvesInGroup);
				ensure(!CurvesInGroup.IsEmpty());
				FLinearColor CurveBackgroundColor = FLinearColor::Transparent;
				CurveTrack = MakeShared<FAnimTimelineTrack_CompoundCurve>(MoveTemp(CurvesInGroup), FText::FromStringView(Name()), FText::FromString(FullName), CurveBackgroundColor, InModel);
				CurveTrack->SetExpanded(false);
			}

			if (CurveTrack) {
				ParentTrack->AddChild(CurveTrack.ToSharedRef());
				ParentTrack = CurveTrack.Get();
			}
		}

		for (FCurveGroup& Subgroup : Subgroups)
		{
			Subgroup.AddCurvesToTrack(InModel, ParentTrack);
		}
	}
};

void FAnimTimelineTrack_CompoundCurve::AddGroupedCurveTracks(TArrayView<const FFloatCurve> FloatCurves, FAnimTimelineTrack& ParentTrack, const TSharedRef<FAnimModel>& InModel, FStringView Delimiters)
{
	FCurveGroup CurveGroups = FCurveGroup::MakeCurveGroup(FloatCurves, Delimiters);
	CurveGroups.AddCurvesToTrack(InModel, &ParentTrack);
}
