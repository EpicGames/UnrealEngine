// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "AnimTimelineTrack_Curve.h"
#include "AnimTimelineTrack_FloatCurve.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"

struct FFloatCurve;

//! \brief Tree view of a group of float curves, use AddGroupedCurveTracks(FloatCurves, ...) to create a tree view for curves
class FAnimTimelineTrack_CompoundCurve : public FAnimTimelineTrack_Curve
{
	ANIMTIMELINE_DECLARE_TRACK(FAnimTimelineTrack_CompoundCurve, FAnimTimelineTrack_Curve);

public:
	FAnimTimelineTrack_CompoundCurve(TArray<const FFloatCurve*> InCurves, const FText& InCurveName, const FText& InFullCurveName, const FLinearColor& InBackgroundColor, const TSharedRef<FAnimModel>& InModel);

	/** FAnimTimelineTrack_Curve interface */
	virtual FLinearColor GetCurveColor(int32 InCurveIndex) const override;
	virtual FText GetFullCurveName(int32 InCurveIndex) const override;
	virtual bool CanEditCurve(int32 InCurveIndex) const override;
	virtual void GetCurveEditInfo(int32 InCurveIndex, FName& OutName, ERawCurveTrackTypes& OutType, int32& OutCurveIndex) const override;
	/** End FAnimTimelineTrack_Curve interface */

	static constexpr auto DefaultDelimiters = TEXT("._/\\|");
	//! \brief Add grouped view curve tracks for a list of curves named like 'A.B.C'
	//! \param[in] FloatCurves The curves to display
	//! \param[inout] ParentTrack The parent track to add the curves, typically a FAnimTimelineTrack_Curve
	//! \param[in] InModel FAnimModel to display the track
	static void AddGroupedCurveTracks(TArrayView<const FFloatCurve> FloatCurves, FAnimTimelineTrack& ParentTrack, const TSharedRef<FAnimModel>& InModel, FStringView Delimiters = DefaultDelimiters);

	const TArray<FName>& GetCurveNames() const { return CurveNames; }
private:
	TArray<const FFloatCurve*> Curves;
	TArray<FName> CurveNames; // Curve names used when removing curves (in case the FFloatCurve was removed)

	static TArray<const FRichCurve*> ToRichCurves(TArrayView<FFloatCurve const* const> InCurves);
	struct FCurveGroup;
};

// FloatCurve with customizable display name, e.g. showing "C" for curve "A.B.C"
class FAnimTimelineTrack_FloatCurveWithDisplayName : public FAnimTimelineTrack_FloatCurve
{
	ANIMTIMELINE_DECLARE_TRACK(FAnimTimelineTrack_FloatCurveWithDisplayName, FAnimTimelineTrack_FloatCurve);

public:
	FAnimTimelineTrack_FloatCurveWithDisplayName(const FFloatCurve* InCurve, FText InDisplayName, const TSharedRef<FAnimModel>& InModel);
	virtual FText GetLabel() const override;
	virtual bool ShowCurves() const override;
};
