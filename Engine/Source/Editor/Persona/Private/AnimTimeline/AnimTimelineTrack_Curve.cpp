// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimTimeline/AnimTimelineTrack_Curve.h"
#include "CurveEditor.h"
#include "SCurveViewerPanel.h"
#include "RichCurveEditorModel.h"
#include "Animation/AnimSequenceBase.h"
#include "Widgets/Layout/SBorder.h"
#include "Styling/AppStyle.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "AnimSequenceTimelineCommands.h"
#include "ScopedTransaction.h"
#include "PersonaUtils.h"
#include "Animation/SmartName.h"
#include "Animation/AnimMontage.h"
#include "Fonts/FontMeasure.h"
#include "Animation/AnimSequence.h"
#include "AnimTimeline/AnimModel_AnimSequenceBase.h"
#include "Preferences/PersonaOptions.h"
#include "IPersonaPreviewScene.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "AnimPreviewInstance.h"
#include "AnimTimelineClipboard.h"
#include "SAnimSequenceCurveEditor.h"
#include "Framework/Commands/GenericCommands.h"

#define LOCTEXT_NAMESPACE "FAnimTimelineTrack_Curve"

ANIMTIMELINE_IMPLEMENT_TRACK(FAnimTimelineTrack_Curve);

class FAnimModelCurveEditorBounds : public ICurveEditorBounds
{
public:
	FAnimModelCurveEditorBounds(const TSharedRef<FAnimModel>& InModel)
		: Model(InModel)
	{}

	virtual void GetInputBounds(double& OutMin, double& OutMax) const override
	{
		FAnimatedRange ViewRange = Model.Pin()->GetViewRange();
		OutMin = ViewRange.GetLowerBoundValue();
		OutMax = ViewRange.GetUpperBoundValue();
	}

	virtual void SetInputBounds(double InMin, double InMax) override
	{
		Model.Pin()->SetViewRange(TRange<double>(InMin, InMax));
	}

	TWeakPtr<FAnimModel> Model;
};

/** Widget used for drawing bounds on top of the curve viewer */
class SCurveBoundsOverlay : public SLeafWidget
{
public:
	SLATE_BEGIN_ARGS(SCurveBoundsOverlay) 
		: _BoundsLabelFormat(LOCTEXT("BoundsFormat2D", "{0}s, {1}"))
	{
	}

	SLATE_ATTRIBUTE(FText, BoundsLabelFormat)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedRef<FCurveEditor> InCurveEditor)
	{
		BoundsLabelFormatAttribute = InArgs._BoundsLabelFormat;
		CurveEditor = InCurveEditor;

		BuildBoundsLabels();
	}

	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override
	{
		// Rendering info
		constexpr float LabelOffsetPx = 2.0f;
		const float Width = static_cast<float>(AllottedGeometry.GetLocalSize().X);
		const float Height = static_cast<float>(AllottedGeometry.GetLocalSize().Y);
		const FPaintGeometry PaintGeometry  = AllottedGeometry.ToPaintGeometry();
		const FSlateFontInfo FontInfo = FCoreStyle::GetDefaultFontStyle("Regular", 8);
		TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
		const ESlateDrawEffect DrawEffects = ShouldBeEnabled(bParentEnabled) ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

		auto DrawLabel = [&AllottedGeometry, &LayerId, &OutDrawElements, &FontInfo, DrawEffects](const FText& InText, const FPaintGeometry& InLabelGeometry)
		{
			const FLinearColor LabelColor = FLinearColor::White.CopyWithNewOpacity(0.65f);

			FSlateDrawElement::MakeText(
				OutDrawElements,
				LayerId,
				InLabelGeometry,
				InText,
				FontInfo,
				DrawEffects,
				LabelColor
			);
		};

		DrawLabel(TopLeftLabel, AllottedGeometry.ToPaintGeometry(FSlateLayoutTransform(FVector2D(LabelOffsetPx, LabelOffsetPx))));

		FVector2D LabelSize = FontMeasureService->Measure(TopRightLabel, FontInfo);
		DrawLabel(TopRightLabel, AllottedGeometry.ToPaintGeometry(FSlateLayoutTransform(FVector2D(Width - (LabelSize.X + LabelOffsetPx), LabelOffsetPx))));

		LabelSize = FontMeasureService->Measure(BottomLeftLabel, FontInfo);
		DrawLabel(BottomLeftLabel, AllottedGeometry.ToPaintGeometry(FSlateLayoutTransform(FVector2D(LabelOffsetPx, Height - (LabelSize.Y + LabelOffsetPx)))));

		LabelSize = FontMeasureService->Measure(BottomRightLabel, FontInfo);
		DrawLabel(BottomRightLabel, AllottedGeometry.ToPaintGeometry(FSlateLayoutTransform(FVector2D(Width - (LabelSize.X + LabelOffsetPx), Height - (LabelSize.Y + LabelOffsetPx)))));

		return LayerId + 1;
	}

	FVector2D ComputeDesiredSize(float) const override
	{
		return FVector2D(8.0f, 8.0f);
	}

	void BuildBoundsLabels()
	{
		double InputMin = TNumericLimits<double>::Max(), InputMax = TNumericLimits<double>::Lowest(), OutputMin = TNumericLimits<double>::Max(), OutputMax = TNumericLimits<double>::Lowest();

		for(const TPair<FCurveModelID, TUniquePtr<FCurveModel>>& ModelPair : CurveEditor.Pin()->GetCurves())
		{
			double LocalMin, LocalMax;
			ModelPair.Value->GetTimeRange(LocalMin, LocalMax);
			InputMin = FMath::Min(InputMin, LocalMin);
			InputMax = FMath::Max(InputMax, LocalMax);
			ModelPair.Value->GetValueRange(LocalMin, LocalMax);
			OutputMin = FMath::Min(OutputMin, LocalMin);
			OutputMax = FMath::Max(OutputMax, LocalMax);
		}

		const FText BoundsLabelFormat = BoundsLabelFormatAttribute.Get();

		TopLeftLabel = FText::Format(BoundsLabelFormat, FText::AsNumber(InputMin), FText::AsNumber(OutputMax));
		TopRightLabel = FText::Format(BoundsLabelFormat, FText::AsNumber(InputMax), FText::AsNumber(OutputMax));
		BottomLeftLabel = FText::Format(BoundsLabelFormat, FText::AsNumber(InputMin), FText::AsNumber(OutputMin));
		BottomRightLabel = FText::Format(BoundsLabelFormat, FText::AsNumber(InputMax), FText::AsNumber(OutputMin));
	}

	// The curve editor we are using to get out values
	TWeakPtr<FCurveEditor> CurveEditor;

	// The format to use to display the bounds
	TAttribute<FText> BoundsLabelFormatAttribute;

	// Labels for each corner of the bounds
	FText TopLeftLabel;
	FText TopRightLabel;
	FText BottomLeftLabel;
	FText BottomRightLabel;
};

FAnimTimelineTrack_Curve::FAnimTimelineTrack_Curve(const FText& InCurveName, const FText& InFullCurveName, const FLinearColor& InColor, const FLinearColor& InBackgroundColor, const TSharedRef<FAnimModel>& InModel)
	: FAnimTimelineTrack(InCurveName, InCurveName, InModel)
	, Color(InColor)
	, BackgroundColor(InBackgroundColor)
	, FullCurveName(InFullCurveName)
{
	SetHeight(32.0f);
}

FAnimTimelineTrack_Curve::FAnimTimelineTrack_Curve(const FRichCurve* InCurve, const FName& InName, int32 InCurveIndex, ERawCurveTrackTypes InType, const FText& InCurveName, const FText& InFullCurveName, const FLinearColor& InColor, const FLinearColor& InBackgroundColor, const TSharedRef<FAnimModel>& InModel)
	: FAnimTimelineTrack(InCurveName, InCurveName, InModel)
	, Color(InColor)
	, BackgroundColor(InBackgroundColor)
	, FullCurveName(InFullCurveName)
	, OuterCurveName(InName)
	, OuterCurveIndex(InCurveIndex)
	, OuterType(InType)
{
	Curves.Add(InCurve);
	SetHeight(32.0f);
}

FAnimTimelineTrack_Curve::FAnimTimelineTrack_Curve(const TArray<const FRichCurve*>& InCurves, const FText& InCurveName, const FText& InFullCurveName, const FLinearColor& InColor, const FLinearColor& InBackgroundColor, const TSharedRef<FAnimModel>& InModel)
	: FAnimTimelineTrack(InCurveName, InCurveName, InModel)
	, Curves(InCurves)
	, Color(InColor)
	, BackgroundColor(InBackgroundColor)
	, FullCurveName(InFullCurveName)
{
	SetHeight(32.0f);
}

TSharedRef<SWidget> FAnimTimelineTrack_Curve::GenerateContainerWidgetForTimeline()
{
	CurveEditor = MakeShared<FCurveEditor>();
	CurveEditor->SetBounds(MakeUnique<FAnimModelCurveEditorBounds>(GetModel()));
	CurveEditor->InputZoomToFitPadding = 0.0f;
	CurveEditor->OutputZoomToFitPadding = 0.01f;
	FCurveEditorInitParams CurveEditorInitParams;
	CurveEditor->InitCurveEditor(CurveEditorInitParams);

	for(int32 CurveIndex = 0; CurveIndex < Curves.Num(); ++CurveIndex)
	{
		const FRichCurve* Curve = Curves[CurveIndex];

		FName Name;
		ERawCurveTrackTypes Type;
		int32 EditIndex;
		GetCurveEditInfo(CurveIndex, Name, Type, EditIndex);

		TUniquePtr<FRichCurveEditorModelNamed> NewCurveModel = MakeUnique<FRichCurveEditorModelNamed>(Name, Type, EditIndex, GetModel()->GetAnimSequenceBase());
		NewCurveModel->SetColor(GetCurveColor(CurveIndex));
		NewCurveModel->SetIsKeyDrawEnabled(MakeAttributeLambda([](){ return GetDefault<UPersonaOptions>()->bTimelineDisplayCurveKeys; }));
		CurveEditor->AddCurve(MoveTemp(NewCurveModel));
	}

	TSharedRef<SWidget> TimelineWidget = MakeTimelineWidgetContainer();

	UAnimMontage* AnimMontage = Cast<UAnimMontage>(GetModel()->GetAnimSequenceBase());
	if(!(AnimMontage && AnimMontage->HasParentAsset()))
	{
		TimelineWidget->SetOnMouseDoubleClick(FPointerEventHandler::CreateSP(this, &FAnimTimelineTrack_Curve::HandleDoubleClicked));
		TimelineWidget->SetOnMouseButtonUp(FPointerEventHandler::CreateSP(this, &FAnimTimelineTrack_Curve::HandleMouseButtonUp));
	}

	if (GetModel()->GetPreviewScene()->GetPreviewMeshComponent()->PreviewInstance != nullptr)
	{		
		GetModel()->GetPreviewScene()->GetPreviewMeshComponent()->PreviewInstance->AddKeyCompleteDelegate(FSimpleDelegate::CreateSP(this, &FAnimTimelineTrack_Curve::HandleCurveChanged));
	}

	return TimelineWidget;
}

FLinearColor FAnimTimelineTrack_Curve::GetCurveColor(int32 InCurveIndex) const
{
	return Color;
}

TSharedRef<SWidget> FAnimTimelineTrack_Curve::MakeTimelineWidgetContainer()
{
	TSharedRef<SWidget> CurveWidget = MakeCurveWidget();

	// zoom to fit now we have a view
	CurveEditor->ZoomToFit(EAxisList::Y);

	return 
		SAssignNew(TimelineWidgetContainer, SBorder)
		.Padding(0.0f)
		.BorderImage(FAppStyle::GetBrush("AnimTimeline.Outliner.DefaultBorder"))
		.BorderBackgroundColor_Lambda([this](){ return GetModel()->IsTrackSelected(AsShared()) ? FAppStyle::GetSlateColor("SelectionColor").GetSpecifiedColor().CopyWithNewOpacity(0.75f) : BackgroundColor.Desaturate(0.75f); })
		[
			CurveWidget
		];
}

TSharedRef<SWidget> FAnimTimelineTrack_Curve::MakeCurveWidget()
{
	return
		SNew(SOverlay)
		+SOverlay::Slot()
		[
			SNew(SCurveViewerPanel, CurveEditor.ToSharedRef())
			.Visibility_Lambda([this]()
			{  
				// Dont show curves in parent tracks when children are expanded
				return ShowCurves() ? EVisibility::Visible : EVisibility::Hidden;
			})
			.CurveThickness_Lambda([this]()
			{
				return IsHovered() ? 2.0f : 1.0f;
			})
		]
		+SOverlay::Slot()
		[
			SAssignNew(CurveOverlay, SCurveBoundsOverlay, CurveEditor.ToSharedRef())
			.BoundsLabelFormat(LOCTEXT("BoundsFormat", "{1}"))	// Only want to display the Y axis
			.Visibility_Lambda([this]()
			{  
				// Dont show curves in parent tracks when children are expanded
				return ShowCurves() && IsHovered() ? EVisibility::Visible : EVisibility::Hidden;
			})
		];
}

TSharedRef<SWidget> FAnimTimelineTrack_Curve::GenerateContainerWidgetForOutliner(const TSharedRef<SAnimOutlinerItem>& InRow)
{
	TSharedPtr<SBorder> OuterBorder;
	TSharedPtr<SHorizontalBox> InnerHorizontalBox;
	TSharedRef<SWidget> OutlinerWidget = GenerateStandardOutlinerWidget(InRow, true, OuterBorder, InnerHorizontalBox);

	UAnimMontage* AnimMontage = Cast<UAnimMontage>(GetModel()->GetAnimSequenceBase());
	if(!(AnimMontage && AnimMontage->HasParentAsset()))
	{
		OuterBorder->SetOnMouseDoubleClick(FPointerEventHandler::CreateSP(this, &FAnimTimelineTrack_Curve::HandleDoubleClicked));
		AddCurveTrackButton(InnerHorizontalBox);
	}

	return OutlinerWidget;
}

void FAnimTimelineTrack_Curve::AddCurveTrackButton(TSharedPtr<SHorizontalBox> InnerHorizontalBox)
{
	InnerHorizontalBox->AddSlot()
	.AutoWidth()
	.HAlign(HAlign_Right)
	.VAlign(VAlign_Center)
	.Padding(OutlinerRightPadding, 1.0f)
	[
		PersonaUtils::MakeTrackButton(LOCTEXT("EditCurveButtonText", "Curve"), FOnGetContent::CreateSP(this, &FAnimTimelineTrack_Curve::BuildCurveTrackMenu), MakeAttributeSP(this, &FAnimTimelineTrack_Curve::IsHovered))
	];
}

bool FAnimTimelineTrack_Curve::ShowCurves() const
{
	// Dont show curves in parent tracks when children are expanded
	return !IsExpanded() || Children.Num() == 0;
}

TSharedRef<SWidget> FAnimTimelineTrack_Curve::BuildCurveTrackMenu()
{
	FMenuBuilder MenuBuilder(true, GetModel()->GetCommandList());

	MenuBuilder.BeginSection("Curve", LOCTEXT("CurveMenuSection", "Curve"));
	{
		MenuBuilder.AddMenuEntry(
			FAnimSequenceTimelineCommands::Get().EditCurve->GetLabel(),
			FAnimSequenceTimelineCommands::Get().EditCurve->GetDescription(),
			FAnimSequenceTimelineCommands::Get().EditCurve->GetIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &FAnimTimelineTrack_Curve::HendleEditCurve)));
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void FAnimTimelineTrack_Curve::AddToContextMenu(FMenuBuilder& InMenuBuilder, TSet<FName>& InOutExistingMenuTypes) const
{
	if(!InOutExistingMenuTypes.Contains(FAnimTimelineTrack_Curve::GetTypeName()))
	{
		InMenuBuilder.BeginSection("EditCurve", LOCTEXT("CurveEditMenuSection", "Curve Edit"));
		{
			InMenuBuilder.AddMenuEntry(FAnimSequenceTimelineCommands::Get().PasteDataIntoCurve);
		}
		InMenuBuilder.EndSection();
		
		InMenuBuilder.BeginSection("EditSelection", LOCTEXT("CurveSelectionEditMenuSection", "Selection Edit"));
		{
			InMenuBuilder.AddMenuEntry(FGenericCommands::Get().Cut);
			InMenuBuilder.AddMenuEntry(FGenericCommands::Get().Copy);
			InMenuBuilder.AddMenuEntry(FGenericCommands::Get().Paste);
			InMenuBuilder.AddMenuEntry(FGenericCommands::Get().Delete);

			InMenuBuilder.AddSeparator();
			
			InMenuBuilder.AddMenuEntry(FAnimSequenceTimelineCommands::Get().EditSelectedCurves);
			InMenuBuilder.AddMenuEntry(FAnimSequenceTimelineCommands::Get().CopySelectedCurveNames);
		}
		InMenuBuilder.EndSection();
		
		InOutExistingMenuTypes.Add(FAnimTimelineTrack_Curve::GetTypeName());
	}
}

void FAnimTimelineTrack_Curve::Copy(UAnimTimelineClipboardContent* InOutClipboard) const
{
	check(InOutClipboard != nullptr)
	
	if (Curves.Num() == 1)
	{
		UFloatCurveCopyObject * CopyableCurve = UAnimCurveBaseCopyObject::Create<UFloatCurveCopyObject>();
		const FRichCurve* InCurve = Curves[0];

		// Copy raw curve data
		CopyableCurve->Curve.SetName(FName(FullCurveName.ToString()));
		CopyableCurve->Curve.FloatCurve = *InCurve;
		CopyableCurve->Curve.SetCurveTypeFlags(AACF_Editable);

		// Copy curve identifier data
		CopyableCurve->CurveName = CopyableCurve->Curve.GetName();
		CopyableCurve->CurveType = ERawCurveTrackTypes::RCT_Float;

		// Origin data
		CopyableCurve->OriginName = GetModel()->GetAnimSequenceBase()->GetFName();
			
		InOutClipboard->Curves.Add(CopyableCurve);
	}
	else
	{
		UE_LOGF(LogAnimation, Warning, "Copying multiple curves from a FAnimTimelineTrack_Curve not supported. Curve: %ls", *FullCurveName.ToString())
	}
}

FReply FAnimTimelineTrack_Curve::HandleMouseButtonUp(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent)
{
	if(InPointerEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		FMenuBuilder MenuBuilder(true, GetModel()->GetCommandList());

		GetModel()->BuildContextMenu(MenuBuilder);

		FWidgetPath WidgetPath = InPointerEvent.GetEventPath() != nullptr ? *InPointerEvent.GetEventPath() : FWidgetPath();
		FSlateApplication::Get().PushMenu(TimelineWidgetContainer.ToSharedRef(), WidgetPath, MenuBuilder.MakeWidget(), FSlateApplication::Get().GetCursorPos(), FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu));

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FReply FAnimTimelineTrack_Curve::HandleDoubleClicked(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent)
{
	if(InPointerEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		GetModel()->GetCommandList()->ExecuteAction(FAnimSequenceTimelineCommands::Get().EditSelectedCurves.ToSharedRef());

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void FAnimTimelineTrack_Curve::HandleCurveChanged()
{
	ZoomToFit();
}

void FAnimTimelineTrack_Curve::PostUndoRedo()
{
	ZoomToFit();
}

void FAnimTimelineTrack_Curve::HendleEditCurve()
{
	TArray<IAnimationEditor::FCurveEditInfo> EditCurveInfo;
	for(int32 CurveIndex = 0; CurveIndex < Curves.Num(); ++CurveIndex)
	{
		FName Name;
		ERawCurveTrackTypes Type;
		int32 EditCurveIndex;
		GetCurveEditInfo(CurveIndex, Name, Type, EditCurveIndex);

		EditCurveInfo.Emplace(GetFullCurveName(CurveIndex), GetCurveColor(CurveIndex), Name, Type, EditCurveIndex, FSimpleDelegate::CreateSP(this, &FAnimTimelineTrack_Curve::HandleCurveChanged));
	}
	StaticCastSharedRef<FAnimModel_AnimSequenceBase>(GetModel())->OnEditCurves.ExecuteIfBound(GetModel()->GetAnimSequenceBase(), EditCurveInfo, nullptr);
}

void FAnimTimelineTrack_Curve::GetCurveEditInfo(int32 InCurveIndex, FName& OutName, ERawCurveTrackTypes& OutType, int32& OutCurveIndex) const
{
	OutName = OuterCurveName;
	OutType = OuterType;
	OutCurveIndex = OuterCurveIndex;
}

void FAnimTimelineTrack_Curve::ZoomToFit()
{
	if(CurveEditor.IsValid())
	{
		CurveEditor->ZoomToFit(EAxisList::Y);
	}
	if(CurveOverlay.IsValid())
	{
		CurveOverlay->BuildBoundsLabels();
	}
}

#undef LOCTEXT_NAMESPACE
