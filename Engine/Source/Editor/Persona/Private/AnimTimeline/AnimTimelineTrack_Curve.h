// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimTimeline/AnimTimelineTrack.h"
#include "Animation/Skeleton.h"
#include "EditorUndoClient.h"
#include "AnimTimeline/AnimModel.h"
#include "Animation/AnimData/CurveIdentifier.h"

class SBorder;
class FCurveEditor;
struct FRichCurve;
class SHorizontalBox;
class SCurveBoundsOverlay;

class FAnimTimelineTrack_Curve : public FAnimTimelineTrack, public FSelfRegisteringEditorUndoClient
{
	ANIMTIMELINE_DECLARE_TRACK(FAnimTimelineTrack_Curve, FAnimTimelineTrack);

public:
	FAnimTimelineTrack_Curve(const FText& InCurveName, const FText& InFullCurveName, const FLinearColor& InColor, const FLinearColor& InBackgroundColor, const TSharedRef<FAnimModel>& InModel);
	FAnimTimelineTrack_Curve(const FRichCurve* InCurve, const FName& InName, int32 InCurveIndex, ERawCurveTrackTypes InType, const FText& InCurveName, const FText& InFullCurveName, const FLinearColor& InColor, const FLinearColor& InBackgroundColor, const TSharedRef<FAnimModel>& InModel);
	FAnimTimelineTrack_Curve(const TArray<const FRichCurve*>& InCurves, const FText& InCurveName, const FText& InFullCurveName, const FLinearColor& InColor, const FLinearColor& InBackgroundColor, const TSharedRef<FAnimModel>& InModel);

	/** FAnimTimelineTrack interface */
	virtual TSharedRef<SWidget> GenerateContainerWidgetForTimeline() override;
	virtual TSharedRef<SWidget> GenerateContainerWidgetForOutliner(const TSharedRef<SAnimOutlinerItem>& InRow) override;
	virtual bool SupportsSelection() const override { return true; }
	virtual void AddToContextMenu(FMenuBuilder& InMenuBuilder, TSet<FName>& InOutExistingMenuTypes) const override;
	virtual bool SupportsCopy() const override { return true; }
	virtual void Copy(UAnimTimelineClipboardContent* InOutClipboard) const override;
	
	/** FEditorUndoClient interface */
	virtual void PostUndo(bool bSuccess) override { PostUndoRedo(); }
	virtual void PostRedo(bool bSuccess) override { PostUndoRedo(); }

	/** Access the curve we are editing */
	const TArray<const FRichCurve*>& GetCurves() const { return Curves; }

	/** Get a color for the curve widget (and edited curves) */
	virtual FLinearColor GetCurveColor(int32 InCurveIndex) const;

	/** Get whether a specified curve can be edited */
	virtual bool CanEditCurve(int32 InCurveIndex) const { return true; }

	/** Get the name of the curve when edited (includes path, e.g. BoneName.Translation.X) */
	virtual FText GetFullCurveName(int32 InCurveIndex) const { return FullCurveName; }

	/** Get the information needed to reference this curve for editing */
	virtual void GetCurveEditInfo(int32 InCurveIndex, FName& OutName, ERawCurveTrackTypes& OutType, int32& OutCurveIndex) const;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.3, "Please use GetCurveEditInfo that takes a FName.")
	virtual void GetCurveEditInfo(int32 InCurveIndex, FSmartName& OutName, ERawCurveTrackTypes& OutType, int32& OutCurveIndex) const {}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** Delegate handing curve changing externally */
	void HandleCurveChanged();

protected:
	/** Make the container (border) widget for the curve timeline */
	virtual TSharedRef<SWidget> MakeTimelineWidgetContainer();

	/** Make the curve widget itself */
	virtual TSharedRef<SWidget> MakeCurveWidget();

	/** Build the track menu for this curve */
	virtual TSharedRef<SWidget> BuildCurveTrackMenu();

	/** Helper function for building outliner widget */
	virtual void AddCurveTrackButton(TSharedPtr<SHorizontalBox> InnerHorizontalBox);

	/** Should this track draw its curves, by default, hide curves in parent tracks when children are expanded. */
	virtual bool ShowCurves() const;

	/** Edit this curve when double clicked */
	FReply HandleDoubleClicked(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent);

	/** Push a popup menu when we right-click */
	FReply HandleMouseButtonUp(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent);

	/** Edit this curve in the curve editor */
	void HendleEditCurve();

	/** Handle undo/redo */
	void PostUndoRedo();

	/** Zoom the view to fit the bounds of the curve */
	void ZoomToFit();

protected:
	/** The curve we are editing */
	TArray<const FRichCurve*> Curves;

	/** The color of the curve */
	FLinearColor Color;

	/** The color of the curve background */
	FLinearColor BackgroundColor;

	/** Name of the curve when edited (includes path, e.g. BoneName.Translation.X) */
	FText FullCurveName;

	/** The name of the outer curve */
	FName OuterCurveName;

	/** The index into the outer curve */
	int32 OuterCurveIndex;

	/** The type of this curves outer (currently always RCT_Transform) */
	ERawCurveTrackTypes OuterType;

	/** Container widget for timeline */
	TSharedPtr<SBorder> TimelineWidgetContainer;

	/** Curve editor */
	TSharedPtr<FCurveEditor> CurveEditor;

	/** Overlay widget used to display bounds */
	TSharedPtr<SCurveBoundsOverlay> CurveOverlay;
};
