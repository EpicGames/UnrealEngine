// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimTimeline/AnimTimelineTrack_Curves.h"
#include "PersonaUtils.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Input/SSearchBox.h"
#include "AnimSequenceTimelineCommands.h"
#include "IEditableSkeleton.h"
#include "SAnimCurvePicker.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Animation/AnimSequenceBase.h"
#include "Widgets/Input/STextEntryPopup.h"
#include "Framework/Application/SlateApplication.h"
#include "ScopedTransaction.h"
#include "Animation/AnimMontage.h"
#include "AnimTimeline/SAnimOutlinerItem.h"
#include "Preferences/PersonaOptions.h"
#include "SListViewSelectorDropdownMenu.h"
#include "Animation/AnimData/IAnimationDataModel.h"

#define LOCTEXT_NAMESPACE "FAnimTimelineTrack_Notifies"

ANIMTIMELINE_IMPLEMENT_TRACK(FAnimTimelineTrack_Curves);

FAnimTimelineTrack_Curves::FAnimTimelineTrack_Curves(const TSharedRef<FAnimModel>& InModel)
	: FAnimTimelineTrack(LOCTEXT("CurvesRootTrackLabel", "Curves"), LOCTEXT("CurvesRootTrackToolTip", "Curve data contained in this asset"), InModel)
{
}

TSharedRef<SWidget> FAnimTimelineTrack_Curves::GenerateContainerWidgetForOutliner(const TSharedRef<SAnimOutlinerItem>& InRow)
{
	TSharedPtr<SBorder> OuterBorder;
	TSharedPtr<SHorizontalBox> InnerHorizontalBox;
	OutlinerWidget = GenerateStandardOutlinerWidget(InRow, false, OuterBorder, InnerHorizontalBox);

	OuterBorder->SetBorderBackgroundColor(FAppStyle::GetColor("AnimTimeline.Outliner.HeaderColor"));

	InnerHorizontalBox->AddSlot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		.Padding(2.0f, 1.0f)
		.AutoWidth()
		[
			SNew(STextBlock)
			.TextStyle(&FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("AnimTimeline.Outliner.Label"))
			.Text(this, &FAnimTimelineTrack_Curves::GetLabel)
			.HighlightText(InRow->GetHighlightText())
		];

	InnerHorizontalBox->AddSlot()
		.FillWidth(1.0f)
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(5.0f, 1.0f)
		[
			SNew(STextBlock)
			.TextStyle(&FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("TinyText"))
			.Text_Lambda([this]()
			{ 
				UAnimSequenceBase* AnimSequenceBase = GetModel()->GetAnimSequenceBase();
				return FText::Format(LOCTEXT("CurveCountFormat", "({0})"), FText::AsNumber(AnimSequenceBase->GetDataModel()->GetNumberOfFloatCurves())); 
			})
		];

	UAnimMontage* AnimMontage = Cast<UAnimMontage>(GetModel()->GetAnimSequenceBase());
	if(!(AnimMontage && AnimMontage->HasParentAsset()))
	{
		InnerHorizontalBox->AddSlot()
			.AutoWidth()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.Padding(OutlinerRightPadding, 1.0f)
			[
				PersonaUtils::MakeTrackButton(LOCTEXT("EditCurvesButtonText", "Curves"), FOnGetContent::CreateSP(this, &FAnimTimelineTrack_Curves::BuildCurvesSubMenu), MakeAttributeSP(this, &FAnimTimelineTrack_Curves::IsHovered))
			];
	}

	return OutlinerWidget.ToSharedRef();
}

void FAnimTimelineTrack_Curves::DeleteAllCurves()
{
	UAnimSequenceBase* AnimSequenceBase = GetModel()->GetAnimSequenceBase();
	IAnimationDataController& Controller = AnimSequenceBase->GetController();
	Controller.RemoveAllCurvesOfType(ERawCurveTrackTypes::RCT_Float);
}

TSharedRef<SWidget> FAnimTimelineTrack_Curves::BuildCurvesSubMenu()
{
	FMenuBuilder MenuBuilder(true, GetModel()->GetCommandList());

	MenuBuilder.BeginSection("Curves", LOCTEXT("CurvesMenuSection", "Curves"));
	{
		MenuBuilder.AddSubMenu(
			FAnimSequenceTimelineCommands::Get().AddCurve->GetLabel(),
			FAnimSequenceTimelineCommands::Get().AddCurve->GetDescription(),
			FNewMenuDelegate::CreateSP(this, &FAnimTimelineTrack_Curves::FillVariableCurveMenu)
		);

		MenuBuilder.AddSubMenu(
			FAnimSequenceTimelineCommands::Get().AddMetadata->GetLabel(),
			FAnimSequenceTimelineCommands::Get().AddMetadata->GetDescription(),
			FNewMenuDelegate::CreateSP(this, &FAnimTimelineTrack_Curves::FillMetadataEntryMenu)
		);

		UAnimSequenceBase* AnimSequenceBase = GetModel()->GetAnimSequenceBase();
		if(AnimSequenceBase->GetDataModel()->GetNumberOfFloatCurves() > 0)
		{
			MenuBuilder.AddMenuEntry(
				FAnimSequenceTimelineCommands::Get().RemoveAllCurves->GetLabel(),
				FAnimSequenceTimelineCommands::Get().RemoveAllCurves->GetDescription(),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateSP(this, &FAnimTimelineTrack_Curves::DeleteAllCurves))
			);
		}
	}

	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("Options", LOCTEXT("OptionsMenuSection", "Options"));
	{
		MenuBuilder.AddMenuEntry(
			FAnimSequenceTimelineCommands::Get().ShowCurveKeys->GetLabel(),
			FAnimSequenceTimelineCommands::Get().ShowCurveKeys->GetDescription(),
			FAnimSequenceTimelineCommands::Get().ShowCurveKeys->GetIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &FAnimTimelineTrack_Curves::HandleShowCurvePoints),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &FAnimTimelineTrack_Curves::IsShowCurvePointsEnabled)
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);

		MenuBuilder.AddMenuEntry(
			FAnimSequenceTimelineCommands::Get().UseTreeView->GetLabel(),
			FAnimSequenceTimelineCommands::Get().UseTreeView->GetDescription(),
			FAnimSequenceTimelineCommands::Get().UseTreeView->GetIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &FAnimTimelineTrack_Curves::HandleUseTreeView),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &FAnimTimelineTrack_Curves::IsUseTreeViewEnabled)
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void FAnimTimelineTrack_Curves::FillMetadataEntryMenu(FMenuBuilder& Builder)
{
	// Add new metadata entry menu button
	{
		const FText Description = LOCTEXT("NewMetadataCreateNew_ToolTip", "Create a new metadata entry");
		const FText Label = LOCTEXT("NewMetadataCreateNew_Label","Create Metadata Entry");
		FUIAction UIAction;
		UIAction.ExecuteAction.BindRaw(this, &FAnimTimelineTrack_Curves::CreateNewMetadataEntryClicked);

		Builder.AddMenuEntry(Label, Description, FSlateIcon(), UIAction);
	}
	
	Builder.BeginSection(NAME_None, LOCTEXT("MetadataMenu_ListHeading", "Available Names"));

	// Add existing curve to timeline using curve picker
	{
		const TSharedRef<SWidget> CurvePickerWidget = SNew(SAnimCurvePicker, &GetModel()->GetEditableSkeleton()->GetSkeleton())
		.OnCurvesPicked_Lambda([this](const TConstArrayView<FName> InCurves)
		{
			check(InCurves.Num() == 1);
			return OnMetadataCurveNamePicked(InCurves[0]);
		})
		.IsCurveNameMarkedForExclusion(this, &FAnimTimelineTrack_Curves::IsCurveMarkedForExclusion);
		Builder.AddWidget(CurvePickerWidget, FText::GetEmpty(), true);
	}
	
	Builder.EndSection();
}

void FAnimTimelineTrack_Curves::FillVariableCurveMenu(FMenuBuilder& Builder)
{
	// Menu entry to create a new curve
	{
		FText Description = LOCTEXT("NewVariableCurveCreateNew_ToolTip", "Create a new variable curve");
		FText Label = LOCTEXT("NewVariableCurveCreateNew_Label", "Create Curve");
		FUIAction UIAction;
		UIAction.ExecuteAction.BindRaw(this, &FAnimTimelineTrack_Curves::CreateNewCurveClicked);
		Builder.AddMenuEntry(Label, Description, FSlateIcon(), UIAction);
	}
	
	Builder.BeginSection(NAME_None, LOCTEXT("VariableMenu_ListHeading", "Available Names"));

	// Add existing curve to timeline using curve picker
	{
		const TSharedRef<SWidget> CurvePickerWidget = SNew(SAnimCurvePicker, &GetModel()->GetEditableSkeleton()->GetSkeleton())
		.OnCurvesPicked_Lambda([this](const TConstArrayView<FName> InCurves)
		{
			check(InCurves.Num() == 1);
			OnVariableCurveNamePicked(InCurves[0]);
		})
		.IsCurveNameMarkedForExclusion(this, &FAnimTimelineTrack_Curves::IsCurveMarkedForExclusion);
		
		Builder.AddWidget(CurvePickerWidget, FText::GetEmpty(), true);
	}
	
	Builder.EndSection();
}

void FAnimTimelineTrack_Curves::AddMetadataEntry(const FName& InCurveName)
{
	UAnimSequenceBase* AnimSequenceBase = GetModel()->GetAnimSequenceBase();

	IAnimationDataController& Controller = AnimSequenceBase->GetController();
	IAnimationDataController::FScopedBracket ScopedBracket(Controller, LOCTEXT("AddCurveMetadata", "Add Curve Metadata"));

	const FAnimationCurveIdentifier MetadataCurveId(InCurveName, ERawCurveTrackTypes::RCT_Float);
	Controller.AddCurve(MetadataCurveId, AACF_Metadata);
	Controller.SetCurveKeys(MetadataCurveId, { FRichCurveKey(0.f, 1.f) });
}

void FAnimTimelineTrack_Curves::CreateNewMetadataEntryClicked()
{
	TSharedRef<STextEntryPopup> TextEntry =
		SNew(STextEntryPopup)
		.Label(LOCTEXT("NewMetadataCurveEntryLabal", "Metadata Name"))
		.OnTextCommitted(this, &FAnimTimelineTrack_Curves::CreateNewMetadataEntry);

	FSlateApplication& SlateApp = FSlateApplication::Get();
	SlateApp.PushMenu(
		OutlinerWidget.ToSharedRef(),
		FWidgetPath(),
		TextEntry,
		SlateApp.GetCursorPos(),
		FPopupTransitionEffect::TypeInPopup
		);
}

void FAnimTimelineTrack_Curves::CreateNewMetadataEntry(const FText& CommittedText, ETextCommit::Type CommitType)
{
	FSlateApplication::Get().DismissAllMenus();
	if(CommitType == ETextCommit::OnEnter)
	{
		// Add the name to the skeleton and then add the new curve to the sequence
		UAnimSequenceBase* AnimSequenceBase = GetModel()->GetAnimSequenceBase();
		USkeleton* Skeleton = AnimSequenceBase->GetSkeleton();
		if(Skeleton && !CommittedText.IsEmpty())
		{
			AddMetadataEntry(FName(*CommittedText.ToString()));
		}
	}
}

void FAnimTimelineTrack_Curves::CreateNewCurveClicked()
{
	TSharedRef<STextEntryPopup> TextEntry =
		SNew(STextEntryPopup)
		.Label(LOCTEXT("NewCurveEntryLabal", "Curve Name"))
		.OnTextCommitted(this, &FAnimTimelineTrack_Curves::CreateTrack);

	FSlateApplication& SlateApp = FSlateApplication::Get();
	SlateApp.PushMenu(
		OutlinerWidget.ToSharedRef(),
		FWidgetPath(),
		TextEntry,
		SlateApp.GetCursorPos(),
		FPopupTransitionEffect::TypeInPopup
		);
}

void FAnimTimelineTrack_Curves::CreateTrack(const FText& ComittedText, ETextCommit::Type CommitInfo)
{
	if ( CommitInfo == ETextCommit::OnEnter )
	{
		UAnimSequenceBase* AnimSequenceBase = GetModel()->GetAnimSequenceBase();
		USkeleton* Skeleton = AnimSequenceBase->GetSkeleton();
		if(Skeleton && !ComittedText.IsEmpty())
		{
			const FScopedTransaction Transaction(LOCTEXT("AnimCurve_AddTrack", "Add New Curve"));

			AddVariableCurve(FName(*ComittedText.ToString()));
		}

		FSlateApplication::Get().DismissAllMenus();
	}
}

void FAnimTimelineTrack_Curves::AddVariableCurve(const FName& InCurveName)
{
	FScopedTransaction Transaction(LOCTEXT("AddCurve", "Add Curve"));

	UAnimSequenceBase* AnimSequenceBase = GetModel()->GetAnimSequenceBase();
	AnimSequenceBase->Modify();

	IAnimationDataController& Controller = AnimSequenceBase->GetController();
	const FAnimationCurveIdentifier FloatCurveId(InCurveName, ERawCurveTrackTypes::RCT_Float);
	Controller.AddCurve(FloatCurveId);
}

void FAnimTimelineTrack_Curves::HandleShowCurvePoints()
{
	GetMutableDefault<UPersonaOptions>()->bTimelineDisplayCurveKeys = !GetDefault<UPersonaOptions>()->bTimelineDisplayCurveKeys;
}

bool FAnimTimelineTrack_Curves::IsShowCurvePointsEnabled() const
{
	return GetDefault<UPersonaOptions>()->bTimelineDisplayCurveKeys;
}

void FAnimTimelineTrack_Curves::HandleUseTreeView()
{
	GetMutableDefault<UPersonaOptions>()->bUseTreeViewForAnimationCurves = !GetDefault<UPersonaOptions>()->bUseTreeViewForAnimationCurves;
	GetModel()->RefreshTracks();
}

bool FAnimTimelineTrack_Curves::IsUseTreeViewEnabled() const
{
	return GetDefault<UPersonaOptions>()->bUseTreeViewForAnimationCurves;
}

void FAnimTimelineTrack_Curves::OnMetadataCurveNamePicked(const FName& InCurveName)
{
	FSlateApplication::Get().DismissAllMenus();

	if(InCurveName != NAME_None)
	{
		AddMetadataEntry(InCurveName);
	}
}

void FAnimTimelineTrack_Curves::OnVariableCurveNamePicked(const FName& InCurveName)
{
	FSlateApplication::Get().DismissAllMenus();
	
	if(InCurveName != NAME_None)
	{
		AddVariableCurve(InCurveName);
	}
}

bool FAnimTimelineTrack_Curves::IsCurveMarkedForExclusion(const FName& InCurveName)
{
	return GetModel()->GetAnimSequenceBase()->GetDataModel()->FindFloatCurve(FAnimationCurveIdentifier(InCurveName, ERawCurveTrackTypes::RCT_Float)) != nullptr;
}
#undef LOCTEXT_NAMESPACE
