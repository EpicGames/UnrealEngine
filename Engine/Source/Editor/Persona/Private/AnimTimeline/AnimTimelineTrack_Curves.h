// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimTimeline/AnimTimelineTrack.h"
#include "Animation/Skeleton.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"

class FAnimTimelineTrack_Curves : public FAnimTimelineTrack
{
	ANIMTIMELINE_DECLARE_TRACK(FAnimTimelineTrack_Curves, FAnimTimelineTrack);

public:
	FAnimTimelineTrack_Curves(const TSharedRef<FAnimModel>& InModel);

	/** FAnimTimelineTrack interface */
	virtual TSharedRef<SWidget> GenerateContainerWidgetForOutliner(const TSharedRef<SAnimOutlinerItem>& InRow) override;

private:
	TSharedRef<SWidget> BuildCurvesSubMenu();
	void FillMetadataEntryMenu(FMenuBuilder& Builder);
	void FillVariableCurveMenu(FMenuBuilder& Builder);
	void AddMetadataEntry(const FName& InCurveName);
	void CreateNewMetadataEntryClicked();
	void CreateNewMetadataEntry(const FText& CommittedText, ETextCommit::Type CommitType);
	void CreateNewCurveClicked();
	void CreateTrack(const FText& ComittedText, ETextCommit::Type CommitInfo);
	void AddVariableCurve(const FName& InCurveName);
	void DeleteAllCurves();

	/** Handlers for showing curve points */
	void HandleShowCurvePoints();
	bool IsShowCurvePointsEnabled() const;

	/** Handlers for switching to and from tree view */
	void HandleUseTreeView();
	bool IsUseTreeViewEnabled() const;
	
	/** Curve Picker Callbacks */
	void OnMetadataCurveNamePicked(const FName& InCurveName);
	void OnVariableCurveNamePicked(const FName& InCurveName);
	bool IsCurveMarkedForExclusion(const FName& InCurveName);
	
private:
	TSharedPtr<SWidget>	OutlinerWidget;
};
