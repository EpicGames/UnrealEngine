// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimTimeline/AnimTimelineTrack_FloatCurve.h"
#include "CurveEditor.h"
#include "Animation/AnimSequenceBase.h"
#include "Widgets/Layout/SBorder.h"
#include "Styling/AppStyle.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "AnimSequenceTimelineCommands.h"
#include "ScopedTransaction.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Animation/Skeleton.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Animation/AnimMontage.h"
#include "Widgets/Colors/SColorBlock.h"
#include "PersonaUtils.h"
#include "Widgets/Colors/SColorPicker.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBox.h"
#include "AnimTimeline/AnimModel_AnimSequenceBase.h"
#include "AnimTimelineClipboard.h"
#include "Animation/AnimData/AnimDataModel.h"
#include "AnimTimeline/SAnimOutlinerItem.h"
#include "Fonts/FontMeasure.h"

#define LOCTEXT_NAMESPACE "FAnimTimelineTrack_FloatCurve"

ANIMTIMELINE_IMPLEMENT_TRACK(FAnimTimelineTrack_FloatCurve);

FAnimTimelineTrack_FloatCurve::FAnimTimelineTrack_FloatCurve(const FFloatCurve* InCurve, const TSharedRef<FAnimModel>& InModel)
	: FAnimTimelineTrack_Curve(&InCurve->FloatCurve, InCurve->GetName(), 0, ERawCurveTrackTypes::RCT_Float, FText::FromName(InCurve->GetName()), FText::FromName(InCurve->GetName()), InCurve->GetColor(), InCurve->GetColor(), InModel)
	, FloatCurve(InCurve)
	, CurveName(InCurve->GetName())
	, CurveId(FAnimationCurveIdentifier(InCurve->GetName(), ERawCurveTrackTypes::RCT_Float))
	, Color(InCurve->Color)
	, Comment(InCurve->Comment) 
	, bIsMetadata(InCurve->GetCurveTypeFlag(AACF_Metadata))
{
	SetHeight(32.0f);
}

TSharedRef<SWidget> FAnimTimelineTrack_FloatCurve::MakeTimelineWidgetContainer()
{
	TSharedRef<SWidget> CurveWidget = MakeCurveWidget();

	// zoom to fit now we have a view
	CurveEditor->ZoomToFit(EAxisList::Y);

	return
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SBox)
			.HeightOverride(Height)
			[
				SAssignNew(TimelineWidgetContainer, SBorder)
				.Padding(0.0f)
				.BorderImage(bIsMetadata ? FAppStyle::GetBrush("Sequencer.Section.CollapsedSelectedSectionOverlay") : FAppStyle::GetBrush("AnimTimeline.Outliner.DefaultBorder"))
				.BorderBackgroundColor(this, &FAnimTimelineTrack_FloatCurve::GetTrackColor, false)
				[
					CurveWidget
				]
			]
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SBox)
			.HeightOverride(this, &FAnimTimelineTrack_FloatCurve::GetCommentSize)
			[
				SNew(SBorder)
				.Padding(5.0f, 0.0f)
				.Visibility(this, &FAnimTimelineTrack_FloatCurve::GetCommentVisibility)
				.ToolTipText(this, &FAnimTimelineTrack_FloatCurve::GetToolTipText)
				.BorderImage(FAppStyle::GetBrush("AnimTimeline.Outliner.DefaultBorder"))
				.BorderBackgroundColor(this, &FAnimTimelineTrack_FloatCurve::GetTrackColor, true)
				[
					SAssignNew(EditableTextComment, SInlineEditableTextBlock)
					.Text(this, &FAnimTimelineTrack_FloatCurve::GetCommentText)
					.IsSelected(this, &FAnimTimelineTrack_FloatCurve::IsSelected)
					.OnTextCommitted(this, &FAnimTimelineTrack_FloatCurve::OnCommitCurveComment)
				]
			]
		];
}

float FAnimTimelineTrack_FloatCurve::GetCommentHeight() const
{
	if(Comment.IsEmpty())
	{
		return 0.0f;
	}

	const TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
	UE::Slate::FDeprecateVector2DResult Result = FontMeasureService->Measure(Comment, FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>( "NormalText" ).Font);
	return Result.Y + 6.0f;
}

FOptionalSize FAnimTimelineTrack_FloatCurve::GetCommentSize() const
{
	return GetCommentHeight();
}

float FAnimTimelineTrack_FloatCurve::GetHeight() const
{
	return Height + GetCommentHeight();
}

TSharedRef<SWidget> FAnimTimelineTrack_FloatCurve::GenerateContainerWidgetForOutliner(const TSharedRef<SAnimOutlinerItem>& InRow)
{
	TSharedPtr<SBorder> OuterBorder;
	TSharedPtr<SHorizontalBox> InnerHorizontalBox;
	TSharedRef<SWidget> OutlinerWidget = GenerateStandardOutlinerWidget(InRow, false, OuterBorder, InnerHorizontalBox);

	UAnimMontage* AnimMontage = Cast<UAnimMontage>(GetModel()->GetAnimSequenceBase());
	bool bChildAnimMontage = AnimMontage && AnimMontage->HasParentAsset();

	InnerHorizontalBox->AddSlot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		.Padding(2.0f, 1.0f)
		.FillWidth(1.0f)
		[
			SAssignNew(EditableTextLabel, SInlineEditableTextBlock)
			.IsReadOnly(bChildAnimMontage)
			.Text(this, &FAnimTimelineTrack_FloatCurve::GetLabel)
			.IsSelected(this, &FAnimTimelineTrack_FloatCurve::IsSelected)
			.OnTextCommitted(this, &FAnimTimelineTrack_FloatCurve::OnCommitCurveName)
			.HighlightText(InRow->GetHighlightText())
		];

	if(!bChildAnimMontage)
	{
		OuterBorder->SetOnMouseDoubleClick(FPointerEventHandler::CreateSP(this, &FAnimTimelineTrack_FloatCurve::HandleDoubleClicked));
		AddCurveTrackButton(InnerHorizontalBox);
	}

	return OutlinerWidget;
}

TSharedRef<SWidget> FAnimTimelineTrack_FloatCurve::BuildCurveTrackMenu()
{
	FMenuBuilder MenuBuilder(true, GetModel()->GetCommandList());

	bIsMetadata = FloatCurve->GetCurveTypeFlag(AACF_Metadata);

	MenuBuilder.BeginSection("Curve", bIsMetadata ? LOCTEXT("CurveMetadataMenuSection", "Curve Metadata") : LOCTEXT("CurveMenuSection", "Curve"));
	{
		if(bIsMetadata)
		{
			MenuBuilder.AddMenuEntry(
				FAnimSequenceTimelineCommands::Get().ConvertMetaDataToCurve->GetLabel(),
				FAnimSequenceTimelineCommands::Get().ConvertMetaDataToCurve->GetDescription(),
				FAnimSequenceTimelineCommands::Get().ConvertMetaDataToCurve->GetIcon(),
				FUIAction(
					FExecuteAction::CreateSP(this, &FAnimTimelineTrack_FloatCurve::ConvertMetaDataToCurve)
				)
			);
		}
		else
		{
			MenuBuilder.AddMenuEntry(
				FAnimSequenceTimelineCommands::Get().EditCurve->GetLabel(),
				FAnimSequenceTimelineCommands::Get().EditCurve->GetDescription(),
				FAnimSequenceTimelineCommands::Get().EditCurve->GetIcon(),
				FUIAction(FExecuteAction::CreateSP(this, &FAnimTimelineTrack_FloatCurve::HendleEditCurve)));

			MenuBuilder.AddMenuEntry(
				FAnimSequenceTimelineCommands::Get().ConvertCurveToMetaData->GetLabel(),
				FAnimSequenceTimelineCommands::Get().ConvertCurveToMetaData->GetDescription(),
				FAnimSequenceTimelineCommands::Get().ConvertCurveToMetaData->GetIcon(),
				FUIAction(
					FExecuteAction::CreateSP(this, &FAnimTimelineTrack_FloatCurve::ConvertCurveToMetaData)
				)
			);
		}

		MenuBuilder.AddMenuEntry(
			FAnimSequenceTimelineCommands::Get().RemoveCurve->GetLabel(),
			FAnimSequenceTimelineCommands::Get().RemoveCurve->GetDescription(),
			FAnimSequenceTimelineCommands::Get().RemoveCurve->GetIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &FAnimTimelineTrack_FloatCurve::RemoveCurve)
			)
		);

		MenuBuilder.AddMenuEntry(
			MakeAttributeLambda([this]()
			{
				return FloatCurve->Comment.IsEmpty() ? FAnimSequenceTimelineCommands::Get().AddComment->GetLabel() : LOCTEXT("EditComment", "Edit Comment");
			}),
			FAnimSequenceTimelineCommands::Get().AddComment->GetDescription(),
			FAnimSequenceTimelineCommands::Get().AddComment->GetIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &FAnimTimelineTrack_FloatCurve::HandleAddComment)
			)
		);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void FAnimTimelineTrack_FloatCurve::ConvertCurveToMetaData()
{
	UAnimSequenceBase* AnimSequenceBase = GetModel()->GetAnimSequenceBase();

	IAnimationDataController& Controller = AnimSequenceBase->GetController();
	IAnimationDataController::FScopedBracket ScopedBracket(Controller, LOCTEXT("ConvertCurveToMetaData_Bracket", "Converting curve to metadata"));
	Controller.SetCurveFlag(CurveId, AACF_Metadata, true);
	Controller.SetCurveKeys(CurveId, { FRichCurveKey(0.f, 1.f) });	

	bIsMetadata = true;

	ZoomToFit();
}

void FAnimTimelineTrack_FloatCurve::ConvertMetaDataToCurve()
{
	UAnimSequenceBase* AnimSequenceBase = GetModel()->GetAnimSequenceBase();

	IAnimationDataController& Controller = AnimSequenceBase->GetController();
	IAnimationDataController::FScopedBracket ScopedBracket(Controller, LOCTEXT("CurvePanel_ConvertMetaDataToCurve", "Convert metadata to curve"));
	Controller.SetCurveFlag(CurveId, AACF_Metadata, false);
	bIsMetadata = false;
}

void FAnimTimelineTrack_FloatCurve::RemoveCurve()
{
	UAnimSequenceBase* AnimSequenceBase = GetModel()->GetAnimSequenceBase();

	if(AnimSequenceBase->GetDataModel()->FindFloatCurve(FAnimationCurveIdentifier(FloatCurve->GetName(), ERawCurveTrackTypes::RCT_Float)))
	{
		// Stop editing this curve in the external editor window
		IAnimationEditor::FCurveEditInfo EditInfo(CurveName, ERawCurveTrackTypes::RCT_Float, 0);

		AnimSequenceBase->Modify(true);

		IAnimationDataController& Controller = AnimSequenceBase->GetController();
		Controller.RemoveCurve(CurveId);

		AnimSequenceBase->PostEditChange();
	}
}

void FAnimTimelineTrack_FloatCurve::OnCommitCurveName(const FText& InText, ETextCommit::Type CommitInfo)
{
	UAnimSequenceBase* AnimSequenceBase = GetModel()->GetAnimSequenceBase();
	
	// only do this if the name isn't same
	FText CurrentCurveName = GetLabel();
	if (!CurrentCurveName.EqualToCaseIgnored(InText))
	{
		// Check that the name doesn't already exist
		const FName RequestedName = FName(*InText.ToString());
		
		const TArray<FFloatCurve>& FloatCurves = AnimSequenceBase->GetDataModel()->GetFloatCurves();
		if (FloatCurves.ContainsByPredicate([RequestedName](const FFloatCurve& Curve)
		{
			return Curve.GetName() == RequestedName;
		}))
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("InvalidName"), FText::FromName(RequestedName));
			FNotificationInfo Info(FText::Format(LOCTEXT("AnimCurveRenamedInUse", "The name \"{InvalidName}\" is already used."), Args));

			Info.bUseLargeFont = false;
			Info.ExpireDuration = 5.0f;

			TSharedPtr<SNotificationItem> Notification = FSlateNotificationManager::Get().AddNotification(Info);
			if (Notification.IsValid())
			{
				Notification->SetCompletionState(SNotificationItem::CS_Fail);
			}
			return;
		}
	

		IAnimationDataController& Controller = AnimSequenceBase->GetController();
		IAnimationDataController::FScopedBracket ScopedBracket(Controller, LOCTEXT("CurveEditor_RenameCurve", "Renaming Curve"));
      
		FAnimationCurveIdentifier NewCurveId(RequestedName, ERawCurveTrackTypes::RCT_Float);
		Controller.RenameCurve(CurveId, NewCurveId);
	}
}

FText FAnimTimelineTrack_FloatCurve::GetLabel() const
{
	return FText::FromName(FloatCurve->GetName());
}

void FAnimTimelineTrack_FloatCurve::Copy(UAnimTimelineClipboardContent* InOutClipboard) const
{
	check(InOutClipboard != nullptr)
	
	UFloatCurveCopyObject * CopyableCurve = UAnimCurveBaseCopyObject::Create<UFloatCurveCopyObject>();

	// Copy raw curve data
	CopyableCurve->Curve.SetName(FloatCurve->GetName());
	CopyableCurve->Curve.SetCurveTypeFlags(FloatCurve->GetCurveTypeFlags());
	CopyableCurve->Curve.CopyCurve(*FloatCurve);
	CopyableCurve->Curve.Comment = FloatCurve->Comment;
	
	// Copy curve identifier data
	CopyableCurve->CurveName = CurveName;
	CopyableCurve->CurveType = ERawCurveTrackTypes::RCT_Float;
	CopyableCurve->Channel = CurveId.Channel;
	CopyableCurve->Axis = CurveId.Axis;
	
	// Origin data
	CopyableCurve->OriginName = GetModel()->GetAnimSequenceBase()->GetFName();
	
	InOutClipboard->Curves.Add(CopyableCurve);
}

bool FAnimTimelineTrack_FloatCurve::CanEditCurve(int32 InCurveIndex) const
{
	return !bIsMetadata;
}

void FAnimTimelineTrack_FloatCurve::RequestRename()
{
	if(EditableTextLabel.IsValid())
	{
		EditableTextLabel->EnterEditingMode();
	}
}

void FAnimTimelineTrack_FloatCurve::AddCurveTrackButton(TSharedPtr<SHorizontalBox> InnerHorizontalBox)
{
	InnerHorizontalBox->AddSlot()
	.AutoWidth()
	.HAlign(HAlign_Right)
	.VAlign(VAlign_Center)
	.Padding(0.0f, 1.0f)
	[
		PersonaUtils::MakeTrackButton(LOCTEXT("EditCurveButtonText", "Curve"), FOnGetContent::CreateSP(this, &FAnimTimelineTrack_FloatCurve::BuildCurveTrackMenu), MakeAttributeSP(this, &FAnimTimelineTrack_FloatCurve::IsHovered))
	];

	FLinearColor CurveColor = FloatCurve->Color;
	auto SetValue = [this](FLinearColor InNewColor)
	{
		UAnimSequenceBase* AnimSequenceBase = GetModel()->GetAnimSequenceBase();
		AnimSequenceBase->GetController().SetCurveColor(CurveId, InNewColor);
		Color = InNewColor;

		// Set display curves too
		for(const TPair<FCurveModelID, TUniquePtr<FCurveModel>>& CurvePair : CurveEditor->GetCurves())
		{
			CurvePair.Value->SetColor(InNewColor);
		}
	};

	auto OnGetMenuContent = [this, CurveColor, SetValue]()
	{
		// Open a color picker
		return SNew(SColorPicker)
			.TargetColorAttribute(CurveColor)
			.UseAlpha(false)
			.DisplayInlineVersion(true)
			.OnColorCommitted_Lambda(SetValue)
			.OnInteractivePickBegin_Lambda([this]()
			{
				UAnimSequenceBase* AnimSequenceBase = GetModel()->GetAnimSequenceBase();
				AnimSequenceBase->GetController().OpenBracket(LOCTEXT("EditCurveColor_Bracket", "Editing Curve Color"));				
			})
			.OnInteractivePickEnd_Lambda([this]()
			{
				UAnimSequenceBase* AnimSequenceBase = GetModel()->GetAnimSequenceBase();
				AnimSequenceBase->GetController().CloseBracket();
			});
	};

	InnerHorizontalBox->AddSlot()
	.AutoWidth()
	.HAlign(HAlign_Right)
	.VAlign(VAlign_Fill)
	.Padding(2.0f, 0.0f, 0.0f, 0.0f)
	[
		SNew(SComboButton)
		.ToolTipText(LOCTEXT("EditCurveColor", "Edit Curve Color"))
		.ContentPadding(0.0f)
		.HasDownArrow(false)
		.ButtonStyle(FAppStyle::Get(), "Sequencer.AnimationOutliner.ColorStrip")
		.OnGetMenuContent_Lambda(OnGetMenuContent)
		.CollapseMenuOnParentFocus(true)
		.VAlign(VAlign_Fill)
		.ButtonContent()
		[
			SNew(SColorBlock)
			.Color(CurveColor)
			.ShowBackgroundForAlpha(false)
			.AlphaDisplayMode(EColorBlockAlphaDisplayMode::Ignore)
			.Size(FVector2D(OutlinerRightPadding - 2.0f, OutlinerRightPadding))
		]
	];
}

FLinearColor FAnimTimelineTrack_FloatCurve::GetCurveColor(int32 InCurveIndex) const
{ 
	return Color; 
}

void FAnimTimelineTrack_FloatCurve::GetCurveEditInfo(int32 InCurveIndex, FName& OutName, ERawCurveTrackTypes& OutType, int32& OutCurveIndex) const
{
	OutName = CurveName;
	OutType = ERawCurveTrackTypes::RCT_Float;
	OutCurveIndex = InCurveIndex;
}

void FAnimTimelineTrack_FloatCurve::HandleAddComment()
{
	if(Comment.IsEmpty())
	{
		UAnimSequenceBase* AnimSequenceBase = GetModel()->GetAnimSequenceBase();
		FString NewComment = LOCTEXT("DefaultComment", "Comment").ToString();
		AnimSequenceBase->GetController().SetCurveComment(CurveId, NewComment);
		Comment = NewComment;
	}

	ExecuteOnGameThread(UE_SOURCE_LOCATION, [WeakThis = TWeakPtr<FAnimTimelineTrack_FloatCurve>(SharedThis(this))]()
	{
		if(TSharedPtr<FAnimTimelineTrack_FloatCurve> This = WeakThis.Pin())
		{
			This->EditableTextComment->EnterEditingMode();
		}
	});
}

void FAnimTimelineTrack_FloatCurve::OnCommitCurveComment(const FText& InText, ETextCommit::Type CommitInfo)
{
	if(Comment != InText.ToString())
	{
		UAnimSequenceBase* AnimSequenceBase = GetModel()->GetAnimSequenceBase();
		AnimSequenceBase->GetController().SetCurveComment(CurveId, InText.ToString());
		Comment = FloatCurve->Comment;
	}
}

FText FAnimTimelineTrack_FloatCurve::GetCommentText() const
{
	return FText::FromString(Comment);
}

EVisibility FAnimTimelineTrack_FloatCurve::GetCommentVisibility() const
{
	return Comment.IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible;
}

bool FAnimTimelineTrack_FloatCurve::IsSelected() const
{
	return GetModel()->IsTrackSelected(SharedThis(this));
}

FSlateColor FAnimTimelineTrack_FloatCurve::GetTrackColor(bool bForComment) const
{
	if(GetModel()->IsTrackSelected(AsShared()))
	{
		return FAppStyle::GetSlateColor("SelectionColor").GetSpecifiedColor().CopyWithNewOpacity(bForComment ? 0.5f : 0.75f);
	}
	else
	{
		FLinearColor CurveColor = bIsMetadata ? Color.Desaturate(0.25f) : Color.Desaturate(0.75f);
		return bForComment ? CurveColor.CopyWithNewOpacity(CurveColor.A * 0.5f) : CurveColor;
	}
}

#undef LOCTEXT_NAMESPACE
