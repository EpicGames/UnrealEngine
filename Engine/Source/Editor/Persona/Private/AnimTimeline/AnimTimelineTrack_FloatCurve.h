// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimTimeline/AnimTimelineTrack_Curve.h"

struct FFloatCurve;
class SBorder;
class FCurveEditor;

class FAnimTimelineTrack_FloatCurve : public FAnimTimelineTrack_Curve
{
	ANIMTIMELINE_DECLARE_TRACK(FAnimTimelineTrack_FloatCurve, FAnimTimelineTrack_Curve);

public:
	FAnimTimelineTrack_FloatCurve(const FFloatCurve* InCurve, const TSharedRef<FAnimModel>& InModel);

	/** FAnimTimelineTrack_Curve interface */
	virtual TSharedRef<SWidget> MakeTimelineWidgetContainer() override;
	virtual TSharedRef<SWidget> GenerateContainerWidgetForOutliner(const TSharedRef<SAnimOutlinerItem>& InRow) override;
	virtual TSharedRef<SWidget> BuildCurveTrackMenu() override;
	virtual FText GetLabel() const override;
	virtual bool CanEditCurve(int32 InCurveIndex) const override;
	virtual bool CanRename() const override { return true; }
	virtual void RequestRename() override;
	virtual void AddCurveTrackButton(TSharedPtr<SHorizontalBox> InnerHorizontalBox) override;
	virtual FLinearColor GetCurveColor(int32 InCurveIndex) const override;
	virtual void GetCurveEditInfo(int32 InCurveIndex, FName& OutName, ERawCurveTrackTypes& OutType, int32& OutCurveIndex) const override;
	virtual bool SupportsCopy() const override { return true; }
	virtual void Copy(UAnimTimelineClipboardContent* InOutClipboard) const override;
	virtual float GetHeight() const override;
	
	/** Access the curve we are editing */
	const FFloatCurve* GetFloatCurve() { return FloatCurve; }

	/** Get this curves name */
	FName GetFName() const { return CurveName; }

private:
	void ConvertCurveToMetaData();
	void ConvertMetaDataToCurve();
	void RemoveCurve();
	void OnCommitCurveName(const FText& InText, ETextCommit::Type CommitInfo);
	float GetCommentHeight() const;
	FOptionalSize GetCommentSize() const;
	void HandleAddComment();
	void OnCommitCurveComment(const FText& InText, ETextCommit::Type CommitInfo);
	FText GetCommentText() const;
	EVisibility GetCommentVisibility() const;
	bool IsSelected() const;
	FSlateColor GetTrackColor(bool bForComment) const;

private:
	/** The curve we are editing */
	const FFloatCurve* FloatCurve;

	/** The curve name and identifier */
	FName CurveName;
	FAnimationCurveIdentifier CurveId;

	/** Cached color */
	FLinearColor Color;

	/** Cached comment */
	FString Comment;

	/** Cached metadata flag */
	bool bIsMetadata = false;

	/** Label we can edit */
	TSharedPtr<SInlineEditableTextBlock> EditableTextLabel;

	/** Comment we can edit */
	TSharedPtr<SInlineEditableTextBlock> EditableTextComment;
};
