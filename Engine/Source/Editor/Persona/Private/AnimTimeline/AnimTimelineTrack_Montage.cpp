// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimTimeline/AnimTimelineTrack_Montage.h"
#include "SAnimMontagePanel.h"
#include "Animation/AnimComposite.h"
#include "PersonaUtils.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Layout/SBorder.h"
#include "AnimTimeline/AnimTimelineTrack_MontagePanel.h"
#include "AnimTimeline/AnimModel_AnimMontage.h"
#include "Fonts/FontMeasure.h"
#include "Framework/Application/SlateApplication.h"
#include "Preferences/PersonaOptions.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "EditorFontGlyphs.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "SAnimTimingPanel.h"

#define LOCTEXT_NAMESPACE "FAnimTimelineTrack_Montage"

ANIMTIMELINE_IMPLEMENT_TRACK(FAnimTimelineTrack_Montage);

namespace MontageSectionsConstants
{
	const FVector2D TextBorderMargin(2.0f, 2.0f);
	const FVector2D TextOffset(2.0f, 2.0f);
}

class SMontageSections : public SLeafWidget
{
	SLATE_BEGIN_ARGS(SMontageSections) {}

	SLATE_ATTRIBUTE(float, ViewInputMin)

	SLATE_ATTRIBUTE(float, ViewInputMax)

	SLATE_ATTRIBUTE(bool, DisplayTiming)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<FAnimModel_AnimMontage>& InModel)
	{
		WeakModel = InModel;
		AnimMontage = CastChecked<UAnimMontage>(InModel->GetAnimSequenceBase());
		ViewInputMin = InArgs._ViewInputMin;
		ViewInputMax = InArgs._ViewInputMax;
		SelectedSectionIndex = INDEX_NONE;
		DraggedSectionIndex = INDEX_NONE;
		DraggedSectionTime = 0.0f;
		bIsSelecting = false;
		bDisplayTiming = InArgs._DisplayTiming;

		LabelFont = FCoreStyle::GetDefaultFontStyle("Regular", 10);
		IconFont = FAppStyle::Get().GetFontStyle("FontAwesome.10");

		InModel->OnHandleObjectsSelected().AddSP(this, &SMontageSections::HandleObjectsSelected);
		InModel->OnSectionTimeDragged.BindSP(this, &SMontageSections::HandleSectionTimeDragged);
	}

	FText GetSectionIconText(int32 InSectionIndex) const
	{
		auto GetNextSectionIndex = [this](UAnimMontage* InAnimMontage, int32 InSectionIndex)
		{
			const FCompositeSection& CompositeSection = InAnimMontage->CompositeSections[InSectionIndex];
		
			for(int32 OtherSectionIndex = 0; OtherSectionIndex < AnimMontage->CompositeSections.Num(); ++OtherSectionIndex)
			{
				const FCompositeSection& OtherCompositeSection = InAnimMontage->CompositeSections[OtherSectionIndex];
				if(OtherCompositeSection.SectionName == CompositeSection.NextSectionName)
				{
					return OtherSectionIndex;
				}
			}
			
			return (int32)INDEX_NONE;
		};

		// Find other section to determine what icon to give this section
		int32 NextSectionIndex = GetNextSectionIndex(AnimMontage, InSectionIndex);

		FText IconText;
		if(NextSectionIndex == INDEX_NONE)
		{
			IconText = FText::GetEmpty();
		}
		else if(NextSectionIndex == InSectionIndex)
		{
			IconText = FEditorFontGlyphs::Undo;
		}
		else if(NextSectionIndex > InSectionIndex)
		{
			IconText = FEditorFontGlyphs::Arrow_Right;
		}
		else if(NextSectionIndex < InSectionIndex)
		{
			IconText = FEditorFontGlyphs::Arrow_Left;
		}

		return IconText;
	}

	int32 GetSectionTiming(int32 InSectionIndex) const
	{
		TArray<TSharedPtr<FTimingRelevantElementBase>> TimingElements;
		SAnimTimingPanel::GetTimingRelevantElements(AnimMontage, TimingElements);

		for(int32 Idx = 0 ; Idx < TimingElements.Num() ; ++Idx)
		{
			TSharedPtr<FTimingRelevantElementBase> Element = TimingElements[Idx];

			if(Element->GetType() == ENotifyTimingElementType::Section)
			{
				// Only the notify type will return the type flags above
				FTimingRelevantElement_Section* SectionElement = static_cast<FTimingRelevantElement_Section*>(Element.Get());
				if(InSectionIndex == SectionElement->SectionIdx)
				{
					return Idx;
				}
			}
		}

		return INDEX_NONE;
	}

	virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override
	{
		FTrackScaleInfo ScaleInfo(ViewInputMin.Get(), ViewInputMax.Get(), 0, 0, AllottedGeometry.GetLocalSize());
		const TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
		const FSlateBrush* BorderBrush = FAppStyle::GetBrush("SpecialEditableTextImageNormal");
		const FLinearColor SelectedColor = FAppStyle::GetSlateColor("SelectionColor").GetSpecifiedColor();
	
		const FLinearColor MontageColor = AnimMontage->HasParentAsset() ? GetDefault<UPersonaOptions>()->SectionTimingNodeColor.Desaturate(0.75f) : GetDefault<UPersonaOptions>()->SectionTimingNodeColor;

		for(int32 SectionIndex = 0; SectionIndex < AnimMontage->CompositeSections.Num(); ++SectionIndex)
		{
			const FCompositeSection& CompositeSection = AnimMontage->CompositeSections[SectionIndex];

			FText NameText;
			if(bDisplayTiming.Get())
			{
				int32 TimingIndex = GetSectionTiming(SectionIndex);
				if(TimingIndex != INDEX_NONE)
				{
					NameText = FText::Format(LOCTEXT("SectionTimingFormat", "{0}  {1}"), FText::AsNumber(TimingIndex), FText::FromName(CompositeSection.SectionName));
				}
				else
				{
					NameText = FText::FromName(CompositeSection.SectionName);
				}
			}
			else
			{
				NameText = FText::FromName(CompositeSection.SectionName);
			}

			const FVector2D TextSize = FontMeasureService->Measure(NameText, LabelFont);
			const FVector2D TextBorderSize = TextSize + (MontageSectionsConstants::TextBorderMargin * 2.0f);

			const FText IconText = GetSectionIconText(SectionIndex);
			const FVector2D TextIconSize = FontMeasureService->Measure(IconText, LabelFont);
			const FVector2D TextIconBorderSize = TextIconSize + (MontageSectionsConstants::TextBorderMargin * 2.0f);

			const FVector2D TotalBorderSize(TextBorderSize.X + TextIconBorderSize.X, FMath::Max(TextBorderSize.Y, TextIconBorderSize.Y));

			const float LabelPosX = ScaleInfo.InputToLocalX(DraggedSectionIndex == SectionIndex ? DraggedSectionTime : CompositeSection.GetTime());
			const float LabelPosY = static_cast<float>((AllottedGeometry.GetLocalSize().Y * 0.5) - (TextBorderSize.Y * 0.5));
			const float IconPosY = static_cast<float>((AllottedGeometry.GetLocalSize().Y * 0.5) - (TextIconBorderSize.Y * 0.5));

			const float RightEdgeToNotify = static_cast<float>(AllottedGeometry.Size.X - (LabelPosX + TextBorderSize.X));
			const bool bDrawLabelOnLeft = RightEdgeToNotify < 0.0f;

			FSlateDrawElement::MakeBox( 
				OutDrawElements,
				++LayerId,
				AllottedGeometry.ToPaintGeometry(TotalBorderSize, FSlateLayoutTransform(FVector2D(bDrawLabelOnLeft ? LabelPosX - TotalBorderSize.X : LabelPosX, LabelPosY))),
				BorderBrush,
				ESlateDrawEffect::None,
				SelectedSectionIndex == SectionIndex ? SelectedColor : MontageColor);

			FSlateDrawElement::MakeText( 
				OutDrawElements,
				++LayerId,
				AllottedGeometry.ToPaintGeometry(TextSize, FSlateLayoutTransform(FVector2D(bDrawLabelOnLeft ? (LabelPosX - TotalBorderSize.X) + MontageSectionsConstants::TextOffset.X : LabelPosX + MontageSectionsConstants::TextOffset.X, LabelPosY + MontageSectionsConstants::TextOffset.Y))),
				NameText,
				LabelFont,
				ESlateDrawEffect::None,
				FLinearColor::Black);

			FSlateDrawElement::MakeText( 
				OutDrawElements,
				++LayerId,
				AllottedGeometry.ToPaintGeometry(TextIconSize, FSlateLayoutTransform(FVector2D(bDrawLabelOnLeft ? (LabelPosX - TotalBorderSize.X) + TextBorderSize.X + (MontageSectionsConstants::TextBorderMargin.X * 2.0f) + MontageSectionsConstants::TextOffset.X : LabelPosX + MontageSectionsConstants::TextOffset.X + (MontageSectionsConstants::TextBorderMargin.X * 2.0f) + TextBorderSize.X, IconPosY + MontageSectionsConstants::TextOffset.Y))),
				IconText,
				IconFont,
				ESlateDrawEffect::None,
				FLinearColor::Black);
		}

		return LayerId;
	}

	virtual FVector2D ComputeDesiredSize(float LayoutScaleMultiplier) const override
	{
		return FVector2D(5000.0f, 24.0f);
	}

	bool HitTestSections(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, int32& OutHitSectionIndex) const
	{
		FTrackScaleInfo ScaleInfo(ViewInputMin.Get(), ViewInputMax.Get(), 0, 0, MyGeometry.GetLocalSize());
		const TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();

		for(int32 SectionIndex = 0; SectionIndex < AnimMontage->CompositeSections.Num(); ++SectionIndex)
		{
			const FCompositeSection& CompositeSection = AnimMontage->CompositeSections[SectionIndex];

			const FText NameText = FText::FromName(CompositeSection.SectionName);
			const FVector2D TextSize = FontMeasureService->Measure(NameText, LabelFont);
			const FVector2D TextBorderSize = TextSize + (MontageSectionsConstants::TextBorderMargin * 2.0f);

			const FText IconText = GetSectionIconText(SectionIndex);
			const FVector2D TextIconSize = FontMeasureService->Measure(IconText, LabelFont);
			const FVector2D TextIconBorderSize = TextIconSize + (MontageSectionsConstants::TextBorderMargin * 2.0f);

			const FVector2D TotalBorderSize(TextBorderSize.X + TextIconBorderSize.X, FMath::Max(TextBorderSize.Y, TextIconBorderSize.Y));

			const double LabelPosX = ScaleInfo.InputToLocalX(CompositeSection.GetTime());
			const double LabelPosY = (MyGeometry.GetLocalSize().Y * 0.5) - (TotalBorderSize.Y * 0.5);

			const double RightEdgeToNotify = MyGeometry.Size.X - (LabelPosX + TotalBorderSize.X);
			const bool bDrawLabelOnLeft = RightEdgeToNotify < 0.0;

			const FGeometry LabelGeometry = MyGeometry.MakeChild(
				TotalBorderSize,
				FSlateLayoutTransform(FVector2D(bDrawLabelOnLeft ? LabelPosX - TotalBorderSize.X: LabelPosX, LabelPosY))
			);

			if(LabelGeometry.IsUnderLocation(MouseEvent.GetScreenSpacePosition()))
			{
				OutHitSectionIndex = SectionIndex;
				return true;
			}
		}

		return false;
	}

	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		int32 HitSectionIndex = INDEX_NONE;
		if(HitTestSections(MyGeometry, MouseEvent, HitSectionIndex))
		{
			return FReply::Handled().CaptureMouse(AsShared());
		}

		return FReply::Unhandled();	
	}

	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		if(!AnimMontage->HasParentAsset())
		{
			SelectedSectionIndex = INDEX_NONE;

			int32 HitSectionIndex = INDEX_NONE;
			if(HitTestSections(MyGeometry, MouseEvent, HitSectionIndex))
			{
				TGuardValue<bool> GuardValue(bIsSelecting, true);
				SelectedSectionIndex = HitSectionIndex;
				WeakModel.Pin()->ShowSectionInDetailsView(HitSectionIndex);
			}
			else
			{
				TGuardValue<bool> GuardValue(bIsSelecting, true);
				WeakModel.Pin()->ClearDetailsView();
			}

			if(MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
			{
				FMenuBuilder MenuBuilder(true, nullptr);

				FTrackScaleInfo ScaleInfo(ViewInputMin.Get(), ViewInputMax.Get(), 0, 0, MyGeometry.GetLocalSize());
				SummonTrackContextMenu(MenuBuilder, ScaleInfo.LocalXToInput(static_cast<float>(MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition()).X)), SelectedSectionIndex);

				FSlateApplication::Get().PushMenu(
					AsShared(),
					FWidgetPath(),
					MenuBuilder.MakeWidget(),
					FSlateApplication::Get().GetCursorPos(),
					FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu)
					);
			}
		}

		return FReply::Handled().ReleaseMouseCapture();
	}

	virtual bool SupportsKeyboardFocus() const override
	{
		return true;
	}

	void HandleObjectsSelected(const TArray<UObject*>& InObjects)
	{
		if(!bIsSelecting)
		{
			SelectedSectionIndex = INDEX_NONE;
		}
	}

	void HandleSectionTimeDragged(int32 SectionIndex, float Time, bool bIsDragging)
	{
		if(bIsDragging)
		{
			DraggedSectionIndex = SectionIndex;
			DraggedSectionTime = Time;
		}
		else
		{
			DraggedSectionIndex = INDEX_NONE;
		}
	}

	void SummonTrackContextMenu(FMenuBuilder& MenuBuilder, float DataPosX, int32 SectionIndex)
	{
		TSharedPtr<SAnimMontagePanel> MontagePanel = WeakModel.Pin()->GetMontagePanel()->GetAnimMontagePanel();

		// Sections
		MenuBuilder.BeginSection("AnimMontageSections", LOCTEXT("Sections", "Sections"));
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("NewMontageSection", "New Montage Section"), 
				LOCTEXT("NewMontageSectionToolTip", "Adds a new Montage Section"),
				FSlateIcon(), 
				FUIAction(
					FExecuteAction::CreateSP(MontagePanel.Get(), &SAnimMontagePanel::OnNewSectionClicked, DataPosX),
					FCanExecuteAction::CreateSP(MontagePanel.Get(), &SAnimMontagePanel::CanAddNewSection)
				)
			);

			if (SectionIndex != INDEX_NONE && AnimMontage->CompositeSections.Num() > 1) // Can't delete the last section!
			{
				MenuBuilder.AddMenuEntry(
					LOCTEXT("DeleteMontageSection", "Delete Montage Section"), 
					LOCTEXT("DeleteMontageSectionToolTip", "Deletes Montage Section"), 
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateSP(MontagePanel.Get(), &SAnimMontagePanel::RemoveSection, SectionIndex)
					)
				);

				FCompositeSection& Section = AnimMontage->CompositeSections[SectionIndex];

				// Add item to directly set section time
				TSharedRef<SWidget> TimeWidget = 
					SNew(SBox)
					.HAlign(HAlign_Right)
					.ToolTipText(LOCTEXT("SetSectionTimeToolTip", "Set the time of this section directly"))
					[
						SNew(SBox)
						.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
						.WidthOverride(100.0f)
						[
							SNew(SNumericEntryBox<float>)
							.Font(FAppStyle::GetFontStyle(TEXT("MenuItem.Font")))
							.MinValue(0.0f)
							.MaxValue(AnimMontage->GetPlayLength())
							.Value(Section.GetTime())
							.AllowSpin(true)
							.OnValueCommitted_Lambda([this, SectionIndex](float InValue, ETextCommit::Type InCommitType)
							{
								if (AnimMontage->CompositeSections.IsValidIndex(SectionIndex))
								{
									WeakModel.Pin()->GetMontagePanel()->GetAnimMontagePanel()->SetSectionTime(SectionIndex, InValue);
								}

								FSlateApplication::Get().DismissAllMenus();
							})
						]
					];

				MenuBuilder.AddWidget(TimeWidget, LOCTEXT("SectionTimeMenuText", "Section Time"));

				// Add item to directly set section frame
				TSharedRef<SWidget> FrameWidget = 
					SNew(SBox)
					.HAlign( HAlign_Right )
					.ToolTipText(LOCTEXT("SetFrameToolTip", "Set the frame of this section directly"))
					[
						SNew(SBox)
						.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
						.WidthOverride(100.0f)
						[
							SNew(SNumericEntryBox<int32>)
							.Font(FAppStyle::GetFontStyle(TEXT("MenuItem.Font")))
							.MinValue(0)
							.MaxValue(AnimMontage->GetNumberOfSampledKeys())
							.Value(AnimMontage->GetFrameAtTime(Section.GetTime()))
							.AllowSpin(true)						
							.OnValueCommitted_Lambda([this, SectionIndex](int32 InValue, ETextCommit::Type InCommitType)
							{
								if (AnimMontage->CompositeSections.IsValidIndex(SectionIndex))
								{
									float NewTime = FMath::Clamp(AnimMontage->GetTimeAtFrame(InValue), 0.0f, AnimMontage->GetPlayLength());
									WeakModel.Pin()->GetMontagePanel()->GetAnimMontagePanel()->SetSectionTime(SectionIndex, NewTime);
								}

								FSlateApplication::Get().DismissAllMenus();
							})
						]
					];

				MenuBuilder.AddWidget(FrameWidget, LOCTEXT("SectionFrameMenuText", "Section Frame"));
			}
		}
		MenuBuilder.EndSection();
	}

	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override
	{
		if(InKeyEvent.GetKey() == EKeys::Delete && SelectedSectionIndex != INDEX_NONE)
		{
			WeakModel.Pin()->GetMontagePanel()->GetAnimMontagePanel()->RemoveSection(SelectedSectionIndex);
			SelectedSectionIndex = INDEX_NONE;

			return FReply::Handled();
		}

		return FReply::Unhandled();
	}

	TAttribute<float> ViewInputMin;
	TAttribute<float> ViewInputMax;
	TAttribute<bool> bDisplayTiming;

	UAnimMontage* AnimMontage;

	TWeakPtr<FAnimModel_AnimMontage> WeakModel;

	FSlateFontInfo LabelFont;

	FSlateFontInfo IconFont;

	int32 SelectedSectionIndex;

	int32 DraggedSectionIndex;

	float DraggedSectionTime;

	bool bIsSelecting;
};


FAnimTimelineTrack_Montage::FAnimTimelineTrack_Montage(const TSharedRef<FAnimModel_AnimMontage>& InModel)
	: FAnimTimelineTrack(FText::GetEmpty(), FText::GetEmpty(), InModel)
{
	SetHeight(24.0f);
}

TSharedRef<SWidget> FAnimTimelineTrack_Montage::GenerateContainerWidgetForOutliner(const TSharedRef<SAnimOutlinerItem>& InRow)
{
	TSharedPtr<SBorder> OuterBorder;
	TSharedPtr<SHorizontalBox> InnerHorizontalBox;
	TSharedPtr<SWidget> OutlinerWidget = GenerateStandardOutlinerWidget(InRow, true, OuterBorder, InnerHorizontalBox);

	OuterBorder->SetBorderBackgroundColor(FAppStyle::GetColor("AnimTimeline.Outliner.HeaderColor"));

	InnerHorizontalBox->AddSlot()
		.AutoWidth()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		.Padding(OutlinerRightPadding, 1.0f)
		[
			PersonaUtils::MakeTrackButton(LOCTEXT("EditMontageButtonText", "Montage"), FOnGetContent::CreateSP(this, &FAnimTimelineTrack_Montage::BuildMontageSubMenu), MakeAttributeSP(this, &FAnimTimelineTrack_Montage::IsHovered))
		];

	return OutlinerWidget.ToSharedRef();
}

TSharedRef<SWidget> FAnimTimelineTrack_Montage::GenerateContainerWidgetForTimeline()
{
	UAnimMontage* AnimMontage = CastChecked<UAnimMontage>(GetModel()->GetAnimSequenceBase());

	return SNew(SMontageSections, StaticCastSharedRef<FAnimModel_AnimMontage>(GetModel()))
		.IsEnabled(!AnimMontage->HasParentAsset())
		.ViewInputMin(this, &FAnimTimelineTrack_Montage::GetViewMinInput)
		.ViewInputMax(this, &FAnimTimelineTrack_Montage::GetViewMaxInput)
		.DisplayTiming(this, &FAnimTimelineTrack_Montage::IsShowSectionTimingEnabled);
}

TSharedRef<SWidget> FAnimTimelineTrack_Montage::BuildMontageSubMenu()
{
	UAnimMontage* AnimMontage = CastChecked<UAnimMontage>(GetModel()->GetAnimSequenceBase());
	TSharedRef<FAnimModel_AnimMontage> MontageModel = StaticCastSharedRef<FAnimModel_AnimMontage>(GetModel());

	FMenuBuilder MenuBuilder(true, GetModel()->GetCommandList());

	bool bIsChildAnimMontage = AnimMontage->HasParentAsset();
	if(bIsChildAnimMontage)
	{
		MenuBuilder.BeginSection("ParentMontage", LOCTEXT("ParentMontageMenuSection", "Parent"));
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("FindParent", "Find parent"),
				LOCTEXT("FindParentInCBToolTip", "Find parent in Content Browser"),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Search"),
				FUIAction(FExecuteAction::CreateSP(this, &FAnimTimelineTrack_Montage::OnFindParentClassInContentBrowserClicked))
			);

			MenuBuilder.AddMenuEntry(
				LOCTEXT("EditParent", "Edit parent"),
				LOCTEXT("EditParentToolTip", "Open parent in editor"),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Edit"),
				FUIAction(FExecuteAction::CreateSP(this, &FAnimTimelineTrack_Montage::OnEditParentClassClicked))
			);
		}
		MenuBuilder.EndSection();
	}
	else
	{
		// Slots
		MenuBuilder.BeginSection("AnimMontageSlots", LOCTEXT("Slots", "Slots") );
		{
			MenuBuilder.AddSubMenu(
				LOCTEXT("NewSlot", "New Slot"),
				LOCTEXT("NewSlotToolTip", "Adds a new Slot"), 
				FNewMenuDelegate::CreateSP(&MontageModel->GetMontagePanel()->GetAnimMontagePanel().Get(), &SAnimMontagePanel::BuildNewSlotMenu)
			);

			MenuBuilder.AddMenuEntry(
				LOCTEXT("OpenAnimSlotManager", "Slot Manager..."),
				LOCTEXT("OpenAnimSlotManagerToolTip", "Open Anim Slot Manager to edit Slots and Groups."),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateSP(&MontageModel->GetMontagePanel()->GetAnimMontagePanel().Get(), &SAnimMontagePanel::OnOpenAnimSlotManager))
			);
		}
		MenuBuilder.EndSection();
	}

	MenuBuilder.BeginSection("TimingPanelOptions", LOCTEXT("TimingPanelOptionsHeader", "Options"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("ToggleTimingNodes_Sections", "Show Section Timing Nodes"),
			LOCTEXT("ShowSectionTimingNodes", "Show or hide the timing display for sections"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &FAnimTimelineTrack_Montage::OnShowSectionTiming),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &FAnimTimelineTrack_Montage::IsShowSectionTimingEnabled)
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

FText FAnimTimelineTrack_Montage::GetLabel() const
{
	UAnimMontage* AnimMontage = CastChecked<UAnimMontage>(GetModel()->GetAnimSequenceBase());
	bool bIsChildAnimMontage = AnimMontage->HasParentAsset();

	return bIsChildAnimMontage ? 
		LOCTEXT("ChildMontageTitle", "Child Montage") : 
		FText::Format(LOCTEXT("MontageTitle", "Montage ({0})"), FText::FromName(AnimMontage->GetGroupName()));
}

FText FAnimTimelineTrack_Montage::GetToolTipText() const
{
	UAnimMontage* AnimMontage = CastChecked<UAnimMontage>(GetModel()->GetAnimSequenceBase());
	bool bIsChildAnimMontage = AnimMontage->HasParentAsset();

	return bIsChildAnimMontage ? 
		LOCTEXT("ChildMontageTooltip", "This is a child anim montage. To edit the layout, please go to the parent montage") : 
		FText::Format(LOCTEXT("MontageTooltip", "This montage uses the group '{0}'. The group is set by the first slot's group."), FText::FromName(AnimMontage->GetGroupName()));
}

void FAnimTimelineTrack_Montage::OnFindParentClassInContentBrowserClicked()
{
	UAnimMontage* AnimMontage = CastChecked<UAnimMontage>(GetModel()->GetAnimSequenceBase());

	UObject* ParentClass = AnimMontage->ParentAsset;
	if (ParentClass != nullptr)
	{
		TArray<UObject*> ParentObjectList;
		ParentObjectList.Add(ParentClass);
		GEditor->SyncBrowserToObjects(ParentObjectList);
	}
}

void FAnimTimelineTrack_Montage::OnEditParentClassClicked()
{
	UAnimMontage* AnimMontage = CastChecked<UAnimMontage>(GetModel()->GetAnimSequenceBase());

	UObject* ParentClass = AnimMontage->ParentAsset;
	if (ParentClass != nullptr)
	{
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(ParentClass);
	}
}

void FAnimTimelineTrack_Montage::OnShowSectionTiming()
{
	StaticCastSharedRef<FAnimModel_AnimMontage>(GetModel())->ToggleSectionTimingDisplay();
}

bool FAnimTimelineTrack_Montage::IsShowSectionTimingEnabled() const
{
	return StaticCastSharedRef<FAnimModel_AnimMontage>(GetModel())->IsSectionTimingDisplayEnabled();
}

#undef LOCTEXT_NAMESPACE
