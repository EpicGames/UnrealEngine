// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimTimeline/AnimTimelineTrack.h"

class FAnimTimelineTrack_MontagePanel;
class SVerticalBox;
class FAnimModel_AnimMontage;

/** Header track for montage information */
class FAnimTimelineTrack_Montage : public FAnimTimelineTrack
{
	ANIMTIMELINE_DECLARE_TRACK(FAnimTimelineTrack_Montage, FAnimTimelineTrack);

public:
	FAnimTimelineTrack_Montage(const TSharedRef<FAnimModel_AnimMontage>& InModel);

	/** FAnimTimelineTrack interface */
	virtual TSharedRef<SWidget> GenerateContainerWidgetForOutliner(const TSharedRef<SAnimOutlinerItem>& InRow) override;
	virtual TSharedRef<SWidget> GenerateContainerWidgetForTimeline() override;
	virtual FText GetLabel() const override;
	virtual FText GetToolTipText() const override;
	virtual bool SupportsFiltering() const override { return false; }

	TSharedRef<SWidget> BuildMontageSubMenu();

	void OnFindParentClassInContentBrowserClicked();

	void OnEditParentClassClicked();

	void SetMontagePanel(const TSharedRef<FAnimTimelineTrack_MontagePanel>& InMontagePanel) { MontagePanel = InMontagePanel; }

private:
	void OnShowSectionTiming();
	bool IsShowSectionTimingEnabled() const;

private:
	TWeakPtr<FAnimTimelineTrack_MontagePanel> MontagePanel;
};
