// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimTimeline/AnimTimelineTrack_MontagePanel.h"
#include "Animation/Skeleton.h"
#include "SAnimMontagePanel.h"
#include "Animation/AnimComposite.h"
#include "PersonaUtils.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Images/SImage.h"
#include "AnimTimeline/AnimModel_AnimMontage.h"

#define LOCTEXT_NAMESPACE "FAnimTimelineTrack_MontagePanel"

ANIMTIMELINE_IMPLEMENT_TRACK(FAnimTimelineTrack_MontagePanel);

namespace
{
	static const float MontageSlotTrackHeight = 48.0f;
}

FAnimTimelineTrack_MontagePanel::FAnimTimelineTrack_MontagePanel(const TSharedRef<FAnimModel>& InModel)
	: FAnimTimelineTrack(FText::GetEmpty(), LOCTEXT("MontageSlotTooltip", "Montage slot"), InModel)
{
	UAnimMontage* AnimMontage = CastChecked<UAnimMontage>(GetModel()->GetAnimSequenceBase());

	SetHeight(AnimMontage->SlotAnimTracks.Num() * MontageSlotTrackHeight);
}

TSharedRef<SWidget> FAnimTimelineTrack_MontagePanel::GenerateContainerWidgetForTimeline()
{
	return GetAnimMontagePanel();
}

TSharedRef<SWidget> FAnimTimelineTrack_MontagePanel::GenerateContainerWidgetForOutliner(const TSharedRef<SAnimOutlinerItem>& InRow)
{
	TSharedRef<SWidget> Widget = 
		SNew(SHorizontalBox)
		.ToolTipText(this, &FAnimTimelineTrack::GetToolTipText)
		+SHorizontalBox::Slot()
		[
			SAssignNew(OutlinerWidget, SVerticalBox)
		];

	RefreshOutlinerWidget();

	return Widget;
}

void FAnimTimelineTrack_MontagePanel::RefreshOutlinerWidget()
{
	UAnimMontage* AnimMontage = CastChecked<UAnimMontage>(GetModel()->GetAnimSequenceBase());
	USkeleton* Skeleton = AnimMontage->GetSkeleton();

	// Make sure all slots defined in the montage are registered in our skeleton.
	for(FSlotAnimationTrack& SlotAnimationTrack : AnimMontage->SlotAnimTracks)
	{
		Skeleton->RegisterSlotNode(SlotAnimationTrack.SlotName);
	}

	OutlinerWidget->ClearChildren();

	int32 SlotIndex = 0;
	for(FSlotAnimationTrack& SlotAnimationTrack : AnimMontage->SlotAnimTracks)
	{
		TSharedPtr<SBox> SlotBox;

		OutlinerWidget->AddSlot()
			.AutoHeight()
			[
				SAssignNew(SlotBox, SBox)
				.HeightOverride(MontageSlotTrackHeight)
			];

		auto LabelLambda = [AnimMontage, Skeleton, SlotIndex]()
		{ 
			if(AnimMontage->SlotAnimTracks.IsValidIndex(SlotIndex))
			{
				return FText::FromString(FString::Printf(TEXT("%s.%s"), *Skeleton->GetSlotGroupName(AnimMontage->SlotAnimTracks[SlotIndex].SlotName).ToString(), *AnimMontage->SlotAnimTracks[SlotIndex].SlotName.ToString()));
			}
			else
			{
				return FText::GetEmpty();
			}
		};

		TSharedPtr<SHorizontalBox> HorizontalBox;

		SlotBox->SetContent(
			SNew(SBorder)
			.ToolTipText_Lambda(LabelLambda)
			.BorderImage(FAppStyle::GetBrush("Sequencer.Section.BackgroundTint"))
			.BorderBackgroundColor(FAppStyle::GetColor("AnimTimeline.Outliner.ItemColor"))
			[
				SAssignNew(HorizontalBox, SHorizontalBox)
				+SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Left)
				.Padding(30.0f, 0.0f, 0.0f, 0.0f)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.Padding(0.0f, 0.0f, 2.0f, 0.0f)
					.AutoWidth()
					[
						SNew(SImage)
						.Image(FAppStyle::GetBrush("AnimSlotManager.Warning"))
						.Visibility(this, &FAnimTimelineTrack_MontagePanel::GetSlotWarningVisibility, SlotIndex)
						.ToolTipText(this, &FAnimTimelineTrack_MontagePanel::GetSlotWarningText, SlotIndex)
					]
					+SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.FillWidth(1.0f)
					[
						SNew(STextBlock)
						.TextStyle(&FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("AnimTimeline.Outliner.Label"))
						.Text_Lambda(LabelLambda)
					]
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Right)
				.Padding(OutlinerRightPadding, 1.0f)
				[
					PersonaUtils::MakeTrackButton(LOCTEXT("AddSlotButtonText", "Slot"), FOnGetContent::CreateSP(this, &FAnimTimelineTrack_MontagePanel::BuildMontageSlotSubMenu, SlotIndex), MakeAttributeSP(SlotBox.Get(), &SWidget::IsHovered))
				]
			]
		);

		SlotIndex++;
	}

	UpdateSlotGroupWarning();
}

TSharedRef<SWidget> FAnimTimelineTrack_MontagePanel::BuildMontageSlotSubMenu(int32 InSlotIndex)
{
	FMenuBuilder MenuBuilder(true, GetModel()->GetCommandList());

	UAnimMontage* AnimMontage = CastChecked<UAnimMontage>(GetModel()->GetAnimSequenceBase());

	if(!AnimMontage->HasParentAsset())
	{
		MenuBuilder.BeginSection("AnimMontageEditSlots", LOCTEXT("EditSlot", "Edit Slot"));
		{
			MenuBuilder.AddSubMenu(
				LOCTEXT("SlotName", "Slot name"),
				LOCTEXT("SlotNameTooltip", "Change the name of this slot"),
				FNewMenuDelegate::CreateSP(this, &FAnimTimelineTrack_MontagePanel::BuildSlotNameMenu, InSlotIndex)
			);

			MenuBuilder.AddSubMenu(
				LOCTEXT("NewSlot", "New Slot"),
			 	LOCTEXT("NewSlotToolTip", "Adds a new Slot"), 
				FNewMenuDelegate::CreateSP(AnimMontagePanel.Get(), &SAnimMontagePanel::BuildNewSlotMenu)
			);

			if(InSlotIndex != INDEX_NONE)
			{
				MenuBuilder.AddMenuEntry(
					LOCTEXT("DeleteSlot", "Delete Slot"), 
					LOCTEXT("DeleteSlotToolTip", "Deletes Slot"), 
					FSlateIcon(), 
					FUIAction(
						FExecuteAction::CreateSP(AnimMontagePanel.Get(), &SAnimMontagePanel::RemoveMontageSlot, InSlotIndex),
						FCanExecuteAction::CreateSP(AnimMontagePanel.Get(), &SAnimMontagePanel::CanRemoveMontageSlot, InSlotIndex)
					)
				);

				MenuBuilder.AddMenuEntry(
					LOCTEXT("DuplicateSlot", "Duplicate Slot"), 
					LOCTEXT("DuplicateSlotToolTip", "Duplicates the selected slot"), 
					FSlateIcon(), 
					FUIAction(FExecuteAction::CreateSP(AnimMontagePanel.Get(), &SAnimMontagePanel::DuplicateMontageSlot, InSlotIndex))
				);
			}
		}
		MenuBuilder.EndSection();
	}

	MenuBuilder.BeginSection("AnimMontageOtherOptions", LOCTEXT("OtherOptions", "Other Options"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("PreviewSlot", "Preview Slot"), 
			LOCTEXT("PreviewSlotToolTip", "Preview this slot in the viewport"), 
			FSlateIcon("CoreStyle", "ToggleButtonCheckbox"), 
			FUIAction(
				FExecuteAction::CreateSP(AnimMontagePanel.Get(), &SAnimMontagePanel::OnSlotPreviewedChanged, InSlotIndex),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(AnimMontagePanel.Get(), &SAnimMontagePanel::IsSlotPreviewed, InSlotIndex)
			),
			NAME_None,
			EUserInterfaceActionType::Check
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("OpenAnimSlotManager", "Slot Manager..."), 
			LOCTEXT("OpenAnimSlotManagerToolTip", "Open Anim Slot Manager to edit Slots and Groups."), 
			FSlateIcon(), 
			FUIAction(FExecuteAction::CreateSP(AnimMontagePanel.Get(), &SAnimMontagePanel::OnOpenAnimSlotManager))
		);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void FAnimTimelineTrack_MontagePanel::BuildSlotNameMenu(FMenuBuilder& InMenuBuilder, int32 InSlotIndex)
{
	UAnimMontage* AnimMontage = CastChecked<UAnimMontage>(GetModel()->GetAnimSequenceBase());
	USkeleton* Skeleton = AnimMontage->GetSkeleton();

	InMenuBuilder.BeginSection("AnimMontageAvailableSlots", LOCTEXT("AvailableSlots", "Available Slots") );
	{
		for (const FAnimSlotGroup& SlotGroup : Skeleton->GetSlotGroups())
		{
			for (const FName& SlotName : SlotGroup.SlotNames)
			{
				FText SlotItemText = FText::FromString(FString::Printf(TEXT("%s.%s"), *SlotGroup.GroupName.ToString(), *SlotName.ToString()));

				InMenuBuilder.AddMenuEntry(
					SlotItemText,
					FText::Format(LOCTEXT("SlotTooltipFormat", "Set this slot's name to '{0}'"), SlotItemText),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateSP(this, &FAnimTimelineTrack_MontagePanel::OnSetSlotName, SlotName, InSlotIndex),
						FCanExecuteAction(),
						FIsActionChecked::CreateSP(this, &FAnimTimelineTrack_MontagePanel::IsSlotNameSet, SlotName, InSlotIndex)
					),
					NAME_None,
					EUserInterfaceActionType::RadioButton
				);
			}
		}
	}
	InMenuBuilder.EndSection();
}

void FAnimTimelineTrack_MontagePanel::OnSetSlotName(FName InName, int32 InSlotIndex)
{
	UAnimMontage* AnimMontage = CastChecked<UAnimMontage>(GetModel()->GetAnimSequenceBase());
	if(AnimMontage->SlotAnimTracks.IsValidIndex(InSlotIndex))
	{
		if (AnimMontage->GetSkeleton()->ContainsSlotName(InName))
		{
			AnimMontagePanel->RenameSlotNode(InSlotIndex, InName.ToString());

			UpdateSlotGroupWarning();
		}
	}
}

bool FAnimTimelineTrack_MontagePanel::IsSlotNameSet(FName InName, int32 InSlotIndex)
{
	UAnimMontage* AnimMontage = CastChecked<UAnimMontage>(GetModel()->GetAnimSequenceBase());
	if(AnimMontage->SlotAnimTracks.IsValidIndex(InSlotIndex))
	{
		return AnimMontage->SlotAnimTracks[InSlotIndex].SlotName == InName;
	}

	return false;
}

EVisibility FAnimTimelineTrack_MontagePanel::GetSlotWarningVisibility(int32 InSlotIndex) const
{
	return WarningInfo[InSlotIndex].bWarning ? EVisibility::Visible : EVisibility::Collapsed;
}

FText FAnimTimelineTrack_MontagePanel::GetSlotWarningText(int32 InSlotIndex) const
{
	return WarningInfo[InSlotIndex].WarningText;
}

void FAnimTimelineTrack_MontagePanel::UpdateSlotGroupWarning()
{
	UAnimMontage* AnimMontage = CastChecked<UAnimMontage>(GetModel()->GetAnimSequenceBase());
	FName MontageGroupName = AnimMontage->GetGroupName();

	WarningInfo.Empty();

	int32 NumAnimTracks = AnimMontage->SlotAnimTracks.Num();
	if (NumAnimTracks > 0)
	{
		WarningInfo.SetNum(NumAnimTracks);

		TArray<FName> UniqueSlotNameList;
		for (int32 TrackIndex = 0; TrackIndex < NumAnimTracks; TrackIndex++)
		{
			FName CurrentSlotName = AnimMontage->SlotAnimTracks[TrackIndex].SlotName;
			FName CurrentSlotGroupName = AnimMontage->GetSkeleton()->GetSlotGroupName(CurrentSlotName);
			
			// Verify that slot names are unique.
			bool bSlotNameAlreadyInUse = UniqueSlotNameList.Contains(CurrentSlotName);
			if (!bSlotNameAlreadyInUse)
			{
				UniqueSlotNameList.Add(CurrentSlotName);
			}

			bool bDifferentGroupName = (CurrentSlotGroupName != MontageGroupName);
			WarningInfo[TrackIndex].bWarning = bDifferentGroupName || bSlotNameAlreadyInUse;

			FTextBuilder TextBuilder;

			if (bDifferentGroupName)
			{
				TextBuilder.AppendLine(FText::Format(LOCTEXT("AnimMontagePanel_SlotGroupMismatchToolTipText", "Slot's group '{0}' is different than the Montage's group '{1}'. All slots must belong to the same group."), FText::FromName(CurrentSlotGroupName), FText::FromName(MontageGroupName)));
			}
			if (bSlotNameAlreadyInUse)
			{
				TextBuilder.AppendLine(FText::Format(LOCTEXT("AnimMontagePanel_SlotNameAlreadyInUseToolTipText", "Slot named '{0}' is already used in this Montage. All slots must be unique"), FText::FromName(CurrentSlotName)));
			}

			WarningInfo[TrackIndex].WarningText = TextBuilder.ToText();
		}
	}
}

TSharedRef<SAnimMontagePanel> FAnimTimelineTrack_MontagePanel::GetAnimMontagePanel()
{ 
	if(!AnimMontagePanel.IsValid())
	{
		UAnimMontage* AnimMontage = CastChecked<UAnimMontage>(GetModel()->GetAnimSequenceBase());

		AnimMontagePanel = SNew(SAnimMontagePanel, StaticCastSharedRef<FAnimModel_AnimMontage>(GetModel()))
			.Montage(AnimMontage)
			.bChildAnimMontage(AnimMontage->HasParentAsset())
			.InputMin(this, &FAnimTimelineTrack_MontagePanel::GetMinInput)
			.InputMax(this, &FAnimTimelineTrack_MontagePanel::GetMaxInput)
			.ViewInputMin(this, &FAnimTimelineTrack_MontagePanel::GetViewMinInput)
			.ViewInputMax(this, &FAnimTimelineTrack_MontagePanel::GetViewMaxInput)
			.OnSetInputViewRange(this, &FAnimTimelineTrack_MontagePanel::OnSetInputViewRange)
			.OnInvokeTab(GetModel()->OnInvokeTab);
	}

	return AnimMontagePanel.ToSharedRef(); 
}

#undef LOCTEXT_NAMESPACE
