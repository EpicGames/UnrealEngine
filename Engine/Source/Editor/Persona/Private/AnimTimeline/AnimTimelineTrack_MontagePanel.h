// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimTimeline/AnimTimelineTrack.h"

class SAnimMontagePanel;
class SVerticalBox;
class FMenuBuilder;

/** A timeline track that re-uses the legacy panel widget to display anim montages */
class FAnimTimelineTrack_MontagePanel : public FAnimTimelineTrack
{
	ANIMTIMELINE_DECLARE_TRACK(FAnimTimelineTrack_MontagePanel, FAnimTimelineTrack);

public:
	FAnimTimelineTrack_MontagePanel(const TSharedRef<FAnimModel>& InModel);

	/** FAnimTimelineTrack interface */
	virtual TSharedRef<SWidget> GenerateContainerWidgetForTimeline() override;
	virtual TSharedRef<SWidget> GenerateContainerWidgetForOutliner(const TSharedRef<SAnimOutlinerItem>& InRow) override;
	virtual bool SupportsFiltering() const override { return false; }

	TSharedRef<SAnimMontagePanel> GetAnimMontagePanel();

	TSharedRef<SWidget> BuildMontageSlotSubMenu(int32 InSlotIndex);

	void BuildSlotNameMenu(FMenuBuilder& InMenuBuilder, int32 InSlotIndex);

	void RefreshOutlinerWidget();

	void OnSetSlotName(FName InName, int32 InSlotIndex);

	bool IsSlotNameSet(FName InName, int32 InSlotIndex);

	EVisibility GetSlotWarningVisibility(int32 InSlotIndex) const;

	FText GetSlotWarningText(int32 InSlotIndex) const;

	void UpdateSlotGroupWarning();

private:
	/** The legacy montage panel */
	TSharedPtr<SAnimMontagePanel> AnimMontagePanel;

	/** The outliner widget to allow for dynamic refresh */
	TSharedPtr<SVerticalBox> OutlinerWidget;

	/** Cached info about a slot warning */
	struct FSlotWarningInfo
	{
		FText WarningText;
		bool bWarning;
	};

	/** Cached warning info for all slots */
	TArray<FSlotWarningInfo> WarningInfo;
};
