// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimTimeline/AnimTimelineTrack_Notifies.h"
#include "PersonaUtils.h"
#include "Animation/AnimSequenceBase.h"
#include "Widgets/SBoxPanel.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "AnimSequenceTimelineCommands.h"
#include "SAnimNotifyPanel.h"
#include "AnimTimeline/AnimTimelineTrack_NotifiesPanel.h"
#include "Widgets/Layout/SBorder.h"
#include "ScopedTransaction.h"
#include "Animation/AnimMontage.h"
#include "AnimTimeline/AnimModel_AnimSequenceBase.h"

#define LOCTEXT_NAMESPACE "FAnimTimelineTrack_Notifies"

ANIMTIMELINE_IMPLEMENT_TRACK(FAnimTimelineTrack_Notifies);

FAnimTimelineTrack_Notifies::FAnimTimelineTrack_Notifies(const TSharedRef<FAnimModel>& InModel)
	: FAnimTimelineTrack(LOCTEXT("NotifiesRootTrackLabel", "Notifies"), LOCTEXT("NotifiesRootTrackToolTip", "Notifies and sync markers"), InModel)
{
}

TSharedRef<SWidget> FAnimTimelineTrack_Notifies::GenerateContainerWidgetForOutliner(const TSharedRef<SAnimOutlinerItem>& InRow)
{
	TSharedPtr<SBorder> OuterBorder;
	TSharedPtr<SHorizontalBox> InnerHorizontalBox;
	TSharedRef<SWidget> OutlinerWidget = GenerateStandardOutlinerWidget(InRow, true, OuterBorder, InnerHorizontalBox);

	OuterBorder->SetBorderBackgroundColor(FAppStyle::GetColor("AnimTimeline.Outliner.HeaderColor"));

	UAnimMontage* AnimMontage = Cast<UAnimMontage>(GetModel()->GetAnimSequenceBase());
	if(!(AnimMontage && AnimMontage->HasParentAsset()))
	{
		InnerHorizontalBox->AddSlot()
			.AutoWidth()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.Padding(OutlinerRightPadding, 1.0f)
			[
				PersonaUtils::MakeTrackButton(LOCTEXT("AddTrackButtonText", "Track"), FOnGetContent::CreateSP(this, &FAnimTimelineTrack_Notifies::BuildNotifiesSubMenu), MakeAttributeSP(this, &FAnimTimelineTrack_Notifies::IsHovered))
			];
	}

	return OutlinerWidget;
}

TSharedRef<SWidget> FAnimTimelineTrack_Notifies::BuildNotifiesSubMenu()
{
	FMenuBuilder MenuBuilder(true, GetModel()->GetCommandList());

	MenuBuilder.BeginSection("Notifies", LOCTEXT("NotifiesMenuSection", "Notifies"));
	{
		MenuBuilder.AddMenuEntry(
			FAnimSequenceTimelineCommands::Get().AddNotifyTrack->GetLabel(),
			FAnimSequenceTimelineCommands::Get().AddNotifyTrack->GetDescription(),
			FAnimSequenceTimelineCommands::Get().AddNotifyTrack->GetIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &FAnimTimelineTrack_Notifies::AddTrack)
			)
		);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("TimingPanelOptions", LOCTEXT("TimingPanelOptionsHeader", "Options"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("ToggleTimingNodes_Notifies", "Show Notify Timing Nodes"),
			LOCTEXT("ShowNotifyTimingNodes", "Show or hide the timing display for notifies in the notify panel"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(&StaticCastSharedRef<FAnimModel_AnimSequenceBase>(GetModel()).Get(), &FAnimModel_AnimSequenceBase::ToggleNotifiesTimingElementDisplayEnabled, ENotifyTimingElementType::QueuedNotify),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(&StaticCastSharedRef<FAnimModel_AnimSequenceBase>(GetModel()).Get(), &FAnimModel_AnimSequenceBase::IsNotifiesTimingElementDisplayEnabled, ENotifyTimingElementType::QueuedNotify)
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void FAnimTimelineTrack_Notifies::AddTrack()
{
	UAnimSequenceBase* AnimSequenceBase = GetModel()->GetAnimSequenceBase();

	FScopedTransaction Transaction(LOCTEXT("AddNotifyTrack", "Add Notify Track"));
	AnimSequenceBase->Modify();

	FAnimNotifyTrack NewItem;
	NewItem.TrackName = GetNewTrackName(AnimSequenceBase);
	NewItem.TrackColor = FLinearColor::White;

	AnimSequenceBase->AnimNotifyTracks.Add(NewItem);

	NotifiesPanel.Pin()->RequestTrackRename(AnimSequenceBase->AnimNotifyTracks.Num() - 1);

	NotifiesPanel.Pin()->Update();
}

FName FAnimTimelineTrack_Notifies::GetNewTrackName(UAnimSequenceBase* InAnimSequenceBase)
{
	TArray<FName> TrackNames;
	TrackNames.Reserve(50);

	for (const FAnimNotifyTrack& Track : InAnimSequenceBase->AnimNotifyTracks)
	{
		TrackNames.Add(Track.TrackName);
	}

	FName NameToTest;
	int32 TrackIndex = 1;
	
	do 
	{
		NameToTest = *FString::FromInt(TrackIndex++);
	} while (TrackNames.Contains(NameToTest));

	return NameToTest;
}

#undef LOCTEXT_NAMESPACE
