// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimTimeline/AnimTimelineTrack.h"
#include "SAnimTimingPanel.h"

class FAnimTimelineTrack_NotifiesPanel;

class FAnimTimelineTrack_Notifies : public FAnimTimelineTrack
{
	ANIMTIMELINE_DECLARE_TRACK(FAnimTimelineTrack_Notifies, FAnimTimelineTrack);

public:
	FAnimTimelineTrack_Notifies(const TSharedRef<FAnimModel>& InModel);

	/** FAnimTimelineTrack interface */
	virtual TSharedRef<SWidget> GenerateContainerWidgetForOutliner(const TSharedRef<SAnimOutlinerItem>& InRow) override;

	/** Get a new, unused track name using the specified anim sequence */
	static FName GetNewTrackName(UAnimSequenceBase* InAnimSequenceBase);

	void SetAnimNotifyPanel(const TSharedRef<FAnimTimelineTrack_NotifiesPanel>& InNotifiesPanel) { NotifiesPanel = InNotifiesPanel; }

	/** Controls timing visibility for notify tracks */
	EVisibility OnGetTimingNodeVisibility(ENotifyTimingElementType::Type InType) const;

private:
	/** Populate the track menu */
	TSharedRef<SWidget> BuildNotifiesSubMenu();

	/** Add a new track */
	void AddTrack();
	
	/** The legacy notifies panel we are linked to */
	TWeakPtr<FAnimTimelineTrack_NotifiesPanel> NotifiesPanel;
};
