// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimTimeline/AnimTimelineTrack_NotifiesPanel.h"
#include "SAnimNotifyPanel.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Layout/SBox.h"
#include "PersonaUtils.h"
#include "AnimSequenceTimelineCommands.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "AnimTimeline/AnimTimelineTrack_Notifies.h"
#include "ScopedTransaction.h"
#include "Widgets/Views/SExpanderArrow.h"
#include "Widgets/Layout/SBorder.h"
#include "AnimTimeline/SAnimOutlinerItem.h"
#include "Animation/AnimMontage.h"
#include "AnimTimeline/AnimModel_AnimSequenceBase.h"

#define LOCTEXT_NAMESPACE "FAnimTimelineTrack_NotifiesPanel"

const float FAnimTimelineTrack_NotifiesPanel::NotificationTrackHeight = 24.0f;
const FName FAnimTimelineTrack_NotifiesPanel::AnimationEditorStatusBarName = FName(TEXT("AssetEditor.AnimationEditor.MainMenu"));

ANIMTIMELINE_IMPLEMENT_TRACK(FAnimTimelineTrack_NotifiesPanel);

FAnimTimelineTrack_NotifiesPanel::FAnimTimelineTrack_NotifiesPanel(const TSharedRef<FAnimModel>& InModel)
	: FAnimTimelineTrack(FText::GetEmpty(), FText::GetEmpty(), InModel)
	, PendingRenameTrackIndex(INDEX_NONE)
{
	SetHeight((float)InModel->GetAnimSequenceBase()->AnimNotifyTracks.Num() * NotificationTrackHeight);
}

TSharedRef<SWidget> FAnimTimelineTrack_NotifiesPanel::GenerateContainerWidgetForTimeline()
{
	GetAnimNotifyPanel();

	AnimNotifyPanel->Update();

	return AnimNotifyPanel.ToSharedRef();
}

TSharedRef<SWidget> FAnimTimelineTrack_NotifiesPanel::GenerateContainerWidgetForOutliner(const TSharedRef<SAnimOutlinerItem>& InRow)
{
	TSharedRef<SWidget> Widget = 
		SNew(SHorizontalBox)
		.ToolTipText(this, &FAnimTimelineTrack::GetToolTipText)
		+SHorizontalBox::Slot()
		[
			SAssignNew(OutlinerWidget, SVerticalBox)
		];

	RefreshOutlinerWidget();

	return Widget;
}

void FAnimTimelineTrack_NotifiesPanel::RefreshOutlinerWidget()
{
	OutlinerWidget->ClearChildren();

	int32 TrackIndex = 0;
	UAnimSequenceBase* AnimSequence = GetModel()->GetAnimSequenceBase();
	for(FAnimNotifyTrack& AnimNotifyTrack : AnimSequence->AnimNotifyTracks)
	{
		TSharedPtr<SBox> SlotBox;
		TSharedPtr<SInlineEditableTextBlock> InlineEditableTextBlock;

		OutlinerWidget->AddSlot()
			.AutoHeight()
			[
				SAssignNew(SlotBox, SBox)
				.HeightOverride(NotificationTrackHeight)
			];

		TSharedPtr<SHorizontalBox> HorizontalBox;

		SlotBox->SetContent(
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("Sequencer.Section.BackgroundTint"))
			.BorderBackgroundColor(FAppStyle::GetColor("AnimTimeline.Outliner.ItemColor"))
			[
				SAssignNew(HorizontalBox, SHorizontalBox)
				+SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Left)
				.Padding(30.0f, 0.0f, 0.0f, 0.0f)
				[
					SAssignNew(InlineEditableTextBlock, SInlineEditableTextBlock)
					.Text_Lambda([TrackIndex, AnimSequence](){ return AnimSequence->AnimNotifyTracks.IsValidIndex(TrackIndex) ? FText::FromName(AnimSequence->AnimNotifyTracks[TrackIndex].TrackName) : FText::GetEmpty(); }) 
					.IsSelected(FIsSelected::CreateLambda([](){ return true; }))
					.OnTextCommitted(this, &FAnimTimelineTrack_NotifiesPanel::OnCommitTrackName, TrackIndex)
				]

			]
		);

		UAnimMontage* AnimMontage = Cast<UAnimMontage>(GetModel()->GetAnimSequenceBase());
		if(!(AnimMontage && AnimMontage->HasParentAsset()))
		{
			HorizontalBox->AddSlot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Right)
				.Padding(OutlinerRightPadding, 1.0f)
				[
					PersonaUtils::MakeTrackButton(LOCTEXT("AddTrackButtonText", "Track"), FOnGetContent::CreateSP(this, &FAnimTimelineTrack_NotifiesPanel::BuildNotifiesPanelSubMenu, TrackIndex), MakeAttributeSP(SlotBox.Get(), &SWidget::IsHovered))
				];
		}

		if(PendingRenameTrackIndex == TrackIndex)
		{
			TWeakPtr<SInlineEditableTextBlock> WeakInlineEditableTextBlock = InlineEditableTextBlock;
			InlineEditableTextBlock->RegisterActiveTimer(0.0f, FWidgetActiveTimerDelegate::CreateSP(this, &FAnimTimelineTrack_NotifiesPanel::HandlePendingRenameTimer, WeakInlineEditableTextBlock));
		}

		TrackIndex++;
	}
}

EActiveTimerReturnType FAnimTimelineTrack_NotifiesPanel::HandlePendingRenameTimer(double InCurrentTime, float InDeltaTime, TWeakPtr<SInlineEditableTextBlock> InInlineEditableTextBlock)
{
	if(InInlineEditableTextBlock.IsValid())
	{
		InInlineEditableTextBlock.Pin()->EnterEditingMode();
	}

	PendingRenameTrackIndex = INDEX_NONE;

	return EActiveTimerReturnType::Stop;
}

TSharedRef<SWidget> FAnimTimelineTrack_NotifiesPanel::BuildNotifiesPanelSubMenu(int32 InTrackIndex)
{
	UAnimSequenceBase* AnimSequence = GetModel()->GetAnimSequenceBase();

	FMenuBuilder MenuBuilder(true, GetModel()->GetCommandList());

	MenuBuilder.BeginSection("NotifyTrack", LOCTEXT("NotifyTrackMenuSection", "Notify Track"));
	{
		MenuBuilder.AddMenuEntry(
			FAnimSequenceTimelineCommands::Get().InsertNotifyTrack->GetLabel(),
			FAnimSequenceTimelineCommands::Get().InsertNotifyTrack->GetDescription(),
			FAnimSequenceTimelineCommands::Get().InsertNotifyTrack->GetIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &FAnimTimelineTrack_NotifiesPanel::InsertTrack, InTrackIndex)
			)
		);

		if(AnimSequence->AnimNotifyTracks.Num() > 1)
		{
			MenuBuilder.AddMenuEntry(
				FAnimSequenceTimelineCommands::Get().RemoveNotifyTrack->GetLabel(),
				FAnimSequenceTimelineCommands::Get().RemoveNotifyTrack->GetDescription(),
				FAnimSequenceTimelineCommands::Get().RemoveNotifyTrack->GetIcon(),
				FUIAction(
					FExecuteAction::CreateSP(this, &FAnimTimelineTrack_NotifiesPanel::RemoveTrack, InTrackIndex)
				)
			);
		}
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void FAnimTimelineTrack_NotifiesPanel::InsertTrack(int32 InTrackIndexToInsert)
{
	UAnimSequenceBase* AnimSequence = GetModel()->GetAnimSequenceBase();

	FScopedTransaction Transaction(LOCTEXT("InsertNotifyTrack", "Insert Notify Track"));
	AnimSequence->Modify();

	// before insert, make sure everything behind is fixed
	for (int32 TrackIndex = InTrackIndexToInsert; TrackIndex < AnimSequence->AnimNotifyTracks.Num(); ++TrackIndex)
	{
		FAnimNotifyTrack& Track = AnimSequence->AnimNotifyTracks[TrackIndex];

		const int32 NewTrackIndex = TrackIndex + 1;

		for (FAnimNotifyEvent* Notify : Track.Notifies)
		{
			// fix notifies indices
			Notify->TrackIndex = NewTrackIndex;
		}

		for (FAnimSyncMarker* SyncMarker : Track.SyncMarkers)
		{
			// fix notifies indices
			SyncMarker->TrackIndex = NewTrackIndex;
		}
	}

	FAnimNotifyTrack NewItem;
	NewItem.TrackName = FAnimTimelineTrack_Notifies::GetNewTrackName(AnimSequence);
	NewItem.TrackColor = FLinearColor::White;

	AnimSequence->AnimNotifyTracks.Insert(NewItem, InTrackIndexToInsert);

	// Request a rename on rebuild
	PendingRenameTrackIndex = InTrackIndexToInsert;

	Update();
}

void FAnimTimelineTrack_NotifiesPanel::RemoveTrack(int32 InTrackIndexToRemove)
{
	UAnimSequenceBase* AnimSequence = GetModel()->GetAnimSequenceBase();

	if (AnimSequence->AnimNotifyTracks.IsValidIndex(InTrackIndexToRemove))
	{
		FScopedTransaction Transaction(LOCTEXT("RemoveNotifyTrack", "Remove Notify Track"));
		AnimSequence->Modify();

		// before insert, make sure everything behind is fixed
		for (int32 TrackIndex = InTrackIndexToRemove; TrackIndex < AnimSequence->AnimNotifyTracks.Num(); ++TrackIndex)
		{
			FAnimNotifyTrack& Track = AnimSequence->AnimNotifyTracks[TrackIndex];
			const int32 NewTrackIndex = FMath::Max(0, TrackIndex - 1);

			for (FAnimNotifyEvent* Notify : Track.Notifies)
			{
				// fix notifies indices
				Notify->TrackIndex = NewTrackIndex;
			}

			for (FAnimSyncMarker* SyncMarker : Track.SyncMarkers)
			{
				// fix notifies indices
				SyncMarker->TrackIndex = NewTrackIndex;
			}
		}

		AnimSequence->AnimNotifyTracks.RemoveAt(InTrackIndexToRemove);

		Update();
	}
}

void FAnimTimelineTrack_NotifiesPanel::Update()
{
	SetHeight((float)GetModel()->GetAnimSequenceBase()->AnimNotifyTracks.Num() * NotificationTrackHeight);
	RefreshOutlinerWidget();
	if(AnimNotifyPanel.IsValid())
	{
		AnimNotifyPanel->Update();
	}
}

void FAnimTimelineTrack_NotifiesPanel::HandleNotifyChanged()
{
	SetHeight((float)GetModel()->GetAnimSequenceBase()->AnimNotifyTracks.Num() * NotificationTrackHeight);
	RefreshOutlinerWidget();
}

void FAnimTimelineTrack_NotifiesPanel::OnCommitTrackName(const FText& InText, ETextCommit::Type CommitInfo, int32 TrackIndexToName)
{
	UAnimSequenceBase* AnimSequence = GetModel()->GetAnimSequenceBase();
	if (AnimSequence->AnimNotifyTracks.IsValidIndex(TrackIndexToName))
	{
		FScopedTransaction Transaction(FText::Format(LOCTEXT("RenameNotifyTrack", "Rename Notify Track to '{0}'"), InText));
		AnimSequence->Modify();

		FText TrimText = FText::TrimPrecedingAndTrailing(InText);
		AnimSequence->AnimNotifyTracks[TrackIndexToName].TrackName = FName(*TrimText.ToString());
	}
}

EVisibility FAnimTimelineTrack_NotifiesPanel::OnGetTimingNodeVisibility(ENotifyTimingElementType::Type ElementType) const
{
	return StaticCastSharedRef<FAnimModel_AnimSequenceBase>(GetModel())->IsNotifiesTimingElementDisplayEnabled(ElementType) ? EVisibility::Visible : EVisibility::Hidden;
}

TSharedRef<SAnimNotifyPanel> FAnimTimelineTrack_NotifiesPanel::GetAnimNotifyPanel()
{
	if(!AnimNotifyPanel.IsValid())
	{
		UAnimMontage* AnimMontage = Cast<UAnimMontage>(GetModel()->GetAnimSequenceBase());
		bool bChildAnimMontage = AnimMontage && AnimMontage->HasParentAsset();

		AnimNotifyPanel = SNew(SAnimNotifyPanel, GetModel())
			.IsEnabled(!bChildAnimMontage)
			.Sequence(GetModel()->GetAnimSequenceBase())
			.InputMin(this, &FAnimTimelineTrack_NotifiesPanel::GetMinInput)
			.InputMax(this, &FAnimTimelineTrack_NotifiesPanel::GetMaxInput)
			.ViewInputMin(this, &FAnimTimelineTrack_NotifiesPanel::GetViewMinInput)
			.ViewInputMax(this, &FAnimTimelineTrack_NotifiesPanel::GetViewMaxInput)
			.OnGetScrubValue(this, &FAnimTimelineTrack_NotifiesPanel::GetScrubValue)
			.OnSelectionChanged(this, &FAnimTimelineTrack_NotifiesPanel::SelectObjects)
			.OnSetInputViewRange(this, &FAnimTimelineTrack_NotifiesPanel::OnSetInputViewRange)
			.OnInvokeTab(GetModel()->OnInvokeTab)
			.OnSnapPosition(&GetModel().Get(), &FAnimModel::Snap)
			.OnGetTimingNodeVisibility(this, &FAnimTimelineTrack_NotifiesPanel::OnGetTimingNodeVisibility)
			.OnNotifiesChanged_Lambda([this]()
			{ 
				Update();
				GetModel()->OnTracksChanged().Broadcast();
				 
				if (StatusBarMessageHandle.IsValid())
				{
					if(UStatusBarSubsystem* StatusBarSubsystem = GEditor->GetEditorSubsystem<UStatusBarSubsystem>())
					{
						StatusBarSubsystem->PopStatusBarMessage(AnimationEditorStatusBarName, StatusBarMessageHandle);
						StatusBarMessageHandle.Reset();
					}
				}
			})
			.OnNotifyStateHandleBeingDragged_Lambda([this](TSharedPtr<SAnimNotifyNode> NotifyNode, const FPointerEvent& Event, ENotifyStateHandleHit::Type Handle, float Time)
			{
				if (Event.IsShiftDown())
				{
					const FFrameTime FrameTime = FFrameTime::FromDecimal(Time * (double)GetModel()->GetTickResolution());
					GetModel()->SetScrubPosition(FrameTime);
				}

				if (!StatusBarMessageHandle.IsValid())
				{
					if (UStatusBarSubsystem* StatusBarSubsystem = GEditor->GetEditorSubsystem<UStatusBarSubsystem>())
					{
						StatusBarMessageHandle = StatusBarSubsystem->PushStatusBarMessage(AnimationEditorStatusBarName,
							LOCTEXT("AutoscrubNotifyStateHandle", "Hold SHIFT while dragging a notify state Begin or End handle to auto scrub the timeline."));
					}
				}
			})
			.OnNotifyNodesBeingDragged_Lambda([this](const TArray<TSharedPtr<SAnimNotifyNode>>& NotifyNodes, const class FDragDropEvent& Event, float DragXPosition, float DragTime)
			{
				if (Event.IsShiftDown())
				{
					const FFrameTime FrameTime = FFrameTime::FromDecimal(DragTime * (double)GetModel()->GetTickResolution());
					GetModel()->SetScrubPosition(FrameTime);
				}

				if (!StatusBarMessageHandle.IsValid())
				{
					if (UStatusBarSubsystem* StatusBarSubsystem = GEditor->GetEditorSubsystem<UStatusBarSubsystem>())
					{
						StatusBarMessageHandle = StatusBarSubsystem->PushStatusBarMessage(AnimationEditorStatusBarName,
							LOCTEXT("AutoscrubNotify", "Hold SHIFT while dragging a notify to auto scrub the timeline."));
					}
				}
			});

		GetModel()->GetAnimSequenceBase()->RegisterOnNotifyChanged(UAnimSequenceBase::FOnNotifyChanged::CreateSP(this, &FAnimTimelineTrack_NotifiesPanel::HandleNotifyChanged));
	}

	return AnimNotifyPanel.ToSharedRef();
}

#undef LOCTEXT_NAMESPACE
