// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimTimeline/AnimTimelineTrack.h"
#include "SAnimTimingPanel.h"
#include "StatusBarSubsystem.h"

class SAnimNotifyPanel;
class SVerticalBox;
class SInlineEditableTextBlock;

/** A timeline track that re-uses the legacy panel widget to display notifies */
class FAnimTimelineTrack_NotifiesPanel : public FAnimTimelineTrack
{
	ANIMTIMELINE_DECLARE_TRACK(FAnimTimelineTrack_NotifiesPanel, FAnimTimelineTrack);

public:
	static const float NotificationTrackHeight;
	static const FName AnimationEditorStatusBarName;

	FAnimTimelineTrack_NotifiesPanel(const TSharedRef<FAnimModel>& InModel);

	/** FAnimTimelineTrack interface */
	virtual TSharedRef<SWidget> GenerateContainerWidgetForTimeline() override;
	virtual TSharedRef<SWidget> GenerateContainerWidgetForOutliner(const TSharedRef<SAnimOutlinerItem>& InRow) override;
	virtual bool SupportsFiltering() const override { return false; }

	TSharedRef<SAnimNotifyPanel> GetAnimNotifyPanel();
	
	void Update();

	/** Request a rename next update */
	void RequestTrackRename(int32 InTrackIndex) { PendingRenameTrackIndex = InTrackIndex; }

private:
	TSharedRef<SWidget> BuildNotifiesPanelSubMenu(int32 InTrackIndex);
	void InsertTrack(int32 InTrackIndexToInsert);
	void RemoveTrack(int32 InTrackIndexToRemove);
	void RefreshOutlinerWidget();
	void OnCommitTrackName(const FText& InText, ETextCommit::Type CommitInfo, int32 TrackIndexToName);
	EVisibility OnGetTimingNodeVisibility(ENotifyTimingElementType::Type ElementType) const;
	EActiveTimerReturnType HandlePendingRenameTimer(double InCurrentTime, float InDeltaTime, TWeakPtr<SInlineEditableTextBlock> InInlineEditableTextBlock);
	void HandleNotifyChanged();

	/** The legacy notify panel */
	TSharedPtr<SAnimNotifyPanel> AnimNotifyPanel;

	/** The outliner widget to allow for dynamic refresh */
	TSharedPtr<SVerticalBox> OutlinerWidget;

	/** Track index we want to trigger a rename for */
	int32 PendingRenameTrackIndex;

	/** Handle to status bar message */
	FStatusBarMessageHandle StatusBarMessageHandle;
};
