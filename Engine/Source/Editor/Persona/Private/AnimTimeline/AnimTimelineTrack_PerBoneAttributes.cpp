// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimTimeline/AnimTimelineTrack_PerBoneAttributes.h"
#include "Framework/Application/SlateApplication.h"
#include "AnimTimeline/SAnimOutlinerItem.h"

#define LOCTEXT_NAMESPACE "FAnimTimelineTrack_PerBoneAttributes"

ANIMTIMELINE_IMPLEMENT_TRACK(FAnimTimelineTrack_PerBoneAttributes);

FAnimTimelineTrack_PerBoneAttributes::FAnimTimelineTrack_PerBoneAttributes(const FName& InBoneName, const TSharedRef<FAnimModel>& InModel)
	: FAnimTimelineTrack(LOCTEXT("BoneAttributesTrackLabel", "Bone Attributes"), LOCTEXT("BoneAttributesToolTip", "Contained Attributes for specific Bone"), InModel), BoneName(InBoneName)
{
}

TSharedRef<SWidget> FAnimTimelineTrack_PerBoneAttributes::GenerateContainerWidgetForOutliner(const TSharedRef<SAnimOutlinerItem>& InRow)
{
	TSharedPtr<SBorder> OuterBorder;
	TSharedPtr<SHorizontalBox> InnerHorizontalBox;
	OutlinerWidget = GenerateStandardOutlinerWidget(InRow, false, OuterBorder, InnerHorizontalBox);

	OuterBorder->SetBorderBackgroundColor(FAppStyle::GetColor("AnimTimeline.Outliner.HeaderColor"));

	InnerHorizontalBox->AddSlot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		.Padding(2.0f, 1.0f)
		.AutoWidth()
		[
			SNew(STextBlock)
			.TextStyle(&FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("AnimTimeline.Outliner.Label"))
			.Text(this, &FAnimTimelineTrack_PerBoneAttributes::GetLabel)
			.HighlightText(InRow->GetHighlightText())
		];

	return OutlinerWidget.ToSharedRef();
}

FText FAnimTimelineTrack_PerBoneAttributes::GetLabel() const
{
	return FText::FromName(BoneName);
}

FText FAnimTimelineTrack_PerBoneAttributes::GetToolTipText() const
{
	return FText::Format(LOCTEXT("BoneAttributesTooltipFormat", "Attributes for Bone: {0}\nNumber of Attributes: {1}"), FText::FromName(BoneName), FText::AsNumber(Children.Num()));
}

#undef LOCTEXT_NAMESPACE
