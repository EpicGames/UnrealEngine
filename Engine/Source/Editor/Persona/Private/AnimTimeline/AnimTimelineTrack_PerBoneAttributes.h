// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimTimeline/AnimTimelineTrack.h"

/** Animation timeline track inserted for each unique bone containing animated attributes (inserted as child of FAnimTimelineTrack_Attributes) */
class FAnimTimelineTrack_PerBoneAttributes : public FAnimTimelineTrack
{
	ANIMTIMELINE_DECLARE_TRACK(FAnimTimelineTrack_PerBoneAttributes, FAnimTimelineTrack);

public:
	FAnimTimelineTrack_PerBoneAttributes(const FName& InBoneName, const TSharedRef<FAnimModel>& InModel);

	/** FAnimTimelineTrack interface */
	virtual TSharedRef<SWidget> GenerateContainerWidgetForOutliner(const TSharedRef<SAnimOutlinerItem>& InRow) override;
	virtual FText GetLabel() const override;
	virtual FText GetToolTipText() const override;
	   	   
private:
	FName BoneName;
	TSharedPtr<SWidget> OutlinerWidget;
};
