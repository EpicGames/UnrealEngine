// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimTimeline/AnimTimelineTrack_TimingPanel.h"
#include "SAnimTimingPanel.h"
#include "Animation/AnimMontage.h"
#include "PersonaUtils.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "AnimTimeline/AnimModel_AnimMontage.h"

#define LOCTEXT_NAMESPACE "FAnimTimelineTrack_TimingPanel"

ANIMTIMELINE_IMPLEMENT_TRACK(FAnimTimelineTrack_TimingPanel);

FAnimTimelineTrack_TimingPanel::FAnimTimelineTrack_TimingPanel(const TSharedRef<FAnimModel_AnimMontage>& InModel)
	: FAnimTimelineTrack(LOCTEXT("TimingLabel", "Timing"), LOCTEXT("TimingTooltip", "The timing of the various elements in this sequence"), InModel)
{
	SetHeight(24.0f);
}

TSharedRef<SWidget> FAnimTimelineTrack_TimingPanel::GenerateContainerWidgetForTimeline()
{
	UAnimMontage* Montage = CastChecked<UAnimMontage>(GetModel()->GetAnimSequenceBase());
	bool bIsChildAnimMontage = Montage && Montage->HasParentAsset();

	return SAssignNew(AnimTimingPanel, SAnimTimingPanel, StaticCastSharedRef<FAnimModel_AnimMontage>(GetModel()))
		.IsEnabled(!bIsChildAnimMontage)
		.InSequence(GetModel()->GetAnimSequenceBase())
		.InputMin(this, &FAnimTimelineTrack_TimingPanel::GetMinInput)
		.InputMax(this, &FAnimTimelineTrack_TimingPanel::GetMaxInput)
		.ViewInputMin(this, &FAnimTimelineTrack_TimingPanel::GetViewMinInput)
		.ViewInputMax(this, &FAnimTimelineTrack_TimingPanel::GetViewMaxInput)
		.OnSetInputViewRange(this, &FAnimTimelineTrack_TimingPanel::OnSetInputViewRange);
}

TSharedRef<SWidget> FAnimTimelineTrack_TimingPanel::GenerateContainerWidgetForOutliner(const TSharedRef<SAnimOutlinerItem>& InRow)
{
	TSharedPtr<SBorder> OuterBorder;
	TSharedPtr<SHorizontalBox> InnerHorizontalBox;
	TSharedPtr<SWidget> OutlinerWidget = GenerateStandardOutlinerWidget(InRow, true, OuterBorder, InnerHorizontalBox);

	InnerHorizontalBox->AddSlot()
		.AutoWidth()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		.Padding(OutlinerRightPadding, 1.0f)
		[
			PersonaUtils::MakeTrackButton(LOCTEXT("EditCurvesButtonText", "Timing"), FOnGetContent::CreateSP(this, &FAnimTimelineTrack_TimingPanel::BuildTimingSubMenu), MakeAttributeSP(this, &FAnimTimelineTrack_TimingPanel::IsHovered))
		];

	return OutlinerWidget.ToSharedRef();
}

TSharedRef<SWidget> FAnimTimelineTrack_TimingPanel::BuildTimingSubMenu()
{
	FMenuBuilder Builder(true, nullptr);

	Builder.BeginSection("TimingPanelOptions", LOCTEXT("TimingPanelOptionsHeader", "Options"));
	{
		Builder.AddMenuEntry(
			LOCTEXT("ToggleTimingNodes_Sections", "Show Section Timing Nodes"),
			LOCTEXT("ShowSectionTimingNodes", "Show or hide the timing display for sections"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(AnimTimingPanel.Get(), &SAnimTimingPanel::OnElementDisplayEnabledChanged, ENotifyTimingElementType::Section),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(AnimTimingPanel.Get(), &SAnimTimingPanel::IsElementDisplayEnabled, ENotifyTimingElementType::Section)
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);

		Builder.AddMenuEntry(
			LOCTEXT("ToggleTimingNodes_Notifies", "Show Notify Timing Nodes"),
			LOCTEXT("ShowNotifyTimingNodes", "Show or hide the timing display for notifies"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(AnimTimingPanel.Get(), &SAnimTimingPanel::OnElementDisplayEnabledChanged, ENotifyTimingElementType::QueuedNotify),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(AnimTimingPanel.Get(), &SAnimTimingPanel::IsElementDisplayEnabled, ENotifyTimingElementType::QueuedNotify)
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
	}
	Builder.EndSection();

	return Builder.MakeWidget();
}

#undef LOCTEXT_NAMESPACE
