// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimTimeline/AnimTimelineTrack.h"

class SAnimTimingPanel;
class SVerticalBox;
class FAnimModel_AnimMontage;

/** A timeline track that re-uses the legacy panel widget to display anim timing panel */
class FAnimTimelineTrack_TimingPanel : public FAnimTimelineTrack
{
	ANIMTIMELINE_DECLARE_TRACK(FAnimTimelineTrack_TimingPanel, FAnimTimelineTrack);

public:
	FAnimTimelineTrack_TimingPanel(const TSharedRef<FAnimModel_AnimMontage>& InModel);

	/** FAnimTimelineTrack interface */
	virtual TSharedRef<SWidget> GenerateContainerWidgetForTimeline() override;
	virtual TSharedRef<SWidget> GenerateContainerWidgetForOutliner(const TSharedRef<SAnimOutlinerItem>& InRow) override;
	virtual bool SupportsFiltering() const override { return false; }

	TSharedPtr<SAnimTimingPanel> GetAnimTimingPanel() { return AnimTimingPanel; }

private:
	TSharedRef<SWidget> BuildTimingSubMenu();

	/** The legacy timing panel */
	TSharedPtr<SAnimTimingPanel> AnimTimingPanel;
};
