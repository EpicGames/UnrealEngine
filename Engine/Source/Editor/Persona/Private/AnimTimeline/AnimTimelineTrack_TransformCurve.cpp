// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimTimeline/AnimTimelineTrack_TransformCurve.h"
#include "Animation/AnimSequenceBase.h"
#include "Widgets/Layout/SBorder.h"
#include "Styling/AppStyle.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "AnimSequenceTimelineCommands.h"
#include "ScopedTransaction.h"
#include "Animation/AnimSequence.h"
#include "IPersonaPreviewScene.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "AnimPreviewInstance.h"
#include "AnimTimeline/AnimModel_AnimSequenceBase.h"
#include "AnimTimelineClipboard.h"

#define LOCTEXT_NAMESPACE "FAnimTimelineTrack_TransformCurve"

ANIMTIMELINE_IMPLEMENT_TRACK(FAnimTimelineTrack_TransformCurve);

FAnimTimelineTrack_TransformCurve::FAnimTimelineTrack_TransformCurve(const FTransformCurve* InCurve, const TSharedRef<FAnimModel>& InModel)
	: FAnimTimelineTrack_Curve(FText::FromName(InCurve->GetName()), FText::FromName(InCurve->GetName()), InCurve->GetColor(), InCurve->GetColor(), InModel)
	, TransformCurve(InCurve)
	, CurveName(InCurve->GetName())
	, CurveId(InCurve->GetName(), ERawCurveTrackTypes::RCT_Transform)
{
	Curves.Add(&InCurve->TranslationCurve.FloatCurves[0]);
	Curves.Add(&InCurve->TranslationCurve.FloatCurves[1]);
	Curves.Add(&InCurve->TranslationCurve.FloatCurves[2]);
	Curves.Add(&InCurve->RotationCurve.FloatCurves[0]);
	Curves.Add(&InCurve->RotationCurve.FloatCurves[1]);
	Curves.Add(&InCurve->RotationCurve.FloatCurves[2]);
	Curves.Add(&InCurve->ScaleCurve.FloatCurves[0]);
	Curves.Add(&InCurve->ScaleCurve.FloatCurves[1]);
	Curves.Add(&InCurve->ScaleCurve.FloatCurves[2]);
}

FLinearColor FAnimTimelineTrack_TransformCurve::GetCurveColor(int32 InCurveIndex) const
{
	static const FLinearColor Colors[3] =
	{
		FLinearColor::Red,
		FLinearColor::Green,
		FLinearColor::Blue,
	};

	return Colors[InCurveIndex % 3];
}

FText FAnimTimelineTrack_TransformCurve::GetFullCurveName(int32 InCurveIndex) const 
{
	check(InCurveIndex >= 0 && InCurveIndex < 9);

	static const FText TrackNames[9] =
	{
		LOCTEXT("TranslationXTrackName", "Translation.X"),
		LOCTEXT("TranslationYTrackName", "Translation.Y"),
		LOCTEXT("TranslationZTrackName", "Translation.Z"),
		LOCTEXT("RotationRollTrackName", "Rotation.Roll"),
		LOCTEXT("RotationPitchTrackName", "Rotation.Pitch"),
		LOCTEXT("RotationYawTrackName", "Rotation.Yaw"),
		LOCTEXT("ScaleXTrackName", "Scale.X"),
		LOCTEXT("ScaleYTrackName", "Scale.Y"),
		LOCTEXT("ScaleZTrackName", "Scale.Z"),
	};
			
	return FText::Format(LOCTEXT("TransformVectorFormat", "{0}.{1}"), FullCurveName, TrackNames[InCurveIndex]);
}

void FAnimTimelineTrack_TransformCurve::Copy(UAnimTimelineClipboardContent* InOutClipboard) const
{
	check(InOutClipboard != nullptr)
	
	UTransformCurveCopyObject * CopyableCurve = UAnimCurveBaseCopyObject::Create<UTransformCurveCopyObject>();

	// Copy raw curve data
	CopyableCurve->Curve.SetName(TransformCurve->GetName());
	CopyableCurve->Curve.SetCurveTypeFlags(TransformCurve->GetCurveTypeFlags());
	CopyableCurve->Curve.CopyCurve(*TransformCurve);
	CopyableCurve->Curve.Comment = TransformCurve->Comment;

	// Copy curve identifier data
	CopyableCurve->CurveName = CurveName;
	CopyableCurve->CurveType = ERawCurveTrackTypes::RCT_Transform;
	CopyableCurve->Channel = CurveId.Channel;
	CopyableCurve->Axis = CurveId.Axis;

	// Origin data
	CopyableCurve->OriginName = GetModel()->GetAnimSequenceBase()->GetFName();
	
	InOutClipboard->Curves.Add(CopyableCurve);
}

TSharedRef<SWidget> FAnimTimelineTrack_TransformCurve::BuildCurveTrackMenu()
{
	FMenuBuilder MenuBuilder(true, GetModel()->GetCommandList());

	MenuBuilder.BeginSection("Curve", LOCTEXT("CurveMenuSection", "Curve"));
	{
			MenuBuilder.AddMenuEntry(
				FAnimSequenceTimelineCommands::Get().EditCurve->GetLabel(),
				FAnimSequenceTimelineCommands::Get().EditCurve->GetDescription(),
				FAnimSequenceTimelineCommands::Get().EditCurve->GetIcon(),
				FUIAction(FExecuteAction::CreateSP(this, &FAnimTimelineTrack_TransformCurve::HendleEditCurve)));

		MenuBuilder.AddMenuEntry(
			LOCTEXT("RemoveTrack", "Remove Track"),
			LOCTEXT("RemoveTrackTooltip", "Remove this track"),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &FAnimTimelineTrack_TransformCurve::DeleteTrack)));

		MenuBuilder.AddMenuEntry(
			LOCTEXT("TrackEnabled", "Enabled"),
			LOCTEXT("TrackEnabledTooltip", "Enable/disable this track"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &FAnimTimelineTrack_TransformCurve::ToggleEnabled),
				FCanExecuteAction(), 
				FIsActionChecked::CreateSP(this, &FAnimTimelineTrack_TransformCurve::IsEnabled)),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
			);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void FAnimTimelineTrack_TransformCurve::DeleteTrack()
{
	UAnimSequenceBase* AnimSequenceBase = GetModel()->GetAnimSequenceBase();
	TSharedRef<FAnimModel_AnimSequenceBase> BaseModel = StaticCastSharedRef<FAnimModel_AnimSequenceBase>(GetModel());

	if(AnimSequenceBase->GetDataModel()->FindTransformCurve(CurveId))
	{
		const FScopedTransaction Transaction(LOCTEXT("AnimCurve_DeleteTrack", "Delete Curve"));

		IAnimationDataController& Controller = AnimSequenceBase->GetController();
		Controller.RemoveCurve(CurveId);

		if (GetModel()->GetPreviewScene()->GetPreviewMeshComponent()->PreviewInstance != nullptr)
		{
			GetModel()->GetPreviewScene()->GetPreviewMeshComponent()->PreviewInstance->RefreshCurveBoneControllers();
		}
	}
}

bool FAnimTimelineTrack_TransformCurve::IsEnabled() const
{
	UAnimSequenceBase* AnimSequenceBase = GetModel()->GetAnimSequenceBase();
	const FAnimCurveBase* Curve = AnimSequenceBase->GetDataModel()->FindTransformCurve(CurveId);
	return Curve && !Curve->GetCurveTypeFlag(AACF_Disabled);
}

void FAnimTimelineTrack_TransformCurve::ToggleEnabled()
{
	UAnimSequenceBase* AnimSequenceBase = GetModel()->GetAnimSequenceBase();

	IAnimationDataController& Controller = AnimSequenceBase->GetController();
	Controller.SetCurveFlag(CurveId, AACF_Disabled, IsEnabled());

	// need to update curves, otherwise they're not disabled
	if (GetModel()->GetPreviewScene()->GetPreviewMeshComponent()->PreviewInstance != nullptr)
	{
		GetModel()->GetPreviewScene()->GetPreviewMeshComponent()->PreviewInstance->RefreshCurveBoneControllers();
	}
}

void FAnimTimelineTrack_TransformCurve::GetCurveEditInfo(int32 InCurveIndex, FName& OutName, ERawCurveTrackTypes& OutType, int32& OutCurveIndex) const
{
	OutName = TransformCurve->GetName();
	OutType = ERawCurveTrackTypes::RCT_Transform;
	OutCurveIndex = InCurveIndex;
}

#undef LOCTEXT_NAMESPACE
