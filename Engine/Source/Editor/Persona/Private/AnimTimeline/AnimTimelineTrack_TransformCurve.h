// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimTimeline/AnimTimelineTrack_Curve.h"

struct FTransformCurve;
class SBorder;
class FCurveEditor;

class FAnimTimelineTrack_TransformCurve : public FAnimTimelineTrack_Curve
{
	ANIMTIMELINE_DECLARE_TRACK(FAnimTimelineTrack_TransformCurve, FAnimTimelineTrack_Curve);

public:
	FAnimTimelineTrack_TransformCurve(const FTransformCurve* InCurve, const TSharedRef<FAnimModel>& InModel);

	/** FAnimTimelineTrack_Curve interface */
	virtual FLinearColor GetCurveColor(int32 InCurveIndex) const override;
	virtual FText GetFullCurveName(int32 InCurveIndex) const override;
	virtual TSharedRef<SWidget> BuildCurveTrackMenu() override;
	virtual bool CanEditCurve(int32 InCurveIndex) const override { return true; }
	virtual void GetCurveEditInfo(int32 InCurveIndex, FName& OutName, ERawCurveTrackTypes& OutType, int32& OutCurveIndex) const override;
	virtual bool SupportsCopy() const override { return true; }
	virtual void Copy(UAnimTimelineClipboardContent* InOutClipboard) const override;
	
	/** Access the curve we are editing */
	const FTransformCurve& GetTransformCurve() { return *TransformCurve; }

	FName GetFName() const { return CurveName; }

private:
	/** Delete this track via the track menu */
	void DeleteTrack();

	/** Show enabled state in the menu */
	bool IsEnabled() const;

	/** Toggle enabled state via the menu */
	void ToggleEnabled();

private:
	/** The curve we are editing */
	const FTransformCurve* TransformCurve;

	/** The curve name and identifier */
	FName CurveName;
	FAnimationCurveIdentifier CurveId;
};
