// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimTimeline/AnimTimelineTrack_VectorCurve.h"
#include "Animation/AnimSequenceBase.h"
#include "Widgets/Layout/SBorder.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "FAnimTimelineTrack_VectorCurve"

ANIMTIMELINE_IMPLEMENT_TRACK(FAnimTimelineTrack_VectorCurve);

FAnimTimelineTrack_VectorCurve::FAnimTimelineTrack_VectorCurve(const FVectorCurve* InCurve, const FName& InName, int32 InCurveIndex, ERawCurveTrackTypes InType, const FText& InCurveName, const FText& InFullCurveName, const FLinearColor& InColor, const TSharedRef<FAnimModel>& InModel)
	: FAnimTimelineTrack_Curve(InCurveName, InFullCurveName, InColor, InColor, InModel)
	, VectorCurve(InCurve)
{
	OuterCurveName = InName;
	OuterCurveIndex = InCurveIndex;
	OuterType = InType;

	Curves.Add(&InCurve->FloatCurves[0]);
	Curves.Add(&InCurve->FloatCurves[1]);
	Curves.Add(&InCurve->FloatCurves[2]);
}

FLinearColor FAnimTimelineTrack_VectorCurve::GetCurveColor(int32 InCurveIndex) const
{
	static const FLinearColor Colors[3] =
	{
		FLinearColor::Red,
		FLinearColor::Green,
		FLinearColor::Blue,
	};

	return Colors[InCurveIndex % 3];
}

FText FAnimTimelineTrack_VectorCurve::GetFullCurveName(int32 InCurveIndex) const 
{ 
	check(InCurveIndex >= 0 && InCurveIndex < 3);

	static const FText TrackNames[3] =
	{
		LOCTEXT("VectorXTrackName", "X"),
		LOCTEXT("VectorYTrackName", "Y"),
		LOCTEXT("VectorZTrackName", "Z")
	};
			
	return FText::Format(LOCTEXT("TransformVectorFormat", "{0}.{1}"), FullCurveName, TrackNames[InCurveIndex]);
}

void FAnimTimelineTrack_VectorCurve::GetCurveEditInfo(int32 InCurveIndex, FName& OutName, ERawCurveTrackTypes& OutType, int32& OutCurveIndex) const
{
	OutName = OuterCurveName;
	OutType = OuterType;
	OutCurveIndex = OuterCurveIndex + InCurveIndex;
}

#undef LOCTEXT_NAMESPACE
