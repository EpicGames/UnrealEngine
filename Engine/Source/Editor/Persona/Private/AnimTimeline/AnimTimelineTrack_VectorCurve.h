// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimTimeline/AnimTimelineTrack_Curve.h"

struct FVectorCurve;
class SBorder;
class FCurveEditor;

class FAnimTimelineTrack_VectorCurve : public FAnimTimelineTrack_Curve
{
	ANIMTIMELINE_DECLARE_TRACK(FAnimTimelineTrack_VectorCurve, FAnimTimelineTrack_Curve);

public:
	FAnimTimelineTrack_VectorCurve(const FVectorCurve* InCurve, const FName& InName, int32 InCurveIndex, ERawCurveTrackTypes InType, const FText& InCurveName, const FText& InFullCurveName, const FLinearColor& InColor, const TSharedRef<FAnimModel>& InModel);

	/** FAnimTimelineTrack_Curve interface */
	virtual FLinearColor GetCurveColor(int32 InCurveIndex) const override;
	virtual FText GetFullCurveName(int32 InCurveIndex) const override;
	virtual bool CanEditCurve(int32 InCurveIndex) const override { return true; }
	virtual void GetCurveEditInfo(int32 InCurveIndex, FName& OutName, ERawCurveTrackTypes& OutType, int32& OutCurveIndex) const override;

	/** Access the curve we are editing */
	const FVectorCurve& GetVectorCurve() { return *VectorCurve; }

private:
	/** The curve we are editing */
	const FVectorCurve* VectorCurve;
};
