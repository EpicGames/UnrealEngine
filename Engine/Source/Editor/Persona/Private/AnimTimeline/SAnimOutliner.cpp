// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimTimeline/SAnimOutliner.h"
#include "AnimTimeline/AnimModel.h"
#include "AnimTimeline/AnimTimelineTrack.h"
#include "AnimTimeline/SAnimOutlinerItem.h"
#include "AnimTimeline/SAnimTrackArea.h"
#include "Widgets/Input/SButton.h"
#include "AnimTimeline/SAnimTrack.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Misc/TextFilterExpressionEvaluator.h"

#define LOCTEXT_NAMESPACE "SAnimOutliner"

SAnimOutliner::~SAnimOutliner()
{
	if(AnimModel.IsValid())
	{
		AnimModel.Pin()->OnTracksChanged().Remove(TracksChangedDelegateHandle);
	}
}

void SAnimOutliner::Construct(const FArguments& InArgs, const TSharedRef<FAnimModel>& InAnimModel, const TSharedRef<SAnimTrackArea>& InTrackArea)
{	
	AnimModel = InAnimModel;
	TrackArea = InTrackArea;
	FilterText = InArgs._FilterText;
	bPhysicalTracksNeedUpdate = false;

	TracksChangedDelegateHandle = InAnimModel->OnTracksChanged().AddSP(this, &SAnimOutliner::HandleTracksChanged);

	TextFilter = MakeShareable(new FTextFilterExpressionEvaluator(ETextFilterExpressionEvaluatorMode::BasicString));

	HeaderRow = SNew(SHeaderRow)
		.Visibility(EVisibility::Collapsed);

	HeaderRow->AddColumn(
		SHeaderRow::Column(TEXT("Outliner"))
		.FillWidth(1.0f)
	);

	STreeView::Construct
	(
		STreeView::FArguments()
		.TreeItemsSource(&InAnimModel->GetAllRootTracks())
		.SelectionMode(ESelectionMode::Multi)
		.OnGenerateRow(this, &SAnimOutliner::HandleGenerateRow) 
		.OnGetChildren(this, &SAnimOutliner::HandleGetChildren)
		.HeaderRow(HeaderRow)
		.ExternalScrollbar(InArgs._ExternalScrollbar)
		.OnExpansionChanged(this, &SAnimOutliner::HandleExpansionChanged)
		.AllowOverscroll(EAllowOverscroll::No)
		.OnContextMenuOpening(this, &SAnimOutliner::HandleContextMenuOpening)
	);

	// expand all
	InAnimModel->ForEachRootTrack([this](FAnimTimelineTrack& InRootTrack)
	{
		InRootTrack.Traverse_ParentFirst([this](FAnimTimelineTrack& InTrack){ SetItemExpansion(InTrack.AsShared(), InTrack.IsExpanded()); return true; });
	});
}

void SAnimOutliner::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	STreeView::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	// These are updated in both tick and paint since both calls can cause changes to the cached rows and the data needs
	// to be kept synchronized so that external measuring calls get correct and reliable results.
	if (bPhysicalTracksNeedUpdate)
	{
		PhysicalTracks.Reset();
		CachedTrackGeometry.GenerateValueArray(PhysicalTracks);

		PhysicalTracks.Sort([](const FCachedGeometry& A, const FCachedGeometry& B)
		{
			return A.Top < B.Top;
		});

		bPhysicalTracksNeedUpdate = false;
	}
}

int32 SAnimOutliner::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	LayerId = STreeView::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);

	// These are updated in both tick and paint since both calls can cause changes to the cached rows and the data needs
	// to be kept synchronized so that external measuring calls get correct and reliable results.
	if (bPhysicalTracksNeedUpdate)
	{
		PhysicalTracks.Reset();
		CachedTrackGeometry.GenerateValueArray(PhysicalTracks);

		PhysicalTracks.Sort([](const FCachedGeometry& A, const FCachedGeometry& B) 
		{
			return A.Top < B.Top;
		});

		bPhysicalTracksNeedUpdate = false;
	}

	return LayerId + 1;
}

TSharedRef<ITableRow> SAnimOutliner::HandleGenerateRow(TSharedRef<FAnimTimelineTrack> InTrack, const TSharedRef<STableViewBase>& OwnerTable)
{
	TSharedRef<SAnimOutlinerItem> Row =
		SNew(SAnimOutlinerItem, OwnerTable, InTrack)
		.OnGenerateWidgetForColumn(this, &SAnimOutliner::GenerateWidgetForColumn)
		.HighlightText(FilterText);

	// Ensure the track area is kept up to date with the virtualized scroll of the tree view
	TSharedPtr<SAnimTrack> TrackWidget = TrackArea->FindTrackSlot(InTrack);

	if (!TrackWidget.IsValid())
	{
		// Add a track slot for the row
		TrackWidget = SNew(SAnimTrack, InTrack, SharedThis(this))
			.ViewRange(AnimModel.Pin().Get(), &FAnimModel::GetViewRange)
		[
			 InTrack->GenerateContainerWidgetForTimeline()
		];

		TrackArea->AddTrackSlot(InTrack, TrackWidget);
	}

	if (ensure(TrackWidget.IsValid()))
	{
		Row->AddTrackAreaReference(TrackWidget);
	}

	return Row;
}

TSharedRef<SWidget> SAnimOutliner::GenerateWidgetForColumn(const TSharedRef<FAnimTimelineTrack>& InTrack, const FName& ColumnId, const TSharedRef<SAnimOutlinerItem>& Row) const
{
	return InTrack->GenerateContainerWidgetForOutliner(Row);
}

void SAnimOutliner::HandleGetChildren(TSharedRef<FAnimTimelineTrack> Item, TArray<TSharedRef<FAnimTimelineTrack>>& OutChildren)
{
	if(!FilterText.Get().IsEmpty())
	{
		for(const TSharedRef<FAnimTimelineTrack>& Child : Item->GetChildren())
		{
			if(!Child->SupportsFiltering() || TextFilter->TestTextFilter(FBasicStringFilterExpressionContext(Child->GetLabel().ToString())))
			{
				OutChildren.Add(Child);
			}
		}
	}
	else
	{
		OutChildren.Append(Item->GetChildren());
	}
}

void SAnimOutliner::HandleExpansionChanged(TSharedRef<FAnimTimelineTrack> InTrack, bool bIsExpanded)
{
	InTrack->SetExpanded(bIsExpanded);
	
	// Expand any children that are also expanded
	for (const TSharedRef<FAnimTimelineTrack>& Child : InTrack->GetChildren())
	{
		if (Child->IsExpanded())
		{
			SetItemExpansion(Child, true);
		}
	}
}

TSharedPtr<SWidget> SAnimOutliner::HandleContextMenuOpening()
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, AnimModel.Pin()->GetCommandList());

	AnimModel.Pin()->BuildContextMenu(MenuBuilder);

	// > 1 because the search widget is always added
	return MenuBuilder.GetMultiBox()->GetBlocks().Num() > 1 ? MenuBuilder.MakeWidget() : TSharedPtr<SWidget>();
}

void SAnimOutliner::HandleTracksChanged()
{
	RequestTreeRefresh();
}

void SAnimOutliner::ReportChildRowGeometry(const TSharedRef<FAnimTimelineTrack>& InTrack, const FGeometry& InGeometry)
{
	const float ChildOffset = static_cast<float>(TransformPoint(
		Concatenate(
			InGeometry.GetAccumulatedLayoutTransform(),
			GetCachedGeometry().GetAccumulatedLayoutTransform().Inverse()
		),
		FVector2D(0,0)
	).Y);

	const FCachedGeometry* ExistingGeometry = CachedTrackGeometry.Find(InTrack);
	if(ExistingGeometry == nullptr || (ExistingGeometry->Top != ChildOffset || ExistingGeometry->Height != InGeometry.Size.Y))
	{
		CachedTrackGeometry.Add(InTrack, FCachedGeometry(InTrack, ChildOffset, static_cast<float>(InGeometry.Size.Y)));
		bPhysicalTracksNeedUpdate = true;
	}
}

void SAnimOutliner::OnChildRowRemoved(const TSharedRef<FAnimTimelineTrack>& InTrack)
{
	CachedTrackGeometry.Remove(InTrack);
	bPhysicalTracksNeedUpdate = true;
}

TOptional<SAnimOutliner::FCachedGeometry> SAnimOutliner::GetCachedGeometryForTrack(const TSharedRef<FAnimTimelineTrack>& InTrack) const
{
	if (const FCachedGeometry* FoundGeometry = CachedTrackGeometry.Find(InTrack))
	{
		return *FoundGeometry;
	}

	return TOptional<FCachedGeometry>();
}

TOptional<float> SAnimOutliner::ComputeTrackPosition(const TSharedRef<FAnimTimelineTrack>& InTrack) const
{
	// Positioning strategy:
	// Attempt to root out any visible track in the specified track's sub-hierarchy, and compute the track's offset from that
	float NegativeOffset = 0.f;
	TOptional<float> Top;
	
	// Iterate parent first until we find a tree view row we can use for the offset height
	auto Iter = [&](FAnimTimelineTrack& InTrack)
	{		
		TOptional<FCachedGeometry> ChildRowGeometry = GetCachedGeometryForTrack(InTrack.AsShared());
		if (ChildRowGeometry.IsSet())
		{
			Top = ChildRowGeometry->Top;
			// Stop iterating
			return false;
		}

		NegativeOffset -= InTrack.GetHeight() + InTrack.GetPadding().Combined();
		return true;
	};

	InTrack->TraverseVisible_ParentFirst(Iter);

	if (Top.IsSet())
	{
		return NegativeOffset + Top.GetValue();
	}

	return Top;
}

void SAnimOutliner::ScrollByDelta(float DeltaInSlateUnits)
{
	ScrollBy(GetCachedGeometry(), DeltaInSlateUnits, EAllowOverscroll::No);
}

void SAnimOutliner::Private_SetItemSelection( TSharedRef<FAnimTimelineTrack> TheItem, bool bShouldBeSelected, bool bWasUserDirected )
{
	if(TheItem->SupportsSelection())
	{
		AnimModel.Pin()->SetTrackSelected(TheItem, bShouldBeSelected);

		STreeView::Private_SetItemSelection(TheItem, bShouldBeSelected, bWasUserDirected);
	}
}

void SAnimOutliner::Private_ClearSelection()
{
	AnimModel.Pin()->ClearTrackSelection();

	STreeView::Private_ClearSelection();
}

void SAnimOutliner::Private_SelectRangeFromCurrentTo( TSharedRef<FAnimTimelineTrack> InRangeSelectionEnd )
{
	STreeView::Private_SelectRangeFromCurrentTo(InRangeSelectionEnd);

	for(TSet<TSharedRef<FAnimTimelineTrack>>::TIterator Iter = SelectedItems.CreateIterator(); Iter; ++Iter)
	{
		if(!(*Iter)->SupportsSelection())
		{
			Iter.RemoveCurrent();
		}
	}

	for(const TSharedRef<FAnimTimelineTrack>& SelectedItem : SelectedItems)
	{
		AnimModel.Pin()->SetTrackSelected(SelectedItem, true);
	}
}

void SAnimOutliner::RefreshFilter()
{
	TextFilter->SetFilterText(FilterText.Get());

	RequestTreeRefresh();
}

#undef LOCTEXT_NAMESPACE
