// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "Input/Reply.h"
#include "Styling/ISlateStyle.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STreeView.h"
#include "AnimTimeline/AnimTimelineTrack.h"

class FAnimModel;
class SAnimTrackArea;
class SAnimOutlinerItem;
class FMenuBuilder;
class FTextFilterExpressionEvaluator;

/** A delegate that is executed when adding menu content. */
DECLARE_DELEGATE_OneParam(FOnGetContextMenuContent, FMenuBuilder& /*MenuBuilder*/);

class SAnimOutliner : public STreeView<TSharedRef<FAnimTimelineTrack>>
{
public:
	SLATE_BEGIN_ARGS(SAnimOutliner) {}

	/** Externally supplied scroll bar */
	SLATE_ARGUMENT(TSharedPtr<SScrollBar>, ExternalScrollbar)

	/** Called to populate the context menu. */
	SLATE_EVENT(FOnGetContextMenuContent, OnGetContextMenuContent)

	/** The filter test used for searching */
	SLATE_ATTRIBUTE(FText, FilterText)

	SLATE_END_ARGS()

	~SAnimOutliner();

	void Construct(const FArguments& InArgs, const TSharedRef<FAnimModel>& InAnimModel, const TSharedRef<SAnimTrackArea>& InTrackArea);

	/** SWidget interface */
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime);
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const;

	/** STreeView interface */
	virtual void Private_SetItemSelection( TSharedRef<FAnimTimelineTrack> TheItem, bool bShouldBeSelected, bool bWasUserDirected = false ) override;
	virtual void Private_ClearSelection() override;
	virtual void Private_SelectRangeFromCurrentTo( TSharedRef<FAnimTimelineTrack> InRangeSelectionEnd ) override;

	/** Structure used to cache physical geometry for a particular track */
	struct FCachedGeometry
	{
		FCachedGeometry(TSharedRef<FAnimTimelineTrack> InTrack, float InTop, float InHeight)
			: Track(MoveTemp(InTrack))
			, Top(InTop)
			, Height(InHeight)
		{}

		TSharedRef<FAnimTimelineTrack> Track;
		float Top;
		float Height;
	};

	/** Get the cached geometry for the specified track */
	TOptional<FCachedGeometry> GetCachedGeometryForTrack(const TSharedRef<FAnimTimelineTrack>& InTrack) const;

	/** Compute the position of the specified track using its cached geometry */
	TOptional<float> ComputeTrackPosition(const TSharedRef<FAnimTimelineTrack>& InTrack) const;

	/** Report geometry for a child row */
	void ReportChildRowGeometry(const TSharedRef<FAnimTimelineTrack>& InTrack, const FGeometry& InGeometry);

	/** Called when a child row widget has been added/removed */
	void OnChildRowRemoved(const TSharedRef<FAnimTimelineTrack>& InTrack);

	/** Scroll this tree view by the specified number of slate units */
	void ScrollByDelta(float DeltaInSlateUnits);

	/** Apply any changed filter */
	void RefreshFilter();

private:
	/** Generate a row for the outliner */
	TSharedRef<ITableRow> HandleGenerateRow(TSharedRef<FAnimTimelineTrack> Item, const TSharedRef<STableViewBase>& OwnerTable);

	/** Get the children of an outliner item */
	void HandleGetChildren(TSharedRef<FAnimTimelineTrack> Item, TArray<TSharedRef<FAnimTimelineTrack>>& OutChildren);

	/** Record the expansion state when it changes */
	void HandleExpansionChanged(TSharedRef<FAnimTimelineTrack> InItem, bool bIsExpanded);

	/** Handle context menu */
	TSharedPtr<SWidget> HandleContextMenuOpening();

	/** Handle tracks changing */
	void HandleTracksChanged();

	/** Generate a widget for the specified column */
	TSharedRef<SWidget> GenerateWidgetForColumn(const TSharedRef<FAnimTimelineTrack>& InTrack, const FName& ColumnId, const TSharedRef<SAnimOutlinerItem>& Row) const;

private:
	/** The anim timeline model */
	TWeakPtr<FAnimModel> AnimModel;

	/** The track area */
	TSharedPtr<SAnimTrackArea> TrackArea;

	/** The header row */
	TSharedPtr<SHeaderRow> HeaderRow;

	/** Delegate handle for track changes */
	FDelegateHandle TracksChangedDelegateHandle;

	/** Map of cached geometries for visible tracks */
	TMap<TSharedRef<FAnimTimelineTrack>, FCachedGeometry> CachedTrackGeometry;

	/** Linear, sorted array of tracks that we currently have generated widgets for */
	mutable TArray<FCachedGeometry> PhysicalTracks;

	/** A flag indicating that the physical tracks need to be updated. */
	mutable bool bPhysicalTracksNeedUpdate;

	/** The filter text used when we are searching the tree */
	TAttribute<FText> FilterText;

	/** Compiled filter search terms. */
	TSharedPtr<FTextFilterExpressionEvaluator> TextFilter;
};
