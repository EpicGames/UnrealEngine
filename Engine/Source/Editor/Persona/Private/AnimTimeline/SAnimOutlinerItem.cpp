// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimTimeline/SAnimOutlinerItem.h"
#include "AnimTimeline/AnimTimelineTrack.h"
#include "Widgets/Text/STextBlock.h"
#include "AnimTimeline/SAnimOutliner.h"
#include "Widgets/SOverlay.h"
#include "AnimTimeline/SAnimTrackResizeArea.h"

SAnimOutlinerItem::~SAnimOutlinerItem()
{
	TSharedPtr<SAnimOutliner> Outliner = StaticCastSharedPtr<SAnimOutliner>(OwnerTablePtr.Pin());
	TSharedPtr<FAnimTimelineTrack> PinnedTrack = Track.Pin();
	if (Outliner.IsValid() && PinnedTrack.IsValid())
	{
		Outliner->OnChildRowRemoved(PinnedTrack.ToSharedRef());
	}
}

void SAnimOutlinerItem::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, const TSharedRef<FAnimTimelineTrack>& InTrack)
{
	Track = InTrack;
	OnGenerateWidgetForColumn = InArgs._OnGenerateWidgetForColumn;
	HighlightText = InArgs._HighlightText;

	bHovered = false;
	SetHover(TAttribute<bool>::CreateSP(this, &SAnimOutlinerItem::ShouldAppearHovered));

	SMultiColumnTableRow::Construct(
		SMultiColumnTableRow::FArguments()
			.ShowSelection(true),
		InOwnerTableView);
}

TSharedRef<SWidget> SAnimOutlinerItem::GenerateWidgetForColumn(const FName& ColumnId)
{
	TSharedPtr<FAnimTimelineTrack> PinnedTrack = Track.Pin();
	if (PinnedTrack.IsValid())
	{
		TSharedPtr<SWidget> ColumnWidget = SNullWidget::NullWidget;
		if(OnGenerateWidgetForColumn.IsBound())
		{
			ColumnWidget = OnGenerateWidgetForColumn.Execute(PinnedTrack.ToSharedRef(), ColumnId, SharedThis(this));
		}

		return SNew(SOverlay)
		+SOverlay::Slot()
		[
			ColumnWidget.ToSharedRef()
		];
	}

	return SNullWidget::NullWidget;
}

void SAnimOutlinerItem::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	StaticCastSharedPtr<SAnimOutliner>(OwnerTablePtr.Pin())->ReportChildRowGeometry(Track.Pin().ToSharedRef(), AllottedGeometry);
}

FVector2D SAnimOutlinerItem::ComputeDesiredSize(float LayoutScaleMultiplier) const
{
	TSharedPtr<FAnimTimelineTrack> PinnedTrack = Track.Pin();
	if(PinnedTrack.IsValid())
	{
		return FVector2D(100.0f, PinnedTrack->GetHeight() + PinnedTrack->GetPadding().Combined());
	}

	return FVector2D(100.0f, 16.0f);
}

void SAnimOutlinerItem::AddTrackAreaReference(const TSharedPtr<SAnimTrack>& InTrackWidget)
{
	TrackWidget = InTrackWidget;
}

void SAnimOutlinerItem::OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	bHovered = true;
	SMultiColumnTableRow<TSharedRef<FAnimTimelineTrack>>::OnMouseEnter(MyGeometry, MouseEvent);

	TSharedPtr<FAnimTimelineTrack> PinnedTrack = Track.Pin();
	if(PinnedTrack.IsValid())
	{
		PinnedTrack->SetHovered(true);
	}
}

void SAnimOutlinerItem::OnMouseLeave(const FPointerEvent& MouseEvent)
{
	bHovered = false;
	SMultiColumnTableRow<TSharedRef<FAnimTimelineTrack>>::OnMouseLeave(MouseEvent);

	TSharedPtr<FAnimTimelineTrack> PinnedTrack = Track.Pin();
	if(PinnedTrack.IsValid())
	{
		PinnedTrack->SetHovered(false);
	}
}

bool SAnimOutlinerItem::ShouldAppearHovered() const
{
	if(TSharedPtr<FAnimTimelineTrack> PinnedTrack = Track.Pin())
	{
		return bHovered || PinnedTrack->IsHovered();
	}

	return bHovered;
}
