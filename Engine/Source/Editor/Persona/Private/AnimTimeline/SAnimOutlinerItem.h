// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STreeView.h"
#include "AnimTimeline/AnimTimelineTrack.h"

class FAnimObject;
class SAnimTrack;

class SAnimOutlinerItem : public SMultiColumnTableRow<TSharedRef<FAnimTimelineTrack>>
{
public:
	/** Delegate to invoke to create a new column for this row */
	DECLARE_DELEGATE_RetVal_ThreeParams(TSharedRef<SWidget>, FOnGenerateWidgetForColumn, const TSharedRef<FAnimTimelineTrack>&, const FName&, const TSharedRef<SAnimOutlinerItem>&);

	SLATE_BEGIN_ARGS(SAnimOutlinerItem) {}

	/** Delegate to invoke to create a new column for this row */
	SLATE_EVENT(FOnGenerateWidgetForColumn, OnGenerateWidgetForColumn)

	/** Text to highlight when searching */
	SLATE_ATTRIBUTE(FText, HighlightText)

	SLATE_END_ARGS()

	~SAnimOutlinerItem();

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, const TSharedRef<FAnimTimelineTrack>& InTrack);

	/** SWidget interface */
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	virtual FVector2D ComputeDesiredSize(float LayoutScaleMultiplier) const override;
	virtual void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override;

	/** SMultiColumnTableRow interface */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnId) override;

	/** Get the track this item represents */
	TSharedRef<FAnimTimelineTrack> GetTrack() const { return Track.Pin().ToSharedRef(); }

	/** Add a reference to the specified track, keeping it alive until this row is destroyed */
	void AddTrackAreaReference(const TSharedPtr<SAnimTrack>& InTrackWidget);

	/** Get the text to highlight when searching */
	TAttribute<FText> GetHighlightText() const { return HighlightText; }

private:
	/** Get the label text for this item */
	FText GetLabelText() const;

	/** Get the hover state for this item */
	bool ShouldAppearHovered() const;

private:
	/** The track that we represent */
	TWeakPtr<FAnimTimelineTrack> Track;

	/** Cached reference to a track lane that we relate to. This keeps the track lane alive (it's a weak widget) as long as we are in view. */
	TSharedPtr<SAnimTrack> TrackWidget;

	/** Delegate to invoke to create a new column for this row */
	FOnGenerateWidgetForColumn OnGenerateWidgetForColumn;

	/** Text to highlight when searching */
	TAttribute<FText> HighlightText;
	
	/** Keep an internal IsHovered flag*/
	bool bHovered;
};
