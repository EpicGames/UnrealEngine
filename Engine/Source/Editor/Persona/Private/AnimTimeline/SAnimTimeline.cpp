// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimTimeline/SAnimTimeline.h"
#include "Styling/ISlateStyle.h"
#include "Widgets/SWidget.h"
#include "AnimTimeline/AnimModel.h"
#include "AnimTimeline/SAnimOutliner.h"
#include "AnimTimeline/SAnimTrackArea.h"
#include "SAnimTrack.h"
#include "SAnimTrack.h"
#include "Widgets/Layout/SSplitter.h"
#include "AnimTimeline/SAnimTimelineOverlay.h"
#include "AnimTimeline/SAnimTimelineSplitterOverlay.h"
#include "ISequencerWidgetsModule.h"
#include "FrameNumberNumericInterface.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SScrollBorder.h"
#include "Styling/AppStyle.h"
#include "Fonts/FontMeasure.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SSpacer.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Input/SButton.h"
#include "Modules/ModuleManager.h"
#include "Preferences/PersonaOptions.h"
#include "IPersonaPreviewScene.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "AnimPreviewInstance.h"
#include "EditorWidgetsModule.h"
#include "AnimationEditorPreviewScene.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ScopedTransaction.h"
#include "Widgets/Input/STextEntryPopup.h"
#include "AnimSequenceTimelineCommands.h"
#include "AnimTimelineTrack_Curve.h"
#include "Widgets/Input/SSpinBox.h"
#include "AnimTimeline/SAnimTimelineTransportControls.h"
#include "Animation/AnimSequence.h"
#include "Animation/AnimData/IAnimationDataModel.h"
#include "Animation/AnimSequenceHelpers.h"
#include "Widgets/Layout/SScrollBox.h"
#include "AnimTimeline/AnimTimeSliderController.h"
#include "Framework/Commands/GenericCommands.h"
#include "MovieSceneFwd.h"

#define LOCTEXT_NAMESPACE "SAnimTimeline"

void SAnimTimeline::Construct(const FArguments& InArgs, const TSharedRef<FAnimModel>& InModel)
{
	TWeakPtr<FAnimModel> WeakModel = InModel;

	Model = InModel;
	OnReceivedFocus = InArgs._OnReceivedFocus;


	if (InModel->GetPreviewScene()->GetPreviewMeshComponent()->PreviewInstance)
	{
		InModel->GetPreviewScene()->GetPreviewMeshComponent()->PreviewInstance->AddKeyCompleteDelegate(FSimpleDelegate::CreateSP(this, &SAnimTimeline::HandleKeyComplete));
	}

	ViewRange = MakeAttributeLambda([WeakModel](){ return WeakModel.IsValid() ? WeakModel.Pin()->GetViewRange() : FAnimatedRange(0.0, 0.0); });

	const TAttribute<EFrameNumberDisplayFormats> DisplayFormat = MakeAttributeLambda([]()
	{
		return GetDefault<UPersonaOptions>()->TimelineDisplayFormat;
	});

	const TAttribute<EFrameNumberDisplayFormats> DisplayFormatSecondary = MakeAttributeLambda([]()
	{
		return GetDefault<UPersonaOptions>()->TimelineDisplayFormat == EFrameNumberDisplayFormats::Frames ? EFrameNumberDisplayFormats::Seconds : EFrameNumberDisplayFormats::Frames;
	});

	const TAttribute<FFrameRate> TickResolution = MakeAttributeLambda([this]()
	{
		return FFrameRate(Model.Pin()->GetTickResolution(), 1);
	});

	const TAttribute<FFrameRate> DisplayRate = MakeAttributeLambda([this]()
	{
		return Model.Pin()->GetFrameRate();
	});

	// Create our numeric type interface so we can pass it to the time slider below.
	NumericTypeInterface = MakeShareable(new FFrameNumberInterface(DisplayFormat, 0, TickResolution, DisplayRate));
	SecondaryNumericTypeInterface = MakeShareable(new FFrameNumberInterface(DisplayFormatSecondary, 0, TickResolution, DisplayRate));

	FTimeSliderArgs TimeSliderArgs;
	{
		TimeSliderArgs.ScrubPosition = MakeAttributeLambda([WeakModel](){ return WeakModel.IsValid() ? WeakModel.Pin()->GetScrubPosition() : FFrameTime(0); });
		TimeSliderArgs.ViewRange = ViewRange;
		TimeSliderArgs.PlaybackRange = MakeAttributeLambda([WeakModel](){ return WeakModel.IsValid() ? WeakModel.Pin()->GetPlaybackRange() : TRange<FFrameNumber>(0, 0); });
		TimeSliderArgs.ClampRange = MakeAttributeLambda([WeakModel](){ return WeakModel.IsValid() ? WeakModel.Pin()->GetWorkingRange() : FAnimatedRange(0.0, 0.0); });
		TimeSliderArgs.DisplayRate = DisplayRate;
		TimeSliderArgs.TickResolution = TickResolution;
		TimeSliderArgs.OnViewRangeChanged = FOnViewRangeChanged::CreateSP(&InModel.Get(), &FAnimModel::HandleViewRangeChanged);
		TimeSliderArgs.OnClampRangeChanged = FOnTimeRangeChanged::CreateSP(&InModel.Get(), &FAnimModel::HandleWorkingRangeChanged);
		TimeSliderArgs.IsPlaybackRangeLocked = true;
		TimeSliderArgs.PlaybackStatus = EMovieScenePlayerStatus::Stopped;
		TimeSliderArgs.NumericTypeInterface = NumericTypeInterface;
		TimeSliderArgs.OnScrubPositionChanged = FOnScrubPositionChanged::CreateSP(this, &SAnimTimeline::HandleScrubPositionChanged);
	}

	TimeSliderController = MakeShareable(new FAnimTimeSliderController(TimeSliderArgs, InModel, SharedThis(this), SecondaryNumericTypeInterface));
	
	TSharedRef<FAnimTimeSliderController> TimeSliderControllerRef = TimeSliderController.ToSharedRef();

	// Create the top slider
	const bool bMirrorLabels = false;
	ISequencerWidgetsModule& SequencerWidgets = FModuleManager::Get().LoadModuleChecked<ISequencerWidgetsModule>("SequencerWidgets");
	TopTimeSlider = SequencerWidgets.CreateTimeSlider(TimeSliderControllerRef, bMirrorLabels);

	// Create bottom time range slider
	TSharedRef<ITimeSlider> BottomTimeRange = SequencerWidgets.CreateTimeRange(
		FTimeRangeArgs(
			EShowRange::ViewRange | EShowRange::WorkingRange | EShowRange::PlaybackRange,
			EShowRange::ViewRange | EShowRange::WorkingRange,
			TimeSliderControllerRef,
			EVisibility::Visible,
			NumericTypeInterface.ToSharedRef()
		),
		SequencerWidgets.CreateTimeRangeSlider(TimeSliderControllerRef)
	);

	TSharedRef<SScrollBar> ScrollBar = SNew(SScrollBar)
		.Thickness(FVector2D(5.0f, 5.0f));

	InModel->RefreshTracks();

	TrackArea = SNew(SAnimTrackArea, InModel, TimeSliderControllerRef);
	Outliner = SNew(SAnimOutliner, InModel, TrackArea.ToSharedRef())
		.ExternalScrollbar(ScrollBar)
		.Clipping(EWidgetClipping::ClipToBounds)
		.FilterText_Lambda([this](){ return FilterText; });

	TrackArea->SetOutliner(Outliner);

	ColumnFillCoefficients[0] = 0.25f;
	ColumnFillCoefficients[1] = 0.75f;

	TAttribute<float> FillCoefficient_0, FillCoefficient_1;
	{
		FillCoefficient_0.Bind(TAttribute<float>::FGetter::CreateSP(this, &SAnimTimeline::GetColumnFillCoefficient, 0));
		FillCoefficient_1.Bind(TAttribute<float>::FGetter::CreateSP(this, &SAnimTimeline::GetColumnFillCoefficient, 1));
	}

	const int32 Column0 = 0, Column1 = 1;
	const int32 Row0 = 0, Row1 = 1, Row2 = 2, Row3 = 3, Row4 = 4;

	const float CommonPadding = 3.f;
	const FMargin ResizeBarPadding(4.f, 0, 0, 0);

	ChildSlot
	[
		SNew(SOverlay)
		+SOverlay::Slot()
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			[
				SNew(SOverlay)
				+SOverlay::Slot()
				[
					SNew(SGridPanel)
					.FillRow(1, 1.0f)
					.FillColumn(0, FillCoefficient_0)
					.FillColumn(1, FillCoefficient_1)

					// outliner search box
					+SGridPanel::Slot(Column0, Row0, SGridPanel::Layer(10))
					[
						SNew(SHorizontalBox)
						+SHorizontalBox::Slot()
						.FillWidth(1.0f)
						.VAlign(VAlign_Center)
						[
							SAssignNew(SearchBox, SSearchBox)
							.HintText(LOCTEXT("FilterTracksHint", "Filter"))
							.OnTextChanged(this, &SAnimTimeline::OnOutlinerSearchChanged)
						]
						+SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.HAlign(HAlign_Center)
						.AutoWidth()
						.Padding(2.0f, 0.0f, 2.0f, 0.0f)
						[
							SNew(SBox)
							.MinDesiredWidth(30.0f)
							.VAlign(VAlign_Center)
							.HAlign(HAlign_Center)
							[
								// Current Play Time 
								SNew(SSpinBox<double>)
								.Style(&FAppStyle::GetWidgetStyle<FSpinBoxStyle>("Sequencer.PlayTimeSpinBox"))
								.Value_Lambda([this]() -> double
								{
									return Model.Pin()->GetScrubPosition().Value;
								})
								.OnValueChanged(this, &SAnimTimeline::SetPlayTime)
								.OnValueCommitted_Lambda([this](double InFrame, ETextCommit::Type)
								{
									SetPlayTime(InFrame);
								})
								.MinValue(TOptional<double>())
								.MaxValue(TOptional<double>())
								.TypeInterface(NumericTypeInterface)
								.Delta(this, &SAnimTimeline::GetSpinboxDelta)
								.LinearDeltaSensitivity(25)
							]
						]
					]
					// main timeline area
					+SGridPanel::Slot(Column0, Row1, SGridPanel::Layer(10))
					.ColumnSpan(2)
					[
						SNew(SHorizontalBox)
						+SHorizontalBox::Slot()
						[
							SNew(SOverlay)
							+SOverlay::Slot()
							[
								SNew(SVerticalBox)
								+ SVerticalBox::Slot()
								.FillHeight(1.f)
								[
									SNew(SScrollBorder, Outliner.ToSharedRef())
									[
										SNew(SHorizontalBox)
										
										// outliner tree
										+ SHorizontalBox::Slot()
										.FillWidth(FillCoefficient_0)
										[
											SNew(SBox)
											[
												Outliner.ToSharedRef()
											]
										]

										// track area
										+ SHorizontalBox::Slot()
										.FillWidth(FillCoefficient_1)
										[
											SNew(SBox)
											.Padding(ResizeBarPadding)
											.Clipping(EWidgetClipping::ClipToBounds)
											[
												TrackArea.ToSharedRef()
											]
										]
									]
								]
							]

							+SOverlay::Slot()
							.HAlign(HAlign_Right)
							[
								ScrollBar
							]
						]
					]

					// Transport controls
					+SGridPanel::Slot(Column0, Row3, SGridPanel::Layer(10))
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Fill)
					.Layer(-1)
					[
						SNew(SAnimTimelineTransportControls, InModel->GetPreviewScene(), InModel->GetAnimSequenceBase())
					]

					// Second column
					+SGridPanel::Slot(Column1, Row0)
					.Padding(ResizeBarPadding)
					.RowSpan(2)
					[
						SNew(SBorder)
						.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
						[
							SNew(SSpacer)
						]
					]

					+SGridPanel::Slot(Column1, Row0, SGridPanel::Layer(10))
					.Padding(ResizeBarPadding)
					[
						SNew( SBorder )
						.BorderImage( FAppStyle::GetBrush("ToolPanel.GroupBorder") )
						.BorderBackgroundColor( FLinearColor(.50f, .50f, .50f, 1.0f ) )
						.Padding(0.f)
						.Clipping(EWidgetClipping::ClipToBounds)
						[
							TopTimeSlider.ToSharedRef()
						]
					]

					// Overlay that draws the tick lines
					+SGridPanel::Slot(Column1, Row1, SGridPanel::Layer(10))
					.Padding(ResizeBarPadding)
					[
						SNew(SAnimTimelineOverlay, TimeSliderControllerRef)
						.Visibility( EVisibility::HitTestInvisible )
						.DisplayScrubPosition( false )
						.DisplayTickLines( true )
						.Clipping(EWidgetClipping::ClipToBounds)
						.PaintPlaybackRangeArgs(FPaintPlaybackRangeArgs(FAppStyle::GetBrush("Sequencer.Timeline.PlayRange_L"), FAppStyle::GetBrush("Sequencer.Timeline.PlayRange_R"), 6.f))
					]

					// Overlay that draws the scrub position
					+SGridPanel::Slot(Column1, Row1, SGridPanel::Layer(20))
					.Padding(ResizeBarPadding)
					[
						SNew(SAnimTimelineOverlay, TimeSliderControllerRef)
						.Visibility( EVisibility::HitTestInvisible )
						.DisplayScrubPosition( true )
						.DisplayTickLines( false )
						.Clipping(EWidgetClipping::ClipToBounds)
					]

					// play range slider
					+SGridPanel::Slot(Column1, Row3, SGridPanel::Layer(10))
					.Padding(ResizeBarPadding)
					[
						SNew(SBorder)
						.BorderImage( FAppStyle::GetBrush("ToolPanel.GroupBorder") )
						.BorderBackgroundColor( FLinearColor(0.5f, 0.5f, 0.5f, 1.0f ) )
						.Clipping(EWidgetClipping::ClipToBounds)
						.Padding(0)
						[
							BottomTimeRange
						]
					]
				]
				+SOverlay::Slot()
				[
					// track area virtual splitter overlay
					SNew(SAnimTimelineSplitterOverlay)
					.Style(FAppStyle::Get(), "AnimTimeline.Outliner.Splitter")
					.Visibility(EVisibility::SelfHitTestInvisible)

					+ SSplitter::Slot()
					.Value(FillCoefficient_0)
					.OnSlotResized(SSplitter::FOnSlotResized::CreateSP(this, &SAnimTimeline::OnColumnFillCoefficientChanged, 0))
					[
						SNew(SSpacer)
					]

					+ SSplitter::Slot()
					.Value(FillCoefficient_1)
					.OnSlotResized(SSplitter::FOnSlotResized::CreateSP(this, &SAnimTimeline::OnColumnFillCoefficientChanged, 1))
					[
						SNew(SSpacer)
					]
				]
			]
		]
	];
}

FReply SAnimTimeline::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if(MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		FWidgetPath WidgetPath = MouseEvent.GetEventPath() != nullptr ? *MouseEvent.GetEventPath() : FWidgetPath();

		const bool bCloseAfterSelection = true;
		FMenuBuilder MenuBuilder(bCloseAfterSelection, Model.Pin()->GetCommandList());
		
		MenuBuilder.BeginSection("SelectionEdit", LOCTEXT("TimelineSelectionEditSection", "Selection Edit"));
		{
			MenuBuilder.AddMenuEntry(FGenericCommands::Get().Paste);
		}
		MenuBuilder.EndSection();
		
		MenuBuilder.BeginSection("SnapOptions", LOCTEXT("SnapOptions", "Snapping"));
		{
			MenuBuilder.AddMenuEntry(FAnimSequenceTimelineCommands::Get().SnapToFrames);
			if (GetDefault<UPersonaOptions>()->bExposeNotifiesUICommands)
			{
				MenuBuilder.AddMenuEntry(FAnimSequenceTimelineCommands::Get().SnapToNotifies);
			}
			MenuBuilder.AddMenuEntry(FAnimSequenceTimelineCommands::Get().SnapToCompositeSegments);
			MenuBuilder.AddMenuEntry(FAnimSequenceTimelineCommands::Get().SnapToMontageSections);
		}
		MenuBuilder.EndSection();

		MenuBuilder.BeginSection("TimelineOptions", LOCTEXT("TimelineOptions", "Timeline Options") );
		{
			MenuBuilder.AddSubMenu(
				LOCTEXT("TimeFormat", "Time Format"),
				LOCTEXT("TimeFormatTooltip", "Choose the format of times we display in the timeline"),
				FNewMenuDelegate::CreateLambda([](FMenuBuilder& InMenuBuilder)
				{
					InMenuBuilder.BeginSection("TimeFormat", LOCTEXT("TimeFormat", "Time Format") );
					{
						InMenuBuilder.AddMenuEntry(FAnimSequenceTimelineCommands::Get().DisplaySeconds);
						InMenuBuilder.AddMenuEntry(FAnimSequenceTimelineCommands::Get().DisplayFrames);
					}
					InMenuBuilder.EndSection();

					InMenuBuilder.BeginSection("TimelineAdditional", LOCTEXT("TimelineAdditional", "Additional Display") );
					{
						InMenuBuilder.AddMenuEntry(FAnimSequenceTimelineCommands::Get().DisplayPercentage);
						InMenuBuilder.AddMenuEntry(FAnimSequenceTimelineCommands::Get().DisplaySecondaryFormat);
					}
					InMenuBuilder.EndSection();
				})
			);
		}
		MenuBuilder.EndSection();

		const UAnimSequence* AnimSequence = Cast<UAnimSequence>(Model.Pin()->GetAnimSequenceBase());
		if( AnimSequence )
		{
			FFrameTime MouseTime = TimeSliderController->GetFrameTimeFromMouse(MyGeometry, MouseEvent.GetScreenSpacePosition());
			const float CurrentFrameTime = static_cast<float>(MouseTime.AsDecimal() / static_cast<double>(Model.Pin()->GetTickResolution()));
			const float SequenceLength = AnimSequence->GetPlayLength();

			const FFrameTime MouseFrameTime = TimeSliderController->GetFrameTimeFromMouse(MyGeometry, MouseEvent.GetScreenSpacePosition());
			const FFrameTime AssetFrameTime = ConvertFrameTime(MouseFrameTime, FFrameRate(Model.Pin()->GetTickResolution(), 1), Model.Pin()->GetFrameRate());
			const FFrameNumber RoundedAssetFrame = AssetFrameTime.RoundToFrame();
			const int32 NumKeys = AnimSequence->GetNumberOfSampledKeys();

			MenuBuilder.BeginSection("SequenceEditingContext", LOCTEXT("SequenceEditing", "Sequence Editing") );
			{
				FUIAction Action;
				FText Label;

				//Menu - "Remove Before"
				//Only show this option if the selected frame is greater than frame 1 (first frame)
				if (RoundedAssetFrame > 0)
				{
					//Corrected frame time based on selected frame number
					Action = FUIAction(FExecuteAction::CreateSP(this, &SAnimTimeline::OnCropAnimSequence, true, RoundedAssetFrame));
					Label = FText::Format(LOCTEXT("RemoveTillFrame", "Remove frames in range 0 to {0}"), FText::AsNumber(RoundedAssetFrame.Value));
					MenuBuilder.AddMenuEntry(Label, LOCTEXT("RemoveBefore_ToolTip", "Remove sequence before current position"), FSlateIcon(), Action);
				}

				const FFrameNumber NextFrameNumber = RoundedAssetFrame.Value + 1;

				//Menu - "Remove After"
				//Only show this option if next frame (CurrentKeyIndex + 1) is valid
				if (NextFrameNumber < NumKeys)
				{
					Action = FUIAction(FExecuteAction::CreateSP(this, &SAnimTimeline::OnCropAnimSequence, false, NextFrameNumber));
					Label = FText::Format(LOCTEXT("RemoveFromFrame", "Remove frames in range {0} to {1}"), FText::AsNumber(NextFrameNumber.Value), FText::AsNumber(NumKeys - 1));
					MenuBuilder.AddMenuEntry(Label, LOCTEXT("RemoveAfter_ToolTip", "Remove sequence after current position"), FSlateIcon(), Action);
				}

				MenuBuilder.AddMenuSeparator();

				//Corrected frame time based on selected frame number

				Action = FUIAction(FExecuteAction::CreateSP(this, &SAnimTimeline::OnPromptUserForNumberOfFrames, WidgetPath, TFunction<void(const int32)>([this, RoundedAssetFrame](const int32 NumberOfFrames) { OnInsertAnimSequence(true, RoundedAssetFrame.Value, NumberOfFrames); })));
				Label = FText::Format(LOCTEXT("InsertBeforeCurrentFrame", "Insert frames before {0}"), FText::AsNumber(RoundedAssetFrame.Value));
				MenuBuilder.AddMenuEntry(Label, LOCTEXT("InsertBefore_ToolTip", "Insert a frame before current position"), FSlateIcon(), Action);

				Action = FUIAction(FExecuteAction::CreateSP(this, &SAnimTimeline::OnPromptUserForNumberOfFrames, WidgetPath, TFunction<void(const int32)>([this, RoundedAssetFrame](const int32 NumberOfFrames) { OnInsertAnimSequence(false, RoundedAssetFrame.Value, NumberOfFrames); })));
				Label = FText::Format(LOCTEXT("InsertAfterCurrentFrame", "Insert frames after {0}"), FText::AsNumber(RoundedAssetFrame.Value));
				MenuBuilder.AddMenuEntry(Label, LOCTEXT("InsertAfter_ToolTip", "Insert a frame after current position"), FSlateIcon(), Action);

				MenuBuilder.AddMenuSeparator();

				//Corrected frame time based on selected frame number
				Action = FUIAction(FExecuteAction::CreateSP(this, &SAnimTimeline::OnPromptUserForNumberOfFrames, WidgetPath, TFunction<void(const int32)>([this](const int32 NumberOfFrames) { OnAppendAnimSequence(true, NumberOfFrames); })));
				MenuBuilder.AddMenuEntry(LOCTEXT("AppendBegin", "Add frames to start of sequence"), LOCTEXT("AppendBegin_ToolTip", "Adds user provided number of frames at the start of the sequence"), FSlateIcon(), Action);

				Action = FUIAction(FExecuteAction::CreateSP(this, &SAnimTimeline::OnPromptUserForNumberOfFrames, WidgetPath, TFunction<void(const int32)>([this](const int32 NumberOfFrames) { OnAppendAnimSequence(false, NumberOfFrames); })));
				MenuBuilder.AddMenuEntry(LOCTEXT("AppendEnd", "Append frames at end of sequence"), LOCTEXT("AppendEnd_ToolTip", "AAdds user provided number of frames at the end of the sequence"), FSlateIcon(), Action);

				MenuBuilder.AddMenuSeparator();
				//Menu - "ReZero"
				Action = FUIAction(FExecuteAction::CreateSP(this, &SAnimTimeline::OnReZeroAnimSequence, RoundedAssetFrame.Value));
				Label = FText::Format(LOCTEXT("ReZeroAtFrame", "Re-zero at frame {0}"), FText::AsNumber(RoundedAssetFrame.Value));
				MenuBuilder.AddMenuEntry(Label, FText::Format(LOCTEXT("ReZeroAtFrame_ToolTip", "Resets the root track to (0, 0, 0) at frame {0} and apply the difference to all root transform of the sequence. It moves whole sequence to the amount of current root transform."), FText::AsNumber(RoundedAssetFrame.Value)), FSlateIcon(), Action);

				const int32 FrameNumberForCurrentTime = INDEX_NONE;
				Action = FUIAction(FExecuteAction::CreateSP(this, &SAnimTimeline::OnReZeroAnimSequence, FrameNumberForCurrentTime));
				Label = LOCTEXT("ReZeroAtCurrentTime", "Re-zero at current time");
				MenuBuilder.AddMenuEntry(Label, LOCTEXT("ReZeroAtCurrentTime_ToolTip", "Resets the root track to (0, 0, 0) at the animation scrub time and apply the difference to all root transform of the sequence. It moves whole sequence to the amount of current root transform."), FSlateIcon(), Action);
			}
			MenuBuilder.EndSection();
		}

		FSlateApplication::Get().PushMenu(SharedThis(this), WidgetPath, MenuBuilder.MakeWidget(), FSlateApplication::Get().GetCursorPos(), FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu));

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void SAnimTimeline::OnCropAnimSequence(bool bFromStart, FFrameNumber StartFrame)
{
	UAnimSingleNodeInstance* PreviewInstance = GetPreviewInstance();
	if(PreviewInstance)
	{
		if (PreviewInstance->GetCurrentAsset())
		{
			UAnimSequence* AnimSequence = Cast<UAnimSequence>( PreviewInstance->GetCurrentAsset() );
			if( AnimSequence )
			{
				const FScopedTransaction Transaction( LOCTEXT("CropAnimSequence", "Crop Animation Sequence") );

				//Call modify to restore slider position
				PreviewInstance->Modify();

				//Call modify to restore anim sequence current state
				AnimSequence->Modify();
				
				const TRange<FFrameNumber> TrimRange(TRangeBound<FFrameNumber>::Inclusive(bFromStart ? 0 : StartFrame),
					 TRangeBound<FFrameNumber>::Exclusive(bFromStart ? StartFrame : AnimSequence->GetNumberOfSampledKeys()));

				// Trim off the user-selected part of the raw anim data.
				UE::Anim::AnimationData::Trim(AnimSequence, TrimRange);

				//Resetting slider position to the first frame
				PreviewInstance->SetPosition( 0.0f, false );
			}
		}
	}
}

void SAnimTimeline::OnAppendAnimSequence( bool bFromStart, int32 NumOfFrames )
{
	UAnimSingleNodeInstance* PreviewInstance = GetPreviewInstance();
	if(PreviewInstance && PreviewInstance->GetCurrentAsset())
	{
		UAnimSequence* AnimSequence = Cast<UAnimSequence>(PreviewInstance->GetCurrentAsset());
		if(AnimSequence)
		{
			const FScopedTransaction Transaction(LOCTEXT("InsertAnimSequence", "Insert Animation Sequence"));

			//Call modify to restore slider position
			PreviewInstance->Modify();

			//Call modify to restore anim sequence current state
			AnimSequence->Modify();

			// Crop the raw anim data.
			const int32 StartFrame = (bFromStart)? 0 : AnimSequence->GetDataModel()->GetNumberOfFrames() - 1;
			UE::Anim::AnimationData::DuplicateKeys(AnimSequence, StartFrame, NumOfFrames, StartFrame);
		}
	}
}

void SAnimTimeline::OnInsertAnimSequence(bool bBefore, int32 CurrentFrame, int32 NumFrames)
{
	UAnimSingleNodeInstance* PreviewInstance = GetPreviewInstance();
	if(PreviewInstance && PreviewInstance->GetCurrentAsset())
	{
		UAnimSequence* AnimSequence = Cast<UAnimSequence>(PreviewInstance->GetCurrentAsset());
		if(AnimSequence)
		{
			const FScopedTransaction Transaction(LOCTEXT("InsertAnimSequence", "Insert Animation Sequence"));

			//Call modify to restore slider position
			PreviewInstance->Modify();

			//Call modify to restore anim sequence current state
			AnimSequence->Modify();

			// Crop the raw anim data.
			const int32 StartFrame = (bBefore)? CurrentFrame : CurrentFrame + 1;
			UE::Anim::AnimationData::DuplicateKeys(AnimSequence, StartFrame, NumFrames, CurrentFrame);
		}
	}
}

void SAnimTimeline::OnReZeroAnimSequence(int32 FrameIndex)
{
	UAnimSingleNodeInstance* PreviewInstance = GetPreviewInstance();
	if(PreviewInstance)
	{
		UDebugSkelMeshComponent* PreviewSkelComp =  Model.Pin()->GetPreviewScene()->GetPreviewMeshComponent();

		if (PreviewInstance->GetCurrentAsset() && PreviewSkelComp )
		{
			if(UAnimSequence* AnimSequence = Cast<UAnimSequence>( PreviewInstance->GetCurrentAsset()))
			{
				const FScopedTransaction Transaction( LOCTEXT("ReZeroAnimation", "ReZero Animation Sequence"));
				if (const USkeleton* Skeleton = AnimSequence->GetSkeleton())
				{
					const FName RootBoneName = Skeleton->GetReferenceSkeleton().GetBoneName(0);

					if(AnimSequence->GetDataModel()->IsValidBoneTrackName(RootBoneName))
					{
						TArray<FVector3f> PosKeys;
						TArray<FQuat4f> RotKeys;
						TArray<FVector3f> ScaleKeys;

						TArray<FTransform> BoneTransforms;
						AnimSequence->GetDataModel()->GetBoneTrackTransforms(RootBoneName, BoneTransforms);

						PosKeys.SetNum(BoneTransforms.Num());
						RotKeys.SetNum(BoneTransforms.Num());
						ScaleKeys.SetNum(BoneTransforms.Num());

						// Find vector that would translate current root bone location onto origin.
						FVector FrameTransform = FVector::ZeroVector;
						if (FrameIndex == INDEX_NONE)
						{
							// Use current transform
							FrameTransform = PreviewSkelComp->GetComponentSpaceTransforms()[0].GetLocation();
						}
						else if(BoneTransforms.IsValidIndex(FrameIndex))
						{
							// Use transform at frame
							FrameTransform = BoneTransforms[FrameIndex].GetLocation();
						}

						FVector ApplyTranslation = -1.f * FrameTransform;

						// Convert into world space
						const FVector WorldApplyTranslation = PreviewSkelComp->GetComponentTransform().TransformVector(ApplyTranslation);
						ApplyTranslation = PreviewSkelComp->GetComponentTransform().InverseTransformVector(WorldApplyTranslation);

						for(int32 KeyIndex = 0; KeyIndex < BoneTransforms.Num(); KeyIndex++)
						{
							PosKeys[KeyIndex] = FVector3f(BoneTransforms[KeyIndex].GetLocation() + ApplyTranslation);
							RotKeys[KeyIndex] = FQuat4f(BoneTransforms[KeyIndex].GetRotation());
							ScaleKeys[KeyIndex] = FVector3f(BoneTransforms[KeyIndex].GetScale3D());
						}

						IAnimationDataController& Controller = AnimSequence->GetController();
						Controller.SetBoneTrackKeys(RootBoneName, PosKeys, RotKeys, ScaleKeys);
					}
				}
			}
		}
	}
}

void SAnimTimeline::OnPromptUserForNumberOfFrames(FWidgetPath WidgetPath, TFunction<void(const int32 NumFrames)> Callback)
{
	TSharedRef<STextEntryPopup> TextEntry =
		SNew(STextEntryPopup)
		.Label(LOCTEXT("AskNumFrames", "Number of Frames to Add"))
		.OnTextCommitted_Lambda([Callback](const FText & InNewGroupText, ETextCommit::Type CommitInfo)
		{
			// handle only onEnter. This is a big thing to apply when implicit focus change or any other event
			if (CommitInfo == ETextCommit::OnEnter)
			{
				const int32 NumFrames = FMath::Clamp(FCString::Atoi(*InNewGroupText.ToString()), 0, 1000);
				Callback(NumFrames);
				FSlateApplication::Get().DismissAllMenus();
			}
		});

	FSlateApplication::Get().PushMenu(
		SharedThis(this),
		WidgetPath,
		TextEntry,
		FSlateApplication::Get().GetCursorPos(),
		FPopupTransitionEffect(FPopupTransitionEffect::TypeInPopup)
		);
	
}

TSharedRef<INumericTypeInterface<double>> SAnimTimeline::GetNumericTypeInterface() const
{
	return NumericTypeInterface.ToSharedRef();
}

// FFrameRate::ComputeGridSpacing doesnt deal well with prime numbers, so we have a custom impl here
static bool ComputeGridSpacing(const FFrameRate& InFrameRate, float PixelsPerSecond, double& OutMajorInterval, int32& OutMinorDivisions, float MinTickPx, float DesiredMajorTickPx)
{
	// First try built-in spacing
	const bool bResult = InFrameRate.ComputeGridSpacing(PixelsPerSecond, OutMajorInterval, OutMinorDivisions, MinTickPx, DesiredMajorTickPx);
	if(!bResult || OutMajorInterval == 1.0)
	{
		if (PixelsPerSecond <= 0.f)
		{
			return false;
		}

		const int32 RoundedFPS = static_cast<int32>(FMath::RoundToInt(InFrameRate.AsDecimal()));

		if (RoundedFPS > 0)
		{
			// Showing frames
			TArray<int32, TInlineAllocator<10>> CommonBases;

			// Divide the rounded frame rate by 2s, 3s or 5s recursively
			{
				const int32 Denominators[] = { 2, 3, 5 };

				int32 LowestBase = RoundedFPS;
				for (;;)
				{
					CommonBases.Add(LowestBase);
	
					if (LowestBase % 2 == 0)      { LowestBase = LowestBase / 2; }
					else if (LowestBase % 3 == 0) { LowestBase = LowestBase / 3; }
					else if (LowestBase % 5 == 0) { LowestBase = LowestBase / 5; }
					else
					{ 
						int32 LowestResult = LowestBase;
						for(int32 Denominator : Denominators)
						{
							int32 Result = LowestBase / Denominator;
							if(Result > 0 && Result < LowestResult)
							{
								LowestResult = Result;
							}
						}

						if(LowestResult < LowestBase)
						{
							LowestBase = LowestResult;
						}
						else
						{
							break;
						}
					}
				}
			}

			Algo::Reverse(CommonBases);

			const int32 Scale     = static_cast<int32>(FMath::CeilToInt(DesiredMajorTickPx / PixelsPerSecond * InFrameRate.AsDecimal()));
			const int32 BaseIndex = FMath::Min(Algo::LowerBound(CommonBases, Scale), CommonBases.Num()-1);
			const int32 Base      = CommonBases[BaseIndex];

			const int32 MajorIntervalFrames = FMath::CeilToInt(Scale / static_cast<float>(Base)) * Base;
			OutMajorInterval  = MajorIntervalFrames * InFrameRate.AsInterval();

			// Find the lowest number of divisions we can show that's larger than the minimum tick size
			OutMinorDivisions = 0;
			for (int32 DivIndex = 0; DivIndex < BaseIndex; ++DivIndex)
			{
				if (Base % CommonBases[DivIndex] == 0)
				{
					const int32 MinorDivisions = MajorIntervalFrames/CommonBases[DivIndex];
					if (OutMajorInterval / MinorDivisions * PixelsPerSecond >= MinTickPx)
					{
						OutMinorDivisions = MinorDivisions;
						break;
					}
				}
			}
		}
	}

	return OutMajorInterval != 0;
}

bool SAnimTimeline::GetGridMetrics(float PhysicalWidth, double& OutMajorInterval, int32& OutMinorDivisions) const
{
	const FSlateFontInfo SmallLayoutFont = FCoreStyle::GetDefaultFontStyle("Regular", 8);
	const TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();

	const FFrameRate DisplayRate = Model.Pin()->GetFrameRate();
	const double BiggestTime = ViewRange.Get().GetUpperBoundValue();
	const FString TickString = NumericTypeInterface->ToString((BiggestTime * DisplayRate).FrameNumber.Value);
	const FVector2D MaxTextSize = FontMeasureService->Measure(TickString, SmallLayoutFont);

	constexpr float MajorTickMultiplier = 2.f;

	const float MinTickPx = static_cast<float>(MaxTextSize.X) + 5.f;
	const float DesiredMajorTickPx = static_cast<float>(MaxTextSize.X) * MajorTickMultiplier;

	if (PhysicalWidth > 0 && DisplayRate.AsDecimal() > 0)
	{
		return ComputeGridSpacing(
			DisplayRate,
			static_cast<float>(PhysicalWidth / ViewRange.Get().Size<double>()),
			OutMajorInterval,
			OutMinorDivisions,
			MinTickPx,
			DesiredMajorTickPx);
	}

	return false;
}

TSharedPtr<ITimeSliderController> SAnimTimeline::GetTimeSliderController() const 
{ 
	return TimeSliderController; 
}

void SAnimTimeline::OnOutlinerSearchChanged( const FText& Filter )
{
	FilterText = Filter;

	Outliner->RefreshFilter();
}

void SAnimTimeline::OnColumnFillCoefficientChanged(float FillCoefficient, int32 ColumnIndex)
{
	ColumnFillCoefficients[ColumnIndex] = FillCoefficient;
}

void SAnimTimeline::HandleKeyComplete()
{
	Model.Pin()->RefreshTracks();
}

class UAnimSingleNodeInstance* SAnimTimeline::GetPreviewInstance() const
{
	UDebugSkelMeshComponent* PreviewMeshComponent = Model.Pin()->GetPreviewScene()->GetPreviewMeshComponent();
	return PreviewMeshComponent && PreviewMeshComponent->IsPreviewOn()? PreviewMeshComponent->PreviewInstance : nullptr;
}

void SAnimTimeline::HandleScrubPositionChanged(FFrameTime NewScrubPosition, bool bIsScrubbing, bool bEvaluate) const
{
	if (UAnimSingleNodeInstance* PreviewInstance = GetPreviewInstance())
	{
		if(PreviewInstance->IsPlaying())
		{
			PreviewInstance->SetPlaying(false);
		}
	}

	Model.Pin()->SetScrubPosition(NewScrubPosition);
}

double SAnimTimeline::GetSpinboxDelta() const
{
	return FFrameRate(Model.Pin()->GetTickResolution(), 1).AsDecimal() * Model.Pin()->GetFrameRate().AsInterval();
}

void SAnimTimeline::SetPlayTime(double InFrameTime)
{
	if (UAnimSingleNodeInstance* PreviewInstance = GetPreviewInstance())
	{
		PreviewInstance->SetPlaying(false);
		PreviewInstance->SetPosition(static_cast<float>(InFrameTime / static_cast<double>(Model.Pin()->GetTickResolution())));
	}
}

#undef LOCTEXT_NAMESPACE
