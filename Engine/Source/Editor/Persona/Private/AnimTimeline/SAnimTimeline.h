// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimatedRange.h"
#include "CoreTypes.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Input/NumericTypeInterface.h"
#include "ITimeSlider.h"
#include "ITransportControl.h"

class FAnimModel;
class SAnimOutliner;
class SAnimTrackArea;
class FAnimTimeSliderController;
class SSearchBox;
struct FPaintPlaybackRangeArgs;
class UAnimSingleNodeInstance;

/**
 * Implements the anim timeline widget.
 */
class SAnimTimeline : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAnimTimeline) { }

	/** The current total range of frame indices */
	SLATE_ATTRIBUTE( FInt32Interval, ViewIndices )
			
	/** Called when any widget contained within the anim timeline has received focus */
	SLATE_EVENT( FSimpleDelegate, OnReceivedFocus )

	SLATE_END_ARGS()

public:

	/**
	 * Construct this widget.
	 *
	 * @param InArgs The declaration data for this widget.
	 * @param InModel The model for the anim timeline
	 */
	void Construct(const FArguments& InArgs, const TSharedRef<FAnimModel>& InModel);

	/** SWidget interface */
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	/** Compute a major grid interval and number of minor divisions to display */
	bool GetGridMetrics(float PhysicalWidth, double& OutMajorInterval, int32& OutMinorDivisions) const;

	/** Get the time slider controller */
	TSharedPtr<ITimeSliderController> GetTimeSliderController() const;

private:
	/**
	 * @return The fill percentage of the animation outliner
	 */
	float GetColumnFillCoefficient(int32 ColumnIndex) const
	{
		return ColumnFillCoefficients[ColumnIndex];
	}

	/** Get numeric Type interface for converting between frame numbers and display formats. */
	TSharedRef<INumericTypeInterface<double>> GetNumericTypeInterface() const;

	/** Called when the outliner search terms change */
	void OnOutlinerSearchChanged(const FText& Filter);

	/** Called when a column fill percentage is changed by a splitter slot. */
	void OnColumnFillCoefficientChanged(float FillCoefficient, int32 ColumnIndex);

	/** Handles an additive layer key being set */
	void HandleKeyComplete();

	UAnimSingleNodeInstance* GetPreviewInstance() const;



	void HandleScrubPositionChanged(FFrameTime NewScrubPosition, bool bIsScrubbing, bool bEvaluate) const;

	void OnCropAnimSequence(bool bFromStart, FFrameNumber StartFrame);

	void OnAppendAnimSequence(bool bFromStart, int32 NumOfFrames);

	void OnInsertAnimSequence(bool bBefore, int32 CurrentFrame, int32 NumFrames);

	void OnReZeroAnimSequence(int32 FrameIndex);

	void OnPromptUserForNumberOfFrames(FWidgetPath WidgetPath, TFunction<void(const int32 NumFrames)> Callback);
	double GetSpinboxDelta() const;

	void SetPlayTime(double InFrameTime);

private:
	/** Anim timeline model */
	TWeakPtr<FAnimModel> Model;

	/** Outliner widget */
	TSharedPtr<SAnimOutliner> Outliner;

	/** Track area widget */
	TSharedPtr<SAnimTrackArea> TrackArea;

	/** The time slider controller */
	TSharedPtr<FAnimTimeSliderController> TimeSliderController;

	/** The top time slider widget */
	TSharedPtr<ITimeSlider> TopTimeSlider;

	/** The search box for filtering tracks. */
	TSharedPtr<SSearchBox> SearchBox;

	/** The fill coefficients of each column in the grid. */
	float ColumnFillCoefficients[2];

	/** Called when the user has begun dragging the selection selection range */
	FSimpleDelegate OnSelectionRangeBeginDrag;

	/** Called when the user has finished dragging the selection selection range */
	FSimpleDelegate OnSelectionRangeEndDrag;

	/** Called when any widget contained within the anim timeline has received focus */
	FSimpleDelegate OnReceivedFocus;

	/** Numeric Type interface for converting between frame numbers and display formats. */
	TSharedPtr<INumericTypeInterface<double>> NumericTypeInterface;

	/** Secondary numeric Type interface for converting between frame numbers and display formats. */
	TSharedPtr<INumericTypeInterface<double>> SecondaryNumericTypeInterface;

	/** The view range */
	TAttribute<FAnimatedRange> ViewRange;

	/** Filter text used to search the tree */
	FText FilterText;
};
