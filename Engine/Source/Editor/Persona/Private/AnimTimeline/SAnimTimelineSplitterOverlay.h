// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "HAL/Platform.h"
#include "Input/CursorReply.h"
#include "Input/Reply.h"
#include "Layout/Margin.h"
#include "Templates/SharedPointer.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/SOverlay.h"

class FArrangedChildren;
struct FCaptureLostEvent;
struct FGeometry;
struct FPointerEvent;

/** Splitter used on the anim timeline as an overlay. Input is disabled on all areas except the draggable positions */
class SAnimTimelineSplitterOverlay : public SOverlay
{
public:
	typedef SSplitter::FArguments FArguments;

	void Construct( const FArguments& InArgs );

	/** SwWidget interface */
	virtual void OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const override;
	virtual FCursorReply OnCursorQuery( const FGeometry& MyGeometry, const FPointerEvent& CursorEvent ) const override;
	virtual FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	virtual void OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent) override;
	virtual FReply OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	virtual FReply OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	virtual void OnMouseLeave( const FPointerEvent& MouseEvent ) override;

private:
	FMargin GetSplitterHandlePadding(int32 Index) const;

	TSharedPtr<SSplitter> Splitter;
	mutable TArray<FMargin> SlotPadding;
};
