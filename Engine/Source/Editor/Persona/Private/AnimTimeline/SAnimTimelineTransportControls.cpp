// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimTimeline/SAnimTimelineTransportControls.h"
#include "EditorWidgetsModule.h"
#include "AnimationEditorPreviewScene.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "Animation/AnimSingleNodeInstance.h"
#include "Animation/AnimSequenceBase.h"
#include "Animation/AnimSequence.h"
#include "AnimPreviewInstance.h"
#include "Modules/ModuleManager.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "UICommandList_Pinnable.h"
#include "ViewportToolbar/AnimViewportContext.h"
#include "ViewportToolbar/AnimationEditorMenus.h"
#include "Widgets/SBoxPanel.h"

void SAnimTimelineTransportControls::Construct(const FArguments& InArgs, const TSharedRef<IPersonaPreviewScene>& InPreviewScene, UAnimSequenceBase* InAnimSequenceBase)
{
	WeakPreviewScene = InPreviewScene;
	AnimSequenceBase = InAnimSequenceBase;

	check(AnimSequenceBase);

	FEditorWidgetsModule& EditorWidgetsModule = FModuleManager::LoadModuleChecked<FEditorWidgetsModule>("EditorWidgets");

	FTransportControlArgs TransportControlArgs;
	TransportControlArgs.OnForwardPlay = FOnClicked::CreateSP(this, &SAnimTimelineTransportControls::OnClick_Forward);
	TransportControlArgs.OnRecord = FOnClicked::CreateSP(this, &SAnimTimelineTransportControls::OnClick_Record);
	TransportControlArgs.OnBackwardPlay = FOnClicked::CreateSP(this, &SAnimTimelineTransportControls::OnClick_Backward);
	TransportControlArgs.OnForwardStep = FOnClicked::CreateSP(this, &SAnimTimelineTransportControls::OnClick_Forward_Step);
	TransportControlArgs.OnBackwardStep = FOnClicked::CreateSP(this, &SAnimTimelineTransportControls::OnClick_Backward_Step);
	TransportControlArgs.OnForwardEnd = FOnClicked::CreateSP(this, &SAnimTimelineTransportControls::OnClick_Forward_End);
	TransportControlArgs.OnBackwardEnd = FOnClicked::CreateSP(this, &SAnimTimelineTransportControls::OnClick_Backward_End);
	TransportControlArgs.OnToggleLooping = FOnClicked::CreateSP(this, &SAnimTimelineTransportControls::OnClick_ToggleLoop);
	TransportControlArgs.OnGetLooping = FOnGetLooping::CreateSP(this, &SAnimTimelineTransportControls::IsLoopStatusOn);
	TransportControlArgs.OnGetPlaybackMode = FOnGetPlaybackMode::CreateSP(this, &SAnimTimelineTransportControls::GetPlaybackMode);
	TransportControlArgs.OnGetRecording = FOnGetRecording::CreateSP(this, &SAnimTimelineTransportControls::IsRecording);

	TSharedPtr<SHorizontalBox> HorizontalBox;
	// clang-format off
	ChildSlot
	[
		SAssignNew(HorizontalBox, SHorizontalBox)
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Center)
		.FillWidth(1.0)
		[
			EditorWidgetsModule.CreateTransportControl(TransportControlArgs)
		]
	];
	// clang-format on

	// Add Playback Speed widget - we need to retrieve the viewport tab and the viewport it hosts
	if (TSharedPtr<FAnimationEditorPreviewScene> PreviewScene =
			StaticCastSharedPtr<FAnimationEditorPreviewScene>(WeakPreviewScene.Pin()))
	{
		const FName TimelinePlaybackMenu = "AnimationEditor.Timeline.PlaybackMenu";
		if (!UToolMenus::Get()->IsMenuRegistered(TimelinePlaybackMenu))
		{
			if (UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(
					TimelinePlaybackMenu, NAME_None, EMultiBoxType::SlimHorizontalToolBar, false
				))
			{
				FToolMenuSection& UnnamedSection = Menu->AddSection(NAME_None);
				UnnamedSection.AddEntry(UE::AnimationEditor::CreatePlaybackSubmenu());
			}
		}

		FToolMenuContext MenuContext;
		{
			MenuContext.AppendCommandList(PreviewScene->GetCommandList());
			UAnimViewportContext* const ContextObject = NewObject<UAnimViewportContext>();
			ContextObject->PersonaPreviewScene = PreviewScene;
			MenuContext.AddObject(ContextObject);
		}

		// clang-format off
		HorizontalBox->AddSlot()
		.HAlign(HAlign_Right)
		.FillWidth(0.4)
		[
			UToolMenus::Get()->GenerateWidget(TimelinePlaybackMenu, MenuContext)
		];
		// clang-format on
	}
}

UAnimSingleNodeInstance* SAnimTimelineTransportControls::GetPreviewInstance() const
{
	UDebugSkelMeshComponent* PreviewMeshComponent = GetPreviewScene()->GetPreviewMeshComponent();
	return PreviewMeshComponent && PreviewMeshComponent->IsPreviewOn()? PreviewMeshComponent->PreviewInstance : nullptr;
}

FReply SAnimTimelineTransportControls::OnClick_Forward_Step()
{
	UDebugSkelMeshComponent* SMC = GetPreviewScene()->GetPreviewMeshComponent();

	if (UAnimSingleNodeInstance* PreviewInstance = GetPreviewInstance())
	{
		bool bShouldStepCloth = FMath::Abs(PreviewInstance->GetLength() - PreviewInstance->GetCurrentTime()) > SMALL_NUMBER;

		PreviewInstance->SetPlaying(false);
		PreviewInstance->StepForward();

		if(SMC && bShouldStepCloth)
		{
			SMC->bPerformSingleClothingTick = true;
		}
	}
	else if (SMC)
	{
		const UAnimSequence* AnimSequence = Cast<UAnimSequence>(AnimSequenceBase);
		const FFrameRate TargetFramerate = AnimSequence ? AnimSequence->GetSamplingFrameRate() : FFrameRate(30, 1);

		// Advance a single frame, leaving it paused afterwards
		SMC->GlobalAnimRateScale = 1.0f;
		SMC->TickAnimation(static_cast<float>(TargetFramerate.AsInterval()), false);
		SMC->GlobalAnimRateScale = 0.0f;
	}

	return FReply::Handled();
}

FReply SAnimTimelineTransportControls::OnClick_Forward_End()
{
	UAnimSingleNodeInstance* PreviewInstance = GetPreviewInstance();
	if (PreviewInstance)
	{
		PreviewInstance->SetPlaying(false);
		PreviewInstance->SetPosition(PreviewInstance->GetLength(), false);
	}

	return FReply::Handled();
}

FReply SAnimTimelineTransportControls::OnClick_Backward_Step()
{
	UAnimSingleNodeInstance* PreviewInstance = GetPreviewInstance();
	UDebugSkelMeshComponent* SMC = GetPreviewScene()->GetPreviewMeshComponent();
	if (PreviewInstance)
	{
		bool bShouldStepCloth = PreviewInstance->GetCurrentTime() > SMALL_NUMBER;

		PreviewInstance->SetPlaying(false);
		PreviewInstance->StepBackward();

		if(SMC && bShouldStepCloth)
		{
			SMC->bPerformSingleClothingTick = true;
		}
	}
	return FReply::Handled();
}

FReply SAnimTimelineTransportControls::OnClick_Backward_End()
{
	UAnimSingleNodeInstance* PreviewInstance = GetPreviewInstance();
	if (PreviewInstance)
	{
		PreviewInstance->SetPlaying(false);
		PreviewInstance->SetPosition(0.f, false);
	}
	return FReply::Handled();
}

FReply SAnimTimelineTransportControls::OnClick_Forward()
{
	UAnimSingleNodeInstance* PreviewInstance = GetPreviewInstance();
	UDebugSkelMeshComponent* SMC = GetPreviewScene()->GetPreviewMeshComponent();

	if (PreviewInstance)
	{
		bool bIsReverse = PreviewInstance->IsReverse();
		bool bIsPlaying = PreviewInstance->IsPlaying();
		// if current bIsReverse and bIsPlaying, we'd like to just turn off reverse
		if (bIsReverse && bIsPlaying)
		{
			PreviewInstance->SetReverse(false);
		}
		// already playing, simply pause
		else if (bIsPlaying) 
		{
			PreviewInstance->SetPlaying(false);
			
			if(SMC && SMC->bPauseClothingSimulationWithAnim)
			{
				SMC->SuspendClothingSimulation();
			}
		}
		// if not playing, play forward
		else 
		{
			//if we're at the end of the animation, jump back to the beginning before playing
			if ( PreviewInstance->GetCurrentTime() >= AnimSequenceBase->GetPlayLength() )
			{
				PreviewInstance->SetPosition(0.0f, false);
			}

			PreviewInstance->SetReverse(false);
			PreviewInstance->SetPlaying(true);

			if(SMC && SMC->bPauseClothingSimulationWithAnim)
			{
				SMC->ResumeClothingSimulation();
			}
		}
	}
	else if(SMC)
	{
		SMC->GlobalAnimRateScale = (SMC->GlobalAnimRateScale > 0.0f) ? 0.0f : 1.0f;
	}

	return FReply::Handled();
}

FReply SAnimTimelineTransportControls::OnClick_Backward()
{
	UAnimSingleNodeInstance* PreviewInstance = GetPreviewInstance();
	if (PreviewInstance)
	{
		bool bIsReverse = PreviewInstance->IsReverse();
		bool bIsPlaying = PreviewInstance->IsPlaying();
		// if currently playing forward, just simply turn on reverse
		if (!bIsReverse && bIsPlaying)
		{
			PreviewInstance->SetReverse(true);
		}
		else if (bIsPlaying)
		{
			PreviewInstance->SetPlaying(false);
		}
		else
		{
			//if we're at the beginning of the animation, jump back to the end before playing
			if ( PreviewInstance->GetCurrentTime() <= 0.0f )
			{
				PreviewInstance->SetPosition(AnimSequenceBase->GetPlayLength(), false);
			}

			PreviewInstance->SetPlaying(true);
			PreviewInstance->SetReverse(true);
		}
	}
	return FReply::Handled();
}

FReply SAnimTimelineTransportControls::OnClick_ToggleLoop()
{
	UAnimSingleNodeInstance* PreviewInstance = GetPreviewInstance();
	if (PreviewInstance)
	{
		bool bIsLooping = PreviewInstance->IsLooping();
		PreviewInstance->SetLooping(!bIsLooping);
	}
	return FReply::Handled();
}

FReply SAnimTimelineTransportControls::OnClick_Record()
{
	StaticCastSharedRef<FAnimationEditorPreviewScene>(GetPreviewScene())->RecordAnimation();

	return FReply::Handled();
}

bool SAnimTimelineTransportControls::IsLoopStatusOn() const
{
	UAnimSingleNodeInstance* PreviewInstance = GetPreviewInstance();
	return (PreviewInstance && PreviewInstance->IsLooping());
}

EPlaybackMode::Type SAnimTimelineTransportControls::GetPlaybackMode() const
{
	if (UAnimSingleNodeInstance* PreviewInstance = GetPreviewInstance())
	{
		if (PreviewInstance->IsPlaying())
		{
			return PreviewInstance->IsReverse() ? EPlaybackMode::PlayingReverse : EPlaybackMode::PlayingForward;
		}
		return EPlaybackMode::Stopped;
	}
	else if (UDebugSkelMeshComponent* SMC = GetPreviewScene()->GetPreviewMeshComponent())
	{
		return (SMC->GlobalAnimRateScale > 0.0f) ? EPlaybackMode::PlayingForward : EPlaybackMode::Stopped;
	}
	
	return EPlaybackMode::Stopped;
}

bool SAnimTimelineTransportControls::IsRecording() const
{
	return StaticCastSharedRef<FAnimationEditorPreviewScene>(GetPreviewScene())->IsRecording();
}
