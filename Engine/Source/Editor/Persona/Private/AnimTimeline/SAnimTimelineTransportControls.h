// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "ITransportControl.h"

class IPersonaPreviewScene;
class UAnimSequenceBase;
class UAnimSingleNodeInstance;

class SAnimTimelineTransportControls : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAnimTimelineTransportControls) {}

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<IPersonaPreviewScene>& InPreviewScene, UAnimSequenceBase* InAnimSequenceBase);

private:
	UAnimSingleNodeInstance* GetPreviewInstance() const;

	TSharedRef<IPersonaPreviewScene> GetPreviewScene() const { return WeakPreviewScene.Pin().ToSharedRef(); }

	FReply OnClick_Forward_Step();

	FReply OnClick_Forward_End();

	FReply OnClick_Backward_Step();

	FReply OnClick_Backward_End();

	FReply OnClick_Forward();

	FReply OnClick_Backward();

	FReply OnClick_ToggleLoop();

	FReply OnClick_Record();

	bool IsLoopStatusOn() const;

	EPlaybackMode::Type GetPlaybackMode() const;

	bool IsRecording() const;

private:
	TWeakPtr<IPersonaPreviewScene> WeakPreviewScene;

	UAnimSequenceBase* AnimSequenceBase;
};