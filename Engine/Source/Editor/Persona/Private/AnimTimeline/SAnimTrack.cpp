// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimTimeline/SAnimTrack.h"
#include "Rendering/DrawElements.h"
#include "Styling/AppStyle.h"
#include "AnimTimeline/SAnimOutliner.h"
#include "AnimTimeline/SAnimTrackResizeArea.h"
#include "Widgets/SOverlay.h"

void SAnimTrack::Construct(const FArguments& InArgs, const TSharedRef<FAnimTimelineTrack>& InTrack, const TSharedRef<SAnimOutliner>& InOutliner)
{
	WeakTrack = InTrack;
	WeakOutliner = InOutliner;
	ViewRange = InArgs._ViewRange;

	ChildSlot
	.HAlign(HAlign_Fill)
	.Padding(0)
	[
		SNew(SOverlay)
		+SOverlay::Slot()
		[
			InArgs._Content.Widget
		]
	];
}

int32 SAnimTrack::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	static const FName BorderName("AnimTimeline.Outliner.DefaultBorder");
	static const FName SelectionColorName("SelectionColor");

	TSharedPtr<FAnimTimelineTrack> Track = WeakTrack.Pin();
	if(Track.IsValid())
	{
		if (Track->IsVisible())
		{
			float TotalNodeHeight = Track->GetHeight() + Track->GetPadding().Combined();

			// draw hovered
			if (Track->IsHovered())
			{
				FSlateDrawElement::MakeBox(
					OutDrawElements,
					LayerId++,
					AllottedGeometry.ToPaintGeometry(
						FVector2D(AllottedGeometry.GetLocalSize().X, TotalNodeHeight),
						FSlateLayoutTransform()
					),
					FAppStyle::GetBrush(BorderName),
					ESlateDrawEffect::None,
					FLinearColor(1.0f, 1.0f, 1.0f, 0.05f)
				);
			}

			// Draw track bottom border
			FSlateDrawElement::MakeLines(
				OutDrawElements,
				LayerId++,
				AllottedGeometry.ToPaintGeometry(),
				TArray<FVector2D>({ FVector2D(0.0f, TotalNodeHeight), FVector2D(AllottedGeometry.GetLocalSize().X, TotalNodeHeight) }),
				ESlateDrawEffect::None,
				FLinearColor(0.1f, 0.1f, 0.1f, 0.3f)
			);
		}
	}

	return SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId + 1, InWidgetStyle, bParentEnabled);
}

void SAnimTrack::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	FVector2D ThisFrameDesiredSize = GetDesiredSize();

	if (LastDesiredSize.IsSet() && ThisFrameDesiredSize.Y != LastDesiredSize.GetValue().Y)
	{
		TSharedPtr<SAnimOutliner> Outliner = WeakOutliner.Pin();
		if (Outliner.IsValid())
		{
			Outliner->RequestTreeRefresh();
		}
	}

	LastDesiredSize = ThisFrameDesiredSize;
}

FVector2D SAnimTrack::ComputeDesiredSize(float LayoutScale) const
{
	TSharedPtr<FAnimTimelineTrack> Track = WeakTrack.Pin();
	if(Track.IsValid())
	{
		return FVector2D(100.f, Track->GetHeight() + Track->GetPadding().Combined());
	}

	return FVector2D::ZeroVector;
}

float SAnimTrack::GetPhysicalPosition() const
{
	TSharedPtr<FAnimTimelineTrack> Track = WeakTrack.Pin();
	if(Track.IsValid())
	{
		return WeakOutliner.Pin()->ComputeTrackPosition(Track.ToSharedRef()).Get(0.0f);
	}
	return 0.0f;
}

void SAnimTrack::OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	SCompoundWidget::OnMouseEnter(MyGeometry, MouseEvent);

	TSharedPtr<FAnimTimelineTrack> Track = WeakTrack.Pin();
	if(Track.IsValid())
	{
		Track->SetHovered(true);
	}
}

void SAnimTrack::OnMouseLeave(const FPointerEvent& MouseEvent)
{
	SCompoundWidget::OnMouseLeave(MouseEvent);

	TSharedPtr<FAnimTimelineTrack> Track = WeakTrack.Pin();
	if(Track.IsValid())
	{
		Track->SetHovered(false);
	}
}

FReply SAnimTrack::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	TSharedPtr<FAnimTimelineTrack> Track = WeakTrack.Pin();
	TSharedPtr<SAnimOutliner> Outliner = WeakOutliner.Pin();
	if(Track.IsValid() && Track->SupportsSelection())
	{
		if(MouseEvent.GetModifierKeys().IsControlDown())
		{
			Outliner->SetItemSelection(Track.ToSharedRef(), true, ESelectInfo::OnMouseClick);
		}
		else if(MouseEvent.GetModifierKeys().IsShiftDown())
		{
			Outliner->Private_SelectRangeFromCurrentTo(Track.ToSharedRef());
		}
		else
		{
			if(MouseEvent.GetEffectingButton() != EKeys::RightMouseButton || !Outliner->IsItemSelected(Track.ToSharedRef()))
			{
				Outliner->ClearSelection();
			}
			Outliner->SetItemSelection(Track.ToSharedRef(), true, ESelectInfo::OnMouseClick);
		}
			
		return FReply::Handled();
	}

	return FReply::Unhandled();
}
