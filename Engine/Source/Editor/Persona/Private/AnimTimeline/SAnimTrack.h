// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnimTimeline/AnimTimelineTrack.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "ITimeSlider.h"

class FPaintArgs;
class FSlateWindowElementList;
class SAnimOutliner;

/**
 * A wrapper widget responsible for positioning a track within the track area
 */
class SAnimTrack : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAnimTrack) {}

	SLATE_DEFAULT_SLOT(FArguments, Content)

	SLATE_ATTRIBUTE(FAnimatedRange, ViewRange)

	SLATE_END_ARGS()

	/** Construct this widget */
	void Construct(const FArguments& InArgs, const TSharedRef<FAnimTimelineTrack>& InTrack, const TSharedRef<SAnimOutliner>& InOutliner);
	
	/** Paint this widget */
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	virtual void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	/** Get the desired physical vertical position of this track */
	float GetPhysicalPosition() const;

protected:
	virtual FVector2D ComputeDesiredSize(float LayoutScale) const override;

private:
	/** The track that we represent */
	TWeakPtr<FAnimTimelineTrack> WeakTrack;

	/** Outliner that we are associated with */
	TWeakPtr<SAnimOutliner> WeakOutliner;

	/** Our desired size last frame */
	TOptional<FVector2D> LastDesiredSize;

	/** The range our track should display */
	TAttribute<FAnimatedRange> ViewRange;

	/** The range of indices our track should display */
	TAttribute<FInt32Interval> ViewIndices;
};
