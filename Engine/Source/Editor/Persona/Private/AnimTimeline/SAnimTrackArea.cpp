// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimTimeline/SAnimTrackArea.h"
#include "Types/PaintArgs.h"
#include "Layout/ArrangedChildren.h"
#include "Rendering/DrawElements.h"
#include "Layout/LayoutUtils.h"
#include "Widgets/SWeakWidget.h"
#include "Styling/AppStyle.h"
#include "AnimTimeline/SAnimTrack.h"
#include "AnimTimeline/SAnimOutliner.h"
#include "AnimTimeline/AnimTimelineTrack.h"
#include "AnimTimeline/AnimModel.h"
#include "AnimTimeline/AnimTimeSliderController.h"

FAnimTrackAreaSlot::FAnimTrackAreaSlot(const TSharedPtr<SAnimTrack>& InSlotContent)
	: TAlignmentWidgetSlotMixin<FAnimTrackAreaSlot>(HAlign_Fill, VAlign_Top)
{
	TrackWidget = InSlotContent;

	AttachWidget(
		SNew(SWeakWidget)
		.Clipping(EWidgetClipping::ClipToBounds)
		.PossiblyNullContent(InSlotContent)
	);
}

float FAnimTrackAreaSlot::GetVerticalOffset() const
{
	TSharedPtr<SAnimTrack> PinnedTrackWidget = TrackWidget.Pin();
	return PinnedTrackWidget.IsValid() ? PinnedTrackWidget->GetPhysicalPosition() : 0.f;
}

void SAnimTrackArea::Construct(const FArguments& InArgs, const TSharedRef<FAnimModel>& InAnimModel, const TSharedRef<FAnimTimeSliderController>& InTimeSliderController)
{
	WeakModel = InAnimModel;
	WeakTimeSliderController = InTimeSliderController;
}

void SAnimTrackArea::SetOutliner(const TSharedPtr<SAnimOutliner>& InOutliner)
{
	WeakOutliner = InOutliner;
}

void SAnimTrackArea::Empty()
{
	TrackSlots.Empty();
	Children.Empty();
}

void SAnimTrackArea::AddTrackSlot(const TSharedRef<FAnimTimelineTrack>& InTrack, const TSharedPtr<SAnimTrack>& InSlot)
{
	TrackSlots.Add(InTrack, InSlot);
	Children.AddSlot(FAnimTrackAreaSlot::FSlotArguments(MakeUnique<FAnimTrackAreaSlot>(InSlot)));
}

TSharedPtr<SAnimTrack> SAnimTrackArea::FindTrackSlot(const TSharedRef<FAnimTimelineTrack>& InTrack)
{
	// Remove stale entries
	TrackSlots.Remove(TWeakPtr<FAnimTimelineTrack>());

	return TrackSlots.FindRef(InTrack).Pin();
}

void SAnimTrackArea::OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const
{
	for (int32 ChildIndex = 0; ChildIndex < Children.Num(); ++ChildIndex)
	{
		const FAnimTrackAreaSlot& CurChild = Children[ChildIndex];

		const EVisibility ChildVisibility = CurChild.GetWidget()->GetVisibility();
		if (!ArrangedChildren.Accepts(ChildVisibility))
		{
			continue;
		}

		const FMargin Padding(0, CurChild.GetVerticalOffset(), 0, 0);

		const AlignmentArrangeResult XResult = AlignChild<Orient_Horizontal>(static_cast<float>(AllottedGeometry.GetLocalSize().X), CurChild, Padding, 1.0f, false);
		const AlignmentArrangeResult YResult = AlignChild<Orient_Vertical>(static_cast<float>(AllottedGeometry.GetLocalSize().Y), CurChild, Padding, 1.0f, false);

		ArrangedChildren.AddWidget(ChildVisibility,
			AllottedGeometry.MakeChild(
				CurChild.GetWidget(),
				FVector2D(XResult.Offset, YResult.Offset),
				FVector2D(XResult.Size, YResult.Size)
			)
		);
	}
}

FVector2D SAnimTrackArea::ComputeDesiredSize( float ) const
{
	FVector2D MaxSize(0.0f, 0.0f);
	for (int32 ChildIndex = 0; ChildIndex < Children.Num(); ++ChildIndex)
	{
		const FAnimTrackAreaSlot& CurChild = Children[ChildIndex];

		const EVisibility ChildVisibilty = CurChild.GetWidget()->GetVisibility();
		if (ChildVisibilty != EVisibility::Collapsed)
		{
			FVector2D ChildDesiredSize = CurChild.GetWidget()->GetDesiredSize();
			MaxSize.X = FMath::Max(MaxSize.X, ChildDesiredSize.X);
			MaxSize.Y = FMath::Max(MaxSize.Y, ChildDesiredSize.Y);
		}
	}

	return MaxSize;
}

FChildren* SAnimTrackArea::GetChildren()
{
	return &Children;
}

int32 SAnimTrackArea::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	// paint the child widgets
	FArrangedChildren ArrangedChildren(EVisibility::Visible);
	ArrangeChildren(AllottedGeometry, ArrangedChildren);

	const FPaintArgs NewArgs = Args.WithNewParent(this);

	for (int32 ChildIndex = 0; ChildIndex < ArrangedChildren.Num(); ++ChildIndex)
	{
		FArrangedWidget& CurWidget = ArrangedChildren[ChildIndex];
		FSlateRect ChildClipRect = MyCullingRect.IntersectionWith(CurWidget.Geometry.GetLayoutBoundingRect());
		const int32 ThisWidgetLayerId = CurWidget.Widget->Paint(NewArgs, CurWidget.Geometry, ChildClipRect, OutDrawElements, LayerId + 2, InWidgetStyle, bParentEnabled);

		LayerId = FMath::Max(LayerId, ThisWidgetLayerId);
	}

	return LayerId;
}

void SAnimTrackArea::UpdateHoverStates( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{

}

FReply SAnimTrackArea::OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	TSharedPtr<FAnimTimeSliderController> TimeSliderController = WeakTimeSliderController.Pin();
	if(TimeSliderController.IsValid())
	{
		WeakOutliner.Pin()->ClearSelection();
		WeakModel.Pin()->ClearDetailsView();

		TimeSliderController->OnMouseButtonDown(*this, MyGeometry, MouseEvent);
	}

	return FReply::Unhandled();
}

FReply SAnimTrackArea::OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	TSharedPtr<FAnimTimeSliderController> TimeSliderController = WeakTimeSliderController.Pin();
	if(TimeSliderController.IsValid())
	{

		return WeakTimeSliderController.Pin()->OnMouseButtonUp(*this, MyGeometry, MouseEvent);
	}

	return FReply::Unhandled();
}

FReply SAnimTrackArea::OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	UpdateHoverStates(MyGeometry, MouseEvent);

	const TSharedPtr<FAnimTimeSliderController> TimeSliderController = WeakTimeSliderController.Pin();
	if(TimeSliderController.IsValid())
	{
		FReply Reply = WeakTimeSliderController.Pin()->OnMouseMove(*this, MyGeometry, MouseEvent);

		// Handle right click scrolling on the track area
		if (Reply.IsEventHandled())
		{
			if (MouseEvent.IsMouseButtonDown(EKeys::RightMouseButton) && HasMouseCapture())
			{
				WeakOutliner.Pin()->ScrollByDelta(static_cast<float>(-MouseEvent.GetCursorDelta().Y));
			}
		}

		return Reply;
	}

	return FReply::Unhandled();
}

FReply SAnimTrackArea::OnMouseWheel( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	TSharedPtr<FAnimTimeSliderController> TimeSliderController = WeakTimeSliderController.Pin();
	if(TimeSliderController.IsValid())
	{
		FReply Reply = WeakTimeSliderController.Pin()->OnMouseWheel(*this, MyGeometry, MouseEvent);
		if (Reply.IsEventHandled())
		{
			return Reply;
		}

		const float ScrollAmount = -MouseEvent.GetWheelDelta() * GetGlobalScrollAmount();
		WeakOutliner.Pin()->ScrollByDelta(ScrollAmount);

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void SAnimTrackArea::OnMouseLeave(const FPointerEvent& MouseEvent)
{
}

FCursorReply SAnimTrackArea::OnCursorQuery( const FGeometry& MyGeometry, const FPointerEvent& CursorEvent ) const
{
	if (CursorEvent.IsMouseButtonDown(EKeys::RightMouseButton) && HasMouseCapture())
	{
		return FCursorReply::Cursor(EMouseCursor::GrabHandClosed);
	}
	else
	{
		TSharedPtr<FAnimTimeSliderController> TimeSliderController = WeakTimeSliderController.Pin();
		if(TimeSliderController.IsValid())
		{
			return TimeSliderController->OnCursorQuery(*this, MyGeometry, CursorEvent);
		}
	}

	return FCursorReply::Unhandled();
}

void SAnimTrackArea::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	CachedGeometry = AllottedGeometry;

	for (int32 Index = 0; Index < Children.Num();)
	{
		if (!StaticCastSharedRef<SWeakWidget>(Children[Index].GetWidget())->ChildWidgetIsValid())
		{
			Children.RemoveAt(Index);
		}
		else
		{
			++Index;
		}
	}
}
