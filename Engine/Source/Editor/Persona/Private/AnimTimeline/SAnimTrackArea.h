// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Geometry.h"
#include "Input/CursorReply.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "SlotBase.h"
#include "Layout/Children.h"
#include "Widgets/SPanel.h"

class FArrangedChildren;
class FPaintArgs;
class FSlateWindowElementList;
class FAnimTimelineTrack;
class SAnimTrack;
class SAnimOutliner;
class FAnimModel;
class FAnimTimeSliderController;

/**
 * Structure representing a slot in the track area.
 */
class FAnimTrackAreaSlot : public TSlotBase<FAnimTrackAreaSlot>, public TAlignmentWidgetSlotMixin<FAnimTrackAreaSlot>
{
public:

	/** Construction from a track lane */
	FAnimTrackAreaSlot(const TSharedPtr<SAnimTrack>& InSlotContent);

	/** Get the vertical position of this slot inside its parent. */
	float GetVerticalOffset() const;

	/** The track that we represent. */
	TWeakPtr<SAnimTrack> TrackWidget;
};


/**
 * The area where tracks are displayed.
 */
class SAnimTrackArea : public SPanel
{
public:

	SLATE_BEGIN_ARGS(SAnimTrackArea)
	{
		_Clipping = EWidgetClipping::ClipToBounds;
	}
	SLATE_END_ARGS()

	SAnimTrackArea()
		: Children(this)
	{}

	/** Construct this widget */
	void Construct(const FArguments& InArgs, const TSharedRef<FAnimModel>& InAnimModel, const TSharedRef<FAnimTimeSliderController>& InTimeSliderController);

public:

	/** Empty the track area */
	void Empty();

	/** Add a new track slot to this area for the given node. The slot will be automatically cleaned up when all external references to the supplied slot are removed. */
	void AddTrackSlot(const TSharedRef<FAnimTimelineTrack>& InTrack, const TSharedPtr<SAnimTrack>& InTrackWidget);

	/** Attempt to find an existing slot relating to the given node */
	TSharedPtr<SAnimTrack> FindTrackSlot(const TSharedRef<FAnimTimelineTrack>& InTrack);

	/** Access the cached geometry for this track area */
	const FGeometry& GetCachedGeometry() const
	{
		return CachedGeometry;
	}

	/** Assign an outliner to this track area. */
	void SetOutliner(const TSharedPtr<SAnimOutliner>& InOutliner);

public:

	/** SWidget interface */
	virtual FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	virtual FReply OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	virtual FReply OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseWheel( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;
	virtual FCursorReply OnCursorQuery( const FGeometry& MyGeometry, const FPointerEvent& CursorEvent ) const override;
	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;
	virtual void OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const override;
	virtual FVector2D ComputeDesiredSize(float) const override;
	virtual FChildren* GetChildren() override;

protected:

	/** Update any hover state required for the track area */
	void UpdateHoverStates(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

private:

	/** The track area's children. */
	TPanelChildren<FAnimTrackAreaSlot> Children;

private:

	/** Cached geometry. */
	FGeometry CachedGeometry;

	/** A map of child slot content that exists in our view. */
	TMap<TWeakPtr<FAnimTimelineTrack>, TWeakPtr<SAnimTrack>> TrackSlots;

	/** Weak pointer to the model. */
	TWeakPtr<FAnimModel> WeakModel;

	/** Weak pointer to the time slider. */
	TWeakPtr<FAnimTimeSliderController> WeakTimeSliderController;

	/** Weak pointer to the outliner (used for scrolling interactions). */
	TWeakPtr<SAnimOutliner> WeakOutliner;
};
