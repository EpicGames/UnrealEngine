// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimTimeline/SAnimTrackResizeArea.h"
#include "AnimTimeline/AnimTimelineTrack.h"
#include "Widgets/Layout/SBox.h"

void SAnimTrackResizeArea::Construct(const FArguments& InArgs, TWeakPtr<FAnimTimelineTrack> InTrack)
{
	Track = InTrack;

	ChildSlot
	[
		SNew(SBox)
		.HeightOverride(5.f)
	];
}

FReply SAnimTrackResizeArea::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	const TSharedPtr<FAnimTimelineTrack> ResizeTarget = Track.Pin();
	if (ResizeTarget.IsValid() && MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
	{
		DragParameters = FDragParameters(ResizeTarget->GetHeight(), static_cast<float>(MouseEvent.GetScreenSpacePosition().Y));
		return FReply::Handled().CaptureMouse(AsShared());
	}
	return FReply::Handled();
}

FReply SAnimTrackResizeArea::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	DragParameters.Reset();
	return FReply::Handled().ReleaseMouseCapture();
}

FReply SAnimTrackResizeArea::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (DragParameters.IsSet() && HasMouseCapture())
	{
		const float NewHeight = DragParameters->OriginalHeight + (static_cast<float>(MouseEvent.GetScreenSpacePosition().Y) - DragParameters->DragStartY);

		const TSharedPtr<FAnimTimelineTrack> ResizeTarget = Track.Pin();
		if (ResizeTarget.IsValid() && FMath::RoundToInt(NewHeight) != FMath::RoundToInt(DragParameters->OriginalHeight))
		{
			ResizeTarget->SetHeight(NewHeight);
		}
	}

	return FReply::Handled();
}

FCursorReply SAnimTrackResizeArea::OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const
{
	return FCursorReply::Cursor(EMouseCursor::ResizeUpDown);
}
