// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FAnimTimelineTrack;

/** Area at the bottom of a track used to resize it */
class SAnimTrackResizeArea : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAnimTrackResizeArea){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TWeakPtr<FAnimTimelineTrack> InTrack);

	/** SWidget interface */
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FCursorReply OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const override;

private:

	struct FDragParameters
	{
		FDragParameters(float InOriginalHeight, float InDragStartY)
			: OriginalHeight(InOriginalHeight)
			, DragStartY(InDragStartY)
		{}

		float OriginalHeight;
		float DragStartY;
	};

	TOptional<FDragParameters> DragParameters;

	TWeakPtr<FAnimTimelineTrack> Track;
};