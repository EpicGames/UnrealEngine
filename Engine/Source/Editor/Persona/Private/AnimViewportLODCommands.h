// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "HAL/Platform.h"
#include "Internationalization/Internationalization.h"
#include "Styling/AppStyle.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

class FUICommandInfo;

/**
 * Class containing commands for persona viewport LOD actions
 */
class FAnimViewportLODCommands : public TCommands<FAnimViewportLODCommands>
{
public:
	FAnimViewportLODCommands() 
		: TCommands<FAnimViewportLODCommands>
		(
			TEXT("AnimViewportLODCmd"), // Context name for fast lookup
			NSLOCTEXT("Contexts", "AnimViewportLODCmd", "Animation Viewport LOD Command"), // Localized context name for displaying
			NAME_None, // Parent context name. 
			FAppStyle::GetAppStyleSetName() // Icon Style Set
		)
	{
	}

	/** LOD Debug */
	TSharedPtr< FUICommandInfo > LODDebug;

	/** LOD Auto */
	TSharedPtr< FUICommandInfo > LODAuto;

	/** LOD 0 */
	TSharedPtr< FUICommandInfo > LOD0;

public:
	/** Registers our commands with the binding system */
	virtual void RegisterCommands() override;
};
