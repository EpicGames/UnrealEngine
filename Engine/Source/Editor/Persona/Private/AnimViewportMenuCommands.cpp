// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimViewportMenuCommands.h"

#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"
#include "GenericPlatform/GenericApplication.h"
#include "InputCoreTypes.h"

#define LOCTEXT_NAMESPACE "AnimViewportMenuCommands"

void FAnimViewportMenuCommands::RegisterCommands()
{
	UI_COMMAND( PreviewSceneSettings, "Preview Scene Settings...", "The Advanced Preview Settings tab will let you alter the preview scene's settings.", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND( CameraFollowNone, "Free Camera", "Camera is free to move.", EUserInterfaceActionType::RadioButton, FInputChord() );
	UI_COMMAND( CameraFollowBounds, "Orbit Bounds", "Camera orbits the bounds of the mesh.", EUserInterfaceActionType::RadioButton, FInputChord() );
	UI_COMMAND( CameraFollowRoot, "Orbit Root", "Camera orbits the root bone while keeping the mesh vertically centered.", EUserInterfaceActionType::RadioButton, FInputChord() );
	UI_COMMAND( CameraFollowBone, "Orbit Bone", "Camera focuses on a specified bone.", EUserInterfaceActionType::RadioButton, FInputChord() );
	
	UI_COMMAND( TogglePauseAnimationOnCameraMove, "Pause Animation On Camera Move", "Pause the preview animation when moving the camera and resume when finished.", EUserInterfaceActionType::ToggleButton, FInputChord() );

	UI_COMMAND( SetCPUSkinning, "CPU Skinning", "Toggles display of CPU skinning in the Preview Pane.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND( SetShowNormals, "Normals", "Toggles display of vertex normals in the Preview Pane.", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( SetShowTangents, "Tangents", "Toggles display of vertex tangents in the Preview Pane.", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( SetShowBinormals, "Binormals", "Toggles display of vertex binormals (orthogonal vector to normal and tangent) in the Preview Pane.", EUserInterfaceActionType::ToggleButton, FInputChord() );

	UI_COMMAND( AnimSetDrawUVs, "UV", "Toggles display of the mesh's UVs for the specified channel.", EUserInterfaceActionType::ToggleButton, FInputChord() );

	UI_COMMAND(SaveCameraAsDefault, "Save Camera As Default", "Save the current camera as default for this mesh.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ClearDefaultCamera, "Clear Default Camera", "Clear default camera for this mesh.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(JumpToDefaultCamera, "Jump To Default Camera", "Jump to the default camera (if set).", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Shift, EKeys::F));
}

#undef LOCTEXT_NAMESPACE
