// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "HAL/Platform.h"
#include "Internationalization/Internationalization.h"
#include "Styling/AppStyle.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

class FUICommandInfo;

/**
 * Class containing commands for viewport menu actions
 */
class FAnimViewportMenuCommands : public TCommands<FAnimViewportMenuCommands>
{
public:
	FAnimViewportMenuCommands() 
		: TCommands<FAnimViewportMenuCommands>
		(
			TEXT("AnimViewportMenu"), // Context name for fast lookup
			NSLOCTEXT("Contexts", "AnimViewportMenu", "Animation Viewport Menu"), // Localized context name for displaying
			NAME_None, // Parent context name.  
			FAppStyle::GetAppStyleSetName() // Icon Style Set
		)
	{
	}

	/** Open settings for the preview scene */
	TSharedPtr< FUICommandInfo > PreviewSceneSettings;
	
	/** Select camera follow mode off */
	TSharedPtr< FUICommandInfo > CameraFollowNone;

	/** Select camera follow mode to follow bounds */
	TSharedPtr< FUICommandInfo > CameraFollowBounds;

	/** Select camera follow mode to follow a named bone */
	TSharedPtr< FUICommandInfo > CameraFollowBone;

	/** Select camera follow mode to orbit the root bone while keeping the mesh vertically centered. */
	TSharedPtr< FUICommandInfo > CameraFollowRoot;

	/** Toggle whether or not to pause the preview animation when moving the camera */
	TSharedPtr< FUICommandInfo > TogglePauseAnimationOnCameraMove;

	/** Show vertex normals */
	TSharedPtr< FUICommandInfo > SetCPUSkinning;

	/** Show vertex normals */
	TSharedPtr< FUICommandInfo > SetShowNormals;

	/** Show vertex tangents */
	TSharedPtr< FUICommandInfo > SetShowTangents;

	/** Show vertex binormals */
	TSharedPtr< FUICommandInfo > SetShowBinormals;

	/** Draw UV mapping to viewport */
	TSharedPtr< FUICommandInfo > AnimSetDrawUVs;

	/** Save current camera as default */
	TSharedPtr< FUICommandInfo > SaveCameraAsDefault;

	/** Clear default camera */
	TSharedPtr< FUICommandInfo > ClearDefaultCamera;

	/** Jump to default camera */
	TSharedPtr< FUICommandInfo > JumpToDefaultCamera;

public:
	/** Registers our commands with the binding system */
	virtual void RegisterCommands() override;
};
