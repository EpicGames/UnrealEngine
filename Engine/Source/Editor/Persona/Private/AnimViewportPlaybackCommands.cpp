// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimViewportPlaybackCommands.h"
#include "Styling/AppStyle.h"
#include "AnimationEditorViewportClient.h"

#define LOCTEXT_NAMESPACE "AnimViewportPlaybackCommands"

FAnimViewportPlaybackCommands::FAnimViewportPlaybackCommands() : TCommands<FAnimViewportPlaybackCommands>
	(
	TEXT("AnimViewportPlayback"), // Context name for fast lookup
	NSLOCTEXT("Contexts", "AnimViewportPlayback", "Animation Viewport Playback"), // Localized context name for displaying
	NAME_None, // Parent context name.  
	FAppStyle::GetAppStyleSetName() // Icon Style Set
	)
{
	PlaybackSpeedCommands.AddZeroed(EAnimationPlaybackSpeeds::NumPlaybackSpeeds);
	TurnTableSpeeds.AddZeroed(EAnimationPlaybackSpeeds::NumPlaybackSpeeds);
}

void FAnimViewportPlaybackCommands::RegisterCommands()
{
	const TSharedRef<FBindingContext> SharedThis = AsShared();
	auto MakeLocalizedCommand =
		[SharedThis](
			TSharedPtr<FUICommandInfo>& OutCommandInfo, FName InCommandName, const FText& InCommandLabel, const FText& InCommandTooltip
		)
	{
		FUICommandInfo::MakeCommandInfo(
			SharedThis,
			OutCommandInfo,
			InCommandName,
			InCommandLabel,
			InCommandTooltip,
			FSlateIcon(),
			EUserInterfaceActionType::RadioButton,
			FInputChord()
		);
	};

	const FNumberFormattingOptions OneDecimalFormatting =
		FNumberFormattingOptions().SetMinimumFractionalDigits(1).SetMaximumFractionalDigits(1);

	// Playback Speed Commands
	{
		MakeLocalizedCommand(
			PlaybackSpeedCommands[EAnimationPlaybackSpeeds::OneTenth],
			"PlaybackSpeed_x0.1",
			FText::Format(LOCTEXT("PlaybackSpeed_0.1_Label", "x{0}"), FText::AsNumber(0.1f)),
			LOCTEXT("PlaybackSpeed_0.1_Tooltip", "Set the animation playback speed to a tenth of normal")
		);
		MakeLocalizedCommand(
			PlaybackSpeedCommands[EAnimationPlaybackSpeeds::Quarter],
			"PlaybackSpeed_x0.25",
			FText::Format(LOCTEXT("PlaybackSpeed_0.25_Label", "x{0}"), FText::AsNumber(0.25f)),
			LOCTEXT("PlaybackSpeed_0.25_Tooltip", "Set the animation playback speed to a tenth of normal")
		);
		MakeLocalizedCommand(
			PlaybackSpeedCommands[EAnimationPlaybackSpeeds::Half],
			"PlaybackSpeed_x0.5",
			FText::Format(LOCTEXT("PlaybackSpeed_0.5_Label", "x{0}"), FText::AsNumber(0.5f)),
			LOCTEXT("PlaybackSpeed_0.5_Tooltip", "Set the animation playback speed to a tenth of normal")
		);
		MakeLocalizedCommand(
			PlaybackSpeedCommands[EAnimationPlaybackSpeeds::ThreeQuarters],
			"PlaybackSpeed_x0.75",
			FText::Format(LOCTEXT("PlaybackSpeed_0.75_Label", "x{0}"), FText::AsNumber(0.75f)),
			LOCTEXT("PlaybackSpeed_0.75_Tooltip", "Set the animation playback speed to a tenth of normal")
		);
		MakeLocalizedCommand(
			PlaybackSpeedCommands[EAnimationPlaybackSpeeds::Normal],
			"PlaybackSpeed_x1.0",
			FText::Format(LOCTEXT("PlaybackSpeed_1.0_Label", "x{0}"), FText::AsNumber(1.0f, &OneDecimalFormatting)),
			LOCTEXT("PlaybackSpeed_1.0_Tooltip", "Set the animation playback speed to a tenth of normal")
		);
		MakeLocalizedCommand(
			PlaybackSpeedCommands[EAnimationPlaybackSpeeds::Double],
			"PlaybackSpeed_x2.0",
			FText::Format(LOCTEXT("PlaybackSpeed_2.0_Label", "x{0}"), FText::AsNumber(2.0f, &OneDecimalFormatting)),
			LOCTEXT("PlaybackSpeed_2.0_Tooltip", "Set the animation playback speed to a tenth of normal")
		);
		MakeLocalizedCommand(
			PlaybackSpeedCommands[EAnimationPlaybackSpeeds::FiveTimes],
			"PlaybackSpeed_x5.0",
			FText::Format(LOCTEXT("PlaybackSpeed_5.0_Label", "x{0}"), FText::AsNumber(5.0f, &OneDecimalFormatting)),
			LOCTEXT("PlaybackSpeed_5.0_Tooltip", "Set the animation playback speed to a tenth of normal")
		);
		MakeLocalizedCommand(
			PlaybackSpeedCommands[EAnimationPlaybackSpeeds::TenTimes],
			"PlaybackSpeed_x10.0",
			FText::Format(LOCTEXT("PlaybackSpeed_10.0_Label", "x{0}"), FText::AsNumber(10.0f, &OneDecimalFormatting)),
			LOCTEXT("PlaybackSpeed_10.0_Tooltip", "Set the animation playback speed to a tenth of normal")
		);
		UI_COMMAND( PlaybackSpeedCommands[EAnimationPlaybackSpeeds::Custom], "xCustom", "Set the animation playback speed to assigned custom speed", EUserInterfaceActionType::RadioButton, FInputChord() );
	}

	// Turntable Speed Commands
	{
		MakeLocalizedCommand(
			TurnTableSpeeds[EAnimationPlaybackSpeeds::OneTenth],
			"Turntable_speed_x0.1",
			FText::Format(LOCTEXT("Turntable_speed_0.1_Label", "x{0}"), FText::AsNumber(0.1f)),
			LOCTEXT("Turntable_speed_0.1_Tooltip", "Set the turn table rotation speed to a tenth of normal")
		);
		MakeLocalizedCommand(
			TurnTableSpeeds[EAnimationPlaybackSpeeds::Quarter],
			"Turntable_speed_x0.25",
			FText::Format(LOCTEXT("Turntable_speed_0.25_Label", "x{0}"), FText::AsNumber(0.25f)),
			LOCTEXT("Turntable_speed_0.25_Tooltip", "Set the turn table rotation speed to a tenth of normal")
		);
		MakeLocalizedCommand(
			TurnTableSpeeds[EAnimationPlaybackSpeeds::Half],
			"Turntable_speed_x0.5",
			FText::Format(LOCTEXT("Turntable_speed_0.5_Label", "x{0}"), FText::AsNumber(0.5f)),
			LOCTEXT("Turntable_speed_0.5_Tooltip", "Set the turn table rotation speed to a tenth of normal")
		);
		MakeLocalizedCommand(
			TurnTableSpeeds[EAnimationPlaybackSpeeds::ThreeQuarters],
			"Turntable_speed_x0.75",
			FText::Format(LOCTEXT("Turntable_speed_0.75_Label", "x{0}"), FText::AsNumber(0.75f)),
			LOCTEXT("Turntable_speed_0.75_Tooltip", "Set the turn table rotation speed to a tenth of normal")
		);
		MakeLocalizedCommand(
			TurnTableSpeeds[EAnimationPlaybackSpeeds::Normal],
			"Turntable_speed_x1.0",
			FText::Format(LOCTEXT("Turntable_speed_1.0_Label", "x{0}"), FText::AsNumber(1.0f, &OneDecimalFormatting)),
			LOCTEXT("Turntable_speed_1.0_Tooltip", "Set the turn table rotation speed to a tenth of normal")
		);
		MakeLocalizedCommand(
			TurnTableSpeeds[EAnimationPlaybackSpeeds::Double],
			"Turntable_speed_x2.0",
			FText::Format(LOCTEXT("Turntable_speed_2.0_Label", "x{0}"), FText::AsNumber(2.0f, &OneDecimalFormatting)),
			LOCTEXT("Turntable_speed_2.0_Tooltip", "Set the turn table rotation speed to a tenth of normal")
		);
		MakeLocalizedCommand(
			TurnTableSpeeds[EAnimationPlaybackSpeeds::FiveTimes],
			"Turntable_speed_x5.0",
			FText::Format(LOCTEXT("Turntable_speed_5.0_Label", "x{0}"), FText::AsNumber(5.0f, &OneDecimalFormatting)),
			LOCTEXT("Turntable_speed_5.0_Tooltip", "Set the turn table rotation speed to a tenth of normal")
		);
		MakeLocalizedCommand(
			TurnTableSpeeds[EAnimationPlaybackSpeeds::TenTimes],
			"Turntable_speed_x10.0",
			FText::Format(LOCTEXT("Turntable_speed_10.0_Label", "x{0}"), FText::AsNumber(10.0f, &OneDecimalFormatting)),
			LOCTEXT("Turntable_speed_10.0_Tooltip", "Set the turn table rotation speed to a tenth of normal")
		);
		UI_COMMAND(TurnTableSpeeds[EAnimationPlaybackSpeeds::Custom], "xCustom", "Set the animation playback speed to assigned custom speed", EUserInterfaceActionType::RadioButton, FInputChord());

		UI_COMMAND(PersonaTurnTablePlay, "Play", "Turn table rotates", EUserInterfaceActionType::RadioButton, FInputChord());
		UI_COMMAND(PersonaTurnTablePause, "Pause", "Freeze with current rotation", EUserInterfaceActionType::RadioButton, FInputChord());
		UI_COMMAND(PersonaTurnTableStop, "Stop", "Stop and Reset orientation", EUserInterfaceActionType::RadioButton, FInputChord());
	}
}

#undef LOCTEXT_NAMESPACE
