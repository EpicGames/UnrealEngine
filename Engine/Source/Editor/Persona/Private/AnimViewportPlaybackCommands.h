// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/Commands.h"

/**
 * Class containing commands for viewport playback actions
 */
class FAnimViewportPlaybackCommands : public TCommands<FAnimViewportPlaybackCommands>
{
public:
	FAnimViewportPlaybackCommands();

	/** Command list for playback speed, indexed by EPlaybackSpeeds*/
	TArray<TSharedPtr< FUICommandInfo >> PlaybackSpeedCommands;

	/** Command list for turn table speed, indexed by EPlaybackSpeeds*/
	TArray<TSharedPtr< FUICommandInfo >> TurnTableSpeeds;
	TSharedPtr<FUICommandInfo> PersonaTurnTablePlay;
	TSharedPtr<FUICommandInfo> PersonaTurnTablePause;
	TSharedPtr<FUICommandInfo> PersonaTurnTableStop;

public:
	/** Registers our commands with the binding system */
	virtual void RegisterCommands() override;
};
