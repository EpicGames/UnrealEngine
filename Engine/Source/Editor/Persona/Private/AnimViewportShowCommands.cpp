// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimViewportShowCommands.h"

#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"
#include "GenericPlatform/GenericApplication.h"
#include "InputCoreTypes.h"

#define LOCTEXT_NAMESPACE "AnimViewportShowCommands"

void FAnimViewportShowCommands::RegisterCommands()
{
	UI_COMMAND( AutoAlignFloorToMesh, "Auto Align Floor to Mesh", "Auto align floor to mesh bounds", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( MuteAudio, "Mute Audio", "Mute audio from the preview", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( UseAudioAttenuation, "Use Audio Attenuation", "Use audio attenuation when playing back audio in the preview", EUserInterfaceActionType::ToggleButton, FInputChord() );

	UI_COMMAND(DoNotProcessRootMotion, "Ignore", "Preview mesh will not consume root motion", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(ProcessRootMotionLoopAndReset, "Loop and Reset", "Preview mesh will consume root motion resetting the position back to the origin every time the animation loops", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(ProcessRootMotionLoop, "Loop", "Preview mesh will consume root motion continually", EUserInterfaceActionType::RadioButton, FInputChord());

	UI_COMMAND(DoNotVisualizeRootMotion, "None", "Do not show root motion", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(VisualizeRootMotionTrajectory, "Visualize Trajectory", "Show root motion trajectory", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(VisualizeRootMotionTrajectoryAndOrientation, "Visualize Trajectory and Orientation", "Show root motion trajectory and orientation", EUserInterfaceActionType::RadioButton, FInputChord());

	UI_COMMAND(DisablePostProcessBlueprint, "Disable Post Process", "Disable the evaluation of post process animation blueprints on the preview instance.", EUserInterfaceActionType::ToggleButton, FInputChord());

	UI_COMMAND( ShowBound, "Bound", "Show bound on preview mesh", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( UseInGameBound, "In-game Bound", "Use in-game bound on preview mesh when showing bounds. Otherwise bounds will always be calculated from bones alone.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND( UseFixedBounds, "Fixed Bounds", "Preview using the 'Fixed Bounds' option, which will use bounds from skel mesh and not bones at all, for speed.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND( UsePreSkinnedBounds, "Pre-skinned Bounds", "Preview using the 'Pre-skinned' option, which will use local space pre-skinned(base pose) bounds from skel mesh. Overrides all other bounds preview settings.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND( ShowPreviewMesh, "Mesh", "Show the preview mesh", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( ShowNaniteFallback, "Nanite Fallback", "Show the preview mesh's Nanite fallback", EUserInterfaceActionType::ToggleButton, FInputChord(EModifierKey::Control, EKeys::N) );
	UI_COMMAND( ShowMorphTargets, "Morph Targets", "Display applied morph targets of the mesh", EUserInterfaceActionType::ToggleButton, FInputChord() );

	UI_COMMAND( ShowBoneNames, "Bone Names", "Display bone names in the viewport", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( ShowBoneColors, "Multi-Color Bones", "Display skeleton with multiple colors", EUserInterfaceActionType::ToggleButton, FInputChord() );

	// below 3 menus are radio button styles
	UI_COMMAND(ShowDisplayInfoBasic, "Basic", "Display basic mesh info in the viewport", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(ShowDisplayInfoDetailed, "Detailed", "Display detailed mesh info in the viewport", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(ShowDisplayInfoSkelControls, "Skeletal Controls", "Display selected skeletal control info in the viewport", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(HideDisplayInfo, "None", "Hide all display info in the viewport", EUserInterfaceActionType::RadioButton, FInputChord());

	UI_COMMAND( ShowOverlayNone, "None", "Clear overlay display", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND( ShowBoneWeight, "Selected Bone Weight", "Display color overlay of the weight from selected bone in the viewport", EUserInterfaceActionType::RadioButton, FInputChord() );
	UI_COMMAND( ShowMorphTargetVerts, "Selected Morph Target Vertices", "Display color overlay with the change of selected morph target in the viewport", EUserInterfaceActionType::RadioButton, FInputChord());

	UI_COMMAND( ShowRawAnimation, "Uncompressed Animation", "Display skeleton with uncompressed animation data", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( ShowNonRetargetedAnimation, "Non-Retargeted Animation", "Display Skeleton With non-retargeted animation data", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( ShowAdditiveBaseBones, "Additive Base", "Display skeleton in additive base pose", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( ShowSourceRawAnimation, "Source Animation", "Display skeleton in source raw animation if you have track curves modified", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( ShowBakedAnimation, "Baked Animation", "Display skeleton in baked raw animation if you have track curves modified", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( ShowSockets, "Sockets", "Display socket hitpoints", EUserInterfaceActionType::ToggleButton, FInputChord() );
	UI_COMMAND( ShowAttributes, "Attributes", "Display transform attributes in the viewport", EUserInterfaceActionType::ToggleButton, FInputChord() );

	UI_COMMAND( ShowBoneDrawNone, "None", "Hide bone selection", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND( ShowBoneDrawSelected, "Selected Only", "Show only the selected bone", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND( ShowBoneDrawSelectedAndParents, "Selected and Parents", "Show the selected bone and its parents, to the root", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND( ShowBoneDrawSelectedAndChildren, "Selected and Children", "Show the selected bone and its children", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND( ShowBoneDrawSelectedAndParentsAndChildren, "Selected and Parents and Children", "Show the selected bone and its parents and children", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND( ShowBoneDrawAll, "All Hierarchy", "Show all hierarchy joints", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND( ToggleShowBoneDrawAll, "Bones", "When enabled displays all bones, otherwise none", EUserInterfaceActionType::ToggleButton, FInputChord(EModifierKey::Shift, EKeys::B));

	UI_COMMAND( ShowLocalAxesNone, "None", "Hide all local hierarchy axis", EUserInterfaceActionType::RadioButton, FInputChord() );
	UI_COMMAND( ShowLocalAxesSelected, "Selected Only", "Show only the local axes of the selected bones", EUserInterfaceActionType::RadioButton, FInputChord() );
	UI_COMMAND( ShowLocalAxesAll, "All", "Show all local hierarchy axes", EUserInterfaceActionType::RadioButton, FInputChord() );

	UI_COMMAND( EnableClothSimulation, "Enable Cloth Simulation", "Show simulated cloth mesh", EUserInterfaceActionType::ToggleButton, FInputChord() );

	UI_COMMAND( ResetClothSimulation, "Reset Cloth Simulation", "Reset simulated cloth to its initial state", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control | EModifierKey::Alt, EKeys::C) );

	UI_COMMAND(PauseClothWithAnim, "Pause with Animation", "If enabled, the clothing simulation will pause when the animation is paused using the scrub panel", EUserInterfaceActionType::ToggleButton, FInputChord());

	// below 3 menus are radio button styles
	UI_COMMAND(ShowAllSections, "Show All Sections", "Display all sections including cloth mapped sections", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(ShowOnlyClothSections, "Show Only Cloth Sections", "Display only cloth mapped sections", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(HideOnlyClothSections, "Hide Only Cloth Sections", "Display all except cloth mapped sections", EUserInterfaceActionType::RadioButton, FInputChord());

	UI_COMMAND(ShowNotificationVisualizations, "Show Notification", "Display animation notifications visualizations in viewport", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ShowAssetUserDataVisualizations, "Show Meta Data", "Display animation meta data visualizations in viewport", EUserInterfaceActionType::ToggleButton, FInputChord());

	UI_COMMAND(ShowTimecode, "Show Timecode", "Display timecode information associated with animation data (if exists).", EUserInterfaceActionType::ToggleButton, FInputChord());
}

#undef LOCTEXT_NAMESPACE
