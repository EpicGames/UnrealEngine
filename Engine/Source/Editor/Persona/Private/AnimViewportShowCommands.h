// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "HAL/Platform.h"
#include "Internationalization/Internationalization.h"
#include "Styling/AppStyle.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

class FUICommandInfo;

#define UE_API PERSONA_API

/**
 * Class containing commands for persona viewport show actions
 */
class UE_API FAnimViewportShowCommands : public TCommands<FAnimViewportShowCommands>
{
public:
	FAnimViewportShowCommands() 
		: TCommands<FAnimViewportShowCommands>
		(
			TEXT("AnimViewportShowCmd"), // Context name for fast lookup
			NSLOCTEXT("Contexts", "AnimViewportShowCmd", "Animation Viewport Show Command"), // Localized context name for displaying
			NAME_None, // Parent context name. 
			FAppStyle::GetAppStyleSetName() // Icon Style Set
		)
	{
	}

	// Silence clang deprecation warnings for implicit default copy operator and constructor
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	~FAnimViewportShowCommands() = default;
	FAnimViewportShowCommands(const FAnimViewportShowCommands&) = delete;
	FAnimViewportShowCommands& operator=(const FAnimViewportShowCommands&) = delete;
	FAnimViewportShowCommands(FAnimViewportShowCommands&&) = delete;
	FAnimViewportShowCommands& operator=(FAnimViewportShowCommands&&) = delete;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** Option to align floor to Mesh */
	TSharedPtr< FUICommandInfo > AutoAlignFloorToMesh;

	/** Option to mute audio in the viewport */
	TSharedPtr< FUICommandInfo > MuteAudio;

	/** Option to use audio attenuation in the viewport */
	TSharedPtr< FUICommandInfo > UseAudioAttenuation;

	/** Option to set ProcessRootMotionMode to Ignore (Preview mesh will not consume root motion) */
	TSharedPtr< FUICommandInfo > DoNotProcessRootMotion;

	/** Option to set ProcessRootMotionMode to LoopAndReset (Preview mesh will consume root motion resetting the position back to the origin every time the animation loops) */
	TSharedPtr< FUICommandInfo > ProcessRootMotionLoopAndReset;

	/** Option to set ProcessRootMotionMode to Loop (Preview mesh will consume root motion continually) */
	TSharedPtr< FUICommandInfo > ProcessRootMotionLoop;

	/** Visualize root motion mode */
	TSharedPtr< FUICommandInfo > DoNotVisualizeRootMotion;
	TSharedPtr< FUICommandInfo > VisualizeRootMotionTrajectory;
	TSharedPtr< FUICommandInfo > VisualizeRootMotionTrajectoryAndOrientation;

	/** Option to enable/disable post process anim blueprint evaluation */
	TSharedPtr< FUICommandInfo > DisablePostProcessBlueprint;
	
	/** Show Bound of preview mesh */
	TSharedPtr< FUICommandInfo > ShowBound;

	/** Use in-game Bound of preview mesh */
	TSharedPtr< FUICommandInfo > UseInGameBound;

	/** Use in-game Bound of preview mesh */
	TSharedPtr< FUICommandInfo > UseFixedBounds;

	/** Use pre-skinned Bound of preview mesh */
	TSharedPtr< FUICommandInfo > UsePreSkinnedBounds;

	/** Show/hide the preview mesh */
	TSharedPtr< FUICommandInfo > ShowPreviewMesh;

	/** Show/hide the preview mesh */
	TSharedPtr< FUICommandInfo > ShowNaniteFallback;

	/** Show Morphtarget */
	TSharedPtr< FUICommandInfo > ShowMorphTargets;

	/** Hide all bones */
	TSharedPtr< FUICommandInfo > ShowBoneDrawNone;

	/** Show only selected bones */
	TSharedPtr< FUICommandInfo > ShowBoneDrawSelected;

	/** Show only selected bones and their parents */
	TSharedPtr< FUICommandInfo > ShowBoneDrawSelectedAndParents;

	/** Show only selected bones and their children */
	TSharedPtr< FUICommandInfo > ShowBoneDrawSelectedAndChildren;

	/** Show only selected bones and their parents and children */
	TSharedPtr< FUICommandInfo > ShowBoneDrawSelectedAndParentsAndChildren;

	/** Toggles showing all bones */
	TSharedPtr< FUICommandInfo > ToggleShowBoneDrawAll;

	/** Show all bones */
	TSharedPtr< FUICommandInfo > ShowBoneDrawAll;

	/** Show raw animation (vs compressed) */
	TSharedPtr< FUICommandInfo > ShowRawAnimation;

	/** Show non retargeted animation. */
	TSharedPtr< FUICommandInfo > ShowNonRetargetedAnimation;

	/** Show additive base pose */
	TSharedPtr< FUICommandInfo > ShowAdditiveBaseBones;

	/** Show non retargeted animation. */
	TSharedPtr< FUICommandInfo > ShowSourceRawAnimation;

	/** Show non retargeted animation. */
	TSharedPtr< FUICommandInfo > ShowBakedAnimation;

	/** Show skeletal mesh bone names */
	TSharedPtr< FUICommandInfo > ShowBoneNames;

	/** Show skeletal mesh bone names */
	TSharedPtr< FUICommandInfo > ShowBoneColors;

	/** Show skeletal mesh info */
	TSharedPtr< FUICommandInfo > ShowDisplayInfoBasic;
	TSharedPtr< FUICommandInfo > ShowDisplayInfoDetailed;
	TSharedPtr< FUICommandInfo > ShowDisplayInfoSkelControls;
	TSharedPtr< FUICommandInfo > HideDisplayInfo;

	/** Show overlay material option */
	TSharedPtr< FUICommandInfo > ShowOverlayNone;
	TSharedPtr< FUICommandInfo > ShowBoneWeight;
	TSharedPtr< FUICommandInfo > ShowMorphTargetVerts;

	/** Show socket hit point diamonds */
	TSharedPtr< FUICommandInfo > ShowSockets;

	/** Show transform attributes */
	TSharedPtr< FUICommandInfo > ShowAttributes;

	/** Hide all local axes */
	TSharedPtr< FUICommandInfo > ShowLocalAxesNone;
	
	/** Show only selected axes */
	TSharedPtr< FUICommandInfo > ShowLocalAxesSelected;
	
	/** Show all local axes */
	TSharedPtr< FUICommandInfo > ShowLocalAxesAll;

	/** Enable cloth simulation */
	TSharedPtr< FUICommandInfo > EnableClothSimulation;

	/** Reset cloth simulation */
	TSharedPtr< FUICommandInfo > ResetClothSimulation;

	/** Enables collision detection between collision primitives in the base mesh 
	  * and clothing on any attachments in the preview scene. 
	*/
	UE_DEPRECATED(5.8, "This functionality has been deprecated")
	TSharedPtr< FUICommandInfo > EnableCollisionWithAttachedClothChildren;

	/** Show all sections which means the original state */
	TSharedPtr< FUICommandInfo > ShowAllSections;
	/** Show only clothing mapped sections */
	TSharedPtr< FUICommandInfo > ShowOnlyClothSections;
	/** Show all except clothing mapped sections */
	TSharedPtr< FUICommandInfo > HideOnlyClothSections;

	TSharedPtr< FUICommandInfo > PauseClothWithAnim;

	/** Show Notification Visualizations */
	TSharedPtr< FUICommandInfo > ShowNotificationVisualizations;

	/** Show AssetUserData Visualizations */
	TSharedPtr< FUICommandInfo > ShowAssetUserDataVisualizations;

	/** Show Animation Timecode Values */
	TSharedPtr< FUICommandInfo > ShowTimecode;

public:
	/** Registers our commands with the binding system */
	virtual void RegisterCommands() override;
};

#undef UE_API
