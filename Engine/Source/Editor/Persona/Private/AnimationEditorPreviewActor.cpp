// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationEditorPreviewActor.h"
#include "AnimationRuntime.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimationEditorPreviewActor)

void AAnimationEditorPreviewActor::K2_DestroyActor()
{
	// Override this to do nothing and warn the user
	UE_LOGF(LogAnimation, Warning, "Attempting to destroy an animation preview actor, skipping.");
}
