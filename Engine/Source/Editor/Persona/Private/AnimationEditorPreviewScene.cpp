// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationEditorPreviewScene.h"

#include "AnimCustomInstanceHelper.h"
#include "AnimPreviewAttacheInstance.h"
#include "AnimPreviewInstance.h"
#include "AnimViewportPlaybackCommands.h"
#include "AudioDevice.h"
#include "AudioDeviceManager.h"
#include "ComponentAssetBroker.h"
#include "IEditableSkeleton.h"
#include "IPersonaToolkit.h"
#include "ISkeletonTreeItem.h"
#include "PersonaModule.h"
#include "PersonaPreviewSceneDefaultController.h"
#include "PersonaPreviewSceneDescription.h"
#include "PersonaUtils.h"
#include "ScopedTransaction.h"
#include "UICommandList_Pinnable.h"
#include "Animation/AnimBlueprint.h"
#include "Animation/PreviewCollectionInterface.h"
#include "Components/StaticMeshComponent.h"
#include "Components/WindDirectionalSourceComponent.h"
#include "Engine/PreviewMeshCollection.h"
#include "Engine/SkeletalMeshSocket.h"
#include "Factories/PreviewMeshCollectionFactory.h"
#include "Framework/Notifications/NotificationManager.h"
#include "GameFramework/WorldSettings.h"
#include "Misc/MessageDialog.h"
#include "Modules/ModuleManager.h"
#include "Particles/ParticleSystemComponent.h"
#include "PhysicsEngine/PhysicsSettings.h"
#include "Preferences/PersonaOptions.h"
#include "Styling/AppStyle.h"
#include "Widgets/Notifications/SNotificationList.h"

#define LOCTEXT_NAMESPACE "AnimationEditorPreviewScene"

/////////////////////////////////////////////////////////////////////////
// FAnimationEditorPreviewScene

FAnimationEditorPreviewScene::FAnimationEditorPreviewScene(const ConstructionValues& CVS, const TSharedPtr<IEditableSkeleton>& InEditableSkeleton, const TSharedRef<IPersonaToolkit>& InPersonaToolkit)
	: IPersonaPreviewScene(CVS)
	, Actor(nullptr)
	, SkeletalMeshComponent(nullptr)
	, EditableSkeletonPtr(InEditableSkeleton)
	, PersonaToolkit(InPersonaToolkit)
	, DefaultMode(EPreviewSceneDefaultAnimationMode::ReferencePose)
	, PrevWindLocation(100.0f, 100.0f, 100.0f)
	, PrevWindRotation(0.0f, 0.0f, 0.0f)
	, PrevWindStrength(0.2f)
	, GravityScale(0.25f)
	, SelectedBoneIndex(INDEX_NONE)
	, bEnableMeshHitProxies(true)
	, LastTickTime(0.0)
	, bSelecting(false)
	, bAllowAdditionalMeshes(true)
	, bAdditionalMeshesSelectable(true)
{
	if (GEditor)
	{
		GEditor->RegisterForUndo(this);
	}
	
	FloorBounds = FloorMeshComponent->CalcBounds(FloorMeshComponent->GetRelativeTransform());

	if(InEditableSkeleton.IsValid())
	{
		InEditableSkeleton->LoadAdditionalPreviewSkeletalMeshes();
	}

	// create the preview scene description
	PreviewSceneDescription = NewObject<UPersonaPreviewSceneDescription>(GetTransientPackage());
	PreviewSceneDescription->SetFlags(RF_Transactional);

	PreviewSceneDescription->SetPreviewController(UPersonaPreviewSceneDefaultController::StaticClass(), this);
	
	PreviewSceneDescription->PreviewMesh = InPersonaToolkit->GetPreviewMesh();
	if(InEditableSkeleton.IsValid())
	{
		PreviewSceneDescription->AdditionalMeshes = InEditableSkeleton->GetSkeleton().GetAdditionalPreviewSkeletalMeshes();
	}

	if(UAnimBlueprint* AnimBlueprint = InPersonaToolkit->GetAnimBlueprint())
	{
		PreviewSceneDescription->PreviewAnimationBlueprint = AnimBlueprint->GetPreviewAnimationBlueprint();
		PreviewSceneDescription->ApplicationMethod = AnimBlueprint->GetPreviewAnimationBlueprintApplicationMethod();
		PreviewSceneDescription->LinkedAnimGraphTag = AnimBlueprint->GetPreviewAnimationBlueprintTag();
	}

	// create a default additional mesh collection so we dont always have to create an asset to edit additional meshes
	UPreviewMeshCollectionFactory* FactoryToUse = NewObject<UPreviewMeshCollectionFactory>();
	if(InEditableSkeleton.IsValid())
	{
		FactoryToUse->CurrentSkeleton = MakeWeakObjectPtr(const_cast<USkeleton*>(&InEditableSkeleton->GetSkeleton()));
	}
	PreviewSceneDescription->DefaultAdditionalMeshes = CastChecked<UPreviewMeshCollection>(FactoryToUse->FactoryCreateNew(UPreviewMeshCollection::StaticClass(), PreviewSceneDescription, "UnsavedCollection", RF_Transient | RF_Transactional, nullptr, nullptr));

	if (!PreviewSceneDescription->AdditionalMeshes.IsValid())
	{
		PreviewSceneDescription->AdditionalMeshes = PreviewSceneDescription->DefaultAdditionalMeshes;
	}

	// Disable killing actors outside of the world
	UWorld* World = GetWorld();
	AWorldSettings* WorldSettings = World->GetWorldSettings(true);
	WorldSettings->bEnableWorldBoundsChecks = false;
	
	// Create a new audio device for this scene
	if (FAudioDeviceManager* AudioDeviceManager = GEngine->GetAudioDeviceManager())
	{
		FAudioDeviceParams DeviceParams = AudioDeviceManager->GetDefaultParamsForNewWorld();
		DeviceParams.AssociatedWorld = World;
		const FAudioDeviceHandle AudioDevice = AudioDeviceManager->RequestAudioDevice(DeviceParams);
		if (AudioDevice.IsValid())
		{
			// Set the base mix of the new device based on the world settings of the world
			AudioDevice.GetAudioDevice()->SetDefaultBaseSoundMix(World->GetWorldSettings()->DefaultBaseSoundMix);

			// Set the world's audio device handle to use so that sounds which play in that world will use the correct audio device
			World->SetAudioDevice(AudioDevice);
				
			FWorldContext& WorldContext = GEngine->GetWorldContextFromWorldChecked(World);

			// Set this audio device handle on the world context so future world's set onto the world context
			// will pass the audio device handle to them and audio will play on the correct audio device
			WorldContext.AudioDeviceID = AudioDevice.GetDeviceID();
		}
	}

	BindCommands();
}

FAnimationEditorPreviewScene::~FAnimationEditorPreviewScene()
{
	bool bInRecording = false;
	FPersonaModule& PersonaModule = FModuleManager::GetModuleChecked<FPersonaModule>(TEXT("Persona"));
	PersonaModule.OnIsRecordingActive().ExecuteIfBound(SkeletalMeshComponent, bInRecording);

	if (bInRecording)
	{
		PersonaModule.OnStopRecording().ExecuteIfBound(SkeletalMeshComponent);
	}

	if (GEditor)
	{
		GEditor->UnregisterForUndo(this);
	}

	UDebugSkelMeshComponent* MeshComponent = GetPreviewMeshComponent();
	if (MeshComponent)
	{
		MeshComponent->SelectionOverrideDelegate.Unbind();
	}
}

void FAnimationEditorPreviewScene::UnregisterForUndo()
{
	if (GEditor)
	{
		GEditor->UnregisterForUndo(this);
	}
}

TArray<UDebugSkelMeshComponent*> FAnimationEditorPreviewScene::GetAllPreviewMeshComponents() const
{
	TArray<UDebugSkelMeshComponent*> PreviewMeshComponents;
	const AActor* MyActor = GetActor();
	if (MyActor)
	{
		MyActor->GetComponents(PreviewMeshComponents, true);
	}
	return PreviewMeshComponents;
}

void FAnimationEditorPreviewScene::ForEachPreviewMesh(TFunction<void (UDebugSkelMeshComponent*)> PerMeshFunction)
{
	TArray<UDebugSkelMeshComponent*> PreviewMeshComponents;
	const AActor* MyActor = GetActor();
	if (MyActor)
	{
		MyActor->GetComponents(PreviewMeshComponents, true);
		for (UDebugSkelMeshComponent* PreviewMesh : PreviewMeshComponents)
		{
			PerMeshFunction(PreviewMesh);
		}
	}
}

void FAnimationEditorPreviewScene::SetPreviewMeshComponent(UDebugSkelMeshComponent* InSkeletalMeshComponent) 
{
	SkeletalMeshComponent = InSkeletalMeshComponent; 

	if(SkeletalMeshComponent)
	{
		SkeletalMeshComponent->SelectionOverrideDelegate = UPrimitiveComponent::FSelectionOverride::CreateRaw(this, &FAnimationEditorPreviewScene::PreviewComponentSelectionOverride);
		SkeletalMeshComponent->PushSelectionToProxy();	
	}
}

void FAnimationEditorPreviewScene::SetPreviewMesh(USkeletalMesh* NewPreviewMesh, bool bAllowOverrideBaseMesh)
{
	if (NewPreviewMesh != nullptr && GetEditableSkeleton().IsValid() && GetEditableSkeleton()->IsSkeletonValid() && 
		!GetEditableSkeleton()->GetSkeleton().IsCompatibleMesh(NewPreviewMesh))
	{
		const USkeleton& Skeleton = GetEditableSkeleton()->GetSkeleton();
		const bool bSkipSkeletonCompatibility = PersonaToolkit.IsValid() && PersonaToolkit.Pin()->CanPreviewMeshUseDifferentSkeleton(); 
		if (NewPreviewMesh->GetSkeleton() && (bSkipSkeletonCompatibility || Skeleton.IsCompatibleForEditor(NewPreviewMesh->GetSkeleton())))
		{
			SetPreviewMeshInternal(NewPreviewMesh);
		}	
		else
		{
			// message box, ask if they'd like to regenerate skeleton
			if (FMessageDialog::Open(EAppMsgType::YesNo, LOCTEXT("RenerateSkeleton", "The preview mesh hierarchy doesn't match with Skeleton anymore. Would you like to regenerate skeleton?")) == EAppReturnType::Yes)
			{
				GetEditableSkeleton()->RecreateBoneTree(NewPreviewMesh);
				SetPreviewMeshInternal(NewPreviewMesh);
			}
			else
			{
				// Send a notification that the skeletal mesh cannot work with the skeleton
				FFormatNamedArguments Args;
				Args.Add(TEXT("PreviewMeshName"), FText::FromString(NewPreviewMesh->GetName()));
				Args.Add(TEXT("TargetSkeletonName"), FText::FromString(Skeleton.GetName()));
				FNotificationInfo Info(FText::Format(LOCTEXT("SkeletalMeshIncompatible", "Skeletal Mesh \"{PreviewMeshName}\" incompatible with Skeleton \"{TargetSkeletonName}\""), Args));
				Info.ExpireDuration = 3.0f;
				Info.bUseLargeFont = false;
				TSharedPtr<SNotificationItem> Notification = FSlateNotificationManager::Get().AddNotification(Info);
				if (Notification.IsValid())
				{
					Notification->SetCompletionState(SNotificationItem::CS_Fail);
				}
			}
		}
	}
	else
	{
		SetPreviewMeshInternal(NewPreviewMesh);
	}

	// changing the main skeletal mesh may mean re-applying the additional meshes
	// as the mesh on the main component may have been substituted by one of the additional meshes
	// we just set main mesh, do not replace
	RefreshAdditionalMeshes(bAllowOverrideBaseMesh);
}

void FAnimationEditorPreviewScene::SetPreviewAnimationBlueprint(UAnimBlueprint* InAnimBlueprint, UAnimBlueprint* InOverlayOrSubAnimBlueprint)
{
	auto ValidateSelfOverlay = [InOverlayOrSubAnimBlueprint, InAnimBlueprint]()
	{
		if (InOverlayOrSubAnimBlueprint == InAnimBlueprint)
		{
			UE_LOGFMT(LogAnimation, Warning, "Anim BP '{AnimBP}' is using itself as a preview overlay, which would cause crashes or infinite recursion. Skipping and applying non-overlaid anim BP instead", InAnimBlueprint->GetName());
			return false;
		}
		return true;
	};

	if (InOverlayOrSubAnimBlueprint && ValidateSelfOverlay())
	{
		SkeletalMeshComponent->SetAnimInstanceClass(InAnimBlueprint ? InAnimBlueprint->GeneratedClass : nullptr);
		EPreviewAnimationBlueprintApplicationMethod ApplicationMethod = InOverlayOrSubAnimBlueprint->GetPreviewAnimationBlueprintApplicationMethod();
		if(ApplicationMethod == EPreviewAnimationBlueprintApplicationMethod::LinkedLayers)
		{
			SkeletalMeshComponent->LinkAnimClassLayers(InOverlayOrSubAnimBlueprint ? InOverlayOrSubAnimBlueprint->GeneratedClass.Get() : nullptr);
		}
		else if(ApplicationMethod == EPreviewAnimationBlueprintApplicationMethod::LinkedAnimGraph)
		{
			SkeletalMeshComponent->LinkAnimGraphByTag(InOverlayOrSubAnimBlueprint->GetPreviewAnimationBlueprintTag(), InOverlayOrSubAnimBlueprint ? InOverlayOrSubAnimBlueprint->GeneratedClass.Get() : nullptr);
		}
	}
	else
	{
		SkeletalMeshComponent->SetAnimInstanceClass(InAnimBlueprint ? InAnimBlueprint->GeneratedClass : nullptr);
	}
}

USkeletalMesh* FAnimationEditorPreviewScene::GetPreviewMesh() const
{
	return PreviewSceneDescription->PreviewMesh.Get();
}

void FAnimationEditorPreviewScene::SetPreviewMeshInternal(USkeletalMesh* NewPreviewMesh)
{
	USkeletalMesh* OldPreviewMesh = SkeletalMeshComponent->GetSkeletalMeshAsset();

	// Store off the old skel mesh we are debugging
	USkeletalMeshComponent* DebuggedSkeletalMeshComponent = nullptr;
	if(SkeletalMeshComponent->GetAnimInstance())
	{
		if(PersonaToolkit.IsValid())
		{
			UAnimBlueprint* SourceBlueprint = PersonaToolkit.Pin()->GetAnimBlueprint();
			if(SourceBlueprint)
			{
				UAnimInstance* DebuggedAnimInstance = Cast<UAnimInstance>(SourceBlueprint->GetObjectBeingDebugged());
				if(DebuggedAnimInstance)
				{
					DebuggedSkeletalMeshComponent = DebuggedAnimInstance->GetSkelMeshComponent();
				}
			}
		}
	}

	// Make sure the desc is up to date as this may have not come from a call to set the value in the desc
	PreviewSceneDescription->PreviewMesh = NewPreviewMesh;

	//Persona skeletal mesh component is the only component that can highlight a particular section
	SkeletalMeshComponent->bCanHighlightSelectedSections = true;

	ValidatePreviewAttachedAssets(NewPreviewMesh);

	if (NewPreviewMesh != SkeletalMeshComponent->GetSkeletalMeshAsset())
	{
		// setting skeletalmesh unregister/re-register, 
		// so I have to save the animation settings and resetting after setting mesh
		UAnimationAsset* AnimAssetToPlay = nullptr;
		float PlayPosition = 0.f;
		bool bPlaying = false;
		bool bNeedsToCopyAnimationData = bIsBeingConstructed == false && SkeletalMeshComponent->GetAnimInstance() && SkeletalMeshComponent->GetAnimInstance() == SkeletalMeshComponent->PreviewInstance;
		if (bNeedsToCopyAnimationData && NewPreviewMesh)
		{
			AnimAssetToPlay = SkeletalMeshComponent->PreviewInstance->GetCurrentAsset();
			PlayPosition = SkeletalMeshComponent->PreviewInstance->GetCurrentTime();
			bPlaying = SkeletalMeshComponent->PreviewInstance->IsPlaying();
		}

		SkeletalMeshComponent->EmptyOverrideMaterials();
		SkeletalMeshComponent->SetSkeletalMesh(NewPreviewMesh);

		if (bNeedsToCopyAnimationData && (NewPreviewMesh == nullptr || AnimAssetToPlay != nullptr))
		{
			SetPreviewAnimationAsset(AnimAssetToPlay);
			SkeletalMeshComponent->PreviewInstance->SetPosition(PlayPosition);
			SkeletalMeshComponent->PreviewInstance->SetPlaying(bPlaying);
		}
	}
	else
	{
		SkeletalMeshComponent->InitAnim(true);
	}

	if (NewPreviewMesh != nullptr)
	{
		AddComponent(SkeletalMeshComponent, FTransform::Identity);

		// Set up the mesh for transactions
		NewPreviewMesh->SetFlags(RF_Transactional);

		AddPreviewAttachedObjects();

		SkeletalMeshComponent->VisibilityBasedAnimTickOption = EVisibilityBasedAnimTickOption::AlwaysTickPoseAndRefreshBones;
	}

	// Setting the skeletal mesh to in the PreviewScene can change AnimScriptInstance so we must re register it
	// with the AnimBlueprint
	if (const TSharedPtr<IPersonaToolkit>& PersonaToolkitPinned = PersonaToolkit.Pin())
	{
		if (UAnimBlueprint* SourceBlueprint = PersonaToolkitPinned->GetAnimBlueprint())
		{
			if (DebuggedSkeletalMeshComponent && DebuggedSkeletalMeshComponent->GetAnimInstance() && DebuggedSkeletalMeshComponent->GetAnimInstance()->IsA(SourceBlueprint->GeneratedClass))
			{
				PersonaUtils::SetObjectBeingDebugged(SourceBlueprint, DebuggedSkeletalMeshComponent->GetAnimInstance());
			}

			// If we didn't have a preview mesh before and we select one now, set it up as the object being debugged
			if (DebuggedSkeletalMeshComponent == nullptr && NewPreviewMesh != nullptr && SkeletalMeshComponent->GetAnimInstance() && SkeletalMeshComponent->GetAnimInstance()->IsA(SourceBlueprint->GeneratedClass))
			{
				PersonaUtils::SetObjectBeingDebugged(SourceBlueprint, SkeletalMeshComponent->GetAnimInstance());
			}
		}
	}

	OnPreviewMeshChanged.Broadcast(OldPreviewMesh, NewPreviewMesh);
}

void FAnimationEditorPreviewScene::ValidatePreviewAttachedAssets(USkeletalMesh* PreviewSkeletalMesh)
{
	// Validate the skeleton/meshes attached objects and display a notification to the user if any were broken
	int32 NumBrokenAssets = GetEditableSkeleton().IsValid() ? GetEditableSkeleton()->ValidatePreviewAttachedObjects() : 0;
	if (PreviewSkeletalMesh)
	{
		NumBrokenAssets += PreviewSkeletalMesh->ValidatePreviewAttachedObjects();
	}

	if (NumBrokenAssets > 0)
	{
		// Tell the user that there were assets that could not be loaded
		FFormatNamedArguments Args;
		Args.Add(TEXT("NumBrokenAssets"), NumBrokenAssets);
		FNotificationInfo Info(FText::Format(LOCTEXT("MissingPreviewAttachedAssets", "{NumBrokenAssets} attached assets could not be found on loading and were removed"), Args));

		Info.bUseLargeFont = false;
		Info.ExpireDuration = 5.0f;

		TSharedPtr<SNotificationItem> Notification = FSlateNotificationManager::Get().AddNotification(Info);
		if (Notification.IsValid())
		{
			Notification->SetCompletionState(SNotificationItem::CS_Fail);
		}
	}
}

void FAnimationEditorPreviewScene::SetAdditionalMeshes(class UDataAsset* InAdditionalMeshes)
{
	if(GetEditableSkeleton().IsValid())
	{
		GetEditableSkeleton()->SetAdditionalPreviewSkeletalMeshes(InAdditionalMeshes);
	}

	RefreshAdditionalMeshes(true);
}

void FAnimationEditorPreviewScene::SetAdditionalMeshesSelectable(bool bSelectable)
{
	bAdditionalMeshesSelectable = bSelectable;
}

void FAnimationEditorPreviewScene::RefreshAdditionalMeshes(bool bAllowOverrideBaseMesh)
{
	// remove all components
	for (USkeletalMeshComponent* Component : AdditionalMeshes)
	{
		UAnimInstance* AnimInst = Component->GetAnimInstance();
		if (AnimInst && AnimInst->IsA(UAnimPreviewAttacheInstance::StaticClass()))
		{
			AnimInst->Montage_Stop(0.0f);

			FAnimCustomInstanceHelper::UnbindFromSkeletalMeshComponent<UAnimPreviewAttacheInstance>(Component);
		}
		
		RemoveComponent(Component);
	}

	AdditionalMeshes.Empty();

	// add new components
	UDataAsset* PreviewSceneAdditionalMeshes = 
		GetEditableSkeleton().IsValid() && GetEditableSkeleton()->IsSkeletonValid() 
		? GetEditableSkeleton()->GetSkeleton().GetAdditionalPreviewSkeletalMeshes() : nullptr;
	if (PreviewSceneAdditionalMeshes == nullptr)
	{
		PreviewSceneAdditionalMeshes = PreviewSceneDescription->DefaultAdditionalMeshes;
	}

	if (PreviewSceneAdditionalMeshes != nullptr)
	{
		const bool bUseCustomAnimBP = GetMutableDefault<UPersonaOptions>()->bAllowPreviewMeshCollectionsToUseCustomAnimBP;

		// get preview interface
		const IPreviewCollectionInterface* PreviewCollection = Cast<IPreviewCollectionInterface>(PreviewSceneAdditionalMeshes);
		if (PreviewCollection)
		{
			if (bAllowOverrideBaseMesh)
			{
				USkeletalMesh* BaseMesh = PreviewCollection->GetPreviewBaseMesh();
				if (BaseMesh)
				{
					SetPreviewMeshInternal(BaseMesh);
				}
			}

			if (bAllowAdditionalMeshes)
			{
				TArray<USkeletalMesh*> ValidMeshes;
				TArray<TSubclassOf<UAnimInstance>> AnimInstances;
				PreviewCollection->GetPreviewSkeletalMeshes(ValidMeshes, AnimInstances);
				const int32 NumMeshes = ValidMeshes.Num();
				for (int32 MeshIndex = 0; MeshIndex < NumMeshes; ++MeshIndex)
				{
					USkeletalMesh* SkeletalMesh = ValidMeshes[MeshIndex];
					if (SkeletalMesh)
					{
						USkeletalMeshComponent* NewComp = NewObject<USkeletalMeshComponent>(Actor);
						NewComp->bSelectable = bAdditionalMeshesSelectable;
						NewComp->RegisterComponent();
						NewComp->SetSkeletalMesh(SkeletalMesh);
						AddComponent(NewComp, FTransform::Identity, true);
						// Use the attach parent bounds if it is valid, as empty bounds would cause mesh to flicker from frustum culling.
						if (NewComp->GetAttachParent() && NewComp->GetAttachParent()->Bounds.SphereRadius > 0)
						{
							NewComp->bUseAttachParentBound = true;
						}
						if (bUseCustomAnimBP && AnimInstances.IsValidIndex(MeshIndex) && AnimInstances[MeshIndex] != nullptr)
						{
							NewComp->SetAnimInstanceClass(AnimInstances[MeshIndex]);
						}

						if (NewComp->GetAnimInstance() == nullptr)
						{
							bool bWasCreated = false;
							FAnimCustomInstanceHelper::BindToSkeletalMeshComponent<UAnimPreviewAttacheInstance>(NewComp,bWasCreated);
						}
						AdditionalMeshes.Add(NewComp);
					}
				}
			}
		}
	}
}

void FAnimationEditorPreviewScene::AddPreviewAttachedObjects()
{
	if(PersonaToolkit.IsValid())
	{
		// Load up mesh attachments...
		USkeletalMesh* Mesh = PersonaToolkit.Pin()->GetMesh();

		if ( Mesh )
		{
			FPreviewAssetAttachContainer& PreviewAssetAttachContainer = Mesh->GetPreviewAttachedAssetContainer();
			for(int32 Index = 0; Index < PreviewAssetAttachContainer.Num(); Index++)
			{
				FPreviewAttachedObjectPair& PreviewAttachedObject = PreviewAssetAttachContainer[Index];

				AttachObjectToPreviewComponent(PreviewAttachedObject.GetAttachedObject(), PreviewAttachedObject.AttachedTo);
			}
		}
	}

	if(GetEditableSkeleton().IsValid() && GetEditableSkeleton()->IsSkeletonValid())
	{
		const USkeleton& Skeleton = GetEditableSkeleton()->GetSkeleton();

		// ...and then skeleton attachments
		for(int32 i = 0; i < Skeleton.PreviewAttachedAssetContainer.Num(); i++)
		{
			const FPreviewAttachedObjectPair& PreviewAttachedObject = Skeleton.PreviewAttachedAssetContainer[i];

			AttachObjectToPreviewComponent(PreviewAttachedObject.GetAttachedObject(), PreviewAttachedObject.AttachedTo);
		}
	}
}

bool FAnimationEditorPreviewScene::AttachObjectToPreviewComponent( UObject* Object, FName AttachTo )
{
	if(PersonaUtils::GetComponentForAttachedObject(SkeletalMeshComponent, Object, AttachTo))
	{
		return false; // Already have this asset attached at this location
	}

	TSubclassOf<UActorComponent> ComponentClass = FComponentAssetBrokerage::GetPrimaryComponentForAsset(Object->GetClass());
	if(SkeletalMeshComponent && *ComponentClass && ComponentClass->IsChildOf(USceneComponent::StaticClass()))
	{
		//set up world info for undo
		AWorldSettings* WorldSettings = GetWorld()->GetWorldSettings();
		WorldSettings->SetFlags(RF_Transactional);
		WorldSettings->Modify();

		USceneComponent* SceneComponent = NewObject<USceneComponent>(WorldSettings, ComponentClass, NAME_None, RF_Transactional);

		FComponentAssetBrokerage::AssignAssetToComponent(SceneComponent, Object);

		if(UParticleSystemComponent* NewPSysComp = Cast<UParticleSystemComponent>(SceneComponent))
		{
			//maybe this should happen in FComponentAssetBrokerage::AssignAssetToComponent?
			NewPSysComp->SetTickGroup(TG_PostUpdateWork);
		}

		//set up preview component for undo
		SkeletalMeshComponent->SetFlags(RF_Transactional);
		SkeletalMeshComponent->Modify();

		// Attach component to the preview component
		SceneComponent->SetupAttachment(SkeletalMeshComponent, AttachTo);
		SceneComponent->RegisterComponent();
		return true;
	}
	return false;
}

void FAnimationEditorPreviewScene::RemoveAttachedObjectFromPreviewComponent(UObject* Object, FName AttachedTo)
{
	// clean up components	
	if (SkeletalMeshComponent)
	{
		AWorldSettings* WorldSettings = GetWorld()->GetWorldSettings();
		WorldSettings->SetFlags(RF_Transactional);
		WorldSettings->Modify();

		//set up preview component for undo
		SkeletalMeshComponent->SetFlags(RF_Transactional);
		SkeletalMeshComponent->Modify();

		for (int32 I= SkeletalMeshComponent->GetAttachChildren().Num()-1; I >= 0; --I) // Iterate backwards because CleanupComponent will remove from AttachChildren
		{
			USceneComponent* ChildComponent = SkeletalMeshComponent->GetAttachChildren()[I];
			UObject* Asset = FComponentAssetBrokerage::GetAssetFromComponent(ChildComponent);

			if( Asset == Object && ChildComponent->GetAttachSocketName() == AttachedTo)
			{
				// PreviewComponet will be cleaned up by PreviewScene, 
				// but if anything is attached, it won't be cleaned up, 
				// so we'll need to clean them up manually
				CleanupComponent(SkeletalMeshComponent->GetAttachChildren()[I]);
				break;
			}
		}
	}
}

void FAnimationEditorPreviewScene::InvalidateViews()
{
	OnInvalidateViews.Broadcast();
}

void FAnimationEditorPreviewScene::FocusViews()
{
	OnFocusViews.Broadcast();
}

void FAnimationEditorPreviewScene::RemoveAttachedComponent( bool bRemovePreviewAttached /* = true */ )
{
	TMap<UObject*, TArray<FName>> PreviewAttachedObjects;

	if( !bRemovePreviewAttached )
	{
		if(GetEditableSkeleton().IsValid() && GetEditableSkeleton()->IsSkeletonValid())
		{
			const USkeleton& Skeleton = GetEditableSkeleton()->GetSkeleton();
			for(auto Iter = Skeleton.PreviewAttachedAssetContainer.CreateConstIterator(); Iter; ++Iter)
			{
				PreviewAttachedObjects.FindOrAdd(Iter->GetAttachedObject()).Add(Iter->AttachedTo);
			}
		}

		if ( USkeletalMesh* PreviewMesh = PersonaToolkit.Pin()->GetMesh() )
		{
			for(auto Iter = PreviewMesh->GetPreviewAttachedAssetContainer().CreateConstIterator(); Iter; ++Iter)
			{
				PreviewAttachedObjects.FindOrAdd(Iter->GetAttachedObject()).Add(Iter->AttachedTo);
			}
		}
	}

	// clean up components	
	if (SkeletalMeshComponent)
	{
		for (int32 I= SkeletalMeshComponent->GetAttachChildren().Num()-1; I >= 0; --I) // Iterate backwards because CleanupComponent will remove from AttachChildren
		{
			USceneComponent* ChildComponent = SkeletalMeshComponent->GetAttachChildren()[I];
			UObject* Asset = FComponentAssetBrokerage::GetAssetFromComponent(ChildComponent);

			bool bRemove = true;

			//are we supposed to leave assets that came from the skeleton
			if(	!bRemovePreviewAttached )
			{
				//could this asset have come from the skeleton
				if(PreviewAttachedObjects.Contains(Asset))
				{
					if(PreviewAttachedObjects.Find(Asset)->Contains(ChildComponent->GetAttachSocketName()))
					{
						bRemove = false;
					}
				}
			}

			// if this component is added by additional meshes, do not remove it. 
			if (AdditionalMeshes.Contains(ChildComponent))
			{
				bRemove = false;
			}

			// you can use delegate to avoid being removed
			if (OnRemoveAttachedComponentFilter.IsBound() && !OnRemoveAttachedComponentFilter.Execute(ChildComponent))
			{
				bRemove = false;
			}

			if(bRemove)
			{
				// PreviewComponet will be cleaned up by PreviewScene, 
				// but if anything is attached, it won't be cleaned up, 
				// so we'll need to clean them up manually
				CleanupComponent(SkeletalMeshComponent->GetAttachChildren()[I]);
			}
		}

		if( bRemovePreviewAttached )
		{
			check(SkeletalMeshComponent->GetAttachChildren().Num() == 0);
		}
	}
}

void FAnimationEditorPreviewScene::CleanupComponent(USceneComponent* Component)
{
	if (Component)
	{
		for (int32 I = Component->GetAttachChildren().Num() - 1; I >= 0; --I) // Iterate backwards because CleanupComponent will remove from AttachChildren
		{
			CleanupComponent(Component->GetAttachChildren()[I]);
		}

		check(Component->GetAttachChildren().Num() == 0);
		// make sure to remove from component list
		RemoveComponent(Component);
		Component->DestroyComponent();
	}
}

void FAnimationEditorPreviewScene::SetPreviewAnimationAsset(UAnimationAsset* AnimAsset, bool bEnablePreview /*= true*/)
{
	if (SkeletalMeshComponent)
	{
		const USkeleton* Skeleton = GetEditableSkeleton().IsValid() && GetEditableSkeleton()->IsSkeletonValid() 
			? &GetEditableSkeleton()->GetSkeleton() : nullptr;

		RemoveAttachedComponent(false);

		if (AnimAsset != nullptr)
		{
			// Early out if the new preview asset is the same as the current one, to avoid replaying from the beginning, etc...
			if (AnimAsset == GetPreviewAnimationAsset() && SkeletalMeshComponent->IsPreviewOn())
			{
				return;
			}

			// Treat it as invalid if it's got a bogus skeleton pointer
			if (AnimAsset->GetSkeleton() == nullptr)
			{
				return;
			}
		}

		SkeletalMeshComponent->EnablePreview(bEnablePreview, AnimAsset);
	}

	OnAnimChanged.Broadcast(AnimAsset);
}

UAnimationAsset* FAnimationEditorPreviewScene::GetPreviewAnimationAsset() const
{
	if (SkeletalMeshComponent)
	{
		// if same, do not overwrite. It will reset time and everything
		if (SkeletalMeshComponent->PreviewInstance != nullptr)
		{
			return SkeletalMeshComponent->PreviewInstance->GetCurrentAsset();
		}
	}

	return nullptr;
}

void FAnimationEditorPreviewScene::SetFloorLocation(const FVector& InPosition)
{
	FloorMeshComponent->SetWorldTransform(FTransform(FQuat::Identity, InPosition, FVector(3.0f, 3.0f, 1.0f)));
}

void FAnimationEditorPreviewScene::ShowReferencePose(bool bShowRefPose, bool bResetBoneTransforms)
{
	if(SkeletalMeshComponent)
	{
		SkeletalMeshComponent->ShowReferencePose(bShowRefPose);

		// Also reset bone transforms
		if(bResetBoneTransforms && SkeletalMeshComponent->GetSkeletalMeshAsset() != nullptr)
		{
			bool bModified = false;
			FScopedTransaction Transaction(LOCTEXT("ResetBoneTransforms", "Reset Bone Transforms"));

			int32 NumBones = SkeletalMeshComponent->GetSkeletalMeshAsset()->GetRefSkeleton().GetNum();
			for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
			{
				FName BoneName = SkeletalMeshComponent->GetSkeletalMeshAsset()->GetRefSkeleton().GetBoneName(BoneIndex);
				const FAnimNode_ModifyBone* ModifiedBone = SkeletalMeshComponent->PreviewInstance->FindModifiedBone(BoneName);
				if (ModifiedBone != nullptr)
				{
					if (!bModified)
					{
						SkeletalMeshComponent->PreviewInstance->SetFlags(RF_Transactional);
						SkeletalMeshComponent->PreviewInstance->Modify();
						bModified = true;
					}

					SkeletalMeshComponent->PreviewInstance->RemoveBoneModification(BoneName);
				}
			}
		}
	}
}

bool FAnimationEditorPreviewScene::IsShowReferencePoseEnabled() const
{
	return SkeletalMeshComponent->IsReferencePoseShown();
}

void FAnimationEditorPreviewScene::SetDefaultAnimationMode(EPreviewSceneDefaultAnimationMode Mode, bool bShowNow)
{
	DefaultMode = Mode;

	if (bShowNow)
	{
		ShowDefaultMode();
	}
}

void FAnimationEditorPreviewScene::ShowDefaultMode()
{
	switch (DefaultMode)
	{
	case EPreviewSceneDefaultAnimationMode::ReferencePose:
		{
			ShowReferencePose(true);
			break;
		}
	case EPreviewSceneDefaultAnimationMode::Animation:
		{
			UObject* PreviewAsset = PersonaToolkit.Pin()->GetAnimationAsset();
			if (PreviewAsset)
			{
				SkeletalMeshComponent->EnablePreview(true, Cast<UAnimationAsset>(PreviewAsset));
			}
		}
		break;
	case EPreviewSceneDefaultAnimationMode::AnimationBlueprint:
		{
			SkeletalMeshComponent->EnablePreview(false, nullptr);

			UAnimBlueprint* AnimBlueprint = PersonaToolkit.Pin()->GetAnimBlueprint();
			if (AnimBlueprint)
			{
				UAnimBlueprint* PreviewAnimBlueprint = AnimBlueprint->GetPreviewAnimationBlueprint();
		
				if (PreviewAnimBlueprint)
				{
					SetPreviewAnimationBlueprint(PreviewAnimBlueprint, AnimBlueprint);
				}
				else
				{
					SetPreviewAnimationBlueprint(AnimBlueprint, nullptr);
				}
			}
		}
		break;
	case EPreviewSceneDefaultAnimationMode::Custom:
		{
			SkeletalMeshComponent->SetCustomDefaultPose();
			break;
		}
	}
}

FText FAnimationEditorPreviewScene::GetPreviewAssetTooltip(bool bEditingAnimBlueprint) const
{
	// if already looking at ref pose
	if (IsShowReferencePoseEnabled())
	{
		FText PreviewFormat(LOCTEXT("PreviewFormat", "Preview {0}"));

		if (bEditingAnimBlueprint)
		{
			UAnimBlueprint* AnimBP = PersonaToolkit.Pin()->GetAnimBlueprint();
			if (AnimBP)
			{
				return FText::Format(PreviewFormat, FText::FromString(AnimBP->GetName()));
			}
		}
		else
		{
			UObject* PreviewAsset = PersonaToolkit.Pin()->GetAnimationAsset();
			if (PreviewAsset)
			{
				return FText::Format(PreviewFormat, FText::FromString(PreviewAsset->GetName()));
			}
		}

		return LOCTEXT("NoPreviewAvailable", "None Available. Please select asset to preview.");
	}
	else
	{
		return FText::Format(LOCTEXT("CurrentlyPreviewingFormat", "Currently previewing {0}"), FText::FromString(SkeletalMeshComponent->GetPreviewText()));
	}
}

void FAnimationEditorPreviewScene::ClearSelectedBone()
{
	TGuardValue<bool> RecursionGuard(bSelecting, true);

	SelectedBoneIndex = INDEX_NONE;

	if (SkeletalMeshComponent)
	{
		SkeletalMeshComponent->BonesOfInterest.Empty();
	}

	OnSelectedBonesChanged.Broadcast({NAME_None}, ESelectInfo::Direct);

	InvalidateViews();
}

void FAnimationEditorPreviewScene::SetSelectedBone(const FName& BoneName, ESelectInfo::Type InSelectInfo)
{
	const int32 BoneIndex = GetEditableSkeleton().IsValid() && GetEditableSkeleton()->IsSkeletonValid() 
		? GetEditableSkeleton()->GetSkeleton().GetReferenceSkeleton().FindBoneIndex(BoneName) : INDEX_NONE;
	if (BoneIndex != INDEX_NONE)
	{
		SetSelectedBones({BoneName}, InSelectInfo);
	}
}

void FAnimationEditorPreviewScene::SetSelectedBones(const TArray<FName>& InBoneNames, ESelectInfo::Type InSelectInfo)
{
	if (!GetEditableSkeleton().IsValid() || !GetEditableSkeleton()->IsSkeletonValid())
	{
		return;
	}

	const FReferenceSkeleton& ReferenceSkeleton = GetEditableSkeleton()->GetSkeleton().GetReferenceSkeleton();
	
	TArray<FName> BoneNames;
	Algo::CopyIf(InBoneNames, BoneNames, [&ReferenceSkeleton](const FName& BoneName)
	{
		return ReferenceSkeleton.FindBoneIndex(BoneName) != INDEX_NONE;
	});

	if (BoneNames.IsEmpty())
	{
		return;
	}
	
	TGuardValue<bool> RecursionGuard(bSelecting, true);
	
	ClearSelectedBone();
	ClearSelectedSocket();
	ClearSelectedActor();
	
	// need to get mesh bone base since BonesOfInterest is saved in SkeletalMeshComponent
	// and it is used by renderer. It is not Skeleton base
	for (const FName& BoneName : BoneNames)
	{
		const int32 MeshBoneIndex = SkeletalMeshComponent->GetBoneIndex(BoneName);
		if (MeshBoneIndex != INDEX_NONE)
		{
			if (SelectedBoneIndex == INDEX_NONE)
			{
				SelectedBoneIndex = MeshBoneIndex;
			}
			SkeletalMeshComponent->BonesOfInterest.Add(SelectedBoneIndex);
		}
	}

	InvalidateViews();
	
	OnSelectedBonesChanged.Broadcast(BoneNames, InSelectInfo);
}

void FAnimationEditorPreviewScene::SetSelectedSocket(const FSelectedSocketInfo& SocketInfo)
{
	TGuardValue<bool> RecursionGuard(bSelecting, true);

	ClearSelectedBone();
	ClearSelectedActor();

	SelectedSocket = SocketInfo;

	OnSelectedSocketChanged.Broadcast(SelectedSocket);

	InvalidateViews();
}

void FAnimationEditorPreviewScene::ClearSelectedSocket()
{
	TGuardValue<bool> RecursionGuard(bSelecting, true);

	SelectedSocket.Reset();

	OnSelectedSocketChanged.Broadcast(SelectedSocket);

	InvalidateViews();
}

void FAnimationEditorPreviewScene::SetSelectedActor(AActor* InActor)
{
	ClearSelectedBone();
	ClearSelectedSocket();

	SelectedActor = InActor;

	InvalidateViews();
}

void FAnimationEditorPreviewScene::ClearSelectedActor()
{
	SelectedActor = nullptr;

	InvalidateViews();
}

void FAnimationEditorPreviewScene::DeselectAll()
{
	ClearSelectedBone();
	ClearSelectedSocket();
	ClearSelectedActor();

	OnDeselectAll.Broadcast();

	InvalidateViews();
}

bool FAnimationEditorPreviewScene::IsRecordAvailable() const
{
	// make sure mesh exists
	return (SkeletalMeshComponent->GetSkeletalMeshAsset() != nullptr);
}

FSlateIcon FAnimationEditorPreviewScene::GetRecordStatusImage() const
{
	if (IsRecording())
	{
		return FSlateIcon(FAppStyle::GetAppStyleSetName(), "Persona.StopRecordAnimation");
	}

	return FSlateIcon(FAppStyle::GetAppStyleSetName(), "Persona.StartRecordAnimation");
}

FText FAnimationEditorPreviewScene::GetRecordMenuLabel() const
{
	if (IsRecording())
	{
		return LOCTEXT("Persona_StopRecordAnimationMenuLabel", "Stop Record Animation");
	}

	return LOCTEXT("Persona_StartRecordAnimationMenuLabel", "Start Record Animation");
}

FText FAnimationEditorPreviewScene::GetRecordStatusLabel() const
{
	bool bInRecording = false;
	FPersonaModule& PersonaModule = FModuleManager::GetModuleChecked<FPersonaModule>(TEXT("Persona"));
	PersonaModule.OnIsRecordingActive().ExecuteIfBound(SkeletalMeshComponent, bInRecording);

	if (bInRecording)
	{
		return LOCTEXT("Persona_StopRecordAnimationLabel", "Stop");
	}

	return LOCTEXT("Persona_StartRecordAnimationLabel", "Record");
}

FText FAnimationEditorPreviewScene::GetRecordStatusTooltip() const
{
	bool bInRecording = false;
	FPersonaModule& PersonaModule = FModuleManager::GetModuleChecked<FPersonaModule>(TEXT("Persona"));
	PersonaModule.OnIsRecordingActive().ExecuteIfBound(SkeletalMeshComponent, bInRecording);

	if (bInRecording)
	{
		return LOCTEXT("Persona_StopRecordAnimation", "Stop Record Animation");
	}

	return LOCTEXT("Persona_StartRecordAnimation", "Start Record Animation");
}

void FAnimationEditorPreviewScene::RecordAnimation()
{
	bool bInRecording = false;
	FPersonaModule& PersonaModule = FModuleManager::GetModuleChecked<FPersonaModule>(TEXT("Persona"));
	PersonaModule.OnIsRecordingActive().ExecuteIfBound(SkeletalMeshComponent, bInRecording);

	if (bInRecording)
	{
		PersonaModule.OnStopRecording().ExecuteIfBound(SkeletalMeshComponent);
	}
	else
	{
		PersonaModule.OnRecord().ExecuteIfBound(SkeletalMeshComponent);
	}

	OnRecordingStateChangedDelegate.Broadcast();
}

bool FAnimationEditorPreviewScene::IsRecording() const
{
	bool bInRecording = false;
	FPersonaModule& PersonaModule = FModuleManager::GetModuleChecked<FPersonaModule>("Persona");
	PersonaModule.OnIsRecordingActive().ExecuteIfBound(SkeletalMeshComponent, bInRecording);

	return bInRecording;
}

void FAnimationEditorPreviewScene::StopRecording()
{
	FPersonaModule& PersonaModule = FModuleManager::GetModuleChecked<FPersonaModule>("Persona");
	PersonaModule.OnStopRecording().ExecuteIfBound(SkeletalMeshComponent);

	OnRecordingStateChangedDelegate.Broadcast();
}

UAnimSequence* FAnimationEditorPreviewScene::GetCurrentRecording() const
{
	FPersonaModule& PersonaModule = FModuleManager::GetModuleChecked<FPersonaModule>("Persona");
	UAnimSequence* Recording = nullptr;
	PersonaModule.OnGetCurrentRecording().ExecuteIfBound(SkeletalMeshComponent, Recording);
	return Recording;
}

float FAnimationEditorPreviewScene::GetCurrentRecordingTime() const
{
	FPersonaModule& PersonaModule = FModuleManager::GetModuleChecked<FPersonaModule>("Persona");
	float RecordingTime = 0.0f;
	PersonaModule.OnGetCurrentRecordingTime().ExecuteIfBound(SkeletalMeshComponent, RecordingTime);
	return RecordingTime;
}

bool FAnimationEditorPreviewScene::PreviewComponentSelectionOverride(const UPrimitiveComponent* InComponent) const
{
	if (InComponent == GetPreviewMeshComponent())
	{
		const USkeletalMeshComponent* Component = CastChecked<USkeletalMeshComponent>(InComponent);
		return (Component->GetSelectedEditorSection() != INDEX_NONE || Component->GetSelectedEditorMaterial() != INDEX_NONE);
	}

	return false;
}

TWeakObjectPtr<AWindDirectionalSource> FAnimationEditorPreviewScene::CreateWindActor(UWorld* World)
{
	TWeakObjectPtr<AWindDirectionalSource> Wind = World->SpawnActor<AWindDirectionalSource>(PrevWindLocation, PrevWindRotation);

	check(Wind.IsValid());
	//initial wind strength value 
	Wind->GetComponent()->Speed = PrevWindStrength;
	return Wind;
}

void FAnimationEditorPreviewScene::OnSetPlaybackSpeed(int32 InPlaybackSpeedMode)
{
	SetAnimationPlaybackSpeedMode(static_cast<EAnimationPlaybackSpeeds::Type>(InPlaybackSpeedMode));
}

void FAnimationEditorPreviewScene::BindCommands()
{
	IPersonaPreviewScene::BindCommands();

	FUICommandList_Pinnable& CommandList = *GetPinnedCommandList();
	FAnimViewportPlaybackCommands::Register();
	const FAnimViewportPlaybackCommands& ViewportPlaybackCommands = FAnimViewportPlaybackCommands::Get();

	// Create a menu item for each playback speed in EAnimationPlaybackSpeeds
	for (int32 i = 0; i < static_cast<int>(EAnimationPlaybackSpeeds::NumPlaybackSpeeds); ++i)
	{
		CommandList.FUICommandList::MapAction(
			ViewportPlaybackCommands.PlaybackSpeedCommands[i],
			FExecuteAction::CreateRaw(this, &FAnimationEditorPreviewScene::OnSetPlaybackSpeed, i),
			FCanExecuteAction(),
			FIsActionChecked::CreateRaw(this, &FAnimationEditorPreviewScene::IsPlaybackSpeedSelected, i)
		);
	}
}

TSharedRef<FUICommandList> FAnimationEditorPreviewScene::CreateCommandList()
{
	return MakeShared<FUICommandList_Pinnable>();
}

TSharedPtr<FUICommandList_Pinnable> FAnimationEditorPreviewScene::GetPinnedCommandList() const
{
	return StaticCastSharedPtr<FUICommandList_Pinnable>(UICommandList);
}

void FAnimationEditorPreviewScene::EnableWind(bool bEnableWind)
{
	UWorld* World = GetWorld();
	check(World);

	if (bEnableWind)
	{
		if (!WindSourceActor.IsValid())
		{
			WindSourceActor = CreateWindActor(World);
		}
	}
	else if (WindSourceActor.IsValid())
	{
		PrevWindLocation = WindSourceActor->GetActorLocation();
		PrevWindRotation = WindSourceActor->GetActorRotation();
		PrevWindStrength = WindSourceActor->GetComponent()->Strength;

		World->DestroyActor(WindSourceActor.Get());
	}
}

bool FAnimationEditorPreviewScene::IsWindEnabled() const
{
	return WindSourceActor.IsValid();
}

void FAnimationEditorPreviewScene::SetWindStrength(float SliderValue)
{
	if (WindSourceActor.IsValid())
	{
		WindSourceActor->GetComponent()->Speed = SliderValue;
		//to apply this new wind strength
		WindSourceActor->UpdateComponentTransforms();
	}
}

float FAnimationEditorPreviewScene::GetWindStrength() const
{
	if (WindSourceActor.IsValid())
	{
		return WindSourceActor->GetComponent()->Speed;
	}

	return 0;
}

void FAnimationEditorPreviewScene::SetGravityScale(float InGravityScale)
{
	GravityScale = InGravityScale;
	float RealGravityScale = InGravityScale * 4.0f;

	UWorld* World = GetWorld();
	AWorldSettings* Setting = World->GetWorldSettings();
	Setting->WorldGravityZ = UPhysicsSettings::Get()->DefaultGravityZ*RealGravityScale;
	Setting->bWorldGravitySet = true;
}

float FAnimationEditorPreviewScene::GetGravityScale() const
{
	return GravityScale;
}

AActor* FAnimationEditorPreviewScene::GetSelectedActor() const
{
	return SelectedActor.Get(); 
}

FSelectedSocketInfo FAnimationEditorPreviewScene::GetSelectedSocket() const
{
	return SelectedSocket; 
}

int32 FAnimationEditorPreviewScene::GetSelectedBoneIndex() const
{ 
	return SelectedBoneIndex;
}

void FAnimationEditorPreviewScene::TogglePlayback()
{
	if (SkeletalMeshComponent)
	{
		if (SkeletalMeshComponent->IsPreviewOn() && SkeletalMeshComponent->PreviewInstance)
		{
			SkeletalMeshComponent->PreviewInstance->SetPlaying(!SkeletalMeshComponent->PreviewInstance->IsPlaying());
		}
		else
		{
			SkeletalMeshComponent->GlobalAnimRateScale = (SkeletalMeshComponent->GlobalAnimRateScale > 0.0f) ? 0.0f : 1.0f;
		}
	}
}

void FAnimationEditorPreviewScene::SetActor(AActor* InActor)
{
	check(Actor == nullptr || !Actor->IsRooted());
	Actor = InActor;
}

AActor* FAnimationEditorPreviewScene::GetActor() const
{
	return Actor;
}

bool FAnimationEditorPreviewScene::AllowMeshHitProxies() const
{
	return bEnableMeshHitProxies;
}

void FAnimationEditorPreviewScene::SetAllowMeshHitProxies(bool bState)
{
	bEnableMeshHitProxies = bState;

	if (GetPreviewMeshComponent())
	{
		GetPreviewMeshComponent()->bSelectable = bEnableMeshHitProxies;
		GetPreviewMeshComponent()->MarkRenderStateDirty();
	}
}

void FAnimationEditorPreviewScene::FlagTickable()
{
	// Set the last tick time so we tick when we are visible in a viewport
	LastTickTime = FPlatformTime::Seconds();
}

void FAnimationEditorPreviewScene::HandleSkeletonTreeSelectionChanged(const TArrayView<TSharedPtr<ISkeletonTreeItem>>& InSelectedItems, ESelectInfo::Type InSelectInfo)
{
	if(!bSelecting)
	{
		if(InSelectedItems.Num() == 0)
		{
			DeselectAll();
		}
		else
		{
			for(const TSharedPtr<ISkeletonTreeItem>& Item : InSelectedItems)
			{
				if(Item->IsOfTypeByName("FSkeletonTreeBoneItem"))
				{
					SetSelectedBone(Item->GetRowItemName(), InSelectInfo);
				}
				else if(Item->IsOfTypeByName("FSkeletonTreeSocketItem"))
				{
					FSelectedSocketInfo SocketInfo;
					SocketInfo.Socket = CastChecked<USkeletalMeshSocket>(Item->GetObject());
					SocketInfo.bSocketIsOnSkeleton = !SocketInfo.Socket->GetOuter()->IsA<USkeletalMesh>();
					SetSelectedSocket(SocketInfo);
				}
			}
		}
	}
}

void FAnimationEditorPreviewScene::Tick(float InDeltaTime)
{
	OnPreTickDelegate.Broadcast();

	IPersonaPreviewScene::Tick(InDeltaTime);

	GetWorld()->Tick(LEVELTICK_All, InDeltaTime);

	if (SkeletalMeshComponent)
	{
		// Handle updating the preview component to represent the effects of root motion	
		const FBoxSphereBounds& Bounds = GetFloorBounds();
		SkeletalMeshComponent->ConsumeRootMotion(Bounds.GetBox().Min, Bounds.GetBox().Max);

		if (LastCachedLODForPreviewComponent != SkeletalMeshComponent->GetPredictedLODLevel())
		{
			OnLODChanged.Broadcast();
			LastCachedLODForPreviewComponent = SkeletalMeshComponent->GetPredictedLODLevel();
		}
	}

	OnPostTickDelegate.Broadcast();
}

bool FAnimationEditorPreviewScene::IsTickable() const
{
	const float VisibilityTimeThreshold = 0.25f;

	// The preview scene is tickable if any viewport can see it
	return  LastTickTime == 0.0	||	// Never been ticked
			FPlatformTime::Seconds() - LastTickTime <= VisibilityTimeThreshold;	// Ticked recently
}

void FAnimationEditorPreviewScene::AddComponent(class UActorComponent* Component, const FTransform& LocalToWorld, bool bAttachToRoot /*= false*/)
{
	if (bAttachToRoot)
	{
		if (USceneComponent* SceneComponent = Cast<USceneComponent>(Component))
		{
			SceneComponent->AttachToComponent(Actor->GetRootComponent(), FAttachmentTransformRules::KeepWorldTransform);
		}
	}

	Actor->AddOwnedComponent(Component);

	IPersonaPreviewScene::AddComponent(Component, LocalToWorld);
}

void FAnimationEditorPreviewScene::RemoveComponent(class UActorComponent* Component)
{
	if (USceneComponent* SceneComponent = Cast<USceneComponent>(Component))
	{
		SceneComponent->DetachFromComponent(FDetachmentTransformRules::KeepWorldTransform);
	}

	Actor->RemoveOwnedComponent(Component);

	IPersonaPreviewScene::RemoveComponent(Component);
}

void FAnimationEditorPreviewScene::PostUndo(bool bSuccess)
{
	// refresh skeletal mesh & animation
	if (PreviewSceneDescription)
	{
		SetPreviewMesh(PreviewSceneDescription->PreviewMesh.Get());
		PreviewSceneDescription->PreviewControllerInstance->InitializeView(PreviewSceneDescription, this);
	}
}

void FAnimationEditorPreviewScene::PostRedo(bool bSuccess)
{
	PostUndo(bSuccess);
}

void FAnimationEditorPreviewScene::AddReferencedObjects( FReferenceCollector& Collector )
{
	IPersonaPreviewScene::AddReferencedObjects(Collector);

	Collector.AddReferencedObject(PreviewSceneDescription);
	Collector.AddReferencedObject(Actor);
	Collector.AddReferencedObject(SkeletalMeshComponent);
	Collector.AddReferencedObjects(AdditionalMeshes);
}

TOptional<float> FAnimationEditorPreviewScene::GetCurrentTime() const
{
	if (SkeletalMeshComponent && SkeletalMeshComponent->PreviewInstance)
	{
		return SkeletalMeshComponent->PreviewInstance->GetCurrentTime();
	}
	return {};
}

void FAnimationEditorPreviewScene::SetCustomAnimationSpeed(float InAnimationSpeed)
{
	CustomAnimationSpeed = InAnimationSpeed;
	SetAnimationPlaybackSpeedMode(EAnimationPlaybackSpeeds::Custom);
}

float FAnimationEditorPreviewScene::GetCustomAnimationSpeed() const
{
	return CustomAnimationSpeed;
}

EAnimationPlaybackSpeeds::Type FAnimationEditorPreviewScene::GetAnimationPlaybackSpeedMode() const
{
	return AnimationPlaybackSpeedMode;
}

void FAnimationEditorPreviewScene::SetAnimationPlaybackSpeedMode(EAnimationPlaybackSpeeds::Type InMode)
{
	AnimationPlaybackSpeedMode = InMode;

	if (GetWorld())
	{
		const float AnimationSpeed = (InMode == EAnimationPlaybackSpeeds::Custom)
									   ? GetCustomAnimationSpeed()
									   : EAnimationPlaybackSpeeds::Values[AnimationPlaybackSpeedMode];
		GetWorld()->GetWorldSettings()->TimeDilation = AnimationSpeed;
	}
}

bool FAnimationEditorPreviewScene::IsPlaybackSpeedSelected(int32 InPlaybackSpeedMode) const
{
	return InPlaybackSpeedMode == AnimationPlaybackSpeedMode;
}

#undef LOCTEXT_NAMESPACE
