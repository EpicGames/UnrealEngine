// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimationEditorTypes.h"
#include "CoreMinimal.h"
#include "Textures/SlateIcon.h"
#include "EditorUndoClient.h"
#include "IPersonaPreviewScene.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "Engine/WindDirectionalSource.h"

class FUICommandList_Pinnable;
class IEditableSkeleton;
class IPersonaToolkit;
class UAnimSequence;



class FAnimationEditorPreviewScene : public IPersonaPreviewScene, public FEditorUndoClient
{
public:
	FAnimationEditorPreviewScene(const ConstructionValues& CVS, const TSharedPtr<class IEditableSkeleton>& InEditableSkeleton, const TSharedRef<class IPersonaToolkit>& InPersonaToolkit);

	~FAnimationEditorPreviewScene();

	/** IPersonaPreviewScene interface */
	virtual void UnregisterForUndo() override;
	virtual TSharedRef<class IPersonaToolkit> GetPersonaToolkit() const override { return PersonaToolkit.Pin().ToSharedRef(); }
	virtual void SetPreviewAnimationAsset(UAnimationAsset* AnimAsset, bool bEnablePreview = true) override;
	virtual UAnimationAsset* GetPreviewAnimationAsset() const override;
	virtual void SetPreviewMesh(USkeletalMesh* NewPreviewMesh, bool bAllowOverrideBaseMesh = true) override;
	virtual USkeletalMesh* GetPreviewMesh() const override;
	virtual void SetPreviewAnimationBlueprint(UAnimBlueprint* InAnimBlueprint, UAnimBlueprint* InOverlayOrSubAnimBlueprint) override;
	virtual bool AttachObjectToPreviewComponent(UObject* Object, FName AttachTo) override;
	virtual void RemoveAttachedObjectFromPreviewComponent(UObject* Object, FName AttachedTo) override;
	virtual void InvalidateViews() override;
	virtual void FocusViews() override;
	virtual UDebugSkelMeshComponent* GetPreviewMeshComponent() const override { return SkeletalMeshComponent; }
	virtual TArray<UDebugSkelMeshComponent*> GetAllPreviewMeshComponents() const override;
	virtual void ForEachPreviewMesh(TFunction<void (UDebugSkelMeshComponent*)> PerMeshFunction) override;
	virtual void SetPreviewMeshComponent(UDebugSkelMeshComponent* InSkeletalMeshComponent) override;
	virtual void SetAdditionalMeshes(class UDataAsset* InAdditionalMeshes) override;
	virtual void SetAdditionalMeshesSelectable(bool bSelectable) override;
	virtual void RefreshAdditionalMeshes(bool bAllowOverrideBaseMesh) override;
	virtual void ShowReferencePose(bool bShowRefPose, bool bResetBoneTransforms = false) override;
	virtual bool IsShowReferencePoseEnabled() const override;
	virtual void SetSelectedBone(const FName& BoneName, ESelectInfo::Type InSelectInfo) override;
	virtual void SetSelectedBones(const TArray<FName>& InBoneNames, ESelectInfo::Type InSelectInfo) override;
	virtual void ClearSelectedBone() override;
	virtual void SetSelectedSocket(const FSelectedSocketInfo& SocketInfo) override;
	virtual void ClearSelectedSocket() override;
	virtual void SetSelectedActor(AActor* InActor) override;
	virtual void ClearSelectedActor() override;
	virtual void DeselectAll() override;

	virtual void RegisterOnAnimChanged(const FOnAnimChanged& Delegate) override
	{
		OnAnimChanged.Add(Delegate);
	}

	virtual void UnregisterOnAnimChanged(FDelegateUserObject Thing) override
	{
		OnAnimChanged.RemoveAll(Thing);
	}

	virtual void RegisterOnPreviewMeshChanged(const FOnPreviewMeshChanged& Delegate) override
	{
		OnPreviewMeshChanged.Add(Delegate);
	}

	virtual void UnregisterOnPreviewMeshChanged(FDelegateUserObject Thing) override
	{
		OnPreviewMeshChanged.RemoveAll(Thing);
	}

	virtual void RegisterOnLODChanged(const FSimpleDelegate& Delegate) override
	{
		OnLODChanged.Add(Delegate);
	}

	virtual void UnregisterOnLODChanged(FDelegateUserObject Thing) override
	{
		OnLODChanged.RemoveAll(Thing);
	}

	virtual void BroadcastOnLODChanged() override
	{
		if (OnLODChanged.IsBound())
		{
			OnLODChanged.Broadcast();
		}
	}

	virtual void RegisterOnMorphTargetsChanged(const FSimpleDelegate& Delegate) override
	{
		OnMorphTargetsChanged.Add(Delegate);
	}

	virtual void UnregisterOnMorphTargetsChanged(FDelegateUserObject Thing) override
	{
		OnMorphTargetsChanged.RemoveAll(Thing);
	}

	virtual void BroadcastOnMorphTargetsChanged() override
	{
		if (OnMorphTargetsChanged.IsBound())
		{
			OnMorphTargetsChanged.Broadcast();
		}
	}

	virtual void RegisterOnInvalidateViews(const FSimpleDelegate& Delegate) override
	{
		OnInvalidateViews.Add(Delegate);
	}

	virtual void UnregisterOnInvalidateViews(FDelegateUserObject Thing) override
	{
		OnInvalidateViews.RemoveAll(Thing);
	}

	virtual void RegisterOnFocusViews(const FSimpleDelegate& Delegate) override
	{
		OnFocusViews.Add(Delegate);
	}

	virtual void UnregisterOnFocusViews(FDelegateUserObject Thing) override
	{
		OnFocusViews.RemoveAll(Thing);
	}

	virtual void RegisterOnMeshClick(const FOnMeshClick& Delegate) override
	{
		OnMeshClick.Add(Delegate);
	}

	virtual void UnregisterOnMeshClick(FDelegateUserObject Thing) override
	{
		OnMeshClick.RemoveAll(Thing);
	}

	virtual FDelegateHandle RegisterOnSelectedBonesChanged(const FOnSelectedBonesChanged& InDelegate) override
	{
		return OnSelectedBonesChanged.Add(InDelegate);
	}

	virtual void UnregisterOnSelectedBonesChanged(FDelegateHandle InHandle) override
	{
		OnSelectedBonesChanged.Remove(InHandle);
	}

	virtual FDelegateHandle RegisterOnSelectedSocketChanged(const FOnSelectedSocketChanged& Delegate) override
	{
		return OnSelectedSocketChanged.Add(Delegate);
	}

	virtual void UnregisterOnSelectedSocketChanged(FDelegateHandle InHandle) override
	{
		OnSelectedSocketChanged.Remove(InHandle);
	}

	virtual FDelegateHandle RegisterOnDeselectAll(const FSimpleDelegate& Delegate) override
	{
		return OnDeselectAll.Add(Delegate);
	}

	virtual void UnregisterOnDeselectAll(FDelegateHandle InHandle) override
	{
		OnDeselectAll.Remove(InHandle);
	}

	virtual void SetDefaultAnimationMode(EPreviewSceneDefaultAnimationMode Mode, bool bShowNow) override;
	virtual void ShowDefaultMode() override;
	virtual void EnableWind(bool bEnableWind) override;
	virtual bool IsWindEnabled() const override;
	virtual void SetWindStrength(float InWindStrength) override;
	virtual float GetWindStrength() const override;
	virtual void SetGravityScale(float InGravityScale) override;
	virtual float GetGravityScale() const override;
	virtual AActor* GetSelectedActor() const override;
	virtual FSelectedSocketInfo GetSelectedSocket() const override;
	virtual int32 GetSelectedBoneIndex() const override;
	virtual void TogglePlayback() override;
	virtual AActor* GetActor() const override;
	virtual void SetActor(AActor* InActor) override;
	virtual bool AllowMeshHitProxies() const override;
	virtual void SetAllowMeshHitProxies(bool bState) override;

	virtual void RegisterOnSelectedLODChanged(const FOnSelectedLODChanged &Delegate) override
	{
		OnSelectedLODChanged.Add(Delegate);
	}
	
	virtual void UnRegisterOnSelectedLODChanged(FDelegateUserObject Thing) override
	{
		OnSelectedLODChanged.RemoveAll(Thing);
	}
	
	virtual void BroadcastOnSelectedLODChanged() override
	{
		if (OnSelectedLODChanged.IsBound())
		{
			OnSelectedLODChanged.Broadcast();
		}
	}

	virtual void RegisterOnCameraOverrideChanged(const FSimpleDelegate& Delegate) override
	{
		OnCameraOverrideChanged.Add(Delegate);
	}

	virtual void UnregisterOnCameraOverrideChanged(FDelegateUserObject Thing) override
	{
		OnCameraOverrideChanged.RemoveAll(Thing);
	}

	virtual void RegisterOnPreTick(const FSimpleDelegate& Delegate) override
	{
		OnPreTickDelegate.Add(Delegate);
	}

	virtual void UnregisterOnPreTick(FDelegateUserObject Thing) override
	{
		OnPreTickDelegate.RemoveAll(Thing);
	}

	virtual void RegisterOnPostTick(const FSimpleDelegate& Delegate) override
	{
		OnPostTickDelegate.Add(Delegate);
	}

	virtual void UnregisterOnPostTick(FDelegateUserObject Thing) override
	{
		OnPostTickDelegate.RemoveAll(Thing);
	}

	virtual void FlagTickable() override;

	void SetCameraOverride(TSharedPtr<class FEditorCameraController> NewCamera)
	{
		CameraOverride = NewCamera;
		OnCameraOverrideChanged.Broadcast();
	}

	TSharedPtr<FEditorCameraController> GetCurrentCameraOverride() const
	{
		return CameraOverride;
	}

	virtual void HandleSkeletonTreeSelectionChanged(const TArrayView<TSharedPtr<ISkeletonTreeItem>>& InSelectedItems, ESelectInfo::Type InSelectInfo) override;
	/** FPreviewScene interface */
	virtual void Tick(float InDeltaTime) override;
	virtual bool IsTickable() const override;
	virtual ETickableTickType GetTickableTickType() const { return ETickableTickType::Conditional; }
	virtual void AddComponent(class UActorComponent* Component, const FTransform& LocalToWorld, bool bAttachToRoot = false) override;
	virtual void RemoveComponent(class UActorComponent* Component) override;

	/** FEditorUndoClient interface */
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;

	/** FGCObject interface */
	virtual void AddReferencedObjects( FReferenceCollector& Collector ) override;

	/** Validate preview attached assets on skeleton and supplied skeletal mesh, notifying user if any are removed */
	void ValidatePreviewAttachedAssets(USkeletalMesh* PreviewSkeletalMesh);

	/** Get the bounds of the floor */
	const FBoxSphereBounds& GetFloorBounds() const { return FloorBounds; }

	/** Set the floor location */
	void SetFloorLocation(const FVector& InPosition);

	/** Get tooltip text for the preview asset button */
	FText GetPreviewAssetTooltip(bool bEditingAnimBlueprint) const;

	/** Get the preview scene description */
	class UPersonaPreviewSceneDescription* GetPreviewSceneDescription() const { return PreviewSceneDescription; }

	/** Begin recording animation form the preview component **/
	void RecordAnimation();

	/** Check whether recording of the preview component is available */
	bool IsRecordAvailable() const;

	/** Get a status image to display for recording in progress */
	FSlateIcon GetRecordStatusImage() const;

	/** Get a tooltip to display while recording */
	FText GetRecordStatusTooltip() const;

	/** Get a the label to display while recording */
	FText GetRecordStatusLabel() const;

	/** Get a the menu label to display while recording */
	FText GetRecordMenuLabel() const;

	/** Check whether this Persona instance is recording */
	bool IsRecording() const;

	/** Stop recording in this Persona instance */
	void StopRecording();

	/** Delegate fired when recording state changes */
	FSimpleMulticastDelegate& OnRecordingStateChanged() { return OnRecordingStateChangedDelegate; }

	/** Get the currently recording animation */
	UAnimSequence* GetCurrentRecording() const;

	/** Get the currently recording animation time */
	float GetCurrentRecordingTime() const;

	/** Get the currently selected actor */
	AActor* GetSelectedActor() { return SelectedActor.Get(); }

	/** Get the currently selected socket */
	FSelectedSocketInfo GetSelectedSocket() { return SelectedSocket; }

	/** Get the currently bone index */
	int32 GetSelectedBoneIndex() { return SelectedBoneIndex; }

	/** Broadcasts that a mesh viewport click occurred */
	virtual bool BroadcastMeshClick(HActor* HitProxy, const FViewportClick& Click) {
		OnMeshClick.Broadcast(HitProxy, Click); return OnMeshClick.IsBound(); }

	/** Override for preview component selection to inform the editor we consider it selected */
	bool PreviewComponentSelectionOverride(const UPrimitiveComponent* InComponent) const;

	virtual void SetAllowAdditionalMeshes(bool bAllow) 
	{
		bAllowAdditionalMeshes = bAllow;
	}

	virtual void SetEditableSkeleton(TSharedPtr<IEditableSkeleton> InEditableSkeleton) override
	{
		EditableSkeletonPtr = InEditableSkeleton;
	}

	void SetIsBeingConstructed(bool bInIsBeingConstructed) 
	{
		bIsBeingConstructed = bInIsBeingConstructed;
	}

	bool IsBeingConstructed() const
	{
		return bIsBeingConstructed;
	}

	/** Toggles state on showing Timecode value in the viewport */
	void ToggleShowTimecode()
	{
		bShowTimecode = !bShowTimecode;
	}

	/** Should we show the timecode in the viewport. */
	bool IsShowTimecode() const
	{
		return bShowTimecode;
	}

	/** Get the current time of the preview instance for the current animation sequence. */
	TOptional<float> GetCurrentTime() const;

	/** Set the current custom animation playback speed */
	virtual void SetCustomAnimationSpeed(float InAnimationSpeed) override;

	/** Get the current custom animation playback speed */
	virtual float GetCustomAnimationSpeed() const override;

	/** Retrieve the current animation playback speed mode */
	virtual  EAnimationPlaybackSpeeds::Type GetAnimationPlaybackSpeedMode() const override;

	/** Set the playback speed mode */
	virtual void SetAnimationPlaybackSpeedMode(EAnimationPlaybackSpeeds::Type InMode) override;

	/** Function to return whether the supplied playback speed is the current active one */
	virtual bool IsPlaybackSpeedSelected(int32 InPlaybackSpeedMode) const override;

	/** Return the list of commands of the preview scene which are bound to delegates */
	TSharedPtr<FUICommandList_Pinnable> GetPinnedCommandList() const;

protected:
	virtual void BindCommands() override;
	virtual TSharedRef<FUICommandList> CreateCommandList() override;

private:
	/** Set preview mesh internal use only. The mesh should be verified by now. */
	void SetPreviewMeshInternal(USkeletalMesh* NewPreviewMesh);

	/** Adds to the viewport all the attached preview objects that the current skeleton and mesh contains */
	void AddPreviewAttachedObjects();

	/** Destroy the supplied component (and its children) */
	void CleanupComponent(USceneComponent* Component);

	/** 
	 * Removes attached components from the preview component. (WARNING: There is a possibility that this function will
	 * remove the wrong component if 2 of the same type (same UObject) are attached at the same location!
	 *
	 * @param bRemovePreviewAttached	Specifies whether all attached components are removed or just the ones
	 *									that haven't come from the target skeleton.
	 */
	void RemoveAttachedComponent(bool bRemovePreviewAttached = true);

	/** Create an actor used to simulate wind (useful for cloth) */
	TWeakObjectPtr<class AWindDirectionalSource> CreateWindActor(UWorld* World);

	TSharedPtr<class IEditableSkeleton> GetEditableSkeleton() const { return EditableSkeletonPtr.Pin(); }

	/* setter/getter for can remove attach component */
	virtual void SetRemoveAttachedComponentFilter(const FOnRemoveAttachedComponentFilter& Delegate) override
	{
		OnRemoveAttachedComponentFilter = Delegate;
	}
	virtual void ClearRemoveAttachedComponentFilter() override
	{
		if (OnRemoveAttachedComponentFilter.IsBound())
		{
			OnRemoveAttachedComponentFilter.Unbind();
		}
	}

	void OnSetPlaybackSpeed(int32 InPlaybackSpeedMode);

private:
	/** The one and only actor we have */
	TObjectPtr<AActor> Actor;

	/** The main preview skeletal mesh component */
	TObjectPtr<UDebugSkelMeshComponent>			SkeletalMeshComponent;

	/** Array of loaded additional meshes */
	TArray<TObjectPtr<USkeletalMeshComponent>>		AdditionalMeshes;

	/** The editable skeleton we are viewing/editing */
	TWeakPtr<class IEditableSkeleton> EditableSkeletonPtr;

	/** The persona toolkit we are embedded in */
	TWeakPtr<class IPersonaToolkit> PersonaToolkit;

	/** Cached bounds of the floor mesh */
	FBoxSphereBounds FloorBounds;

	/** Delegate to be called after the preview animation has been changed */
	FOnAnimChangedMulticaster OnAnimChanged;

	/** Broadcasts whenever the preview mesh changes */
	FOnPreviewMeshChangedMulticaster OnPreviewMeshChanged;

	/** Mode that the preview scene defaults to (usually depending on asset editor context) */
	EPreviewSceneDefaultAnimationMode DefaultMode;

	/** Broadcasts whenever the preview mesh is clicked */
	FOnMeshClickMulticaster OnMeshClick;

	/** Configuration object for editing in details panels */
	TObjectPtr<class UPersonaPreviewSceneDescription> PreviewSceneDescription;

	/** Previous information of a wind actor */
	FVector PrevWindLocation;
	FRotator PrevWindRotation;
	float PrevWindStrength;

	/** Reference to the wind actor */
	TWeakObjectPtr<AWindDirectionalSource> WindSourceActor;

	/** The gravity scale */
	float GravityScale;

	/** The selected actor */
	TWeakObjectPtr<AActor> SelectedActor;

	/** Selected bone */
	int32 SelectedBoneIndex;
	
	/** Selected socket */
	FSelectedSocketInfo SelectedSocket;

	/** LOD index cached & used to check for broadcasting OnLODChanged delegate */
	int32 LastCachedLODForPreviewComponent;

	/** LOD changed delegate */
	FSimpleMulticastDelegate OnLODChanged;

	/** Morph target changed delegate */
	FSimpleMulticastDelegate OnMorphTargetsChanged;

	/** View invalidation delegate */
	FSimpleMulticastDelegate OnInvalidateViews;

	/** View focus delegate */
	FSimpleMulticastDelegate OnFocusViews;

	/** Whether or not mesh section hit proxies should be enabled or not */
	bool bEnableMeshHitProxies;

	/* Selected LOD changed delegate */
	FOnSelectedLODChangedMulticaster OnSelectedLODChanged;

	/** Camera override delegate */
	FSimpleMulticastDelegate OnCameraOverrideChanged;

	/** Currently specified camera override */
	TSharedPtr<FEditorCameraController> CameraOverride;

	/** Delegates fired on pre/post tick of this preview scene */
	FSimpleMulticastDelegate OnPreTickDelegate;
	FSimpleMulticastDelegate OnPostTickDelegate;

	/** Delegate fired when recording state changes */
	FSimpleMulticastDelegate OnRecordingStateChangedDelegate;

	/** The last time we were flagged for ticking */
	double LastTickTime;
	
	/** Bone selection changed delegate */
	FOnSelectedBonesChangedMulticaster OnSelectedBonesChanged;

	/** Socket selection changed delegate */
	FOnSelectedSocketChangedMulticaster OnSelectedSocketChanged;

	/** Deselect all delegate */
	FSimpleMulticastDelegate OnDeselectAll;

	/** Selection recursion guard */
	bool bSelecting;

	/** Allow additional meshes to be attached */
	bool bAllowAdditionalMeshes;

	/** Allow additional meshes to be selectable */
	bool bAdditionalMeshesSelectable;

	/** Delegate Remove attach component */
	FOnRemoveAttachedComponentFilter OnRemoveAttachedComponentFilter;

	/** True during initial creation, so some code can be skipped */
	bool bIsBeingConstructed = false;

	/** Should we display timecode in the preview viewport. */
	bool bShowTimecode = true;

	/** Custom Animation speed in the viewport. Transient setting. */
	float CustomAnimationSpeed = 1.0f;

	/** Selected playback speed mode, used for deciding scale */
	EAnimationPlaybackSpeeds::Type AnimationPlaybackSpeedMode = EAnimationPlaybackSpeeds::Normal;
};
