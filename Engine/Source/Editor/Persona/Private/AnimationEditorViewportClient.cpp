// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationEditorViewportClient.h"
#include "Modules/ModuleManager.h"
#include "EngineGlobals.h"
#include "Animation/AnimSequence.h"
#include "SceneView.h"
#include "Styling/AppStyle.h"
#include "AnimationEditorPreviewScene.h"
#include "Materials/Material.h"
#include "CanvasItem.h"
#include "Editor/EditorPerProjectUserSettings.h"
#include "Animation/AnimBlueprint.h"
#include "Preferences/PersonaOptions.h"
#include "Engine/CollisionProfile.h"
#include "Animation/AnimBlueprintGeneratedClass.h"
#include "Animation/BuiltInAttributeTypes.h"
#include "Animation/MirrorDataTable.h"
#include "GameFramework/WorldSettings.h"
#include "PersonaModule.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "GenericPlatform/GenericPlatformInputDeviceMapper.h"
#include "DynamicMeshBuilder.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Materials/MaterialRenderProxy.h"

#include "SEditorViewport.h"
#include "CanvasTypes.h"
#include "AnimPreviewInstance.h"
#include "ScopedTransaction.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "Engine/SkeletalMeshSocket.h"
#include "Engine/SkinnedAssetCommon.h"
#include "SAnimationEditorViewport.h"
#include "AssetViewerSettings.h"
#include "IPersonaEditorModeManager.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "SkeletalDebugRendering.h"
#include "SkeletalRenderPublic.h"
#include "AudioDevice.h"
#include "RawIndexBuffer.h"
#include "CameraController.h"
#include "ContextObjectStore.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "IPersonaToolkit.h"
#include "Animation/MorphTarget.h"
#include "Rendering/SkeletalMeshModel.h"
#include "UnrealWidget.h"
#include "Engine/PoseWatchRenderData.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "Animation/AnimCompositeBase.h"
#include "AudioEditorSettings.h"
#include "Animation/AnimNotifies/AnimNotifyState.h"
#include "Animation/AnimMontage.h"
#include "Animation/AnimSequenceHelpers.h"
#include "AnimationBlueprintLibrary.h"
#include "ClothingSimulationInstance.h"

TAutoConsoleVariable<bool> CVarShowEngineDebugMessageOnAnimViewport(TEXT("Persona.AnimationEditorViewportClient.ShowEngineDebugMessageOnAnimViewport"), true, TEXT("When true show engine onscreen debug messages on animation editor viewport clients."));

namespace {
	static const float AnimationEditorViewport_RotateSpeed = 0.02f;
	static const float AnimationEditorViewport_TranslateSpeed = 0.25f;
	// follow camera feature
	static const FVector::FReal FollowCamera_InterpSpeed = 4.f;
	static const FVector::FReal FollowCamera_InterpSpeed_Z = 1.f;
}

namespace EAnimationPlaybackSpeeds
{
	// Speed scales for animation playback, must match EAnimationPlaybackSpeeds::Type
	float Values[EAnimationPlaybackSpeeds::NumPlaybackSpeeds] = { 0.1f, 0.25f, 0.5f, 0.75f, 1.0f, 2.0f, 5.0f, 10.0f, 0.f };
}

namespace UE::Private
{
	bool CanDrawPreviewComponents(TArray<UDebugSkelMeshComponent*> PreviewComponents)
	{
		//Avoid drawing if any of the component reference a compiling asset
		for (UDebugSkelMeshComponent* PreviewMeshComponent : PreviewComponents)
		{
			if (PreviewMeshComponent && PreviewMeshComponent->GetSkinnedAsset() && PreviewMeshComponent->GetSkinnedAsset()->IsCompiling())
			{
				return false;
			}
		}
		return true;
	}

	void DrawCoordinateSystem(FPrimitiveDrawInterface* PDI, const FTransform& Transform, const float Thickness, const float Length, const float DepthBias, const bool bScreenSpace, uint8 Alpha)
	{
		const FVector Location = Transform.GetLocation();
		const FVector AxisX = Transform.GetUnitAxis(EAxis::X) * Length;
		const FVector AxisY = Transform.GetUnitAxis(EAxis::Y) * Length;
		const FVector AxisZ = Transform.GetUnitAxis(EAxis::Z) * Length;
		PDI->DrawTranslucentLine(Location, Location + AxisX, FColor::Red.WithAlpha(Alpha), SDPG_World, 1.0f, DepthBias, bScreenSpace);
		PDI->DrawTranslucentLine(Location, Location + AxisY, FColor::Green.WithAlpha(Alpha), SDPG_World, 1.0f, DepthBias, bScreenSpace);
		PDI->DrawTranslucentLine(Location, Location + AxisZ, FColor::Blue.WithAlpha(Alpha), SDPG_World, 1.0f, DepthBias, bScreenSpace);
	}

	FColor GetColorForAxis(EAxis::Type InAxis)
	{
		// Just draw all forward-axis versions as black for now.
		return FColor::Black;
	}

	void DrawFlatArrow(class FPrimitiveDrawInterface* PDI,const FVector& Base,const FVector& XAxis,const FVector& YAxis,FColor Color,float Length,int32 Width, const FMaterialRenderProxy* MaterialRenderProxy, uint8 DepthPriority, float Thickness = 0.0f)
	{
		float DistanceFromBaseToHead = Length/3.0f;
		float DistanceFromBaseToTip = DistanceFromBaseToHead*2.0f;
		float WidthOfBase = Width;
		float WidthOfHead = 2*Width;

		FVector ArrowPoints[7];
		//base points
		ArrowPoints[0] = Base - YAxis*(WidthOfBase*.5f);
		ArrowPoints[1] = Base + YAxis*(WidthOfBase*.5f);
		//inner head
		ArrowPoints[2] = ArrowPoints[0] + XAxis*DistanceFromBaseToHead;
		ArrowPoints[3] = ArrowPoints[1] + XAxis*DistanceFromBaseToHead;
		//outer head
		ArrowPoints[4] = ArrowPoints[2] - YAxis*(WidthOfBase*.5f);
		ArrowPoints[5] = ArrowPoints[3] + YAxis*(WidthOfBase*.5f);
		//tip
		ArrowPoints[6] = Base + XAxis*Length;

		//Draw lines
		{
			//base
			PDI->DrawTranslucentLine(ArrowPoints[0], ArrowPoints[1], Color, DepthPriority, Thickness);
			//base sides																 
			PDI->DrawTranslucentLine(ArrowPoints[0], ArrowPoints[2], Color, DepthPriority, Thickness);
			PDI->DrawTranslucentLine(ArrowPoints[1], ArrowPoints[3], Color, DepthPriority, Thickness);
			//head base																	 
			PDI->DrawTranslucentLine(ArrowPoints[2], ArrowPoints[4], Color, DepthPriority, Thickness);
			PDI->DrawTranslucentLine(ArrowPoints[3], ArrowPoints[5], Color, DepthPriority, Thickness);
			//head sides																 
			PDI->DrawTranslucentLine(ArrowPoints[4], ArrowPoints[6], Color, DepthPriority, Thickness);
			PDI->DrawTranslucentLine(ArrowPoints[5], ArrowPoints[6], Color, DepthPriority, Thickness);

		}

		if (MaterialRenderProxy != nullptr)
		{
			FDynamicMeshBuilder MeshBuilder(PDI->View->GetFeatureLevel());

			//Compute vertices for base circle.
			for(int32 i = 0; i< 7; ++i)
			{
				FDynamicMeshVertex MeshVertex;
				MeshVertex.Position = (FVector3f)ArrowPoints[i];
				MeshVertex.Color = Color;
				MeshVertex.TextureCoordinate[0] = FVector2f(0.0f, 0.0f);;
				MeshVertex.SetTangents(FVector3f(XAxis^YAxis), (FVector3f)YAxis, (FVector3f)XAxis);
				MeshBuilder.AddVertex(MeshVertex); //Add bottom vertex
			}

			//Add triangles / double sided
			{
				MeshBuilder.AddTriangle(0, 2, 1); //base
				MeshBuilder.AddTriangle(0, 1, 2); //base
				MeshBuilder.AddTriangle(1, 2, 3); //base
				MeshBuilder.AddTriangle(1, 3, 2); //base
				MeshBuilder.AddTriangle(4, 5, 6); //head
				MeshBuilder.AddTriangle(4, 6, 5); //head
			}

			MeshBuilder.Draw(PDI, FMatrix::Identity, MaterialRenderProxy, DepthPriority, 0.0f);
		}
	}
}

#define LOCTEXT_NAMESPACE "FAnimationViewportClient"

/////////////////////////////////////////////////////////////////////////
// FAnimationViewportClient

FAnimationViewportClient::FAnimationViewportClient(const TSharedRef<IPersonaPreviewScene>& InPreviewScene, const TSharedRef<SAnimationEditorViewport>& InAnimationEditorViewport, const TSharedRef<FAssetEditorToolkit>& InAssetEditorToolkit, int32 InViewportIndex, bool bInShowStats)
	: FEditorViewportClient(&InAssetEditorToolkit->GetEditorModeManager(), &InPreviewScene.Get(), StaticCastSharedRef<SEditorViewport>(InAnimationEditorViewport))
	, PreviewScenePtr(InPreviewScene)
	, AssetEditorToolkitPtr(InAssetEditorToolkit)
	, bRotateCameraToFollowBone(false)
	, bFocusOnDraw(false)
	, bFocusUsingCustomCamera(false)
	, CachedScreenSize(0.0f)
	, bShowMeshStats(bInShowStats)
	, bInitiallyFocused(false)
	, OrbitRotation(FQuat::Identity)
	, ViewportIndex(InViewportIndex)
	, LastLookAtLocation(FVector::ZeroVector)
	, bResumeAfterTracking(false)
{
	CachedDefaultCameraController = CameraController;

	OnCameraControllerChanged();

	Widget->SetUsesEditorModeTools(ModeTools.Get());
	((FAssetEditorModeManager*)ModeTools.Get())->SetPreviewScene(&InPreviewScene.Get());
	ModeTools->SetDefaultMode(FPersonaEditModes::SkeletonSelection);

	// Default to local space
	SetWidgetCoordSystemSpace(COORD_Local);

	// load config
	ConfigOption = UPersonaOptions::StaticClass()->GetDefaultObject<UPersonaOptions>();
	check (ConfigOption);

	// DrawHelper set up
	DrawHelper.PerspectiveGridSize = UE_OLD_HALF_WORLD_MAX1;
	DrawHelper.AxesLineThickness = ConfigOption->bHighlightOrigin ? 1.0f : 0.0f;
	DrawHelper.bDrawGrid = true;	// Toggling grid now relies on the show flag

	WidgetMode = UE::Widget::WM_Rotate;
	ModeTools->SetWidgetMode(WidgetMode);

	EngineShowFlags.Game = 0;
	EngineShowFlags.ScreenSpaceReflections = 1;
	EngineShowFlags.AmbientOcclusion = 1;
	EngineShowFlags.SetSnap(0);
	EngineShowFlags.Grid = ConfigOption->bShowGrid;

	SetRealtime(true);
	if(GEditor->PlayWorld)
	{
		const bool bShouldBeRealtime = false;
		AddRealtimeOverride(bShouldBeRealtime, LOCTEXT("RealtimeOverride_PIE", "Play in Editor"));
	}

	// @todo double define - fix it
	const float FOVMin = 5.f;
	const float FOVMax = 170.f;

	ViewFOV = FMath::Clamp<float>(ConfigOption->GetAssetEditorOptions(InAssetEditorToolkit->GetEditorName()).ViewportConfigs[ViewportIndex].ViewFOV, FOVMin, FOVMax);
	CameraSpeedSettings = ConfigOption->GetAssetEditorOptions(InAssetEditorToolkit->GetEditorName()).ViewportConfigs[ViewportIndex].CameraSpeedSettings;

	EngineShowFlags.SetSeparateTranslucency(true);
	EngineShowFlags.SetCompositeEditorPrimitives(true);

	EngineShowFlags.SetSelectionOutline(true);

	bDrawUVs = false;
	UVChannelToDraw = 0;

	bAutoAlignFloor = ConfigOption->bAutoAlignFloorToMesh;

	// Set audio mute option
	UWorld* World = PreviewScene->GetWorld();
	if(World)
	{
		World->bAllowAudioPlayback = !ConfigOption->bMuteAudio;

		UAudioEditorSettings* AudioConfigOption = GetMutableDefault<UAudioEditorSettings>();
		check(AudioConfigOption);
		AudioConfigOption->SetUseAudioAttenuation(true);
	}
	
	SetAudioFocus();
}

FAnimationViewportClient::~FAnimationViewportClient()
{
	CameraController = CachedDefaultCameraController;

	// Unregistering the callbacks is mandatory, else we get random crashes
	if (FAnimationEditorPreviewScene* AnimationEditorPreviewScene = GetAnimPreviewScenePtr().Get())
	{
		AnimationEditorPreviewScene->UnregisterOnPreviewMeshChanged(this);
		AnimationEditorPreviewScene->UnregisterOnInvalidateViews(this);
		AnimationEditorPreviewScene->UnregisterOnCameraOverrideChanged(this);
		AnimationEditorPreviewScene->UnregisterOnPreTick(this);
		AnimationEditorPreviewScene->UnregisterOnPostTick(this);

		if (UDebugSkelMeshComponent* PreviewMeshComponent = AnimationEditorPreviewScene->GetPreviewMeshComponent())
		{
			if (OnPhysicsCreatedDelegateHandle.IsValid())
			{
				PreviewMeshComponent->UnregisterOnPhysicsCreatedDelegate(OnPhysicsCreatedDelegateHandle);
			}

			if (OnMeshChangedDelegateHandle.IsValid())
			{
				if (USkeletalMesh* SkelMesh = PreviewMeshComponent->GetSkeletalMeshAsset())
				{
					SkelMesh->GetOnMeshChanged().Remove(OnMeshChangedDelegateHandle);
				}
			}
		}

		AnimationEditorPreviewScene->UnregisterOnSelectedBonesChanged(OnSelectedBonesChangedHandle);
	}
	OnPhysicsCreatedDelegateHandle.Reset();
	OnMeshChangedDelegateHandle.Reset();
}

void FAnimationViewportClient::Initialize()
{
	if (FAnimationEditorPreviewScene* AnimationEditorPreviewScene = GetAnimPreviewScenePtr().Get())
	{
		AnimationEditorPreviewScene->RegisterOnCameraOverrideChanged(FSimpleDelegate::CreateSP(this, &FAnimationViewportClient::OnCameraControllerChanged));
		AnimationEditorPreviewScene->RegisterOnPreviewMeshChanged(FOnPreviewMeshChanged::CreateSP(this, &FAnimationViewportClient::HandleSkeletalMeshChanged));
		if (AnimationEditorPreviewScene->GetPreviewMeshComponent())
		{
			HandleSkeletalMeshChanged(nullptr, AnimationEditorPreviewScene->GetPreviewMeshComponent()->GetSkeletalMeshAsset());
		}
		AnimationEditorPreviewScene->RegisterOnInvalidateViews(FSimpleDelegate::CreateSP(this, &FAnimationViewportClient::HandleInvalidateViews));
		AnimationEditorPreviewScene->RegisterOnFocusViews(FSimpleDelegate::CreateSP(this, &FAnimationViewportClient::HandleFocusViews));
		AnimationEditorPreviewScene->RegisterOnPreTick(FSimpleDelegate::CreateSP(this, &FAnimationViewportClient::HandlePreviewScenePreTick));
		AnimationEditorPreviewScene->RegisterOnPostTick(FSimpleDelegate::CreateSP(this, &FAnimationViewportClient::HandlePreviewScenePostTick));

		OnSelectedBonesChangedHandle = AnimationEditorPreviewScene->RegisterOnSelectedBonesChanged(FOnSelectedBonesChanged::CreateLambda([this](const TArray<FName>&, ESelectInfo::Type)
        {
        	UpdateBonesToDraw();
        }));
	}

	// Setup bones to draw on initialise
	UpdateBonesToDraw();
}

void FAnimationViewportClient::OnToggleAutoAlignFloor()
{
	bAutoAlignFloor = !bAutoAlignFloor;
	UpdateCameraSetup();

	ConfigOption->SetAutoAlignFloorToMesh(bAutoAlignFloor);
}

bool FAnimationViewportClient::IsAutoAlignFloor() const
{
	return bAutoAlignFloor;
}

void FAnimationViewportClient::OnToggleMuteAudio()
{
	UWorld* World = PreviewScene->GetWorld();

	if(World)
	{
		bool bNewAllowAudioPlayback = !World->AllowAudioPlayback();
		World->bAllowAudioPlayback = bNewAllowAudioPlayback;

		ConfigOption->SetMuteAudio(!bNewAllowAudioPlayback);
	}
}

bool FAnimationViewportClient::IsAudioMuted() const
{
	UWorld* World = PreviewScene->GetWorld();

	return (World) ? !World->AllowAudioPlayback() : false;
}

void FAnimationViewportClient::OnToggleUseAudioAttenuation()
{
	UAudioEditorSettings* AudioConfigOption = GetMutableDefault<UAudioEditorSettings>();
	check(AudioConfigOption);
	AudioConfigOption->SetUseAudioAttenuation(!AudioConfigOption->IsUsingAudioAttenuation());
}

bool FAnimationViewportClient::IsUsingAudioAttenuation() const
{
	const UAudioEditorSettings* AudioConfigOption = GetDefault<UAudioEditorSettings>();
	check(AudioConfigOption);
	return AudioConfigOption->IsUsingAudioAttenuation();
}

void FAnimationViewportClient::ToggleRotateCameraToFollowBone()
{
	bRotateCameraToFollowBone = !bRotateCameraToFollowBone;

	if (!bRotateCameraToFollowBone && GetCameraFollowMode() == EAnimationViewportCameraFollowMode::Bone)
	{
		OrbitRotation = FQuat::Identity;
	}
}

bool FAnimationViewportClient::GetShouldRotateCameraToFollowBone() const
{
	return bRotateCameraToFollowBone;
}

void FAnimationViewportClient::SetCameraFollowMode(EAnimationViewportCameraFollowMode InCameraFollowMode, FName InBoneName)
{
	bool bCanFollow = true;
	UDebugSkelMeshComponent* PreviewMeshComponent = GetAnimPreviewScene()->GetPreviewMeshComponent();
	if(InCameraFollowMode == EAnimationViewportCameraFollowMode::Bone && PreviewMeshComponent)
	{
		bCanFollow = PreviewMeshComponent->GetBoneIndex(InBoneName) != INDEX_NONE;
	}

	// When switching out of bone as camera we want to reset our coordinate system to something sensible
	if (InCameraFollowMode != EAnimationViewportCameraFollowMode::BoneAsCamera && CameraFollowMode == EAnimationViewportCameraFollowMode::BoneAsCamera)
	{
		EnableCameraLock(true);
	}

	if(bCanFollow && InCameraFollowMode != EAnimationViewportCameraFollowMode::None)
	{
		ConfigOption->SetViewCameraFollow(AssetEditorToolkitPtr.Pin()->GetEditorName(), InCameraFollowMode, InBoneName, ViewportIndex);

		CameraFollowMode = InCameraFollowMode;
		CameraFollowBoneName = InBoneName;

		bCameraLock = true;
		bUsingOrbitCamera = InCameraFollowMode != EAnimationViewportCameraFollowMode::BoneAsCamera;
		bDisableInput = InCameraFollowMode == EAnimationViewportCameraFollowMode::BoneAsCamera;

		if (PreviewMeshComponent != nullptr)
		{
			FVector LookAtLocation = LastLookAtLocation;

			switch (CameraFollowMode)
			{
			case EAnimationViewportCameraFollowMode::Bounds:
				{
					const FBoxSphereBounds Bounds = PreviewMeshComponent->CalcGameBounds(PreviewMeshComponent->GetComponentTransform());
					LookAtLocation = Bounds.Origin;
				}
				break;
			case EAnimationViewportCameraFollowMode::Root:
				{
					LookAtLocation = PreviewMeshComponent->GetBoneTransform(0).GetLocation();
					LookAtLocation.Z = PreviewMeshComponent->CalcGameBounds(PreviewMeshComponent->GetComponentTransform()).Origin.Z;
				}
				break;
			case EAnimationViewportCameraFollowMode::Bone:
				{
					LookAtLocation = PreviewMeshComponent->GetBoneLocation(InBoneName);
				}
				break;
				case EAnimationViewportCameraFollowMode::BoneAsCamera:
				{
					SetViewLocation(PreviewMeshComponent->GetBoneLocation(CameraFollowBoneName));
					SetViewRotation(PreviewMeshComponent->GetBoneQuaternion(CameraFollowBoneName).Rotator());
				}
				break;
			}
		
			OrbitRotation = FQuat::Identity;
			SetLookAtLocation(LookAtLocation, bUsingOrbitCamera);
			LastLookAtLocation = LookAtLocation;
		}

	}
	else
	{
		ConfigOption->SetViewCameraFollow(AssetEditorToolkitPtr.Pin()->GetEditorName(), EAnimationViewportCameraFollowMode::None, NAME_None, ViewportIndex);

		CameraFollowMode = EAnimationViewportCameraFollowMode::None;
		CameraFollowBoneName = NAME_None;

		OrbitRotation = FQuat::Identity;
		bDisableInput = false;
		EnableCameraLock(false);
		FocusViewportOnPreviewMesh(false);
		Invalidate();
	}
}

void FAnimationViewportClient::OnFocusViewportToSelection()
{
	// If focusing on a bone and using a Camera Follow Mode that orbits a bone, update the bone to follow to the selected bone
	if (CameraFollowMode == EAnimationViewportCameraFollowMode::Root
		|| CameraFollowMode == EAnimationViewportCameraFollowMode::Bone
		|| CameraFollowMode == EAnimationViewportCameraFollowMode::Bounds )
	{
		int32 SelectedBoneIndex = GetAnimPreviewScene()->GetSelectedBoneIndex();
		if (SelectedBoneIndex != INDEX_NONE)
		{
			const FReferenceSkeleton& ReferenceSkeleton = GetAnimPreviewScene()->GetPreviewMeshComponent()->GetReferenceSkeleton();
			const FName SelectedBoneName = ReferenceSkeleton.GetBoneName(SelectedBoneIndex);
			check(SelectedBoneName != NAME_None);
			bRotateCameraToFollowBone = false;
			SetCameraFollowMode(EAnimationViewportCameraFollowMode::Bone, SelectedBoneName);
		}
		else
		{
			SetCameraFollowMode(EAnimationViewportCameraFollowMode::Root, FName());
		}
	}
	else
	{
		SetCameraFollowMode(EAnimationViewportCameraFollowMode::None);
		FocusViewportOnPreviewMesh(false);
	}
	
}

EAnimationViewportCameraFollowMode FAnimationViewportClient::GetCameraFollowMode() const
{
	return CameraFollowMode;
}

FName FAnimationViewportClient::GetCameraFollowBoneName() const
{
	return CameraFollowBoneName;
}

EAxis::Type FAnimationViewportClient::GetBoneAsCameraAxis(EDirection Direction) const
{
	if (Direction == EDirection::Forward)
	{
		return BoneAsCameraForward;
	}
	if (Direction == EDirection::Right)
	{
		return BoneAsCameraRight;
	}
	return BoneAsCameraUp;
}

UE::Math::TVector<double> FAnimationViewportClient::GetBoneAsCameraPositionOffset() const
{
	return BoneAsCameraPositionOffset;
}

FRotator FAnimationViewportClient::GetBoneAsCameraRotationOffset() const
{
	return BoneAsCameraRotationOffset;
}

void FAnimationViewportClient::SetBoneAsCameraAxis(EDirection Direction, EAxis::Type Axis)
{
	if (Direction == EDirection::Forward)
	{
		BoneAsCameraForward = Axis;
	}
	else if (Direction == EDirection::Right)
	{
		BoneAsCameraRight = Axis;
	}
	else
	{
		BoneAsCameraUp = Axis;
	}
}

void FAnimationViewportClient::SetBoneAsCameraPositionOffset(UE::Math::TVector<double> Offset)
{
	BoneAsCameraPositionOffset = Offset;
}

void FAnimationViewportClient::SetBoneAsCameraRotationOffset(const FRotator& Offset)
{
	BoneAsCameraRotationOffset = Offset;
}

void FAnimationViewportClient::JumpToDefaultCamera()
{
	FocusViewportOnPreviewMesh(true);
}


void FAnimationViewportClient::SaveCameraAsDefault()
{
	USkeletalMesh* SkelMesh = GetAnimPreviewScene()->GetPreviewMeshComponent()->GetSkeletalMeshAsset();
	if (SkelMesh)
	{
		FScopedTransaction Transaction(LOCTEXT("SaveCameraAsDefault", "Save Camera As Default"));

		FViewportCameraTransform& ViewTransform = GetViewTransform();
		SkelMesh->Modify();
		SkelMesh->SetDefaultEditorCameraLocation(ViewTransform.GetLocation());
		SkelMesh->SetDefaultEditorCameraRotation(ViewTransform.GetRotation());
		SkelMesh->SetDefaultEditorCameraLookAt(ViewTransform.GetLookAt());
		SkelMesh->SetDefaultEditorCameraOrthoZoom(ViewTransform.GetOrthoZoom());
		SkelMesh->SetHasCustomDefaultEditorCamera(true);

		// Create and display a notification 
		const FText NotificationText = FText::Format(LOCTEXT("SavedDefaultCamera", "Saved default camera for {0}"), FText::AsCultureInvariant(SkelMesh->GetName()));
		FNotificationInfo Info(NotificationText);
		Info.ExpireDuration = 2.0f;
		FSlateNotificationManager::Get().AddNotification(Info);
	}
}

bool FAnimationViewportClient::CanSaveCameraAsDefault() const
{
	return CameraFollowMode == EAnimationViewportCameraFollowMode::None;
}

void FAnimationViewportClient::ClearDefaultCamera()
{
	USkeletalMesh* SkelMesh = GetAnimPreviewScene()->GetPreviewMeshComponent()->GetSkeletalMeshAsset();
	if (SkelMesh)
	{
		FScopedTransaction Transaction(LOCTEXT("ClearDefaultCamera", "Clear Default Camera"));

		SkelMesh->Modify();
		SkelMesh->SetHasCustomDefaultEditorCamera(false);

		// Create and display a notification 
		const FText NotificationText = FText::Format(LOCTEXT("ClearedDefaultCamera", "Cleared default camera for {0}"), FText::AsCultureInvariant(SkelMesh->GetName()));
		FNotificationInfo Info(NotificationText);
		Info.ExpireDuration = 2.0f;
		FSlateNotificationManager::Get().AddNotification(Info);
	}
}

bool FAnimationViewportClient::HasDefaultCameraSet() const
{
	USkeletalMesh* SkelMesh = GetAnimPreviewScene()->GetPreviewMeshComponent()->GetSkeletalMeshAsset();
	return (SkelMesh && SkelMesh->GetHasCustomDefaultEditorCamera());
}

static void DisableAllBodiesSimulatePhysics(UDebugSkelMeshComponent* PreviewMeshComponent)
{
	// Reset simulation state of body instances so we dont actually simulate after recreating the physics state
	for (int32 BodyIdx = 0; BodyIdx < PreviewMeshComponent->Bodies.Num(); ++BodyIdx)
	{
		if (FBodyInstance* BodyInst = PreviewMeshComponent->Bodies[BodyIdx])
		{
			BodyInst->SetInstanceSimulatePhysics(false);
		}
	}
}

void FAnimationViewportClient::HandleSkeletalMeshChanged(USkeletalMesh* OldSkeletalMesh, USkeletalMesh* NewSkeletalMesh)
{
	// Set up our notifications that the mesh we're watching has changed from some external source (like undo/redo)
	if (OldSkeletalMesh)
	{
		OldSkeletalMesh->GetOnMeshChanged().Remove(OnMeshChangedDelegateHandle);
	}

	if (NewSkeletalMesh)
	{
		OnMeshChangedDelegateHandle = NewSkeletalMesh->GetOnMeshChanged().AddSP(this, &FAnimationViewportClient::HandleOnMeshChanged);
	}

	if (OldSkeletalMesh != NewSkeletalMesh || NewSkeletalMesh == nullptr)
	{
		if (!bInitiallyFocused)
		{
			FocusViewportOnPreviewMesh(true);
			bInitiallyFocused = true;
		}

		UpdateCameraSetup();
	}

	// Setup physics data from physics assets if available, clearing any physics setup on the component
	UDebugSkelMeshComponent* PreviewMeshComponent = GetAnimPreviewScene()->GetPreviewMeshComponent();
	UPhysicsAsset* PhysAsset = PreviewMeshComponent->GetPhysicsAsset();
	if(PhysAsset)
	{
		PhysAsset->InvalidateAllPhysicsMeshes();

		if (OnPhysicsCreatedDelegateHandle.IsValid())
		{
			PreviewMeshComponent->UnregisterOnPhysicsCreatedDelegate(OnPhysicsCreatedDelegateHandle);
			OnPhysicsCreatedDelegateHandle.Reset();
		}
		// we need to make sure we monitor any change to the PhysicsState being recreated, as this can happen from path that is external to this class
		// (example: setting a property on a body that is type "simulated" will recreate the state from USkeletalBodySetup::PostEditChangeProperty and let the body simulating (UE-107308)
		OnPhysicsCreatedDelegateHandle = PreviewMeshComponent->RegisterOnPhysicsCreatedDelegate(FOnSkelMeshPhysicsCreated::CreateSP(this, &FAnimationViewportClient::HandleOnSkelMeshPhysicsCreated));

		PreviewMeshComponent->TermArticulated();
		PreviewMeshComponent->InitArticulated(GetWorld()->GetPhysicsScene());
		if (PreviewMeshComponent->CanOverrideCollisionProfile())
		{
			// Set to PhysicsActor to enable tracing regardless of project overrides
			static FName CollisionProfileName(TEXT("PhysicsActor"));
			PreviewMeshComponent->SetCollisionProfileName(CollisionProfileName);
		}
	}

	UpdateBonesToDraw();

	Invalidate();
}

void FAnimationViewportClient::HandleOnMeshChanged()
{
	UpdateCameraSetup();
	UpdateBonesToDraw();
	Invalidate();
}

void FAnimationViewportClient::HandleOnSkelMeshPhysicsCreated()
{
	UDebugSkelMeshComponent* PreviewMeshComponent = GetAnimPreviewScene()->GetPreviewMeshComponent();
	// let's make sure nothing is simulating and that all necessary state are in proper order
	PreviewMeshComponent->SetPhysicsBlendWeight(0.f);
	PreviewMeshComponent->SetSimulatePhysics(false);
	DisableAllBodiesSimulatePhysics(PreviewMeshComponent);
}

void FAnimationViewportClient::Draw(const FSceneView* View, FPrimitiveDrawInterface* PDI)
{
	TArray<UDebugSkelMeshComponent*> PreviewMeshComponents = GetPreviewScene()->GetAllPreviewMeshComponents();
	if (!UE::Private::CanDrawPreviewComponents(PreviewMeshComponents))
	{
		return;
	}

	UpdateBonesToDraw();

	FEditorViewportClient::Draw(View, PDI);

	// draw bones for all debug skeletal meshes
	for (UDebugSkelMeshComponent* PreviewMeshComponent : PreviewMeshComponents)
	{
		const bool bValidComponent = PreviewMeshComponent != nullptr;
		const bool bValidSkeletalMesh = bValidComponent && PreviewMeshComponent->GetSkeletalMeshAsset() != nullptr;

		if (bValidComponent && bValidSkeletalMesh && !PreviewMeshComponent->GetSkeletalMeshAsset()->IsCompiling())
		{
			// Can't have both bones of interest and sockets of interest set
			check( !(GetAnimPreviewScene()->GetSelectedBoneIndex() != INDEX_NONE && GetAnimPreviewScene()->GetSelectedSocket().IsValid() ) )

			const FReferenceSkeleton& RefSkeleton = PreviewMeshComponent->GetReferenceSkeleton();
			const TArray<FBoneIndexType>& DrawBoneIndices = PreviewMeshComponent->GetDrawBoneIndices();
			
			// draw the skeleton normally
			if ( GetBoneDrawMode() != EBoneDrawMode::None )
			{
				DrawMeshBones(PreviewMeshComponent, PDI);
			}

			// special draw modes for debugging various transforms...
			if (PreviewMeshComponent->bDisplayRawAnimation )
			{
				DrawMeshBonesUncompressedAnimation(PreviewMeshComponent, PDI);
			}
			if (PreviewMeshComponent->NonRetargetedSpaceBases.Num() > 0 )
			{
				DrawMeshBonesNonRetargetedAnimation(PreviewMeshComponent, PDI);
			}
			if(PreviewMeshComponent->bDisplayAdditiveBasePose )
			{
				DrawMeshBonesAdditiveBasePose(PreviewMeshComponent, PDI);
			}
			if(PreviewMeshComponent->bDisplayBakedAnimation)
			{
				DrawMeshBonesBakedAnimation(PreviewMeshComponent, PDI);
			}
			if(PreviewMeshComponent->bDisplaySourceAnimation)
			{
				DrawMeshBonesSourceRawAnimation(PreviewMeshComponent, PDI);
			}
			
			DrawWatchedPoses(PreviewMeshComponent, PDI);

			PreviewMeshComponent->DebugDrawClothing(PDI);
			
			// Display socket hit points
			if (PreviewMeshComponent->bDrawSockets )
			{
				if (PreviewMeshComponent->bSkeletonSocketsVisible && PreviewMeshComponent->GetSkeletalMeshAsset()->GetSkeleton() )
				{
					DrawSockets(PreviewMeshComponent, MutableView(PreviewMeshComponent->GetSkeletalMeshAsset()->GetSkeleton()->Sockets), GetAnimPreviewScene()->GetSelectedSocket(), PDI, true);
				}

				if ( PreviewMeshComponent->bMeshSocketsVisible )
				{
					DrawSockets(PreviewMeshComponent, MutableView(PreviewMeshComponent->GetSkeletalMeshAsset()->GetMeshOnlySocketList()), GetAnimPreviewScene()->GetSelectedSocket(), PDI, false);
				}
			}

			if (PreviewMeshComponent->bDrawAttributes)
			{
				DrawAttributes(PreviewMeshComponent, PDI);
			}

			DrawNotifies(PreviewMeshComponent, PDI);

			DrawRootMotionTrajectory(PreviewMeshComponent, PDI);

			DrawAssetUserData(PDI);
		}
		else if (bValidComponent && !bValidSkeletalMesh)
		{
			if (const USkeleton* Skeleton = GetPreviewScene()->GetPersonaToolkit()->GetSkeleton())
			{
				DrawBonesFromSkeleton(PreviewMeshComponent, Skeleton, PreviewMeshComponent->BonesOfInterest, PDI);
			}
		}
	}

	if (bFocusOnDraw)
	{
		bFocusOnDraw = false;
		FocusViewportOnPreviewMesh(bFocusUsingCustomCamera);
	}

	// set camera mode if need be (we need to do this here as focus on draw can take us out of orbit mode)
	FAssetEditorOptions& Options = ConfigOption->GetAssetEditorOptions(AssetEditorToolkitPtr.Pin()->GetEditorName());
	if(Options.ViewportConfigs[ViewportIndex].CameraFollowMode != CameraFollowMode)
	{
		SetCameraFollowMode(Options.ViewportConfigs[ViewportIndex].CameraFollowMode, Options.ViewportConfigs[ViewportIndex].CameraFollowBoneName);
	}
}

void FAnimationViewportClient::DrawCanvas( FViewport& InViewport, FSceneView& View, FCanvas& Canvas )
{
	TArray<UDebugSkelMeshComponent*> PreviewMeshComponents = GetPreviewScene()->GetAllPreviewMeshComponents();
	if (!UE::Private::CanDrawPreviewComponents(PreviewMeshComponents))
	{
		return;
	}

	FEditorViewportClient::DrawCanvas(InViewport, View, Canvas);

	UWorld* World = nullptr;
	
	for  (UDebugSkelMeshComponent* PreviewMeshComponent : PreviewMeshComponents)
	{
		if (!PreviewMeshComponent)
		{
			continue;
		}
		
		// Display bone names
		if (PreviewMeshComponent->bShowBoneNames)
		{
			ShowBoneNames(&Canvas, &View, PreviewMeshComponent);
		}

		// Display attribute names
		if (PreviewMeshComponent->bDrawAttributes)
		{
			ShowAttributeNames(&Canvas, &View, PreviewMeshComponent);
		}

		DrawCanvasNotifies(PreviewMeshComponent, Canvas, View);

		DrawCanvasAssetUserData(Canvas, View);
		
		if (bDrawUVs)
		{
			DrawUVsForMesh(Viewport, &Canvas, 1, PreviewMeshComponent);
		}

		// Debug draw clothing texts
		PreviewMeshComponent->DebugDrawClothingTexts(&Canvas, &View);

		if(World == nullptr)
		{
			World = PreviewMeshComponent->GetWorld();
		}
	}

#if !(UE_BUILD_TEST)
	if (World && GEngine->bEnableOnScreenDebugMessagesDisplay && GEngine->bEnableOnScreenDebugMessages && CVarShowEngineDebugMessageOnAnimViewport->GetBool())
	{
		constexpr int32 MessageX = 20;
		constexpr int32 MessageY = 65;
		GEngine->DrawOnscreenDebugMessages(World, Viewport, &Canvas, nullptr, MessageX, MessageY);
	}
#endif
}

void FAnimationViewportClient::DrawUVsForMesh(FViewport* InViewport, FCanvas* InCanvas, int32 InTextYPos, UDebugSkelMeshComponent* PreviewMeshComponent)
{
	if (!PreviewMeshComponent || !PreviewMeshComponent->GetSkeletalMeshAsset() || PreviewMeshComponent->GetSkeletalMeshAsset()->IsCompiling())
	{
		return;
	}
	
	//use the overridden LOD level
	const uint32 LODLevel = FMath::Clamp(PreviewMeshComponent->GetForcedLOD() - 1, 0, PreviewMeshComponent->GetSkeletalMeshAsset()->GetLODNum() - 1);

	TArray<FVector2D> SelectedEdgeTexCoords; //No functionality in Persona for this (yet?)

	DrawUVs(InViewport, InCanvas, InTextYPos, LODLevel, UVChannelToDraw, SelectedEdgeTexCoords, nullptr, &PreviewMeshComponent->GetSkeletalMeshRenderData()->LODRenderData[LODLevel] );
}

void FAnimationViewportClient::Tick(float DeltaSeconds) 
{
	//Avoid ticking the animation viewport if the skeletalmesh is compiling
	if (GetAnimPreviewScene()->GetPreviewMeshComponent()
		&& GetAnimPreviewScene()->GetPreviewMeshComponent()->GetSkeletalMeshAsset()
		&& GetAnimPreviewScene()->GetPreviewMeshComponent()->GetSkeletalMeshAsset()->IsCompiling())
	{
		return;
	}

	FEditorViewportClient::Tick(DeltaSeconds);

	GetAnimPreviewScene()->FlagTickable();

	TimecodeDisplay.Reset();
	if (GetAnimPreviewScene()->IsShowTimecode())
	{
		UAnimationAsset* AnimationAsset = GetAnimPreviewScene()->GetPreviewAnimationAsset();
		if (UAnimSequence* AnimSequence = Cast<UAnimSequence>(AnimationAsset))
		{
			FName BoneName = UAnimationBlueprintLibrary::FindBoneNameWithTimecodeAttributes(AnimSequence);

			FString SlateName;
			FQualifiedFrameTime QualifiedFrameTime;
			TOptional<float> PlayPosition = GetAnimPreviewScene()->GetCurrentTime();
			if (PlayPosition && UAnimationBlueprintLibrary::EvaluateBoneTimecodeAndSlateAttributesAtTime(BoneName, AnimSequence, *PlayPosition, QualifiedFrameTime, SlateName))
			{
				TimecodeDisplay = {QualifiedFrameTime, SlateName};
			}
		}
	}
}

void FAnimationViewportClient::HandlePreviewScenePreTick()
{
}

void FAnimationViewportClient::HandlePreviewScenePostTick()
{
	if (UDebugSkelMeshComponent* PreviewMeshComponent = GetAnimPreviewScene()->GetPreviewMeshComponent())
	{
		const bool bInBoneOrbitMode = CameraFollowMode != EAnimationViewportCameraFollowMode::None;
		if (IsTracking())
		{
			if (const UAnimSingleNodeInstance* SingleNodeInstance = PreviewMeshComponent->GetSingleNodeInstance())
			{
				if (SingleNodeInstance->IsPlaying() && GetDefault<UPersonaOptions>()->bPauseAnimationOnCameraMove)
				{
					PreviewMeshComponent->Stop();
					bResumeAfterTracking = true;
					return;
				}
			}
		}
		else if (bResumeAfterTracking)
		{
			PreviewMeshComponent->Play(true);
			bResumeAfterTracking = false;
		}

		if (CameraFollowMode != EAnimationViewportCameraFollowMode::None)
		{
			FVector LookAtLocation = LastLookAtLocation;

			switch(CameraFollowMode)
			{
			case EAnimationViewportCameraFollowMode::Bounds:
				{
					const FBoxSphereBounds Bounds = PreviewMeshComponent->CalcGameBounds(PreviewMeshComponent->GetComponentTransform());
					LookAtLocation = Bounds.Origin;
				}
				break;
			case EAnimationViewportCameraFollowMode::Root:
				{
					LookAtLocation = PreviewMeshComponent->GetBoneTransform(0).GetLocation();
					LookAtLocation.Z = PreviewMeshComponent->CalcGameBounds(PreviewMeshComponent->GetComponentTransform()).Origin.Z;
				}
				break;
			case EAnimationViewportCameraFollowMode::Bone:
				{
					const int32 BoneIndex = PreviewMeshComponent->GetBoneIndex(CameraFollowBoneName);
					if (BoneIndex != INDEX_NONE)
					{
						LookAtLocation = PreviewMeshComponent->GetBoneTransform(BoneIndex).GetLocation();

						if (GetShouldRotateCameraToFollowBone())
						{
							OrbitRotation = PreviewMeshComponent->GetBoneQuaternion(CameraFollowBoneName) * FQuat(FVector(0.0f, 1.0f, 0.0f), PI * 0.5f);
						}
					}
					else
					{
						SetCameraFollowMode(EAnimationViewportCameraFollowMode::None);
						return;
					}
				}
				break;
			case EAnimationViewportCameraFollowMode::BoneAsCamera:
				{
					const int32 BoneIndex = PreviewMeshComponent->GetBoneIndex(CameraFollowBoneName);
					if (BoneIndex != INDEX_NONE)
					{
						FTransform BoneTransform = PreviewMeshComponent->GetBoneTransform(CameraFollowBoneName);

						FViewportCameraTransform& ViewTransform = GetViewTransform();

						const FVector Forward = BoneTransform.GetUnitAxis(BoneAsCameraForward);
						const FVector Right = BoneTransform.GetUnitAxis(BoneAsCameraRight);
						const FVector Up = BoneTransform.GetUnitAxis(BoneAsCameraUp);
						const FVector Origin = BoneTransform.GetTranslation() + BoneAsCameraPositionOffset;
						FTransform NewViewTransform = FTransform(Forward, Right, Up, Origin);
						NewViewTransform.SetRotation(BoneAsCameraRotationOffset.Quaternion() * NewViewTransform.Rotator().Quaternion());

						ViewTransform.SetLocation(NewViewTransform.GetLocation());
						ViewTransform.SetRotation(NewViewTransform.GetRotation().Rotator());
					}
				}
				break;
			}

			const FVector Offset = LookAtLocation - LastLookAtLocation;
			SetLookAtLocation(GetLookAtLocation() + Offset);
			LastLookAtLocation = LookAtLocation;
		}
	}
}

void FAnimationViewportClient::SetCameraTargetLocation(const FSphere &BoundSphere, float DeltaSeconds)
{
	FVector OldViewLoc = GetViewLocation();
	FMatrix EpicMat = FTranslationMatrix(-GetViewLocation());
	EpicMat = EpicMat * FInverseRotationMatrix(GetViewRotation());
	FMatrix CamRotMat = EpicMat.InverseFast();
	FVector CamDir = FVector(CamRotMat.M[0][0],CamRotMat.M[0][1],CamRotMat.M[0][2]);
	FVector NewViewLocation = BoundSphere.Center - BoundSphere.W * 2 * CamDir;

	NewViewLocation.X = FMath::FInterpTo(OldViewLoc.X, NewViewLocation.X, (FVector::FReal)DeltaSeconds, FollowCamera_InterpSpeed);
	NewViewLocation.Y = FMath::FInterpTo(OldViewLoc.Y, NewViewLocation.Y, (FVector::FReal)DeltaSeconds, FollowCamera_InterpSpeed);
	NewViewLocation.Z = FMath::FInterpTo(OldViewLoc.Z, NewViewLocation.Z, (FVector::FReal)DeltaSeconds, FollowCamera_InterpSpeed_Z);

	SetViewLocation( NewViewLocation );
}

void FAnimationViewportClient::ShowBoneNames( FCanvas* Canvas, FSceneView* View, UDebugSkelMeshComponent* PreviewMeshComponent )
{
	if (!(PreviewMeshComponent && PreviewMeshComponent->MeshObject))
	{
		return;
	}

	//Most of the code taken from FASVViewportClient::Draw() in AnimSetViewerMain.cpp
	FSkeletalMeshRenderData* SkelMeshRenderData = PreviewMeshComponent->GetSkeletalMeshRenderData();
	check(SkelMeshRenderData);
	const int32 LODIndex = FMath::Clamp(PreviewMeshComponent->GetPredictedLODLevel(), 0, SkelMeshRenderData->LODRenderData.Num()-1);
	FSkeletalMeshLODRenderData& LODData = SkelMeshRenderData->LODRenderData[ LODIndex ];

	// Check if our reference skeleton is out of synch with the one on the loddata
	const FReferenceSkeleton& ReferenceSkeleton = PreviewMeshComponent->GetReferenceSkeleton();
	if (ReferenceSkeleton.GetNum() < LODData.RequiredBones.Num())
	{
		return;
	}

	const int32 HalfX = static_cast<int32>(Viewport->GetSizeXY().X / 2 / GetDPIScale());
	const int32 HalfY = static_cast<int32>(Viewport->GetSizeXY().Y / 2 / GetDPIScale());

	for (int32 i=0; i< LODData.RequiredBones.Num(); i++)
	{
		const int32 BoneIndex = LODData.RequiredBones[i];

		if (!BonesToDraw.IsValidIndex(BoneIndex) || !BonesToDraw[BoneIndex])
		{
			continue;
		}

		// Skip drawing bone name of selected bone, already drawn in SkeletonSelectionEditMode
		if (PreviewMeshComponent == GetAnimPreviewScene()->GetPreviewMeshComponent() && GetAnimPreviewScene()->GetSelectedBoneIndex() == BoneIndex)
		{
			continue;
		}

		// If previewing a specific section, only show the bone names that belong to it
		if ((PreviewMeshComponent->GetSectionPreview() >= 0) && !LODData.RenderSections[PreviewMeshComponent->GetSectionPreview()].BoneMap.Contains(BoneIndex))
		{
			continue;
		}
		if ((PreviewMeshComponent->GetMaterialPreview() >= 0))
		{
			TArray<int32> FoundSectionIndex;
			for (int32 SectionIndex = 0; SectionIndex < LODData.RenderSections.Num(); ++SectionIndex)
			{
				if (LODData.RenderSections[SectionIndex].MaterialIndex == PreviewMeshComponent->GetMaterialPreview())
				{
					FoundSectionIndex.Add(SectionIndex);
					break;
				}
			}
			if (FoundSectionIndex.Num() > 0)
			{
				bool PreviewSectionContainBoneIndex = false;
				for (int32 SectionIndex : FoundSectionIndex)
				{
					if (LODData.RenderSections[SectionIndex].BoneMap.Contains(BoneIndex))
					{
						PreviewSectionContainBoneIndex = true;
						break;
					}
				}
				if (!PreviewSectionContainBoneIndex)
				{
					continue;
				}
			}
		}

		const FColor BoneColor = FColor::White;
		if (BoneColor.A != 0)
		{
			const FVector BonePos = PreviewMeshComponent->GetComponentTransform().TransformPosition(PreviewMeshComponent->GetDrawTransform(BoneIndex).GetLocation());

			const FPlane proj = View->Project(BonePos);
			if (proj.W > 0.f)
			{
				const int32 XPos = static_cast<int32>(HalfX + ( HalfX * proj.X ));
				const int32 YPos = static_cast<int32>(HalfY + ( HalfY * (proj.Y * -1) ));

				const FName BoneName = ReferenceSkeleton.GetBoneName(BoneIndex);
				const FString BoneString = FString::Printf( TEXT("%d: %s"), BoneIndex, *BoneName.ToString() );
				FCanvasTextItem TextItem( FVector2D( XPos, YPos), FText::FromString( BoneString ), GEngine->GetSmallFont(), BoneColor );
				TextItem.EnableShadow(FLinearColor::Black);
				Canvas->DrawItem( TextItem );
			}
		}
	}
}

void FAnimationViewportClient::ShowAttributeNames(FCanvas* Canvas, FSceneView* View, UDebugSkelMeshComponent* MeshComponent) const
{
	if (!MeshComponent || !MeshComponent->GetSkeletalMeshAsset())
	{
		return;
	}
	
	const int32 HalfX = static_cast<int32>(Viewport->GetSizeXY().X / 2 / GetDPIScale());
	const int32 HalfY = static_cast<int32>(Viewport->GetSizeXY().Y / 2 / GetDPIScale());

	const UE::Anim::FMeshAttributeContainer& Attributes = MeshComponent->GetCustomAttributes();

	const int32 TransformAnimationAttributeTypeIndex = Attributes.FindTypeIndex(FTransformAnimationAttribute::StaticStruct());
	if (TransformAnimationAttributeTypeIndex != INDEX_NONE)
	{
		const TArray<UE::Anim::FAttributeId, FDefaultAllocator>& AttributeIdentifiers = Attributes.GetKeys(TransformAnimationAttributeTypeIndex);
		const TArray<UE::Anim::TWrappedAttribute<FDefaultAllocator>>& AttributeValues = Attributes.GetValues(TransformAnimationAttributeTypeIndex);
		check(AttributeIdentifiers.Num() == AttributeValues.Num());

		for (int32 AttributeIndex = 0; AttributeIndex < AttributeValues.Num(); ++AttributeIndex)
		{
			if (const FTransformAnimationAttribute* AttributeValue = AttributeValues[AttributeIndex].GetPtr<FTransformAnimationAttribute>())
			{
				const UE::Anim::FAttributeId& AttributeIdentifier = AttributeIdentifiers[AttributeIndex];

				const FTransform AttributeParentTransform = MeshComponent->GetDrawTransform(AttributeIdentifier.GetIndex()) * MeshComponent->GetComponentTransform();
				const FTransform AttributeTransform = AttributeValue->Value * AttributeParentTransform;

				const FPlane proj = View->Project(AttributeTransform.GetLocation());
				if (proj.W > 0.f)
				{
					const int32 XPos = HalfX + static_cast<int32>(HalfX * proj.X);
					const int32 YPos = HalfY + static_cast<int32>(HalfY * (proj.Y * -1));

					FCanvasTextItem TextItem(FVector2D(XPos, YPos), FText::FromName(AttributeIdentifier.GetName()), GEngine->GetSmallFont(), FLinearColor(0.0f, 1.0f, 1.0f));
					TextItem.EnableShadow(FLinearColor::Black);
					Canvas->DrawItem(TextItem);
				}
			}
		}
	}
}

bool FAnimationViewportClient::ShouldDisplayAdditiveScaleErrorMessage() const
{
	UAnimSequence* AnimSequence = Cast<UAnimSequence>(GetAnimPreviewScene()->GetPreviewAnimationAsset());
	if (AnimSequence)
	{
		if (AnimSequence->IsValidAdditive() && AnimSequence->RefPoseSeq)
		{
			FGuid AnimSeqGuid = AnimSequence->RefPoseSeq->GetDataModel()->GenerateGuid();
			if (RefPoseGuid != AnimSeqGuid)
			{
				RefPoseGuid = AnimSeqGuid;
				bDoesAdditiveRefPoseHaveZeroScale = AnimSequence->DoesSequenceContainZeroScale();
			}
			return bDoesAdditiveRefPoseHaveZeroScale;
		}
	}

	RefPoseGuid.Invalidate();
	return false;
}

extern FText ConcatenateLine(const FText& InText, const FText& InNewLine);

FText FAnimationViewportClient::GetDisplayInfo(bool bDisplayAllInfo) const
{
	FText TextValue;

	const UAssetViewerSettings* Settings = UAssetViewerSettings::Get();	
	const UEditorPerProjectUserSettings* PerProjectUserSettings = GetDefault<UEditorPerProjectUserSettings>();
	const int32 ProfileIndex = Settings->Profiles.IsValidIndex(PerProjectUserSettings->AssetViewerProfileIndex) ? PerProjectUserSettings->AssetViewerProfileIndex : 0;

	// if not valid skeletalmesh
	UDebugSkelMeshComponent* PreviewMeshComponent = GetPreviewScene()->GetPreviewMeshComponent();
	if (PreviewMeshComponent == nullptr || PreviewMeshComponent->GetSkeletalMeshAsset() == nullptr || PreviewMeshComponent->GetSkeletalMeshAsset()->IsCompiling())
	{
		return FText();
	}

	if (ShouldDisplayAdditiveScaleErrorMessage())
	{
		TextValue = ConcatenateLine(TextValue, LOCTEXT("AdditiveRefPoseWarning", "<AnimViewport.WarningText>Additive ref pose contains scales of 0.0, this can cause additive animations to not give the desired results</>"));
	}

	if (PreviewMeshComponent->GetSkeletalMeshAsset()->GetMorphTargets().Num() > 0)
	{
		TArray<UMaterial*> ProcessedMaterials;
		TArray<UMaterial*> MaterialsThatNeedMorphFlagOn;
		TArray<UMaterial*> MaterialsThatNeedSaving;

		const TIndirectArray<FSkeletalMeshLODModel>& LODModels = PreviewMeshComponent->GetSkeletalMeshAsset()->GetImportedModel()->LODModels;
		const TArray<FSkeletalMaterial>& SkeletalMeshMaterials = PreviewMeshComponent->GetSkeletalMeshAsset()->GetMaterials();
		int32 LodNumber = LODModels.Num();
		TArray<UMaterialInterface*> MaterialUsingMorphTarget;
		for (UMorphTarget *MorphTarget : PreviewMeshComponent->GetSkeletalMeshAsset()->GetMorphTargets())
		{
			if (MorphTarget == nullptr)
			{
				continue;
			}
			for (const FMorphTargetLODModel& MorphTargetLODModel : MorphTarget->GetMorphLODModels())
			{
				for (int32 SectionIndex : MorphTargetLODModel.SectionIndices)
				{
					for (int32 LodIdx = 0; LodIdx < LodNumber; LodIdx++)
					{
						const TArray<int32>& LODMaterialMap = PreviewMeshComponent->GetSkeletalMeshAsset()->GetLODInfo(LodIdx)->LODMaterialMap;
						const FSkeletalMeshLODModel& LODModel = LODModels[LodIdx];
						if (LODModel.Sections.IsValidIndex(SectionIndex))
						{
							int32 SectionMaterialIndex = LODModel.Sections[SectionIndex].MaterialIndex;
							if (LODMaterialMap.IsValidIndex(SectionIndex) && LODMaterialMap[SectionIndex] != INDEX_NONE)
							{
								SectionMaterialIndex = LODMaterialMap[SectionIndex];
							}
							if (SkeletalMeshMaterials.IsValidIndex(SectionMaterialIndex))
							{
								MaterialUsingMorphTarget.AddUnique(SkeletalMeshMaterials[SectionMaterialIndex].MaterialInterface);
							}
						}
					}
				}
			}
		}

		for (int i = 0; i < PreviewMeshComponent->GetNumMaterials(); ++i)
		{
			if (UMaterialInterface* MaterialInterface = PreviewMeshComponent->GetMaterial(i))
			{
				UMaterial* Material = MaterialInterface->GetMaterial();
				if ((Material != nullptr) && !ProcessedMaterials.Contains(Material))
				{
					ProcessedMaterials.Add(Material);
					if (MaterialUsingMorphTarget.Contains(MaterialInterface) && !Material->GetUsageByFlag(MATUSAGE_MorphTargets))
					{
						MaterialsThatNeedMorphFlagOn.Add(Material);
					}
					else if (Material->IsUsageFlagDirty(MATUSAGE_MorphTargets))
					{
						MaterialsThatNeedSaving.Add(Material);
					}
				}
			}
		}

		if (MaterialsThatNeedMorphFlagOn.Num() > 0)
		{
			TextValue = ConcatenateLine(TextValue, LOCTEXT("MorphSupportNeeded", "<AnimViewport.WarningText>The following materials need morph support ('Used with Morph Targets' in material editor):</>"));

			for(auto Iter = MaterialsThatNeedMorphFlagOn.CreateIterator(); Iter; ++Iter)
			{
				TextValue = ConcatenateLine(TextValue,FText::FromString((*Iter)->GetPathName()));
			}
		}

		if (MaterialsThatNeedSaving.Num() > 0)
		{
			TextValue = ConcatenateLine(TextValue, LOCTEXT("MaterialsNeedSaving", "<AnimViewport.WarningText>The following materials need saving to fully support morph targets:</>"));

			for(auto Iter = MaterialsThatNeedSaving.CreateIterator(); Iter; ++Iter)
			{
				TextValue = ConcatenateLine(TextValue, FText::FromString((*Iter)->GetPathName()));
			}
		}
	}

	UAnimPreviewInstance* PreviewInstance = PreviewMeshComponent->PreviewInstance;
	if( PreviewInstance )
	{
		// see if you have anim sequence that has transform curves
		if (UAnimSequence* Sequence = Cast<UAnimSequence>(PreviewInstance->GetCurrentAsset()))
		{
			if (Sequence->IsCompressedDataOutOfDate())
			{
				TextValue = ConcatenateLine(TextValue, LOCTEXT("ApplyToCompressedDataWarning", "<AnimViewport.WarningText>Animation is being edited. To apply to compressed data (and recalculate baked additives), click \"Apply\"</>"));
			}
		}

		UAnimCompositeBase* CompositeBase = Cast<UAnimCompositeBase>(PreviewInstance->GetCurrentAsset());
		if (CompositeBase && !CompositeBase->GetCommonTargetFrameRate().IsValid())
		{
			FString AssetString;
			TArray<UAnimationAsset*> Assets;
			if(CompositeBase->GetAllAnimationSequencesReferred(Assets, false))
			{
				for (const UAnimationAsset* AnimAsset : Assets)
				{
					if (const UAnimSequenceBase* AnimSequenceBase = Cast<UAnimSequenceBase>(AnimAsset))
					{
						AssetString.Append(FString::Printf(TEXT("\n\t<AnimViewport.WarningText>%s - %s</>"), *AnimSequenceBase->GetName(), *AnimSequenceBase->GetSamplingFrameRate().ToPrettyText().ToString()));
					}
				}
			}
			
			TextValue = ConcatenateLine(TextValue, FText::Format(LOCTEXT("IncompatibleFrameRatesCompositeWarning", "<AnimViewport.WarningText>{0} is composed of assets with incompatible framerates:</>{1}"), CompositeBase->GetClass()->GetDisplayNameText(), FText::FromString(AssetString)));
		}
	}

	if (PreviewMeshComponent->IsUsingInGameBounds())
	{
		if (!PreviewMeshComponent->CheckIfBoundsAreCorrrect())
		{
			if( PreviewMeshComponent->GetPhysicsAsset() == nullptr )
			{
				TextValue = ConcatenateLine(TextValue, LOCTEXT("NeedToSetupPhysicsAssetForAccurateBounds", "<AnimViewport.WarningText>You may need to setup Physics Asset to use more accurate bounds</>"));
			}
			else
			{
				TextValue = ConcatenateLine(TextValue, LOCTEXT("NeedToSetupBoundsInPhysicsAsset", "<AnimViewport.WarningText>You need to setup bounds in Physics Asset to include whole mesh</>"));
			}
		}
	}

	if (PreviewMeshComponent != nullptr && PreviewMeshComponent->MeshObject != nullptr)
	{
		if (bDisplayAllInfo)
		{
			FSkeletalMeshRenderData* SkelMeshResource = PreviewMeshComponent->GetSkeletalMeshRenderData();
			check(SkelMeshResource);

			// Draw stats about the mesh
			int32 NumBonesInUse;
			int32 NumBonesMappedToVerts;
			int32 NumSectionsInUse;

			const int32 LODIndex = FMath::Clamp(PreviewMeshComponent->GetPredictedLODLevel(), 0, SkelMeshResource->LODRenderData.Num() - 1);
			FSkeletalMeshLODRenderData& LODData = SkelMeshResource->LODRenderData[LODIndex];

			NumBonesInUse = LODData.RequiredBones.Num();
			NumBonesMappedToVerts = LODData.ActiveBoneIndices.Num();
			NumSectionsInUse = LODData.RenderSections.Num();

			// Calculate polys based on non clothing sections so we don't duplicate the counts.
			uint32 NumTotalTriangles = 0;
			int32 NumSections = LODData.RenderSections.Num();
			for(int32 SectionIndex = 0; SectionIndex < NumSections; SectionIndex++)
			{
				NumTotalTriangles += LODData.RenderSections[SectionIndex].NumTriangles;
			}

			TextValue = ConcatenateLine(TextValue, FText::Format(LOCTEXT("MeshInfoFormat", "LOD: {0}, Bones: {1} (Mapped to Vertices: {2}), Polys: {3}"),
				FText::AsNumber(LODIndex),
				FText::AsNumber(NumBonesInUse),
				FText::AsNumber(NumBonesMappedToVerts),
				FText::AsNumber(NumTotalTriangles)));

			TextValue = ConcatenateLine(TextValue, FText::Format(LOCTEXT("ScreenSizeFOVFormat", "Current Screen Size: {0}, FOV: {1}"), FText::AsNumber(CachedScreenSize), FText::AsNumber(ViewFOV)));
			for (int32 SectionIndex = 0; SectionIndex < LODData.RenderSections.Num(); SectionIndex++)
			{
				int32 SectionVerts = LODData.RenderSections[SectionIndex].GetNumVertices();

				FText SectionDisabledText = LODData.RenderSections[SectionIndex].bDisabled ? LOCTEXT("SectionIsDisbable", " Disabled") : FText::GetEmpty();
				TextValue = ConcatenateLine(TextValue, FText::Format(LOCTEXT("SectionFormat", " [Section {0}]{1} Verts: {2}, Bones: {3}, Max Influences: {4}"),
					FText::AsNumber(SectionIndex),
					SectionDisabledText,
					FText::AsNumber(SectionVerts),
					FText::AsNumber(LODData.RenderSections[SectionIndex].BoneMap.Num()),
					FText::AsNumber(LODData.RenderSections[SectionIndex].MaxBoneInfluences)
					));
			}

			TextValue = ConcatenateLine(TextValue, FText::Format(LOCTEXT("TotalVerts", "TOTAL Verts: {0}"),
				FText::AsNumber(LODData.GetNumVertices())));

			TextValue = ConcatenateLine(TextValue, FText::Format(LOCTEXT("Sections", "Sections: {0}"),
				NumSectionsInUse
				));

			TArray<FTransform> LocalBoneTransforms = PreviewMeshComponent->GetBoneSpaceTransforms();
			if (PreviewMeshComponent->BonesOfInterest.Num() > 0)
			{
				int32 BoneIndex = PreviewMeshComponent->BonesOfInterest[0];
				FTransform ReferenceTransform = PreviewMeshComponent->GetReferenceSkeleton().GetRefBonePose()[BoneIndex];
				FTransform LocalTransform = LocalBoneTransforms[BoneIndex];
				FTransform ComponentTransform = PreviewMeshComponent->GetDrawTransform(BoneIndex);

				auto GetDisplayTransform = [](const FTransform& InTransform) -> FText
				{
					FRotator R(InTransform.GetRotation());
					FVector T(InTransform.GetTranslation());
					FVector S(InTransform.GetScale3D());

					FString Output = FString::Printf(TEXT("Rotation: X(Roll) %f Y(Pitch)  %f Z(Yaw) %f\r\n"), R.Roll, R.Pitch, R.Yaw);
					Output += FString::Printf(TEXT("Translation: %f %f %f\r\n"), T.X, T.Y, T.Z);
					Output += FString::Printf(TEXT("Scale3D: %f %f %f\r\n"), S.X, S.Y, S.Z);

					return FText::FromString(Output);
				};

				TextValue = ConcatenateLine(TextValue, FText::Format(LOCTEXT("LocalTransform", "Local: {0}"), GetDisplayTransform(LocalTransform)));

				TextValue = ConcatenateLine(TextValue, FText::Format(LOCTEXT("ComponentTransform", "Component: {0}"), GetDisplayTransform(ComponentTransform)));

				TextValue = ConcatenateLine(TextValue, FText::Format(LOCTEXT("ReferenceTransform", "Reference: {0}"), GetDisplayTransform(ReferenceTransform)));
			}

			TextValue = ConcatenateLine(TextValue, FText::Format(LOCTEXT("ApproximateSize", "Approximate Size: {0}x{1}x{2}"),
				FText::AsNumber(FMath::RoundToInt(PreviewMeshComponent->Bounds.BoxExtent.X * 2.0f)),
				FText::AsNumber(FMath::RoundToInt(PreviewMeshComponent->Bounds.BoxExtent.Y * 2.0f)),
				FText::AsNumber(FMath::RoundToInt(PreviewMeshComponent->Bounds.BoxExtent.Z * 2.0f))));

			uint32 NumNotiesWithErrors = PreviewMeshComponent->AnimNotifyErrors.Num();
			for (uint32 i = 0; i < NumNotiesWithErrors; ++i)
			{
				uint32 NumErrors = PreviewMeshComponent->AnimNotifyErrors[i].Errors.Num();
				for (uint32 ErrorIdx = 0; ErrorIdx < NumErrors; ++ErrorIdx)
				{
					TextValue = ConcatenateLine(TextValue, FText::FromString(PreviewMeshComponent->AnimNotifyErrors[i].Errors[ErrorIdx]));
				}
			}
		}
		else // simplified default display info to be same as static mesh editor
		{
			FSkeletalMeshRenderData* SkelMeshResource = PreviewMeshComponent->GetSkeletalMeshRenderData();
			check(SkelMeshResource);

			const int32 LODIndex = FMath::Clamp(PreviewMeshComponent->GetPredictedLODLevel(), 0, SkelMeshResource->LODRenderData.Num() - 1);
			FSkeletalMeshLODRenderData& LODData = SkelMeshResource->LODRenderData[LODIndex];

			// Current LOD 
			TextValue = ConcatenateLine(TextValue, FText::Format(LOCTEXT("LODFormat", "LOD: {0}"), FText::AsNumber(LODIndex)));

			// current screen size
			TextValue = ConcatenateLine(TextValue, FText::Format(LOCTEXT("ScreenSizeFormat", "Current Screen Size: {0}"), FText::AsNumber(CachedScreenSize)));

			// Triangles
			uint32 NumTotalTriangles = 0;
			int32 NumSections = LODData.RenderSections.Num();
			for (int32 SectionIndex = 0; SectionIndex < NumSections; SectionIndex++)
			{
				NumTotalTriangles += LODData.RenderSections[SectionIndex].NumTriangles;
			}

			TextValue = ConcatenateLine(TextValue, FText::Format(LOCTEXT("TrianglesFormat", "Triangles: {0}"), FText::AsNumber(NumTotalTriangles)));

			// Vertices
			TextValue = ConcatenateLine(TextValue, FText::Format(LOCTEXT("VerticesFormat", "Vertices: {0}"), FText::AsNumber(LODData.GetNumVertices())));

			// UV Channels
			TextValue = ConcatenateLine(TextValue, FText::Format(LOCTEXT("UVChannelsFormat", "UV Channels: {0}"), FText::AsNumber(LODData.GetNumTexCoords())));
			
			// Approx Size 
			TextValue = ConcatenateLine(TextValue, FText::Format(LOCTEXT("ApproxSize", "Approx Size: {0}x{1}x{2}"),
				FText::AsNumber(FMath::RoundToInt(PreviewMeshComponent->Bounds.BoxExtent.X * 2.0f)),
				FText::AsNumber(FMath::RoundToInt(PreviewMeshComponent->Bounds.BoxExtent.Y * 2.0f)),
				FText::AsNumber(FMath::RoundToInt(PreviewMeshComponent->Bounds.BoxExtent.Z * 2.0f))));
		}

		// In case a skin weight profile is currently being previewed show the number of override skin weights it stores
		if (PreviewMeshComponent->IsUsingSkinWeightProfile())
		{
			const FSkeletalMeshRenderData* SkelMeshResource = PreviewMeshComponent->GetSkeletalMeshRenderData();
			check(SkelMeshResource);
			
			const int32 LODIndex = FMath::Clamp(PreviewMeshComponent->GetPredictedLODLevel(), 0, SkelMeshResource->LODRenderData.Num() - 1);
			const FSkeletalMeshLODRenderData& LODData = SkelMeshResource->LODRenderData[LODIndex];
			
			const FName ProfileName = PreviewMeshComponent->GetCurrentSkinWeightProfileName();
			const FRuntimeSkinWeightProfileData* OverrideData = LODData.SkinWeightProfilesData.GetOverrideData(ProfileName);
			TextValue = ConcatenateLine(TextValue, FText::Format(LOCTEXT("NumSkinWeightOverrides", "Skin Weight Profile Weights: {0}"),	(OverrideData && OverrideData->NumWeightsPerVertex > 0) ? FText::AsNumber(OverrideData->BoneWeights.Num() / OverrideData->NumWeightsPerVertex) : LOCTEXT("NoSkinWeightsOverridesForLOD", "no data for LOD")));
		}
		
		const bool bMirroring = PreviewMeshComponent->PreviewInstance && PreviewMeshComponent->PreviewInstance->GetMirrorDataTable();
		if (bMirroring)
		{
			TextValue = ConcatenateLine(TextValue,FText::Format(LOCTEXT("Preview_mirrored", "Mirrored with {0} "), FText::FromString(PreviewMeshComponent->PreviewInstance->GetMirrorDataTable()->GetName())));
		}
	}

	const TArray<FClothingSimulationInstance>& ClothingSimulationInstances = PreviewMeshComponent->GetClothingSimulationInstances();
	for (int32 Index = 0; Index < ClothingSimulationInstances.Num(); ++Index)
	{
		const FClothingSimulationInstance& ClothingSimulationInstance = ClothingSimulationInstances[Index];

		if (const IClothingSimulationInterface* const ClothingSimulation = ClothingSimulationInstance.GetClothingSimulation())
		{
			// Cloth stats
			if (ClothingSimulationInstances.Num() > 1)
			{
				TextValue = ConcatenateLine(TextValue, FText::Format(LOCTEXT("SimulationInstance", "--- Simulation #{0} ---"), Index));
			}
			if (const int32 NumActiveCloths = ClothingSimulation->GetNumCloths())
			{
				TextValue = ConcatenateLine(TextValue, FText::Format(LOCTEXT("NumActiveCloths", "Active Cloths: {0}"), NumActiveCloths));
			}
			if (const int32 NumKinematicParticles = ClothingSimulation->GetNumKinematicParticles())
			{
				TextValue = ConcatenateLine(TextValue, FText::Format(LOCTEXT("NumKinematicParticles", "Kinematic Particles: {0}"), NumKinematicParticles));
			}
			if (const int32 NumDynamicParticles = ClothingSimulation->GetNumDynamicParticles())
			{
				TextValue = ConcatenateLine(TextValue, FText::Format(LOCTEXT("NumDynamicParticles", "Dynamic Particles : {0}"), NumDynamicParticles));
			}
			if (const int32 NumIterations = ClothingSimulation->GetNumIterations())
			{
				TextValue = ConcatenateLine(TextValue, FText::Format(LOCTEXT("NumIterations", "Iterations: {0}"), NumIterations));
			}
			if (const float SimulationTime = ClothingSimulation->GetSimulationTime())
			{
				FNumberFormattingOptions NumberFormatOptions;
				NumberFormatOptions.AlwaysSign = false;
				NumberFormatOptions.UseGrouping = false;
				NumberFormatOptions.RoundingMode = ERoundingMode::HalfFromZero;
				NumberFormatOptions.MinimumIntegralDigits = 1;
				NumberFormatOptions.MaximumIntegralDigits = 6;
				NumberFormatOptions.MinimumFractionalDigits = 2;
				NumberFormatOptions.MaximumFractionalDigits = 2;
				TextValue = ConcatenateLine(TextValue, FText::Format(LOCTEXT("SimulationTime", "Simulation Time: {0}ms"), FText::AsNumber(SimulationTime, &NumberFormatOptions)));
			}
			if (ClothingSimulation->IsTeleported())
			{
				TextValue = ConcatenateLine(TextValue, LOCTEXT("IsTeleported", "Simulation Teleport Activated"));
			}
		}
	}

	if (PreviewMeshComponent->GetSectionPreview() != INDEX_NONE)
	{
		// Notify the user if they are isolating a mesh section.
		TextValue = ConcatenateLine(TextValue, LOCTEXT("MeshSectionsHiddenWarning", "Mesh Sections Hidden"));
	}
	if (PreviewMeshComponent->GetMaterialPreview() != INDEX_NONE)
	{
		// Notify the user if they are isolating a mesh section.
		TextValue = ConcatenateLine(TextValue, LOCTEXT("MeshMaterialHiddenWarning", "Mesh Materials Hidden"));
	}

	if (const UAnimSequenceBase* AnimSequenceBase = Cast<UAnimSequenceBase>(GetAnimPreviewScene()->GetPreviewAnimationAsset()))
	{
		TextValue = ConcatenateLine(TextValue, FText::Format(LOCTEXT("FramerateFormat", "Framerate: {0}"), AnimSequenceBase->GetSamplingFrameRate().ToPrettyText()));
	}

	if (const UPoseAsset* PoseAsset = Cast<UPoseAsset>(GetAnimPreviewScene()->GetPreviewAnimationAsset()))
	{
		if (PoseAsset->GetLinkerCustomVersion(FUE5MainStreamObjectVersion::GUID) >= FUE5MainStreamObjectVersion::PoseAssetRawDataGUIDUpdate && PoseAsset->SourceAnimation && PoseAsset->SourceAnimationRawDataGUID.IsValid() && PoseAsset->GetSourceAnimationGuid() != PoseAsset->SourceAnimationRawDataGUID)
		{
			TextValue = ConcatenateLine(TextValue, LOCTEXT("PoseAssetOutOfDateWarning", "<AnimViewport.WarningText>Poses are out-of-sync with the source animation. To update them click \"Update Source\"</>"));
		}
	}

	if (TimecodeDisplay)
	{
		TextValue = ConcatenateLine(TextValue, FText::Format(LOCTEXT("TimecodeInfo", "Timecode: {0}"), FText::FromString(TimecodeDisplay->QualifiedTime.ToTimecode().ToString())));
		TextValue = ConcatenateLine(TextValue, FText::Format(LOCTEXT("SlateName", "Slate: {0}"), FText::FromString(TimecodeDisplay->Slate)));
		TextValue = ConcatenateLine(TextValue, FText::Format(LOCTEXT("Rate", "Rate: {0}"), FText::AsNumber(TimecodeDisplay->QualifiedTime.Rate.AsDecimal())));
	}
	return TextValue;
}
void FAnimationViewportClient::DrawNodeDebugLines(TArray<FText>& Lines, FCanvas* Canvas, FSceneView* View)
{
	if(Lines.Num() > 0)
	{
		int32 CurrentXOffset = 5;
		int32 CurrentYOffset = 60;

		int32 CharWidth;
		int32 CharHeight;
		StringSize(GEngine->GetSmallFont(), CharWidth, CharHeight, TEXT("0"));

		const int32 LineHeight = CharHeight + 2;

		for(FText& Line : Lines)
		{
			FCanvasTextItem TextItem(FVector2D(CurrentXOffset, CurrentYOffset), Line, GEngine->GetSmallFont(), FLinearColor::White);
			TextItem.EnableShadow(FLinearColor::Black);

			Canvas->DrawItem(TextItem);

			CurrentYOffset += LineHeight;
		}
	}
}

void FAnimationViewportClient::TrackingStarted( const struct FInputEventState& InInputState, bool bIsDraggingWidget, bool bNudge )
{
	ModeTools->StartTracking(this, Viewport);
}

void FAnimationViewportClient::TrackingStopped() 
{
	ModeTools->EndTracking(this, Viewport);

	Invalidate();
}

FVector FAnimationViewportClient::GetWidgetLocation() const
{
	return ModeTools->GetWidgetLocation();
}

FMatrix FAnimationViewportClient::GetWidgetCoordSystem() const
{
	const ECoordSystem Space = GetWidgetCoordSystemSpace();
	
	const bool bComputeOrientation = Space == COORD_Local || Space == COORD_Parent || Space == COORD_Explicit;
	if (bComputeOrientation)
	{
		return ModeTools->GetCustomInputCoordinateSystem();
	}

	return FMatrix::Identity;
}

ECoordSystem FAnimationViewportClient::GetWidgetCoordSystemSpace() const
{ 
	return ModeTools->GetCoordSystem();
}

void FAnimationViewportClient::SetWidgetCoordSystemSpace(ECoordSystem NewCoordSystem)
{
	ModeTools->SetCoordSystem(NewCoordSystem);
	Invalidate();
}

void FAnimationViewportClient::SetViewMode(EViewModeIndex InViewModeIndex)
{
	FEditorViewportClient::SetViewMode(InViewModeIndex);

	ConfigOption->SetViewModeIndex(AssetEditorToolkitPtr.Pin()->GetEditorName(), InViewModeIndex, ViewportIndex);
}

void FAnimationViewportClient::SetViewportType(ELevelViewportType InViewportType)
{
	FEditorViewportClient::SetViewportType(InViewportType);
	FocusViewportOnPreviewMesh(true);

	if(CameraFollowMode != EAnimationViewportCameraFollowMode::None)
	{
		bUsingOrbitCamera = true;
	}

	if (InViewportType != ELevelViewportType::LVT_Perspective)
	{
		SetCameraFollowMode(EAnimationViewportCameraFollowMode::None);
	}
}

void FAnimationViewportClient::RotateViewportType()
{
	FEditorViewportClient::RotateViewportType();
	FocusViewportOnPreviewMesh(true);

	if(CameraFollowMode != EAnimationViewportCameraFollowMode::None)
	{
		bUsingOrbitCamera = true;
	}
}
bool FAnimationViewportClient::InputKey(const FInputKeyEventArgs& EventArgs)
{
	bool bHandled = false;

	FAdvancedPreviewScene* AdvancedScene = static_cast<FAdvancedPreviewScene*>(PreviewScene);
	bHandled |= AdvancedScene->HandleInputKey(EventArgs);

	// Pass keys to standard controls, if we didn't consume input
	return (bHandled)
		? true
		: FEditorViewportClient::InputKey(EventArgs);
}

bool FAnimationViewportClient::InputAxis(const FInputKeyEventArgs& EventArgs)
{
	bool bResult = true;

	if (!bDisableInput)
	{
		FAdvancedPreviewScene* AdvancedScene = (FAdvancedPreviewScene*)PreviewScene;
		bResult = AdvancedScene->HandleViewportInput(EventArgs.Viewport, EventArgs.InputDevice, EventArgs.Key, EventArgs.AmountDepressed, EventArgs.DeltaTime, EventArgs.NumSamples, EventArgs.IsGamepad());
		if (bResult)
		{
			Invalidate();
		}
		else
		{
			bResult = FEditorViewportClient::InputAxis(EventArgs);
		}
	}

	return bResult;
}

void FAnimationViewportClient::ReceivedFocus(FViewport* InViewport)
{
	FEditorViewportClient::ReceivedFocus(InViewport);
	
	UpdateAudioListener();
	SetAudioFocus();
}

void FAnimationViewportClient::LostFocus(FViewport* InViewport)
{
	ClearAudioFocus();
	FEditorViewportClient::LostFocus(InViewport);
}

void FAnimationViewportClient::SetLocalAxesMode(ELocalAxesMode::Type AxesMode)
{
	ConfigOption->SetDefaultLocalAxesSelection( AxesMode );
}

bool FAnimationViewportClient::IsLocalAxesModeSet( ELocalAxesMode::Type AxesMode ) const
{
	return (ELocalAxesMode::Type)ConfigOption->DefaultLocalAxesSelection == AxesMode;
}

ELocalAxesMode::Type FAnimationViewportClient::GetLocalAxesMode() const
{ 
	return (ELocalAxesMode::Type)ConfigOption->DefaultLocalAxesSelection;
}

void FAnimationViewportClient::SetCameraSpeedSettings(const FEditorViewportCameraSpeedSettings& InCameraSpeedSettings)
{
	FEditorViewportClient::SetCameraSpeedSettings(InCameraSpeedSettings);
	ConfigOption->SetCameraSpeedSettings(GetAssetEditorToolkit()->GetEditorName(), GetCameraSpeedSettings(), ViewportIndex);
}

void FAnimationViewportClient::SetBoneDrawSize(const float InBoneDrawSize)
{
	BoneDrawSize = FMath::Max(0.001f,InBoneDrawSize);

	// optionally inform editors that may want to maintain bone size between sessions
	if (OnSetBoneSize.IsBound())
	{
		OnSetBoneSize.Execute(BoneDrawSize);
	}
	
	RedrawRequested(nullptr);
}

float FAnimationViewportClient::GetBoneDrawSize() const
{
	// optionally get bone size from editors that may be storing this between sessions
	if (OnGetBoneSize.IsBound())
	{
		return OnGetBoneSize.Execute();
	}
	
	return BoneDrawSize;
}

void FAnimationViewportClient::SetBoneDrawMode(EBoneDrawMode::Type AxesMode)
{
	ConfigOption->SetDefaultBoneDrawSelection(AxesMode);
	RedrawRequested(Viewport);

	UpdateBonesToDraw();
}

bool FAnimationViewportClient::IsBoneDrawModeSet(EBoneDrawMode::Type AxesMode) const
{
	return (EBoneDrawMode::Type)ConfigOption->DefaultBoneDrawSelection == AxesMode;
}

EBoneDrawMode::Type FAnimationViewportClient::GetBoneDrawMode() const 
{ 
	return (EBoneDrawMode::Type)ConfigOption->DefaultBoneDrawSelection;
}

void FAnimationViewportClient::DrawBonesFromTransforms(
	TArray<FTransform>& Transforms,
	UDebugSkelMeshComponent * MeshComponent,
	FPrimitiveDrawInterface* PDI,
	FLinearColor BoneColour,
	FLinearColor RootBoneColour) const
{
	if (Transforms.IsEmpty() ||
		!MeshComponent->GetSkeletalMeshAsset() ||
		MeshComponent->SkeletonDrawMode == ESkeletonDrawMode::Hidden)
	{
		return;
	}

	if (MeshComponent->SkeletonDrawMode == ESkeletonDrawMode::GreyedOut)
	{
		BoneColour = GetDefault<UPersonaOptions>()->DisabledBoneColor;
	}
	
	TArray<FTransform> WorldTransforms;
	WorldTransforms.AddUninitialized(Transforms.Num());

	TArray<FLinearColor> BoneColours;
	BoneColours.AddUninitialized(Transforms.Num());

	// we could cache parent bones as we calculate, but right now I'm not worried about perf issue of this
	const TArray<FBoneIndexType>& DrawBoneIndices = MeshComponent->GetDrawBoneIndices();
	for ( int32 Index=0; Index < MeshComponent->GetDrawBoneIndices().Num(); ++Index )
	{
		const int32 BoneIndex = DrawBoneIndices[Index];
		const int32 ParentIndex = MeshComponent->GetReferenceSkeleton().GetParentIndex(BoneIndex);

		WorldTransforms[BoneIndex] = Transforms[BoneIndex] * MeshComponent->GetComponentTransform();
		BoneColours[BoneIndex] = (ParentIndex >= 0) ? BoneColour : RootBoneColour;
	}

	constexpr bool bForceDraw = false;
	const bool bAddHitProxy = MeshComponent->SkeletonDrawMode != ESkeletonDrawMode::GreyedOut;
	const bool bUseMuliColors = GetDefault<UPersonaOptions>()->bShowBoneColors;

	DrawBones(
		MeshComponent->GetComponentLocation(),
		DrawBoneIndices,
		MeshComponent->GetReferenceSkeleton(),
		WorldTransforms,
		MeshComponent->BonesOfInterest,
		BoneColours,
		PDI,
		bForceDraw,
		bAddHitProxy,
		bUseMuliColors);
}

void FAnimationViewportClient::DrawBonesFromCompactPose(
	const FCompactHeapPose& Pose,
	UDebugSkelMeshComponent* MeshComponent,
	FPrimitiveDrawInterface* PDI,
	const FLinearColor& DrawColor) const
{
	if (Pose.GetNumBones() == 0 ||
		!MeshComponent ||
		!MeshComponent->GetSkeletalMeshAsset() ||
		MeshComponent->SkeletonDrawMode == ESkeletonDrawMode::Hidden)
	{
		return;
	}
	
	TArray<FTransform> WorldTransforms;
	WorldTransforms.AddUninitialized(Pose.GetBoneContainer().GetNumBones());

	// we could cache parent bones as we calculate, but right now I'm not worried about perf issue of this
	for (FCompactPoseBoneIndex BoneIndex : Pose.ForEachBoneIndex())
	{
		FMeshPoseBoneIndex MeshBoneIndex = Pose.GetBoneContainer().MakeMeshPoseIndex(BoneIndex);

		int32 ParentIndex = Pose.GetBoneContainer().GetParentBoneIndex(MeshBoneIndex.GetInt());

		if (ParentIndex == INDEX_NONE)
		{
			WorldTransforms[MeshBoneIndex.GetInt()] = Pose[BoneIndex] * MeshComponent->GetComponentTransform();
		}
		else
		{
			WorldTransforms[MeshBoneIndex.GetInt()] = Pose[BoneIndex] * WorldTransforms[ParentIndex];
		}
	}

	constexpr bool bForceDraw = true;
	const bool bAddHitProxy = MeshComponent->SkeletonDrawMode != ESkeletonDrawMode::GreyedOut;
	const bool bUseMultiColor = GetDefault<UPersonaOptions>()->bShowBoneColors;

	DrawBones(
		MeshComponent->GetComponentLocation(),
		MeshComponent->GetDrawBoneIndices(),
		MeshComponent->GetReferenceSkeleton(),
		WorldTransforms,
		MeshComponent->BonesOfInterest,
		TArray<FLinearColor>(),
		PDI,
		bForceDraw,
		bAddHitProxy,
		bUseMultiColor);
}

void FAnimationViewportClient::DrawMeshBonesUncompressedAnimation(UDebugSkelMeshComponent * MeshComponent, FPrimitiveDrawInterface* PDI) const
{
	if ( MeshComponent && MeshComponent->GetSkeletalMeshAsset())
	{
		DrawBonesFromTransforms(MeshComponent->UncompressedSpaceBases, MeshComponent, PDI, FColor(255, 127, 39, 255), FColor(255, 127, 39, 255));
	}
}

void FAnimationViewportClient::DrawMeshBonesNonRetargetedAnimation(UDebugSkelMeshComponent * MeshComponent, FPrimitiveDrawInterface* PDI) const
{
	if ( MeshComponent && MeshComponent->GetSkeletalMeshAsset())
	{
		DrawBonesFromTransforms(MeshComponent->NonRetargetedSpaceBases, MeshComponent, PDI, FColor(159, 159, 39, 255), FColor(159, 159, 39, 255));
	}
}

void FAnimationViewportClient::DrawMeshBonesAdditiveBasePose(UDebugSkelMeshComponent * MeshComponent, FPrimitiveDrawInterface* PDI) const
{
	if ( MeshComponent && MeshComponent->GetSkeletalMeshAsset())
	{
		DrawBonesFromTransforms(MeshComponent->AdditiveBasePoses, MeshComponent, PDI, FColor(0, 159, 0, 255), FColor(0, 159, 0, 255));
	}
}

void FAnimationViewportClient::DrawMeshBonesSourceRawAnimation(UDebugSkelMeshComponent * MeshComponent, FPrimitiveDrawInterface* PDI) const
{
	if(MeshComponent && MeshComponent->GetSkeletalMeshAsset())
	{
		DrawBonesFromTransforms(MeshComponent->SourceAnimationPoses, MeshComponent, PDI, FColor(195, 195, 195, 255), FColor(195, 159, 195, 255));
	}
}

void FAnimationViewportClient::DrawWatchedPoses(UDebugSkelMeshComponent * MeshComponent, FPrimitiveDrawInterface* PDI)
{
	if (UAnimBlueprintGeneratedClass* AnimBPGenClass = Cast<UAnimBlueprintGeneratedClass>(MeshComponent->AnimClass))
	{
		if (UAnimBlueprint* Blueprint = Cast<UAnimBlueprint>(AnimBPGenClass->ClassGeneratedBy))
		{
			if (const UAnimInstance* DebuggedAnimInstance = Cast<UAnimInstance>(Blueprint->GetObjectBeingDebugged()))
			{
				if (const USkeletalMeshComponent* DebuggedSkeletalMeshComponent = DebuggedAnimInstance->GetSkelMeshComponent())
				{
					if (const USkeletalMesh* SkeletalMesh = DebuggedSkeletalMeshComponent->GetSkeletalMeshAsset())
					{
						FAnimBlueprintDebugData& DebugData = AnimBPGenClass->GetAnimBlueprintDebugData();
						DebugData.ForEachActiveVisiblePoseWatchPoseElement([PDI, SkeletalMesh](FAnimNodePoseWatch& PoseWatch)
						{
							PoseWatch.CopyPoseWatchData(SkeletalMesh->GetRefSkeleton());
							SkeletalDebugRendering::DrawBonesFromPoseWatch(PDI, PoseWatch, /*bUseWorldTransform*/false);
						});
					}
				}
			}
		}
	}
}

void FAnimationViewportClient::DrawMeshBonesBakedAnimation(UDebugSkelMeshComponent * MeshComponent, FPrimitiveDrawInterface* PDI) const
{
	if(MeshComponent && MeshComponent->GetSkeletalMeshAsset())
	{
		DrawBonesFromTransforms(MeshComponent->BakedAnimationPoses, MeshComponent, PDI, FColor(0, 128, 192, 255), FColor(0, 128, 192, 255));
	}
}

void FAnimationViewportClient::DrawBonesFromSkeleton(UDebugSkelMeshComponent * MeshComponent, const USkeleton* Skeleton, const TArray<int32>& InSelectedBones,FPrimitiveDrawInterface* PDI) const
{
	check(Skeleton);

	// Draw from skeleton ref pose
	const TArray<FTransform>& SkeletonRefPose = Skeleton->GetRefLocalPoses(NAME_None);
	TArray<FTransform> WorldTransforms;
	WorldTransforms.AddUninitialized(SkeletonRefPose.Num());

	TArray<FLinearColor> BoneColours;
	BoneColours.AddUninitialized(SkeletonRefPose.Num());

	TArray<FBoneIndexType> RequiredBones;

	const FReferenceSkeleton& RefSkeleton = Skeleton->GetReferenceSkeleton();

	
	for (FBoneIndexType BoneIndex = 0; BoneIndex < SkeletonRefPose.Num(); ++BoneIndex)
	{
		const int32 ParentIndex = RefSkeleton.GetParentIndex(BoneIndex);

		//add to the list
		RequiredBones.AddUnique(BoneIndex);

		if (ParentIndex >= 0)
		{
			WorldTransforms[BoneIndex] = SkeletonRefPose[BoneIndex] * WorldTransforms[ParentIndex];
		}
		else
		{
			WorldTransforms[BoneIndex] = SkeletonRefPose[BoneIndex];
		}

		BoneColours[BoneIndex] = MeshComponent->GetBoneColor(BoneIndex);
	}

	// color virtual bones
	const FLinearColor VirtualBoneColor = GetDefault<UPersonaOptions>()->VirtualBoneColor;
	for (const int16 VirtualBoneIndex : RefSkeleton.GetRequiredVirtualBones())
	{
		BoneColours[VirtualBoneIndex] = VirtualBoneColor;
	}

	constexpr bool bForceDraw = false;
	constexpr bool bAddHitProxy = true;
	const bool bUseMultiColor = GetDefault<UPersonaOptions>()->bShowBoneColors;
	
	DrawBones(
		FVector::ZeroVector,
		RequiredBones,
		RefSkeleton,
		WorldTransforms,
		InSelectedBones,
		BoneColours,
		PDI,
		bForceDraw,
		bAddHitProxy,
		bUseMultiColor);
}

void FAnimationViewportClient::UpdateBonesToDraw()
{
	if (UDebugSkelMeshComponent* MeshComponent = GetAnimPreviewScene()->GetPreviewMeshComponent())
	{
		const FReferenceSkeleton& RefSkeleton = MeshComponent->GetReferenceSkeleton();

		TArray<int32> ParentIndices;
		ParentIndices.AddUninitialized(RefSkeleton.GetNum());
		for (int32 BoneIndex = 0; BoneIndex < RefSkeleton.GetNum(); ++BoneIndex)
		{
			ParentIndices[BoneIndex] = RefSkeleton.GetParentIndex(BoneIndex);
		}

		SkeletalDebugRendering::CalculateBonesToDraw(
			ParentIndices,
			MeshComponent->BonesOfInterest,
			GetBoneDrawMode(),
			BonesToDraw);
	}
}

void FAnimationViewportClient::DrawMeshBones(UDebugSkelMeshComponent* MeshComponent, FPrimitiveDrawInterface* PDI) const
{
	if (!MeshComponent ||
		!MeshComponent->GetSkeletalMeshAsset() ||
		MeshComponent->GetNumDrawTransform() == 0 ||
		MeshComponent->SkeletonDrawMode == ESkeletonDrawMode::Hidden)
	{
		return;
	}
	
	TArray<FTransform> WorldTransforms;
	WorldTransforms.AddUninitialized(MeshComponent->GetNumDrawTransform());

	TArray<FLinearColor> BoneColours;
	BoneColours.AddUninitialized(MeshComponent->GetNumDrawTransform());

	// factor skeleton draw mode into color selection
	const FLinearColor BoneColor = MeshComponent->SkeletonDrawMode == ESkeletonDrawMode::GreyedOut ? GetDefault<UPersonaOptions>()->DisabledBoneColor : GetDefault<UPersonaOptions>()->DefaultBoneColor;
	const FLinearColor VirtualBoneColor = MeshComponent->SkeletonDrawMode == ESkeletonDrawMode::GreyedOut ? GetDefault<UPersonaOptions>()->DisabledBoneColor : GetDefault<UPersonaOptions>()->VirtualBoneColor;
	
	// we could cache parent bones as we calculate, but right now I'm not worried about perf issue of this
	const TArray<FBoneIndexType>& DrawBoneIndices = MeshComponent->GetDrawBoneIndices();
	for ( int32 Index=0; Index<DrawBoneIndices.Num(); ++Index )
	{
		const int32 BoneIndex = DrawBoneIndices[Index];
		WorldTransforms[BoneIndex] = MeshComponent->GetDrawTransform(BoneIndex) * MeshComponent->GetComponentTransform();
		BoneColours[BoneIndex] = MeshComponent->GetBoneColor(BoneIndex);
	}

	// color virtual bones
	for (int16 VirtualBoneIndex : MeshComponent->GetReferenceSkeleton().GetRequiredVirtualBones())
	{
		BoneColours[VirtualBoneIndex] = VirtualBoneColor;
	}
	
	constexpr bool bForceDraw = false;
	// don't allow selection if the skeleton draw mode is greyed out
	const bool bAddHitProxy = MeshComponent->SkeletonDrawMode != ESkeletonDrawMode::GreyedOut;
	const bool bUseMultiColors = GetDefault<UPersonaOptions>()->bShowBoneColors;

	DrawBones(
		MeshComponent->GetComponentLocation(),
		DrawBoneIndices,
		MeshComponent->GetReferenceSkeleton(),
		WorldTransforms,
		MeshComponent->BonesOfInterest,
		BoneColours,
		PDI,
		bForceDraw,
		bAddHitProxy,
		bUseMultiColors);
}

void FAnimationViewportClient::DrawBones(
	const FVector& ComponentOrigin,
	const TArray<FBoneIndexType>& RequiredBones,
	const FReferenceSkeleton& RefSkeleton,
	const TArray<FTransform>& WorldTransforms,
	const TArray<int32>& InSelectedBones,
	const TArray<FLinearColor>& BoneColors,
	FPrimitiveDrawInterface* PDI,
	bool bForceDraw,
	bool bAddHitProxy,
	bool bUseMultiColors) const
{
	FSkelDebugDrawConfig DrawConfig;
	DrawConfig.BoneDrawMode = GetBoneDrawMode();
	DrawConfig.BoneDrawSize = GetBoneDrawSize();
	DrawConfig.bAddHitProxy = bAddHitProxy;
	DrawConfig.bForceDraw = bForceDraw;
	DrawConfig.bUseMultiColorAsDefaultColor = bUseMultiColors;
	DrawConfig.DefaultBoneColor = GetMutableDefault<UPersonaOptions>()->DefaultBoneColor;
	DrawConfig.AffectedBoneColor = GetMutableDefault<UPersonaOptions>()->AffectedBoneColor;
	DrawConfig.SelectedBoneColor = GetMutableDefault<UPersonaOptions>()->SelectedBoneColor;
	DrawConfig.ParentOfSelectedBoneColor = GetMutableDefault<UPersonaOptions>()->ParentOfSelectedBoneColor;

	TArray<TRefCountPtr<HHitProxy>> HitProxies;

	if (bAddHitProxy)
	{
		HitProxies.Reserve(RefSkeleton.GetNum());
		for (int32 Index = 0; Index < RefSkeleton.GetNum(); ++Index)
		{
			HitProxies.Add(new HPersonaBoneHitProxy(Index, RefSkeleton.GetBoneName(Index)));
		}
	}

	SkeletalDebugRendering::DrawBones(
		PDI,
		ComponentOrigin,
		RequiredBones,
		RefSkeleton,
		WorldTransforms,
		InSelectedBones,
		BoneColors,
		HitProxies,
		DrawConfig,
		BonesToDraw
	);
}

void FAnimationViewportClient::DrawAttributes(UDebugSkelMeshComponent* MeshComponent, FPrimitiveDrawInterface* PDI) const
{
	if (MeshComponent && MeshComponent->GetSkeletalMeshAsset())
	{
		const UE::Anim::FMeshAttributeContainer& Attributes = MeshComponent->GetCustomAttributes();

		const int32 TransformAnimationAttributeTypeIndex = Attributes.FindTypeIndex(FTransformAnimationAttribute::StaticStruct());
		if (TransformAnimationAttributeTypeIndex != INDEX_NONE)
		{
			const TArray<UE::Anim::FAttributeId, FDefaultAllocator>& AttributeIdentifiers = Attributes.GetKeys(TransformAnimationAttributeTypeIndex);
			const TArray<UE::Anim::TWrappedAttribute<FDefaultAllocator>>& AttributeValues = Attributes.GetValues(TransformAnimationAttributeTypeIndex);
			check(AttributeIdentifiers.Num() == AttributeValues.Num());

			for (int32 AttributeIndex = 0; AttributeIndex < AttributeValues.Num(); ++AttributeIndex)
			{
				if (const FTransformAnimationAttribute* AttributeValue = AttributeValues[AttributeIndex].GetPtr<FTransformAnimationAttribute>())
				{
					const UE::Anim::FAttributeId& AttributeIdentifier = AttributeIdentifiers[AttributeIndex];

					const FTransform AttributeParentTransform = MeshComponent->GetDrawTransform(AttributeIdentifier.GetIndex()) * MeshComponent->GetComponentTransform();
					const FTransform AttributeTransform = AttributeValue->Value * AttributeParentTransform;

					DrawWireDiamond(PDI, AttributeTransform.ToMatrixNoScale(), 2.0f, FLinearColor(0.0f, 1.0f, 1.0f), SDPG_Foreground);
					SkeletalDebugRendering::DrawAxes(PDI, AttributeTransform, SDPG_Foreground, 0.0f, 10.0f);
					//DrawDashedLine(PDI, AttributeTransform.GetLocation(), AttributeParentTransform.GetLocation(), FLinearColor(0.0f, 1.0f, 1.0f), 2.0f, SDPG_World);
				}
			}
		}
	}
}

void FAnimationViewportClient::DrawNotifies(UDebugSkelMeshComponent* MeshComponent, FPrimitiveDrawInterface* PDI) const
{
	if (MeshComponent
		&& MeshComponent->IsNotificationVisualizationsEnabled()
		&& MeshComponent->GetSkeletalMeshAsset())
	{
		if (const UAnimSequenceBase* AnimSequenceBase = Cast<UAnimSequenceBase>(GetAnimPreviewScene()->GetPreviewAnimationAsset()))
		{
			for (const FAnimNotifyEvent& Notify : AnimSequenceBase->Notifies)
			{
				if (Notify.Notify)
				{
					Notify.Notify->DrawInEditor(PDI, MeshComponent, AnimSequenceBase, Notify);
				}
				if (Notify.NotifyStateClass)
				{
					Notify.NotifyStateClass->DrawInEditor(PDI, MeshComponent, AnimSequenceBase, Notify);
				}
			}
		}
	}
}

void FAnimationViewportClient::DrawCanvasNotifies(UDebugSkelMeshComponent* MeshComponent, FCanvas& Canvas, FSceneView& View) const
{
	if (MeshComponent
		&& MeshComponent->IsNotificationVisualizationsEnabled()
		&& MeshComponent->GetSkeletalMeshAsset())
	{
		if (const UAnimSequenceBase* AnimSequenceBase = Cast<UAnimSequenceBase>(GetAnimPreviewScene()->GetPreviewAnimationAsset()))
		{
			for (const FAnimNotifyEvent& Notify : AnimSequenceBase->Notifies)
			{
				if (Notify.Notify)
				{
					Notify.Notify->DrawCanvasInEditor(Canvas, View, MeshComponent, AnimSequenceBase, Notify);
				}
				if (Notify.NotifyStateClass)
				{
					Notify.NotifyStateClass->DrawCanvasInEditor(Canvas, View, MeshComponent, AnimSequenceBase, Notify);
				}
			}
		}
	}
}

TArray<IInterface_AssetUserData*> FAnimationViewportClient::GetEditedObjectsWithAssetUserData() const
{
	TArray<IInterface_AssetUserData*> Result;
	if (const TSharedPtr<FAssetEditorToolkit> AssetEditorToolkit = AssetEditorToolkitPtr.Pin())
	{
		if (const TArray<UObject*>* ObjectsCurrentlyBeingEdited = AssetEditorToolkit->GetObjectsCurrentlyBeingEdited())
		{
			for (UObject* Object : *ObjectsCurrentlyBeingEdited)
			{
				if (IInterface_AssetUserData* AssetUserDataInterface = Cast<IInterface_AssetUserData>(Object))
				{
					Result.Add(AssetUserDataInterface);
				}
			}
		}
	}

	return Result;
}

void FAnimationViewportClient::DrawAssetUserData(FPrimitiveDrawInterface* PDI) const
{
	TArray<IInterface_AssetUserData*> AssetsWithUserData = GetEditedObjectsWithAssetUserData();
	for (const IInterface_AssetUserData* AssetUserDataInterface : AssetsWithUserData)
	{
		if (const TArray<UAssetUserData*>* AssetUserDataArray = AssetUserDataInterface->GetAssetUserDataArray())
		{
			for (const UAssetUserData* AssetUserData : *AssetUserDataArray)
			{
				if (AssetUserData)
				{
					AssetUserData->Draw(PDI, PDI->View);
				}
			}
		}
	}	
}

void FAnimationViewportClient::DrawCanvasAssetUserData(FCanvas& Canvas, FSceneView& View) const
{
	TArray<IInterface_AssetUserData*> AssetsWithUserData = GetEditedObjectsWithAssetUserData();
	for (const IInterface_AssetUserData* AssetUserDataInterface : AssetsWithUserData)
	{
		if (const TArray<UAssetUserData*>* AssetUserDataArray = AssetUserDataInterface->GetAssetUserDataArray())
		{
			for (const UAssetUserData* AssetUserData : *AssetUserDataArray)
			{
				if (AssetUserData)
				{
					AssetUserData->DrawCanvas(Canvas, View);
				}
			}
		}
	}
}

void FAnimationViewportClient::DrawRootMotionTrajectory(UDebugSkelMeshComponent* MeshComponent, FPrimitiveDrawInterface* PDI) const
{
	constexpr float DepthBias = 2.0f;
	constexpr bool bScreenSpace = true;

	if (MeshComponent
		&& !MeshComponent->IsVisualizeRootMotionMode(EVisualizeRootMotionMode::None)
		&& MeshComponent->GetSkeletalMeshAsset()
		&& MeshComponent->DoesCurrentAssetHaveRootMotion())
	{
		const EVisualizeRootMotionMode VisMode = MeshComponent->GetVisualizeRootMotionMode();

		const FTransform& ReferenceTransform = MeshComponent->RootMotionReferenceTransform;
		const UMirrorDataTable* MirrorTable = MeshComponent->PreviewInstance ? MeshComponent->PreviewInstance->GetMirrorDataTable() : nullptr;
		
		if (const UAnimSequenceBase* AnimSequenceBase = Cast<UAnimSequenceBase>(GetAnimPreviewScene()->GetPreviewAnimationAsset()))
		{
			// Draw root motion trajectory
			const int32 NumFrames = AnimSequenceBase->GetNumberOfSampledKeys();
			const FFrameRate FrameRate = AnimSequenceBase->GetSamplingFrameRate();
			const float CurrentTime = MeshComponent->GetPosition();
			const USkeleton* Skeleton = AnimSequenceBase->GetSkeleton();
			check(Skeleton);
			EAxis::Type SkeletonForwardAxis = Skeleton->GetPreviewForwardAxis();

			const FColor TrajectoryColor = FColor::Black.WithAlpha(64);

			FVector PrevLocation;
			float PlayLength = AnimSequenceBase->GetPlayLength();
			PlayLength = AnimSequenceBase->GetPlayLength();
			for (int32 Frame = 0; Frame <= NumFrames; Frame++)
			{
				const double Time = FMath::Clamp(FrameRate.AsSeconds(Frame), 0., (double)PlayLength);
				const FTransform Transform = UE::Anim::ExtractRootMotionFromAnimationAsset(AnimSequenceBase, MirrorTable, 0.0, Time) * ReferenceTransform;
				const FVector Location = Transform.GetLocation();

				const bool bFirstOrLastPoint = Frame == 0 || Frame == NumFrames;

				PDI->DrawPoint(Location, TrajectoryColor, bFirstOrLastPoint ? 2.5f : 1.25f, SDPG_World);

				if (VisMode == EVisualizeRootMotionMode::TrajectoryAndOrientation)
				{
					if (bFirstOrLastPoint || (Frame % 3 == 0))
					{
						const FVector XAxis = Transform.GetUnitAxis(SkeletonForwardAxis);
						const FColor AxisColor = UE::Private::GetColorForAxis(SkeletonForwardAxis);

						FVector YAxis, ZAxis;
						XAxis.FindBestAxisVectors(YAxis,ZAxis);
						UE::Private::DrawFlatArrow(PDI, Transform.GetLocation(), XAxis, ZAxis, AxisColor.WithAlpha(64), 15.0f, 8, nullptr, SDPG_World, 1.0f);
					}
				}

				if (Frame > 0)
				{
					PDI->DrawTranslucentLine(PrevLocation, Location, TrajectoryColor, SDPG_World, 1.0f, DepthBias, bScreenSpace);
				}
				PrevLocation = Location;
			}

			// Draw current location on the root motion.
			{
				const FTransform Transform = UE::Anim::ExtractRootMotionFromAnimationAsset(AnimSequenceBase, MirrorTable, 0.0, CurrentTime) * ReferenceTransform;

				const FVector XAxis = Transform.GetUnitAxis(SkeletonForwardAxis);
				const FColor AxisColor = UE::Private::GetColorForAxis(SkeletonForwardAxis);

				FVector YAxis, ZAxis;
				XAxis.FindBestAxisVectors(YAxis,ZAxis);

				if (VisMode == EVisualizeRootMotionMode::TrajectoryAndOrientation)
				{
					UE::Private::DrawFlatArrow(PDI, Transform.GetLocation(), XAxis, ZAxis, AxisColor, 30.0f, 15, GEngine->ArrowMaterialYellow->GetRenderProxy(), SDPG_Foreground, 1.0f);
				}
				UE::Private::DrawCoordinateSystem(PDI, Transform, 10.0f, 20.0f, DepthBias, bScreenSpace, 200);
			}			
		}
	}
}

void FAnimationViewportClient::DrawSockets(const UDebugSkelMeshComponent* InPreviewMeshComponent, TArray<USkeletalMeshSocket*>& InSockets, FSelectedSocketInfo InSelectedSocket, FPrimitiveDrawInterface* PDI, bool bUseSkeletonSocketColor)
{
	if (InPreviewMeshComponent && InPreviewMeshComponent->GetSkeletalMeshAsset())
	{
		ELocalAxesMode::Type LocalAxesMode = (ELocalAxesMode::Type)GetDefault<UPersonaOptions>()->DefaultLocalAxesSelection;

		for ( auto SocketIt = InSockets.CreateConstIterator(); SocketIt; ++SocketIt )
		{
			USkeletalMeshSocket* Socket = *(SocketIt);
			const FReferenceSkeleton& RefSkeleton = InPreviewMeshComponent->GetReferenceSkeleton();

			const int32 ParentIndex = RefSkeleton.FindBoneIndex(Socket->BoneName);

			FTransform WorldTransformSocket = Socket->GetSocketTransform(InPreviewMeshComponent);

			FLinearColor SocketColor;
			FVector Start, End;
			if (ParentIndex >=0)
			{
				FTransform WorldTransformParent = InPreviewMeshComponent->GetDrawTransform(ParentIndex) * InPreviewMeshComponent->GetComponentTransform();
				Start = WorldTransformParent.GetLocation();
				End = WorldTransformSocket.GetLocation();
			}
			else
			{
				Start = FVector::ZeroVector;
				End = WorldTransformSocket.GetLocation();
			}

			const bool bSelectedSocket = (InSelectedSocket.Socket == Socket);

			if( bSelectedSocket )
			{
				SocketColor = FLinearColor(1.0f, 0.34f, 0.0f, 1.0f);
			}
			else
			{
				SocketColor = (bUseSkeletonSocketColor) ? FLinearColor::White : FLinearColor::Red;
			}

			//Render Sphere for bone end point and a cone between it and its parent.
			PDI->DrawLine( Start, End, SocketColor, SDPG_Foreground );
			
			// draw gizmo
			if( (LocalAxesMode == ELocalAxesMode::All) || bSelectedSocket )
			{
				FMatrix SocketMatrix;
				Socket->GetSocketMatrix( SocketMatrix, InPreviewMeshComponent);

				PDI->SetHitProxy(new HPersonaSocketHitProxy(Socket));
				DrawWireDiamond( PDI, SocketMatrix, 2.f, SocketColor, SDPG_Foreground );
				PDI->SetHitProxy(nullptr);
				
				SkeletalDebugRendering::DrawAxes(PDI, FTransform(SocketMatrix), SDPG_Foreground);
			}
		}
	}
}

FSphere FAnimationViewportClient::GetCameraTarget()
{
	// give the editor mode a chance to give us a camera target
	if (IPersonaManagerContext* PersonaModeManagerContext = GetModeTools()->GetInteractiveToolsContext()->ContextObjectStore->FindContext<UPersonaEditorModeManagerContext>())
	{
		FSphere Target;
		if (PersonaModeManagerContext->GetCameraTarget(Target))
		{
			return Target;
		}
	}

	const FSphere DefaultSphere(FVector(0,0,0), 100.0f);

	UDebugSkelMeshComponent* PreviewMeshComponent = GetAnimPreviewScene()->GetPreviewMeshComponent();
	if( !PreviewMeshComponent )
	{
		return DefaultSphere;
	}

	AActor* Actor = PreviewMeshComponent->GetOwner();
	FBox LocalBox(ForceInit);
	if (GetModeTools()->ComputeBoundingBoxForViewportFocus(Actor, PreviewMeshComponent, LocalBox))
	{
		return FBoxSphereBounds(LocalBox).GetSphere();
	}
	
	FBoxSphereBounds Bounds = PreviewMeshComponent->CalcGameBounds(FTransform::Identity);
	return Bounds.GetSphere();
}

void FAnimationViewportClient::UpdateCameraSetup()
{
	static FRotator CustomOrbitRotation(-33.75, -135, 0);
	UDebugSkelMeshComponent* PreviewMeshComponent = GetAnimPreviewScene()->GetPreviewMeshComponent();
	if (PreviewMeshComponent != nullptr && PreviewMeshComponent->GetSkeletalMeshAsset())
	{
		FSphere BoundSphere = GetCameraTarget();
		FVector CustomOrbitZoom(0, BoundSphere.W / (75.0f * (float)PI / 360.0f), 0);
		FVector CustomOrbitLookAt = BoundSphere.Center;

		SetCameraSetup(CustomOrbitLookAt, CustomOrbitRotation, CustomOrbitZoom, CustomOrbitLookAt, GetViewLocation(), GetViewRotation() );

		// Move the floor to the bottom of the bounding box of the mesh, rather than on the origin
		FVector Bottom = PreviewMeshComponent->Bounds.GetBoxExtrema(0);

		FVector FloorPos(0.f, 0.f, GetFloorOffset());
		if (bAutoAlignFloor)
		{
			FloorPos.Z += Bottom.Z;
		}
		GetAnimPreviewScene()->SetFloorLocation(FloorPos);
	}
}

void FAnimationViewportClient::FocusViewportOnSphere( FSphere& Sphere, bool bInstant /*= true*/ )
{
	FBox Box( Sphere.Center - FVector(Sphere.W, 0.0f, 0.0f), Sphere.Center + FVector(Sphere.W, 0.0f, 0.0f) );

	FocusViewportOnBox( Box, bInstant );

	Invalidate();
}

void FAnimationViewportClient::TransformVertexPositionsToWorld(TArray<FFinalSkinVertex>& LocalVertices) const
{
	const UDebugSkelMeshComponent* const PreviewMeshComponent = GetAnimPreviewScene()->GetPreviewMeshComponent();
	if ( !PreviewMeshComponent )
	{
		return;
	}

	const FTransform& LocalToWorldTransform = PreviewMeshComponent->GetComponentTransform();

	for ( int32 VertexIndex = 0; VertexIndex < LocalVertices.Num(); ++VertexIndex )
	{
		FVector3f& VertexPosition = LocalVertices[VertexIndex].Position;
		VertexPosition = (FVector3f)LocalToWorldTransform.TransformPosition((FVector)VertexPosition);
	}
}

void FAnimationViewportClient::GetAllVertexIndicesUsedInSection(const FRawStaticIndexBuffer16or32Interface& IndexBuffer,
																const FSkelMeshRenderSection& SkelMeshSection,
																TArray<int32>& OutIndices) const
{

	const uint32 BaseIndex = SkelMeshSection.BaseIndex;
	const int32 NumWedges = SkelMeshSection.NumTriangles * 3;

	for (int32 WedgeIndex = 0; WedgeIndex < NumWedges; ++WedgeIndex)
	{
		const int32 VertexIndexForWedge = IndexBuffer.Get(SkelMeshSection.BaseIndex + WedgeIndex);
		OutIndices.Add(VertexIndexForWedge);
	}
}

FBox FAnimationViewportClient::ComputeBoundingBoxForSelectedEditorSection() const
{
	UDebugSkelMeshComponent* const PreviewMeshComponent = GetAnimPreviewScene()->GetPreviewMeshComponent();
	if ( !PreviewMeshComponent )
	{
		return FBox(ForceInitToZero);
	}

	USkeletalMesh* const SkeletalMesh = PreviewMeshComponent->GetSkeletalMeshAsset();
	FSkeletalMeshObject* const MeshObject = PreviewMeshComponent->MeshObject;
	if ( !SkeletalMesh || !MeshObject )
	{
		return FBox(ForceInitToZero);
	}

	const int32 LODLevel = PreviewMeshComponent->GetPredictedLODLevel();
	const int32 SelectedEditorSection = PreviewMeshComponent->GetSelectedEditorSection();
	const FSkeletalMeshRenderData& SkelMeshRenderData = MeshObject->GetSkeletalMeshRenderData();

	const FSkeletalMeshLODRenderData& LODData = SkelMeshRenderData.LODRenderData[LODLevel];
	const FSkelMeshRenderSection& SelectedSectionSkelMesh = LODData.RenderSections[SelectedEditorSection];

	// Get us vertices from the entire LOD model.
	TArray<FFinalSkinVertex> SkinnedVertices;
	PreviewMeshComponent->GetCPUSkinnedVertices(SkinnedVertices, LODLevel);
	TransformVertexPositionsToWorld(SkinnedVertices);

	// Find out which of these the selected section actually uses.
	TArray<int32> VertexIndices;
	GetAllVertexIndicesUsedInSection(*LODData.MultiSizeIndexContainer.GetIndexBuffer(), SelectedSectionSkelMesh, VertexIndices);

	// Get their bounds.
	FBox BoundingBox(ForceInitToZero);
	for ( int32 Index = 0; Index < VertexIndices.Num(); ++Index )
	{
		const int32 VertexIndex = VertexIndices[Index];
		BoundingBox += (FVector)SkinnedVertices[VertexIndex].Position;
	}

	return BoundingBox;
}

void FAnimationViewportClient::FocusViewportOnPreviewMesh(bool bUseCustomCamera)
{
	FIntPoint ViewportSize(FIntPoint::ZeroValue);
	if (Viewport != nullptr)
	{
		ViewportSize = Viewport->GetSizeXY();
	}

	if(!(ViewportSize.SizeSquared() > 0))
	{
		// We cannot focus fully right now as the viewport does not know its size
		// and we must have the aspect to correctly focus on the component,
		bFocusOnDraw = true;
		bFocusUsingCustomCamera = bUseCustomCamera;
		return;
	}
	
	if (bHasAudioFocus)
	{
		UpdateAudioListener();
	}

	if (UDebugSkelMeshComponent* const PreviewMeshComponent = GetAnimPreviewScene()->GetPreviewMeshComponent())
	{
		if (USkeletalMesh* const SkelMesh = PreviewMeshComponent->GetSkeletalMeshAsset())
		{
			if (bUseCustomCamera && SkelMesh->GetHasCustomDefaultEditorCamera())
			{
				FViewportCameraTransform& ViewTransform = GetViewTransform();

				ViewTransform.SetLocation(SkelMesh->GetDefaultEditorCameraLocation());
				ViewTransform.SetRotation(SkelMesh->GetDefaultEditorCameraRotation());
				ViewTransform.SetLookAt(SkelMesh->GetDefaultEditorCameraLookAt());
				ViewTransform.SetOrthoZoom(SkelMesh->GetDefaultEditorCameraOrthoZoom());

				Invalidate();
				return;
			}

			if (PreviewMeshComponent->GetSelectedEditorSection() != INDEX_NONE)
			{
				const FBox SelectedSectionBounds = ComputeBoundingBoxForSelectedEditorSection();

				if (SelectedSectionBounds.IsValid)
				{
					FocusViewportOnBox(SelectedSectionBounds);
				}

				return;
			}
		}
	}

	FSphere Sphere = GetCameraTarget();
	FocusViewportOnSphere(Sphere);
}

float FAnimationViewportClient::GetFloorOffset() const
{
	const USkeletalMeshComponent* SkelMeshComponent = GetPreviewScene()->GetPreviewMeshComponent();
	USkeletalMesh* Mesh = SkelMeshComponent ? SkelMeshComponent->GetSkeletalMeshAsset() : nullptr;
	if ( Mesh )
	{
		return Mesh->GetFloorOffset();
	}

	return 0.0f;
}

void FAnimationViewportClient::SetFloorOffset( float NewValue )
{
	const USkeletalMeshComponent* SkelMeshComponent = GetPreviewScene()->GetPreviewMeshComponent();
	USkeletalMesh* Mesh = SkelMeshComponent ? SkelMeshComponent->GetSkeletalMeshAsset() : nullptr;

	if ( Mesh )
	{
		Mesh->Modify();
		Mesh->SetFloorOffset(NewValue);
		UpdateCameraSetup(); // This does the actual moving of the floor mesh
		Invalidate();
	}
}

void FAnimationViewportClient::ToggleCPUSkinning()
{
	GetPreviewScene()->ForEachPreviewMesh([this](UDebugSkelMeshComponent* PreviewMeshComponent)
	{
		const bool bCurVal = PreviewMeshComponent->GetCPUSkinningEnabled();
		PreviewMeshComponent->SetCPUSkinningEnabled(!bCurVal);
	});
	Invalidate();
}

bool FAnimationViewportClient::IsSetCPUSkinningChecked() const
{
	UDebugSkelMeshComponent* PreviewMeshComponent = GetAnimPreviewScene()->GetPreviewMeshComponent();
	if (PreviewMeshComponent != nullptr)
	{
		return PreviewMeshComponent->GetCPUSkinningEnabled();
	}
	return false;
}

void FAnimationViewportClient::ToggleShowNormals()
{
	GetPreviewScene()->ForEachPreviewMesh([this](UDebugSkelMeshComponent* PreviewMeshComponent)
	{
		PreviewMeshComponent->bDrawNormals = !PreviewMeshComponent->bDrawNormals;
		PreviewMeshComponent->MarkRenderStateDirty();
	});
	
	Invalidate();
}

bool FAnimationViewportClient::IsSetShowNormalsChecked() const
{
	UDebugSkelMeshComponent* PreviewMeshComponent = GetAnimPreviewScene()->GetPreviewMeshComponent();
	if (PreviewMeshComponent != nullptr)
	{
		return PreviewMeshComponent->bDrawNormals;
	}
	return false;
}

void FAnimationViewportClient::ToggleShowTangents()
{
	GetPreviewScene()->ForEachPreviewMesh([this](UDebugSkelMeshComponent* PreviewMeshComponent)
	{
		PreviewMeshComponent->bDrawTangents = !PreviewMeshComponent->bDrawTangents;
		PreviewMeshComponent->MarkRenderStateDirty();
	});
	
	Invalidate();
}

bool FAnimationViewportClient::IsSetShowTangentsChecked() const
{
	UDebugSkelMeshComponent* PreviewMeshComponent = GetAnimPreviewScene()->GetPreviewMeshComponent();
	if (PreviewMeshComponent != nullptr)
	{
		return PreviewMeshComponent->bDrawTangents;
	}
	return false;
}

void FAnimationViewportClient::ToggleShowBinormals()
{
	GetPreviewScene()->ForEachPreviewMesh([this](UDebugSkelMeshComponent* PreviewMeshComponent)
	{
		PreviewMeshComponent->bDrawBinormals = !PreviewMeshComponent->bDrawBinormals;
			PreviewMeshComponent->MarkRenderStateDirty();
	});
	
	Invalidate();
}

bool FAnimationViewportClient::IsSetShowBinormalsChecked() const
{
	UDebugSkelMeshComponent* PreviewMeshComponent = GetAnimPreviewScene()->GetPreviewMeshComponent();
	if (PreviewMeshComponent != nullptr)
	{
		return PreviewMeshComponent->bDrawBinormals;
	}
	return false;
}

void FAnimationViewportClient::SetDrawUVOverlay(bool bInDrawUVs)
{
	bDrawUVs = bInDrawUVs;
	Invalidate();
}

bool FAnimationViewportClient::IsSetDrawUVOverlayChecked() const
{
	return bDrawUVs;
}

void FAnimationViewportClient::OnSetShowMeshStats(int32 ShowMode)
{
	ConfigOption->SetShowMeshStats(ShowMode);
}

bool FAnimationViewportClient::IsShowingMeshStats() const
{
	const bool bShouldBeEnabled = ConfigOption->ShowMeshStats != EDisplayInfoMode::None;

	return bShouldBeEnabled && bShowMeshStats;
}

bool FAnimationViewportClient::IsShowingSelectedNodeStats() const
{
	return ConfigOption->ShowMeshStats == EDisplayInfoMode::SkeletalControls;
}

bool FAnimationViewportClient::IsDetailedMeshStats() const
{
	return ConfigOption->ShowMeshStats == EDisplayInfoMode::Detailed;
}

int32 FAnimationViewportClient::GetShowMeshStats() const
{
	return ConfigOption->ShowMeshStats;
}

TSharedPtr<class FAnimationEditorPreviewScene> FAnimationViewportClient::GetAnimPreviewScenePtr() const
{
	return StaticCastSharedPtr<FAnimationEditorPreviewScene>(PreviewScenePtr);
}

TSharedRef<FAnimationEditorPreviewScene> FAnimationViewportClient::GetAnimPreviewScene() const
{
	return StaticCastSharedRef<FAnimationEditorPreviewScene>(GetPreviewScene());
}

void FAnimationViewportClient::PerspectiveCameraMoved()
{
	FEditorViewportClient::PerspectiveCameraMoved();
	
	if (bHasAudioFocus)
	{
		UpdateAudioListener();
	}
}

void FAnimationViewportClient::HandleInvalidateViews()
{
	Invalidate();
}

void FAnimationViewportClient::HandleFocusViews()
{
	SetCameraFollowMode(EAnimationViewportCameraFollowMode::None);
	FocusViewportOnPreviewMesh(false);
}

bool FAnimationViewportClient::CanCycleWidgetMode() const
{
	return ModeTools ? ModeTools->CanCycleWidgetMode() : false;
}

void FAnimationViewportClient::UpdateAudioListener(const FSceneView& View)
{
	UpdateAudioListener();
}

void FAnimationViewportClient::UpdateAudioListener() const
{
	UWorld* ViewportWorld = GetWorld();

	if (ViewportWorld)
	{
		if (FAudioDevice* AudioDevice = ViewportWorld->GetAudioDeviceRaw())
		{
			const FVector& ViewLocation = GetViewLocation();
			const FRotator& ViewRotation = GetViewRotation();

			FTransform ListenerTransform(ViewRotation);
			ListenerTransform.SetLocation(ViewLocation);

			AudioDevice->SetListener(ViewportWorld, 0, ListenerTransform, 0.0f);
		}
	}
}

void FAnimationViewportClient::SetupViewForRendering( FSceneViewFamily& ViewFamily, FSceneView& View )
{
	FEditorViewportClient::SetupViewForRendering( ViewFamily, View );

	// Cache screen size
	const UDebugSkelMeshComponent* PreviewMeshComponent = GetAnimPreviewScene()->GetPreviewMeshComponent();
	if (PreviewMeshComponent != nullptr && PreviewMeshComponent->MeshObject != nullptr)
	{
		const FBoxSphereBounds& SkelBounds = PreviewMeshComponent->Bounds;
		CachedScreenSize = ComputeBoundsScreenSize(SkelBounds.Origin, static_cast<float>(SkelBounds.SphereRadius), View);
	}
}

void FAnimationViewportClient::HandleToggleShowFlag(FEngineShowFlags::EShowFlag EngineShowFlagIndex)
{
	FEditorViewportClient::HandleToggleShowFlag(EngineShowFlagIndex);

	GetPreviewScene()->ForEachPreviewMesh([this](UDebugSkelMeshComponent* InMesh)
	{
		InMesh->bDisplayVertexColors = EngineShowFlags.VertexColors;
		InMesh->MarkRenderStateDirty();
	});

	ConfigOption->SetShowGrid(EngineShowFlags.Grid);
}

void FAnimationViewportClient::OnCameraControllerChanged()
{
	TSharedPtr<FEditorCameraController> Override = GetAnimPreviewScene()->GetCurrentCameraOverride();
	CameraController = Override.IsValid() ? Override.Get() : CachedDefaultCameraController;
}

FMatrix FAnimationViewportClient::CalcViewRotationMatrix(const FRotator& InViewRotation) const
{
	auto ComputeOrbitMatrix = [this](const FViewportCameraTransform& InViewTransform)
	{
		FTransform Transform =
		FTransform( -InViewTransform.GetLookAt() ) * 
		FTransform( OrbitRotation.Inverse() ) *
		FTransform( FRotator(0, InViewTransform.GetRotation().Yaw,0) ) * 
		FTransform( FRotator(0, 0, InViewTransform.GetRotation().Pitch) ) *
		FTransform( FVector(0, (InViewTransform.GetLocation() - InViewTransform.GetLookAt()).Size(), 0) );

		return Transform.ToMatrixNoScale() * FInverseRotationMatrix( FRotator(0,90.f,0) );
	};

	const FViewportCameraTransform& ViewTransform = GetViewTransform();

	if (bUsingOrbitCamera)
	{
		// @todo vreditor: Not stereo friendly yet
		return FTranslationMatrix(ViewTransform.GetLocation()) * ComputeOrbitMatrix(ViewTransform);
	}
	else
	{
		// Create the view matrix
		return FInverseRotationMatrix(InViewRotation);
	}
}

#undef LOCTEXT_NAMESPACE

