// Copyright Epic Games, Inc. All Rights Reserved.
#include "ApexClothingOptionWindow.h"

#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Children.h"
#include "Misc/Attribute.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Types/SlateEnums.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "ApexClothingOption"

void SApexClothingOptionWindow::Construct(const FArguments& InArgs)
{
	WidgetWindow = InArgs._WidgetWindow;

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			SNew(SVerticalBox)

			+SVerticalBox::Slot().AutoHeight() .Padding(5) .HAlign(HAlign_Center)
			[
				SNew(STextBlock)
				.Text( FText::Format( LOCTEXT("MultiLODsExplanation", "This asset has {0} LODs.\nYou can enable or diable clothing LOD by \"Enable Clothing LOD\" check box in Materials section."), FText::AsNumber( InArgs._NumLODs ) ) ) 
			]

			+SVerticalBox::Slot().AutoHeight() .Padding(5) .HAlign(HAlign_Center)
			[
				InArgs._ApexDetails->AsShared()
			]

			+SVerticalBox::Slot().AutoHeight() .Padding(5) .HAlign(HAlign_Center)
			[
				SNew(SHorizontalBox)

				+SHorizontalBox::Slot() .FillWidth(1)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.Text(LOCTEXT("ApexClothingOption_Import", "Import"))
					.OnClicked(this, &SApexClothingOptionWindow::OnImport)
				]

				+SHorizontalBox::Slot() .FillWidth(1)
				[
					SNew(SButton) 
					.HAlign(HAlign_Center)
					.Text(LOCTEXT("ApexClothingOption_Cancel", "Cancel"))
					.OnClicked(this, &SApexClothingOptionWindow::OnCancel)
				]
			]
		]
	];
}

#undef LOCTEXT_NAMESPACE
