// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Input/Reply.h"
#include "Styling/SlateTypes.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWindow.h"

class SUniformGridPanel;

class SApexClothingOptionWindow : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SApexClothingOptionWindow )
		: _WidgetWindow()
		, _NumLODs()
		, _ApexDetails()
	{}

		SLATE_ARGUMENT( TSharedPtr<SWindow>, WidgetWindow )
		SLATE_ARGUMENT( int32, NumLODs )
		SLATE_ARGUMENT(TSharedPtr<SUniformGridPanel>, ApexDetails )
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs);

	FReply OnImport()
	{
		bCanImport = true;
		if ( WidgetWindow.IsValid() )
		{
			WidgetWindow.Pin()->RequestDestroyWindow();
		}
		return FReply::Handled();
	}

	FReply OnCancel()
	{
		bCanImport = false;
		if ( WidgetWindow.IsValid() )
		{
			WidgetWindow.Pin()->RequestDestroyWindow();
		}
		return FReply::Handled();
	}

	ECheckBoxState IsCheckedLOD() const
	{
		return bUseLOD ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	void OnUseLOD(ECheckBoxState CheckState)
	{
		bUseLOD = (CheckState == ECheckBoxState::Checked);
	}

	bool CanImport() const
	{
		return bCanImport;
	}

	bool IsUsingLOD()
	{
		return bUseLOD;
	}

	SApexClothingOptionWindow() 
		: bCanImport(false)
		, bUseLOD(true)
	{}

private:
	bool			bCanImport;
	bool			bReimport;
	bool			bUseLOD;

	TWeakPtr< SWindow > WidgetWindow;
};
