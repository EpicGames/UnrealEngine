// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetSearchBoxUtilPersona.h"
#include "ReferenceSkeleton.h"
#include "Animation/Skeleton.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkeletalMeshSocket.h"
#include "Editor.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Animation/AnimSequence.h"

SAssetSearchBoxForBones::SAssetSearchBoxForBones()
{
	if(GEditor)
	{
		GEditor->RegisterForUndo(this);
	}
}

SAssetSearchBoxForBones::~SAssetSearchBoxForBones()
{
	if(GEditor)
	{
		GEditor->UnregisterForUndo(this);
	}
}

void SAssetSearchBoxForBones::Construct( const FArguments& InArgs, const class UObject* Outer, TSharedPtr<class IPropertyHandle> BoneNameProperty )
{
	check(Outer);

	BonePropertyHandle = BoneNameProperty;
	// set delegate on property change
	// this doesn't work for undo still
	BonePropertyHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &SAssetSearchBoxForBones::RefreshName));

	const USkeleton* Skeleton = NULL;

	TArray<FAssetSearchBoxSuggestion> PossibleSuggestions;
	if( const USkeletalMesh* SkeletalMesh = Cast<const USkeletalMesh>(Outer) )
	{
		if( InArgs._IncludeSocketsForSuggestions.Get() )
		{
			for( auto SocketIt=const_cast<USkeletalMesh*>(SkeletalMesh)->GetMeshOnlySocketList().CreateConstIterator(); SocketIt; ++SocketIt )
			{
				PossibleSuggestions.Add(FAssetSearchBoxSuggestion::MakeSimpleSuggestion((*SocketIt)->SocketName.ToString()));
			}
		}
		Skeleton = SkeletalMesh->GetSkeleton();
	}
	else
	{
		Skeleton = Cast<const USkeleton>(Outer);
	}
	if( Skeleton )
	{
		if (InArgs._IncludeSocketsForSuggestions.Get())
		{
			for( auto SocketIt=Skeleton->Sockets.CreateConstIterator(); SocketIt; ++SocketIt )
			{
				PossibleSuggestions.Add(FAssetSearchBoxSuggestion::MakeSimpleSuggestion((*SocketIt)->SocketName.ToString()));
			}
		}
		
		const TArray<FMeshBoneInfo>& Bones = Skeleton->GetReferenceSkeleton().GetRefBoneInfo();
		for( auto BoneIt=Bones.CreateConstIterator(); BoneIt; ++BoneIt )
		{
			PossibleSuggestions.Add(FAssetSearchBoxSuggestion::MakeSimpleSuggestion(BoneIt->Name.ToString()));
		}
	}

	// create the asset search box
	ChildSlot
	[
		SAssignNew(SearchBox, SAssetSearchBox)
		.InitialText(GetBoneName())
		.HintText(InArgs._HintText)
		.OnTextCommitted(InArgs._OnTextCommitted)
		.PossibleSuggestions(PossibleSuggestions)
		.DelayChangeNotificationsWhileTyping( true )
		.MustMatchPossibleSuggestions(InArgs._MustMatchPossibleSuggestions)
	];
}

void SAssetSearchBoxForBones::RefreshName()
{
	if (SearchBox.IsValid())
	{
		SearchBox->SetText(GetBoneName());
	}
}

FText SAssetSearchBoxForBones::GetBoneName() const
{
	FName CurValue;
	if (BonePropertyHandle.IsValid())
	{
		BonePropertyHandle->GetValue(CurValue);
	}
	return CurValue.IsNone() ? FText::GetEmpty() : FText::FromName(CurValue);
}

SAssetSearchBoxForCurves::SAssetSearchBoxForCurves()
{
	if(GEditor)
	{
		GEditor->RegisterForUndo(this);
	}
}

SAssetSearchBoxForCurves::~SAssetSearchBoxForCurves()
{
	if(GEditor)
	{
		GEditor->UnregisterForUndo(this);
	}
}

void SAssetSearchBoxForCurves::Construct(const FArguments& InArgs, const class USkeleton* InSkeleton, TSharedPtr<class IPropertyHandle> CurveNameProperty)
{
	check(InSkeleton);

	CurveNamePropertyHandle = CurveNameProperty;

	Skeleton = MakeWeakObjectPtr(const_cast<USkeleton*>(InSkeleton));
	
	// create the asset search box
	ChildSlot
		[
			SAssignNew(SearchBox, SAssetSearchBox)
			.InitialText(GetCurveName())
			.HintText(InArgs._HintText)
			.OnTextCommitted(InArgs._OnTextCommitted)
			.PossibleSuggestions(this, &SAssetSearchBoxForCurves::GetCurveSearchSuggestions)
			.DelayChangeNotificationsWhileTyping(true)
			.MustMatchPossibleSuggestions(InArgs._MustMatchPossibleSuggestions)
		];
}

FText SAssetSearchBoxForCurves::GetCurveName() const
{
	FName CurValue;
	if (CurveNamePropertyHandle.IsValid())
	{
		CurveNamePropertyHandle->GetValue(CurValue);
	}
	return CurValue.IsNone() ? FText::GetEmpty() : FText::FromName(CurValue);
}

void SAssetSearchBoxForCurves::RefreshName()
{
	if (SearchBox.IsValid())
	{
		SearchBox->SetText(GetCurveName());
	}
}

TArray<FAssetSearchBoxSuggestion> SAssetSearchBoxForCurves::GetCurveSearchSuggestions() const
{
	TArray<FAssetSearchBoxSuggestion> PossibleSuggestions;
	if (USkeleton* Skel = Skeleton.Get())
	{
		// We use the asset registry to query all assets with the supplied skeleton, and accumulate their curve names
		const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");

		FARFilter Filter;
		Filter.bRecursiveClasses = true;
		Filter.ClassPaths.Add(UAnimSequence::StaticClass()->GetClassPathName());
		Filter.TagsAndValues.Add(TEXT("Skeleton"), FAssetData(Skel).GetExportTextName());

		TArray<FAssetData> Assets;
		AssetRegistryModule.Get().GetAssets(Filter, Assets);
		
		for(const FAssetData& AssetData : Assets)
		{
			const FString TagValue = AssetData.GetTagValueRef<FString>(USkeleton::CurveNameTag);
			if (!TagValue.IsEmpty())
			{
				TArray<FString> ParsedCurveNames;
				if(TagValue.ParseIntoArray(ParsedCurveNames, *USkeleton::CurveTagDelimiter, true))
				{
					for (const FString& CurveString : ParsedCurveNames)
					{
						PossibleSuggestions.Add(FAssetSearchBoxSuggestion::MakeSimpleSuggestion(CurveString));
					}
				}
			}
		}
	}

	return PossibleSuggestions;
}
