// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Framework/SlateDelegates.h"
#include "PropertyHandle.h"
#include "SAssetSearchBox.h"
#include "EditorUndoClient.h"

class USkeleton;

class SAssetSearchBoxForBones : public SCompoundWidget, public FEditorUndoClient
{
public:
	SLATE_BEGIN_ARGS(SAssetSearchBoxForBones)
		: _OnTextCommitted()
		, _HintText()
		, _MustMatchPossibleSuggestions( false )
		, _IncludeSocketsForSuggestions( false )
	{}

	/** Where to place the suggestion list */
	SLATE_ARGUMENT( EMenuPlacement, SuggestionListPlacement )

		/** Invoked whenever the text is committed (e.g. user presses enter) */
		SLATE_EVENT( FOnTextCommitted, OnTextCommitted )

		/** Hint text to display for the search text when there is no value */
		SLATE_ATTRIBUTE( FText, HintText )

		/** Whether the SearchBox allow entries that don't match the possible suggestions */
		SLATE_ATTRIBUTE( bool, MustMatchPossibleSuggestions )

		/** Whether the possible suggestions should include sockets */
		SLATE_ATTRIBUTE( bool, IncludeSocketsForSuggestions )

		SLATE_END_ARGS()

		SAssetSearchBoxForBones();
		~SAssetSearchBoxForBones();

		/** Constructs this widget with InArgs */
		void Construct( const FArguments& InArgs, const class UObject* Outer, TSharedPtr<class IPropertyHandle> ParentBoneProperty );

		/** Refresh bone name */
		void RefreshName();

		virtual void PostUndo( bool bSuccess ) { RefreshName(); };
		virtual void PostRedo( bool bSuccess ) { RefreshName(); };

private:
		TSharedPtr<IPropertyHandle>	BonePropertyHandle;
		TSharedPtr<SAssetSearchBox>			SearchBox;

		/** Get the bone name to display */
		FText GetBoneName() const;
};


class SAssetSearchBoxForCurves : public SCompoundWidget, public FEditorUndoClient
{
public:
	SLATE_BEGIN_ARGS(SAssetSearchBoxForCurves)
		: _OnTextCommitted()
		, _HintText()
		, _MustMatchPossibleSuggestions(false)
		, _IncludeSocketsForSuggestions(false)
	{}

	/** Where to place the suggestion list */
	SLATE_ARGUMENT(EMenuPlacement, SuggestionListPlacement)

	/** Invoked whenever the text is committed (e.g. user presses enter) */
	SLATE_EVENT(FOnTextCommitted, OnTextCommitted)

	/** Hint text to display for the search text when there is no value */
	SLATE_ATTRIBUTE(FText, HintText)

	/** Whether the SearchBox allow entries that don't match the possible suggestions */
	SLATE_ATTRIBUTE(bool, MustMatchPossibleSuggestions)

	/** Whether the possible suggestions should include sockets */
	SLATE_ATTRIBUTE(bool, IncludeSocketsForSuggestions)

	SLATE_END_ARGS()

	SAssetSearchBoxForCurves();
	~SAssetSearchBoxForCurves();

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, const class USkeleton* InSkeleton, TSharedPtr<class IPropertyHandle> ParentCurveProperty);

	/** Refresh curve name */
	void RefreshName();

	virtual void PostUndo( bool bSuccess ) { RefreshName(); };
	virtual void PostRedo( bool bSuccess ) { RefreshName(); };

private:
	TSharedPtr<IPropertyHandle>	CurveNamePropertyHandle;
	TWeakObjectPtr<USkeleton> Skeleton;
	TSharedPtr<SAssetSearchBox>			SearchBox;

	/** Get the search suggestions */
	TArray<FAssetSearchBoxSuggestion> GetCurveSearchSuggestions() const;

	/** Get the curve name to display */
	FText GetCurveName() const;
};
