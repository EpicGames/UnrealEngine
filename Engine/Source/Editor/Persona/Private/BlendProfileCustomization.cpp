// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlendProfileCustomization.h"

#include "Editor.h"
#include "PropertyHandle.h"
#include "DetailWidgetRow.h"
#include "BlendProfilePicker.h"
#include "Animation/BlendProfile.h"
#include "ISkeletonEditorModule.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Animation/AnimBlueprint.h"
#include "Modules/ModuleManager.h"
#include "Engine/PoseWatch.h"
#include "Animation/BlendSpace.h"

#define LOCTEXT_NAMESPACE "BlendProfileCustomization"

namespace BlendProfileCustomizationNames
{
	
	static const FName UseAsBlendProfileName = FName(TEXT("UseAsBlendProfile"));
	static const FName UseAsBlendMaskName = FName(TEXT("UseAsBlendMask"));
}

TSharedRef<IPropertyTypeCustomization> FBlendProfileCustomization::MakeInstance()
{
	return MakeShareable(new FBlendProfileCustomization);
}

void FBlendProfileCustomization::CustomizeHeader(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	TSharedRef<SWidget> ValueCustomWidget = SNullWidget::NullWidget;

	TArray<UObject*> OuterObjects;
	InStructPropertyHandle->GetOuterObjects(OuterObjects);

	if (OuterObjects.Num() > 0)
	{
		// try to get skeleton from first outer
		if (USkeleton* TargetSkeleton = GetSkeletonFromOuter(OuterObjects[0]))
		{
			TWeakPtr<IPropertyHandle> PropertyPtr(InStructPropertyHandle);

			UObject* PropertyValue = nullptr;
			InStructPropertyHandle->GetValue(PropertyValue);
			UBlendProfile* CurrentProfile = Cast<UBlendProfile>(PropertyValue);

			const bool bUseAsBlendMask = InStructPropertyHandle->GetBoolMetaData(BlendProfileCustomizationNames::UseAsBlendMaskName);
			const bool bUseAsBlendProfile = InStructPropertyHandle->GetBoolMetaData(BlendProfileCustomizationNames::UseAsBlendProfileName);

			// If no mode is defined, show both.
			EBlendProfilePickerMode SupportedBlendProfileModes = (bUseAsBlendMask || bUseAsBlendProfile) ? EBlendProfilePickerMode(0) : EBlendProfilePickerMode::AllModes;
			
			if (bUseAsBlendProfile)
			{
				SupportedBlendProfileModes |= EBlendProfilePickerMode::BlendProfile;
			}
			if (bUseAsBlendMask)
			{
				SupportedBlendProfileModes |= EBlendProfilePickerMode::BlendMask;
			}


			FBlendProfilePickerArgs Args;
			Args.bAllowNew = false;
			Args.bAllowModify = false;
			Args.bAllowClear = true;
			Args.OnBlendProfileSelected = FOnBlendProfileSelected::CreateSP(this, &FBlendProfileCustomization::OnBlendProfileChanged, PropertyPtr);
			Args.InitialProfile = CurrentProfile;
			Args.SupportedBlendProfileModes = SupportedBlendProfileModes;
			Args.PropertyHandle = InStructPropertyHandle;

			ISkeletonEditorModule& SkeletonEditorModule = FModuleManager::Get().LoadModuleChecked<ISkeletonEditorModule>("SkeletonEditor");
			ValueCustomWidget = SkeletonEditorModule.CreateBlendProfilePicker(TargetSkeleton, Args);
		}
	}

	HeaderRow.NameContent()
		[
			InStructPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MinDesiredWidth(125.f)
		.MaxDesiredWidth(400.f) //Slightly wider since expected names are a bit longer if users use BlendProfileModes as suffix
		[
			// If we can't find a skeleton, we can't use custom SWidget. Default to regular one.
			ValueCustomWidget != SNullWidget::NullWidget ? ValueCustomWidget : InStructPropertyHandle->CreatePropertyValueWidget()
		];
}

void FBlendProfileCustomization::OnBlendProfileChanged(UBlendProfile* NewProfile, TWeakPtr<IPropertyHandle> WeakPropertyHandle)
{
	if(!GIsTransacting)
	{
		if (TSharedPtr<IPropertyHandle> PropertyHandle = WeakPropertyHandle.Pin())
		{
			PropertyHandle->SetValue(NewProfile);
		}
	}
}

USkeleton* FBlendProfileCustomization::GetSkeletonFromOuter(const UObject* Outer)
{
	const UAnimBlueprint* AnimBlueprint = nullptr;
	if (const UBlendSpace* BlendSpace = Cast<UBlendSpace>(Outer))
	{
		// Check for blend space graph nodes
		if (!BlendSpace->IsAsset())
		{
			AnimBlueprint = BlendSpace->GetTypedOuter<UAnimBlueprint>();
		}			
	}
	if (const UEdGraphNode* OuterEdGraphNode = Cast<UEdGraphNode>(Outer))
	{
		AnimBlueprint = Cast<UAnimBlueprint>(FBlueprintEditorUtils::FindBlueprintForNode(OuterEdGraphNode));
	}
	else if (const UEdGraph* OuterEdGraph = Cast<UEdGraph>(Outer))
	{
		AnimBlueprint = Cast<UAnimBlueprint>(FBlueprintEditorUtils::FindBlueprintForGraph(OuterEdGraph));
	}
	else if (const UPoseWatch* OuterPoseWatch = Cast<UPoseWatch>(Outer))
	{
		AnimBlueprint = Cast<UAnimBlueprint>(FBlueprintEditorUtils::FindBlueprintForNode(OuterPoseWatch->Node.Get()));
	}
	else if (const UPoseWatchPoseElement* OuterPoseElement = Cast<UPoseWatchPoseElement>(Outer))
	{
		if (const UPoseWatch* OuterPoseElementParent = OuterPoseElement->GetParent())
		{
			AnimBlueprint = Cast<UAnimBlueprint>(FBlueprintEditorUtils::FindBlueprintForNode(OuterPoseElementParent->Node.Get()));
		}
	}

	// If outer belongs to an anim blueprint, grab its skeleton.
	if (AnimBlueprint)
	{
		return AnimBlueprint->TargetSkeleton;
	}

	if (const UAnimationAsset* OuterAnimAsset = Cast<UAnimationAsset>(Outer))
	{
		// If outer is an anim asset, grab the skeleton
		return OuterAnimAsset->GetSkeleton();
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE
