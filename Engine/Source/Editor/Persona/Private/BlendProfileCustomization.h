// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IPropertyTypeCustomization.h"

class IPropertyHandle;

class FBlendProfileCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	virtual void CustomizeHeader(TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	
	// Override to do nothing
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override {}

private:

	void OnBlendProfileChanged(class UBlendProfile* NewProfile, TWeakPtr<IPropertyHandle> WeakPropertyHandle);
	class USkeleton* GetSkeletonFromOuter(const UObject* Outer);
};
