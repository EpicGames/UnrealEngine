// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlendProfileInterfaceWrapperDetailsCustomization.h"

#include "Editor.h"
#include "PropertyHandle.h"
#include "DetailWidgetRow.h"
#include "BlendProfilePicker.h"
#include "Animation/BlendProfile.h"
#include "ISkeletonEditorModule.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Animation/AnimBlueprint.h"
#include "Modules/ModuleManager.h"
#include "Engine/PoseWatch.h"
#include "Animation/BlendSpace.h"
#include "Widgets/SWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "IEditableSkeleton.h"
#include "Widgets/SCompoundWidget.h"
#include "EditorUndoClient.h"
#include "BlendProfilePicker.h"
#include "SEnumCombo.h"
#include "PersonaModule.h"
#include "IBlendProfilePickerExtender.h"

#define LOCTEXT_NAMESPACE "BlendProfileStandaloneCustomization"

class UBlendProfile;
enum class EBlendProfileMode : uint8;

class SBlendProfileInterfaceWrapperPicker : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SBlendProfileInterfaceWrapperPicker)
	{
	}
	SLATE_END_ARGS()

	struct FPickerArgs
	{
		FPickerArgs()
			: SupportedBlendProfileModes(EBlendProfilePickerMode::AllModes)
			, Outer(nullptr)
		{
		}

		DECLARE_DELEGATE_OneParam(FOnBlendProfileChosen, UBlendProfile*)

		TObjectPtr<USkeleton> Skeleton;

		IBlendProfilePickerExtender::FPickerWidgetArgs::FOnBlendProfileProviderChanged OnProviderChanged;

		FOnBlendProfileChosen OnBlendProfileChosen;

		TSharedPtr<class IPropertyHandle> PropertyHandle;

		EBlendProfilePickerMode SupportedBlendProfileModes;

		TObjectPtr<UObject> Outer;
	};

	void Construct(const FArguments& InArgs, FPickerArgs ConstructArgs)
	{
		ISkeletonEditorModule& SkeletonEditorModule = FModuleManager::Get().LoadModuleChecked<ISkeletonEditorModule>("SkeletonEditor");

		FBlendProfileInterfaceWrapper* BlendProfileInterface = nullptr;
		{
			void* StructAddress = nullptr;
			if (ConstructArgs.PropertyHandle->GetValueData(/*out*/ StructAddress) == FPropertyAccess::Success)
			{
				BlendProfileInterface = static_cast<FBlendProfileInterfaceWrapper*>(StructAddress);
			}
		}
		check(BlendProfileInterface);

		CustomSource = "Skeleton";
		CustomSourceText = LOCTEXT("Skeleton", "Skeleton");

		FPersonaModule& PersonaModule = FModuleManager::GetModuleChecked<FPersonaModule>("Persona");
		const TArray<TSharedPtr<IBlendProfilePickerExtender>> BlendProfileExtenders = PersonaModule.GetCustomBlendProfiles();

		// Init CustomSources
		{
			CustomSources.Add("Skeleton");

			for (const TSharedPtr<IBlendProfilePickerExtender>& BlendProfileExtender : BlendProfileExtenders)
			{
				CustomSources.Add(BlendProfileExtender->GetId());

				if (!BlendProfileInterface->UsesSkeletonBlendProfile() && BlendProfileExtender->OwnsBlendProfileProvider(BlendProfileInterface->GetCustomProviderObject()))
				{
					CustomSource = BlendProfileExtender->GetId();
					CustomSourceText = BlendProfileExtender->GetDisplayName();
				}
			}
		}

		TSharedPtr<SVerticalBox> VerticalBox;

		ChildSlot
			[
				SAssignNew(VerticalBox, SVerticalBox)
			];

		// Add dropdown to toggle between blend profile types i.e. [Skeleton / Custom1 / ...]
		VerticalBox->AddSlot()
			.AutoHeight()
			[
				SNew(SComboButton)
					.OnGetMenuContent_Lambda([this, ConstructArgs, BlendProfileExtenders]()
						{
							FMenuBuilder MenuBuilder(true, NULL);

							// Action for switching to using skeleton blend profiles
							{
								FUIAction Action(FExecuteAction::CreateLambda([this, ConstructArgs]()
									{
										CustomSource = "Skeleton";
										CustomSourceText = LOCTEXT("Skeleton", "Skeleton");
										CustomWidgetBox->SetContent(SNullWidget::NullWidget);
										ConstructArgs.OnProviderChanged.ExecuteIfBound(nullptr, nullptr);
									}));

								MenuBuilder.AddMenuEntry(LOCTEXT("Skeleton", "Skeleton"), FText::GetEmpty(), FSlateIcon(), Action);
							}

							for (const TSharedPtr<IBlendProfilePickerExtender>& BlendProfileExtender : BlendProfileExtenders)
							{
								FUIAction Action(FExecuteAction::CreateLambda([this, BlendProfileExtender, ConstructArgs]()
									{
										CustomSource = BlendProfileExtender->GetId();
										CustomSourceText = BlendProfileExtender->GetDisplayName();

										IBlendProfilePickerExtender::FPickerWidgetArgs Args;
										Args.InitialSelection = nullptr;
										Args.Outer = ConstructArgs.Outer;
										Args.SupportedBlendProfileModes = ConstructArgs.SupportedBlendProfileModes;
										Args.Skeleton = ConstructArgs.Skeleton;
										Args.OnProviderChanged = IBlendProfilePickerExtender::FPickerWidgetArgs::FOnBlendProfileProviderChanged::CreateLambda([this, ConstructArgs](TObjectPtr<UObject> NewSelection, IBlendProfileProviderInterface* Interface)
											{
												ConstructArgs.OnProviderChanged.ExecuteIfBound(NewSelection, Interface);
											});

										CustomWidgetBox->SetContent(BlendProfileExtender->ConstructPickerWidget(Args));
									}));

								MenuBuilder.AddMenuEntry(BlendProfileExtender->GetDisplayName(), FText::GetEmpty(), FSlateIcon(), Action);
							}

							return MenuBuilder.MakeWidget();
						})
					.ButtonContent()
					[
						SNew(STextBlock)
							.Text_Lambda([this]() { return CustomSourceText; })
					]
			];

		// Add picker for skeleton blend profile to display when in skeleton mode
		{
			FBlendProfilePickerArgs SkeletonPickerArgs;
			SkeletonPickerArgs.bAllowNew = false;
			SkeletonPickerArgs.bAllowModify = false;
			SkeletonPickerArgs.bAllowClear = true;
			SkeletonPickerArgs.OnBlendProfileSelected = FOnBlendProfileSelected::CreateLambda([ConstructArgs](UBlendProfile* InBlendProfile)
				{
					ConstructArgs.OnBlendProfileChosen.ExecuteIfBound(InBlendProfile);
				});
			SkeletonPickerArgs.SupportedBlendProfileModes = ConstructArgs.SupportedBlendProfileModes;
			if (BlendProfileInterface->UsesSkeletonBlendProfile())
			{
				SkeletonPickerArgs.InitialProfile = BlendProfileInterface->GetBlendProfile();
			}

			VerticalBox->AddSlot()
				.AutoHeight()
				[
					SNew(SBox)
						.Visibility_Lambda([this, ConstructArgs]() { return CustomSource == "Skeleton" ? EVisibility::Visible : EVisibility::Collapsed; })
						[
							SkeletonEditorModule.CreateBlendProfilePicker(ConstructArgs.Skeleton, SkeletonPickerArgs)
						]
				];
		}

		// Add picker for custom blend profile to display when in a custom mode, widget is mounted when custom mode is selected
		{
			VerticalBox->AddSlot()
				.AutoHeight()
				[
					SAssignNew(CustomWidgetBox, SBox)
						[
							SNullWidget::NullWidget
						]
				];
		}

		// Setup custom widget if custom extender is being used
		if (!BlendProfileInterface->UsesSkeletonBlendProfile())
		{
			for (const TSharedPtr<IBlendProfilePickerExtender>& BlendProfileExtender : BlendProfileExtenders)
			{
				if (BlendProfileExtender->OwnsBlendProfileProvider(BlendProfileInterface->GetCustomProviderObject()))
				{
					IBlendProfilePickerExtender::FPickerWidgetArgs PickerArgs;
					PickerArgs.InitialSelection = BlendProfileInterface->GetCustomProviderObject();
					PickerArgs.OnProviderChanged = IBlendProfilePickerExtender::FPickerWidgetArgs::FOnBlendProfileProviderChanged::CreateLambda([this, ConstructArgs](TObjectPtr<UObject> NewSelection, IBlendProfileProviderInterface* Interface)
						{
							ConstructArgs.OnProviderChanged.ExecuteIfBound(NewSelection, Interface);
						});
					PickerArgs.Outer = ConstructArgs.Outer;
					PickerArgs.SupportedBlendProfileModes = ConstructArgs.SupportedBlendProfileModes;
					PickerArgs.Skeleton = ConstructArgs.Skeleton;

					CustomWidgetBox->SetContent(BlendProfileExtender->ConstructPickerWidget(PickerArgs));
				}
			}
		}
	}

private:
	TSharedPtr<SBox> CustomWidgetBox;
	TArray<FName> CustomSources;

	FName CustomSource = NAME_None;
	FText CustomSourceText;
};


TSharedRef<IPropertyTypeCustomization> FBlendProfileInterfaceWrapperCustomization::MakeInstance()
{
	return MakeShareable(new FBlendProfileInterfaceWrapperCustomization);
}

void FBlendProfileInterfaceWrapperCustomization::CustomizeHeader(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	TSharedRef<SWidget> ValueCustomWidget = SNullWidget::NullWidget;

	TArray<UObject*> OuterObjects;
	InStructPropertyHandle->GetOuterObjects(OuterObjects);

	if (OuterObjects.Num() > 0)
	{
		// try to get skeleton from first outer
		if (USkeleton* TargetSkeleton = GetSkeletonFromOuter(OuterObjects[0]))
		{
			TWeakPtr<IPropertyHandle> PropertyPtr(InStructPropertyHandle);

			const bool bUseAsBlendMask = InStructPropertyHandle->GetBoolMetaData("UseAsBlendMask");
			const bool bUseAsBlendProfile = InStructPropertyHandle->GetBoolMetaData("UseAsBlendProfile");

			// If no mode is defined, show both.
			EBlendProfilePickerMode SupportedBlendProfileModes = (bUseAsBlendMask || bUseAsBlendProfile) ? EBlendProfilePickerMode(0) : EBlendProfilePickerMode::AllModes;

			if (bUseAsBlendProfile)
			{
				SupportedBlendProfileModes |= EBlendProfilePickerMode::BlendProfile;
			}
			if (bUseAsBlendMask)
			{
				SupportedBlendProfileModes |= EBlendProfilePickerMode::BlendMask;
			}


			/*
			FBlendProfilePickerArgs Args;
			Args.bAllowNew = false;
			Args.bAllowModify = false;
			Args.bAllowClear = true;
			//Args.OnBlendProfileSelected = FOnBlendProfileSelected::CreateSP(this, &FBlendProfileInterfaceWrapperCustomization::OnBlendProfileChanged, PropertyPtr);
			Args.InitialProfile = CurrentProfile;
			Args.SupportedBlendProfileModes = SupportedBlendProfileModes;
			Args.PropertyHandle = InStructPropertyHandle;
			*/

			//ISkeletonEditorModule& SkeletonEditorModule = FModuleManager::Get().LoadModuleChecked<ISkeletonEditorModule>("SkeletonEditor");
			//ValueCustomWidget = SkeletonEditorModule.CreateBlendProfilePicker(TargetSkeleton, Args);

			SBlendProfileInterfaceWrapperPicker::FPickerArgs WrapperArgs;
			WrapperArgs.Skeleton = TargetSkeleton;
			WrapperArgs.SupportedBlendProfileModes = SupportedBlendProfileModes;
			WrapperArgs.PropertyHandle = InStructPropertyHandle;
			WrapperArgs.Outer = OuterObjects[0];
			WrapperArgs.OnProviderChanged = IBlendProfilePickerExtender::FPickerWidgetArgs::FOnBlendProfileProviderChanged::CreateSP(this, &FBlendProfileInterfaceWrapperCustomization::OnBlendProfileProviderChanged, PropertyPtr, OuterObjects[0]);
			WrapperArgs.OnBlendProfileChosen = SBlendProfileInterfaceWrapperPicker::FPickerArgs::FOnBlendProfileChosen::CreateSP(this, &FBlendProfileInterfaceWrapperCustomization::OnBlendProfileChanged, PropertyPtr, OuterObjects[0]);

			ValueCustomWidget = SNew(SBlendProfileInterfaceWrapperPicker, WrapperArgs);
		}
	}

	HeaderRow.NameContent()
		[
			InStructPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MinDesiredWidth(125.f)
		.MaxDesiredWidth(400.f) //Slightly wider since expected names are a bit longer if users use BlendProfileModes as suffix
		[
			ValueCustomWidget
		];
}

void FBlendProfileInterfaceWrapperCustomization::OnBlendProfileChanged(UBlendProfile* NewProfile, TWeakPtr<IPropertyHandle> WeakPropertyHandle, UObject* Outer)
{
	if (!GIsTransacting)
	{
		if (TSharedPtr<IPropertyHandle> PropertyHandle = WeakPropertyHandle.Pin())
		{
			FBlendProfileInterfaceWrapper* BlendProfileInterface = nullptr;
			{
				void* StructAddress = nullptr;
				if (PropertyHandle->GetValueData(/*out*/ StructAddress) == FPropertyAccess::Success)
				{
					BlendProfileInterface = static_cast<FBlendProfileInterfaceWrapper*>(StructAddress);
				}
			}

			BlendProfileInterface->SetSkeletonBlendProfile(NewProfile);
		}
	}
}

void FBlendProfileInterfaceWrapperCustomization::OnBlendProfileProviderChanged(TObjectPtr<UObject> NewProfile, IBlendProfileProviderInterface* Interface, TWeakPtr<IPropertyHandle> WeakPropertyHandle, UObject* Outer)
{
	if (!GIsTransacting)
	{
		if (TSharedPtr<IPropertyHandle> PropertyHandle = WeakPropertyHandle.Pin())
		{
			FBlendProfileInterfaceWrapper* BlendProfileInterface = nullptr;
			{
				void* StructAddress = nullptr;
				if (PropertyHandle->GetValueData(/*out*/ StructAddress) == FPropertyAccess::Success)
				{
					BlendProfileInterface = static_cast<FBlendProfileInterfaceWrapper*>(StructAddress);
				}
			}

			BlendProfileInterface->SetBlendProfileProvider(NewProfile, Interface, Outer);
		}
	}
}

USkeleton* FBlendProfileInterfaceWrapperCustomization::GetSkeletonFromOuter(const UObject* Outer)
{
	const UAnimBlueprint* AnimBlueprint = nullptr;
	if (const UBlendSpace* BlendSpace = Cast<UBlendSpace>(Outer))
	{
		// Check for blend space graph nodes
		if (!BlendSpace->IsAsset())
		{
			AnimBlueprint = BlendSpace->GetTypedOuter<UAnimBlueprint>();
		}
	}
	if (const UEdGraphNode* OuterEdGraphNode = Cast<UEdGraphNode>(Outer))
	{
		AnimBlueprint = Cast<UAnimBlueprint>(FBlueprintEditorUtils::FindBlueprintForNode(OuterEdGraphNode));
	}
	else if (const UEdGraph* OuterEdGraph = Cast<UEdGraph>(Outer))
	{
		AnimBlueprint = Cast<UAnimBlueprint>(FBlueprintEditorUtils::FindBlueprintForGraph(OuterEdGraph));
	}

	if (AnimBlueprint)
	{
		return AnimBlueprint->TargetSkeleton;
	}

	if (const UAnimationAsset* OuterAnimAsset = Cast<UAnimationAsset>(Outer))
	{
		return OuterAnimAsset->GetSkeleton();
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE