// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IPropertyTypeCustomization.h"

class IBlendProfileProviderInterface;
class UBlendProfile;

class FBlendProfileInterfaceWrapperCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	virtual void CustomizeHeader(TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override {}

private:
	void OnBlendProfileChanged(UBlendProfile* NewProfile, TWeakPtr<IPropertyHandle> WeakPropertyHandle, UObject* Outer);

	void OnBlendProfileProviderChanged(TObjectPtr<UObject> NewProfile, IBlendProfileProviderInterface* Interface, TWeakPtr<IPropertyHandle> WeakPropertyHandle, UObject* Outer);

	class USkeleton* GetSkeletonFromOuter(const UObject* Outer);
};
