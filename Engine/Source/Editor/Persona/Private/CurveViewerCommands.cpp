// Copyright Epic Games, Inc. All Rights Reserved.

#include "CurveViewerCommands.h"

#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"

#define LOCTEXT_NAMESPACE "CurveViewerCommands"

void FCurveViewerCommands::RegisterCommands()
{
	UI_COMMAND( AddCurve, "Add Curve", "Add a curve to the Skeleton", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND( FindCurveUses, "Find Curve Uses", "Open the find/replace tab to find all uses of this curve", EUserInterfaceActionType::Button, FInputChord(EKeys::F3));
}

#undef LOCTEXT_NAMESPACE
