// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "HAL/Platform.h"
#include "Internationalization/Internationalization.h"
#include "Styling/AppStyle.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

class FUICommandInfo;

/**
* Class containing commands for curve viewer actions
*/
class FCurveViewerCommands : public TCommands<FCurveViewerCommands>
{
public:
	FCurveViewerCommands()
	: TCommands<FCurveViewerCommands>
		(
			TEXT("CurveViewer"), // Context name for fast lookup
			NSLOCTEXT("Contexts", "CurveViewer", "Curve Viewer"), // Localized context name for displaying
			NAME_None, // Parent context name.  
			FAppStyle::GetAppStyleSetName() // Icon Style Set
		)
	{}

	/** Initialize commands */
	virtual void RegisterCommands() override;

	/** Add curve */
	TSharedPtr< FUICommandInfo > AddCurve;

	/** Fund curve uses */
	TSharedPtr< FUICommandInfo > FindCurveUses;
};


