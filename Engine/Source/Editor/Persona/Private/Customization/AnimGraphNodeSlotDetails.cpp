// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customization/AnimGraphNodeSlotDetails.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/SBoxPanel.h"
#include "Animation/Skeleton.h"
#include "Styling/AppStyle.h"
#include "DetailWidgetRow.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "AnimGraphNode_Base.h"
#include "ScopedTransaction.h"
#include "Widgets/Input/STextComboBox.h"
#include "Widgets/Input/STextEntryPopup.h"
#include "TabSpawners.h"

#define LOCTEXT_NAMESPACE "AnimNodeSlotDetails"

////////////////////////////////////////////////////////////////
FAnimGraphNodeSlotDetails::FAnimGraphNodeSlotDetails(FOnInvokeTab InOnInvokeTab)
	: OnInvokeTab(InOnInvokeTab)
{
}

void FAnimGraphNodeSlotDetails::CustomizeDetails(class IDetailLayoutBuilder& DetailBuilder)
{
	SlotNodeNamePropertyHandle = DetailBuilder.GetProperty(TEXT("Node.SlotName"));

	TArray<UObject*> OuterObjects;
	SlotNodeNamePropertyHandle->GetOuterObjects(OuterObjects);

	check(SlotNodeNamePropertyHandle.IsValid());

	Skeleton = nullptr;

	// only support one object, it doesn't make sense to have same slot node name in multiple nodes
	for (auto Object : OuterObjects)
	{
		// if not this object, we have problem 
		const UAnimGraphNode_Base* Node = CastChecked<UAnimGraphNode_Base>(Object);
		const UAnimBlueprint* AnimBlueprint = Node->GetAnimBlueprint();
		if (AnimBlueprint)
		{
			// if skeleton is already set and it's not same as target skeleton, we have some problem ,return
			if (Skeleton != NULL && Skeleton == AnimBlueprint->TargetSkeleton)
			{
				// abort
				return;
			}

			Skeleton = AnimBlueprint->TargetSkeleton;
		}
	}

	if(Skeleton == nullptr)
	{
		IDetailCategoryBuilder& SlotNameGroup = DetailBuilder.EditCategory(TEXT("Settings"));
		SlotNameGroup.AddCustomRow(LOCTEXT("SlotNameTitleLabel", "Slot Name"))
		.NameContent()
		[
			SlotNodeNamePropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SlotNodeNamePropertyHandle->CreatePropertyValueWidget()
		];
	}
	else
	{
		DetailBuilder.HideProperty(SlotNodeNamePropertyHandle);
		
		// this is used for 2 things, to create name, but also for another pop-up box to show off, it's a bit hacky to use this, but I need widget to parent on
		const TSharedRef<SWidget> SlotNodePropertyNameWidget = SlotNodeNamePropertyHandle->CreatePropertyNameWidget();

		// fill combo with groups and slots
		RefreshComboLists();

		const int32 FoundIndex = SlotNameList.Find(SlotNameComboSelectedName);
		const TSharedPtr<FString> ComboBoxSelectedItem = SlotNameComboListItems[FoundIndex];

		// create combo box
		IDetailCategoryBuilder& SlotNameGroup = DetailBuilder.EditCategory(TEXT("Settings"));
		SlotNameGroup.AddCustomRow(LOCTEXT("SlotNameTitleLabel", "Slot Name"))
		.NameContent()
		[
			SlotNodePropertyNameWidget
		]
		.ValueContent()
		.MinDesiredWidth(125.f * 3.f)
		.MaxDesiredWidth(125.f * 3.f)
		[
			SNew(SHorizontalBox)

			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SAssignNew(SlotNameComboBox, STextComboBox)
				.OptionsSource(&SlotNameComboListItems)
				.OnSelectionChanged(this, &FAnimGraphNodeSlotDetails::OnSlotNameChanged)
				.OnComboBoxOpening(this, &FAnimGraphNodeSlotDetails::OnSlotListOpening)
				.InitiallySelectedItem(ComboBoxSelectedItem)
				.ContentPadding(2.f)
				.ToolTipText(FText::FromString(*ComboBoxSelectedItem))
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
				.Text(LOCTEXT("AnimSlotNode_DetailPanelManageButtonLabel", "Anim Slot Manager"))
				.ToolTipText(LOCTEXT("AnimSlotNode_DetailPanelManageButtonToolTipText", "Open Anim Slot Manager to edit Slots and Groups."))
				.OnClicked(this, &FAnimGraphNodeSlotDetails::OnOpenAnimSlotManager)
				.Content()
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("MeshPaint.FindInCB"))
				]
			]
		];
	}
}

void FAnimGraphNodeSlotDetails::RefreshComboLists(bool bOnlyRefreshIfDifferent /*= false*/)
{
	SlotNodeNamePropertyHandle->GetValue(SlotNameComboSelectedName);

	// Make sure slot node exists in our skeleton.
	Skeleton->RegisterSlotNode(SlotNameComboSelectedName);

	// Refresh Slot Names
	{
		TArray< TSharedPtr< FString > > NewSlotNameComboListItems;
		TArray< FName > NewSlotNameList;
		bool bIsSlotNameListDifferent = false;

		const TArray<FAnimSlotGroup>& SlotGroups = Skeleton->GetSlotGroups();
		for (auto SlotGroup : SlotGroups)
		{
			int32 Index = 0;
			for (auto SlotName : SlotGroup.SlotNames)
			{
				NewSlotNameList.Add(SlotName);

				FString ComboItemString = FString::Printf(TEXT("%s.%s"), *SlotGroup.GroupName.ToString(), *SlotName.ToString());
				NewSlotNameComboListItems.Add(MakeShareable(new FString(ComboItemString)));

				bIsSlotNameListDifferent = bIsSlotNameListDifferent || (!SlotNameComboListItems.IsValidIndex(Index) || (SlotNameComboListItems[Index] != NewSlotNameComboListItems[Index]));
				Index++;
			}
		}

		// Refresh if needed
		if (bIsSlotNameListDifferent || !bOnlyRefreshIfDifferent || (NewSlotNameComboListItems.Num() == 0))
		{
			SlotNameComboListItems = NewSlotNameComboListItems;
			SlotNameList = NewSlotNameList;

			if (SlotNameComboBox.IsValid())
			{
				if (Skeleton->ContainsSlotName(SlotNameComboSelectedName))
				{
					int32 FoundIndex = SlotNameList.Find(SlotNameComboSelectedName);
					TSharedPtr<FString> ComboItem = SlotNameComboListItems[FoundIndex];

					SlotNameComboBox->SetSelectedItem(ComboItem);
					SlotNameComboBox->SetToolTipText(FText::FromString(*ComboItem));
				}
				SlotNameComboBox->RefreshOptions();
			}
		}
	}
}

////////////////////////////////////////////////////////////////

void FAnimGraphNodeSlotDetails::OnSlotNameChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo)
{
	// if it's set from code, we did that on purpose
	if(SelectInfo != ESelectInfo::Direct)
	{
		int32 ItemIndex = SlotNameComboListItems.Find(NewSelection);
		if (ItemIndex != INDEX_NONE)
		{
			SlotNameComboSelectedName = SlotNameList[ItemIndex];
			if (SlotNameComboBox.IsValid())
			{
				SlotNameComboBox->SetToolTipText(FText::FromString(*NewSelection));
			}

			if (Skeleton->ContainsSlotName(SlotNameComboSelectedName))
			{
				// trigger transaction 
				//const FScopedTransaction Transaction(LOCTEXT("ChangeSlotNodeName", "Change Collision Profile"));
				// set profile set up
				ensure(SlotNodeNamePropertyHandle->SetValue(SlotNameComboSelectedName.ToString()) == FPropertyAccess::Result::Success);
			}
		}
	}
}

void FAnimGraphNodeSlotDetails::OnSlotListOpening()
{
	// Refresh Slot Names, in case we used the Anim Slot Manager to make changes.
	RefreshComboLists(true);
}

FReply FAnimGraphNodeSlotDetails::OnOpenAnimSlotManager()
{
	OnInvokeTab.ExecuteIfBound(FPersonaTabs::SkeletonSlotNamesID);
	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
