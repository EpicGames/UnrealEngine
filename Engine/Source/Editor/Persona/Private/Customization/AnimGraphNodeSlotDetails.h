// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "Input/Reply.h"
#include "IDetailCustomization.h"
#include "PersonaDelegates.h"

class IPropertyHandle;
class USkeleton;

/**
 * Customizes a DataTable asset to use a dropdown
 */
class FAnimGraphNodeSlotDetails : public IDetailCustomization
{
public:
	FAnimGraphNodeSlotDetails(FOnInvokeTab InOnInvokeTab);

	static TSharedRef<IDetailCustomization> MakeInstance(FOnInvokeTab InOnInvokeTab) 
	{
		return MakeShareable( new FAnimGraphNodeSlotDetails(InOnInvokeTab) );
	}

	// IDetailCustomization interface
	virtual void CustomizeDetails(class IDetailLayoutBuilder& DetailBuilder) override;
	
private:
	// delegate used to invoke a tab in the containing editor
	FOnInvokeTab OnInvokeTab;

	// property of the two 
	TSharedPtr<IPropertyHandle> SlotNodeNamePropertyHandle;

	// slot node names
	TSharedPtr<class STextComboBox>	SlotNameComboBox;
	TArray< TSharedPtr< FString > > SlotNameComboListItems;
	TArray< FName > SlotNameList;
	FName SlotNameComboSelectedName;

	void OnSlotNameChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo);
	void OnSlotListOpening();

	// slot node names buttons
	FReply OnOpenAnimSlotManager();

	void RefreshComboLists(bool bOnlyRefreshIfDifferent = false);

	USkeleton* Skeleton;
};
