// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customization/BlendParameterDetails.h"

#include "IDetailsView.h"
#include "PropertyHandle.h"
#include "IDetailChildrenBuilder.h"
#include "DetailWidgetRow.h"
#include "Animation/BlendSpace.h"


#define LOCTEXT_NAMESPACE "BlendParameterDetails"

void FBlendParameterDetails::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	uint32 NumChildren;
	StructPropertyHandle->GetNumChildren(NumChildren);

	static const FName RangePropertyNames[NumRangeValues] = { GET_MEMBER_NAME_CHECKED(FBlendParameter, Min), GET_MEMBER_NAME_CHECKED(FBlendParameter, Max) };
	for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
	{
		TSharedRef<IPropertyHandle> ChildHandle = StructPropertyHandle->GetChildHandle(ChildIndex).ToSharedRef();
		const FName ChildName = ChildHandle->GetProperty()->GetFName();

		bool bHandled = false;
		// Pick out Min and Max range value properties
		for (int32 RangeValueIndex = 0; RangeValueIndex < NumRangeValues; ++RangeValueIndex)
		{
			bHandled = (ChildName == RangePropertyNames[RangeValueIndex]);
			if ( bHandled )
			{
				RangeProperties[RangeValueIndex] = ChildHandle;
				bValidRangeValue[RangeValueIndex] = true;
				
				ChildBuilder.AddProperty(ChildHandle)
				.CustomWidget()
				.NameWidget
				[
					ChildHandle->CreatePropertyNameWidget()
				]
				.ValueWidget
				[
					SAssignNew(RangeBoxes[RangeValueIndex], SNumericEntryBox<float>)
					.AllowSpin(false)
					.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
					.Value(this, &FBlendParameterDetails::GetRangeValue, RangeValueIndex)
					.OnValueChanged(this, &FBlendParameterDetails::OnRangeNumValueChanged, RangeValueIndex)
					.OnValueCommitted(this, &FBlendParameterDetails::OnRangeNumValueCommitted, RangeValueIndex)
				];

				break;
			}
		}

		if (!bHandled)
		{
			IDetailPropertyRow& Property = ChildBuilder.AddProperty(ChildHandle);
		}
	}
}

void FBlendParameterDetails::OnRangeNumValueCommitted(float FloatValue, ETextCommit::Type CommitType, const int32 RangeValueIndex) const
{
	// Only commit the new value if it is valid
	if (bValidRangeValue[RangeValueIndex])
	{
		RangeProperties[RangeValueIndex]->SetValue(FloatValue);
	}
}

void FBlendParameterDetails::OnRangeNumValueChanged(float FloatValue, const int32 RangeValueIndex)
{
	// Make sure Max stays bigger than Min (and the other way around)
	float OtherValue;	
	if (RangeProperties[!RangeValueIndex]->GetValue(OtherValue) == FPropertyAccess::Success)
	{
		bValidRangeValue[RangeValueIndex] = true;

		if (RangeValueIndex == 0 && FloatValue >= OtherValue)
		{
			bValidRangeValue[RangeValueIndex] = false;
		}
		else if ( RangeValueIndex == 1 && FloatValue <= OtherValue )
		{
			bValidRangeValue[RangeValueIndex] = false;
		}
	}
}

TOptional<float> FBlendParameterDetails::GetRangeValue(const int32 RangeValueIndex) const
{
	TOptional<float> ReturnValue;	
	float RangeValue = 0.0f;
	if (RangeProperties[RangeValueIndex]->GetValue(RangeValue) == FPropertyAccess::Success)
	{
		ReturnValue = RangeValue;
	}

	return ReturnValue;
}

#undef LOCTEXT_NAMESPACE
