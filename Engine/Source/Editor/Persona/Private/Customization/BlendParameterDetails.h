// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IPropertyTypeCustomization.h"
#include "Widgets/Input/SNumericEntryBox.h"

class IPropertyHandle;

class FBlendParameterDetails : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShareable(new FBlendParameterDetails());
	}

	// IDetailCustomization interface
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override {}
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

private:
	TOptional<float> GetRangeValue(const int32 RangeValueIndex) const;
	void OnRangeNumValueChanged(float FloatValue, const int32 RangeValueIndex);
	void OnRangeNumValueCommitted(float FloatValue, ETextCommit::Type CommitType, const int32 RangeValueIndex) const;

private:
	static const int32 NumRangeValues = 2;
	bool bValidRangeValue[NumRangeValues];

	TSharedPtr<SNumericEntryBox<float>> RangeBoxes[NumRangeValues];
	
	TSharedPtr<IPropertyHandle> RangeProperties[NumRangeValues];
};
