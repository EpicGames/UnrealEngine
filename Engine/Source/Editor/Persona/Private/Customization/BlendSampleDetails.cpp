// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customization/BlendSampleDetails.h"

#include "UObject/ObjectMacros.h"
#include "UObject/Class.h"
#include "IDetailsView.h"
#include "Styling/AppStyle.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "PropertyHandle.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"

#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Input/SVectorInputBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Text/STextBlock.h"

#include "Animation/AnimSequence.h"
#include "Animation/BlendSpace.h"
#include "Animation/BlendSpace1D.h"
#include "SAnimationBlendSpaceGridWidget.h"
#include "PropertyCustomizationHelpers.h"

#include "PackageTools.h"
#include "IDetailGroup.h"
#include "Editor.h"
#include "AnimGraphNode_BlendSpaceGraphBase.h"
#include "Animation/AnimBlueprint.h"
#include "Toolkits/ToolkitManager.h"
#include "BlueprintEditorModule.h"
#include "AnimationGraph.h"
#include "BlendSpaceGraph.h"
#include "PersonaBlendSpaceAnalysis.h"
#include "SAnimationBlendSpaceGridWidget.h"

#define LOCTEXT_NAMESPACE "BlendSampleDetails"

FReply FBlendSampleDetails::HandleAnalyzeAndDuplicateSample()
{
	if (BlendSpace->IsAsset())
	{
		// Dismiss menus so that operations which might affect samples (and cause reallocation etc) won't invalidate
		// data being used in the UI.
		FSlateApplication::Get().DismissAllMenus();
		FVector OrigValue = BlendSpace->GetBlendSample(SampleIndex).SampleValue;
		GridWidget->OnSampleDuplicated.ExecuteIfBound(SampleIndex, OrigValue, true);
	}
	return FReply::Handled();
}

FReply FBlendSampleDetails::HandleAnalyzeAndMoveSample()
{
	if (BlendSpace->IsAsset())
	{
		bool bAnalyzed[3] = { false, false, false };
		FVector OrigValue = BlendSpace->GetBlendSample(SampleIndex).SampleValue;
		FVector NewValue = BlendSpaceAnalysis::CalculateSampleValue(
			*BlendSpace, *BlendSpace->GetBlendSample(SampleIndex).Animation, 
			BlendSpace->GetBlendSample(SampleIndex).RateScale, OrigValue, bAnalyzed);
		NewValue.Z = OrigValue.Z;
		if (NewValue != OrigValue)
		{
			GridWidget->OnSampleMoved.ExecuteIfBound(SampleIndex, NewValue, false);
		}
	}
	return FReply::Handled();
}

FReply FBlendSampleDetails::HandleAnalyzeAndMoveSampleX()
{
	if (BlendSpace->IsAsset())
	{
		bool bAnalyzed[3] = { false, false, false };
		FVector OrigValue = BlendSpace->GetBlendSample(SampleIndex).SampleValue;
		FVector NewValue = BlendSpaceAnalysis::CalculateSampleValue(
			*BlendSpace, *BlendSpace->GetBlendSample(SampleIndex).Animation, 
			BlendSpace->GetBlendSample(SampleIndex).RateScale, OrigValue, bAnalyzed);
		NewValue.Y = OrigValue.Y;
		NewValue.Z = OrigValue.Z;
		if (NewValue != OrigValue)
		{
			GridWidget->OnSampleMoved.ExecuteIfBound(SampleIndex, NewValue, false);
		}
	}
	return FReply::Handled();
}

FReply FBlendSampleDetails::HandleAnalyzeAndMoveSampleY()
{
	if (BlendSpace->IsAsset())
	{
		bool bAnalyzed[3] = { false, false, false };
		FVector OrigValue = BlendSpace->GetBlendSample(SampleIndex).SampleValue;
		FVector NewValue = BlendSpaceAnalysis::CalculateSampleValue(
			*BlendSpace, *BlendSpace->GetBlendSample(SampleIndex).Animation, 
			BlendSpace->GetBlendSample(SampleIndex).RateScale, OrigValue, bAnalyzed);
		NewValue.X = OrigValue.X;
		NewValue.Z = OrigValue.Z;
		if (NewValue != OrigValue)
		{
			GridWidget->OnSampleMoved.ExecuteIfBound(SampleIndex, NewValue, false);
		}
	}
	return FReply::Handled();
}

FBlendSampleDetails::FBlendSampleDetails(const UBlendSpace* InBlendSpace, class SBlendSpaceGridWidget* InGridWidget, int32 InSampleIndex)
	: BlendSpace(InBlendSpace)
	, GridWidget(InGridWidget)
	, SampleIndex(InSampleIndex)
{
	// Retrieve the additive animation type enum
	const UEnum* AdditiveTypeEnum = StaticEnum<EAdditiveAnimationType>();
	// For each type check whether or not the blend space is compatible with it and cache the result
	for (int32 TypeValue = 0; TypeValue < (int32)EAdditiveAnimationType::AAT_MAX; ++TypeValue)
	{
		EAdditiveAnimationType Type = (EAdditiveAnimationType)TypeValue;
		// In case of non additive type make sure the blendspace is made up out of non additive samples only
		const bool bAdditiveFlag = (Type == EAdditiveAnimationType::AAT_None) ? !BlendSpace->IsValidAdditive() : BlendSpace->IsValidAdditive() && BlendSpace->IsValidAdditiveType(Type);		
		bValidAdditiveTypes.Add(AdditiveTypeEnum->GetNameByValue(TypeValue).ToString(), bAdditiveFlag);
	}
}

void FBlendSampleDetails::CustomizeDetails(class IDetailLayoutBuilder& DetailBuilder)
{
	static const FName CategoryName = FName("BlendSample");
	IDetailCategoryBuilder& CategoryBuilder = DetailBuilder.EditCategory(CategoryName);

	// Hide default properties
	TArray<TSharedRef<IPropertyHandle> > DefaultProperties;
	CategoryBuilder.GetDefaultProperties(DefaultProperties);

	// Hide all default properties
	for (TSharedRef<IPropertyHandle> Property : DefaultProperties)
	{
		Property->MarkHiddenByCustomization();
	}
	
	TArray < TSharedPtr<FStructOnScope>> Structs;
	DetailBuilder.GetStructsBeingCustomized(Structs);

	TArray<UPackage*> Packages;
	for ( TSharedPtr<FStructOnScope>& Struct : Structs)
	{	
		Packages.Add(Struct->GetPackage());
	}
	
	TArray<UObject*> Objects;
	UPackageTools::GetObjectsInPackages(&Packages, Objects);

	const UBlendSpace* BlendSpaceBase = nullptr;
	// Find blendspace in found objects
	for ( UObject* Object : Objects )
	{
		BlendSpaceBase = Cast<UBlendSpace>(Object);
		if (BlendSpaceBase)
		{
			break;
		}
	}

	const UBlendSpace* BlendspaceToUse = BlendSpaceBase != nullptr ? BlendSpaceBase : BlendSpace;
	UAnimGraphNode_BlendSpaceGraphBase* BlendSpaceNode = nullptr;
	if(UBlendSpaceGraph* BlendSpaceGraph = Cast<UBlendSpaceGraph>(BlendspaceToUse->GetOuter()))
	{
		check(BlendspaceToUse == BlendSpaceGraph->BlendSpace);
		BlendSpaceNode = CastChecked<UAnimGraphNode_BlendSpaceGraphBase>(BlendSpaceGraph->GetOuter());
	}

	// Sample value
	FBlendSampleDetails::GenerateBlendSampleWidget(
		[&CategoryBuilder, &DetailBuilder]() -> IDetailPropertyRow&
		{
			TSharedPtr<IPropertyHandle> SampleProperty = DetailBuilder.GetProperty(
				GET_MEMBER_NAME_CHECKED(FBlendSample, SampleValue), FBlendSample::StaticStruct());
			return CategoryBuilder.AddProperty(SampleProperty);
		}, GridWidget->OnSampleMoved, (BlendSpaceBase != nullptr) ? BlendSpaceBase : BlendSpace,
		GridWidget->GetSelectedSampleIndex(), false);

	// Animation and rate
	if(BlendspaceToUse->IsAsset())
	{
		TSharedPtr<IPropertyHandle> AnimationProperty = DetailBuilder.GetProperty(
			GET_MEMBER_NAME_CHECKED(FBlendSample, Animation), FBlendSample::StaticStruct());

		IDetailPropertyRow& AnimationRow = CategoryBuilder.AddProperty(AnimationProperty);
		FBlendSampleDetails::GenerateAnimationWidget(AnimationRow, BlendspaceToUse, AnimationProperty);

		TSharedPtr<IPropertyHandle> RateScaleProperty = DetailBuilder.GetProperty(
			GET_MEMBER_NAME_CHECKED(FBlendSample, RateScale), FBlendSample::StaticStruct());
		CategoryBuilder.AddProperty(RateScaleProperty);

		TSharedPtr<IPropertyHandle> UseSingleFrameBlendingProperty = DetailBuilder.GetProperty(
			GET_MEMBER_NAME_CHECKED(FBlendSample, FrameIndexToSample), FBlendSample::StaticStruct());
			 
		IDetailPropertyRow& UseSingleFrameBlendingPropertyRow = CategoryBuilder.AddProperty(UseSingleFrameBlendingProperty);
		FBlendSampleDetails::GenerateSingleFrameBlendingWidget(UseSingleFrameBlendingPropertyRow, UseSingleFrameBlendingProperty, AnimationProperty);

		TSharedPtr<IPropertyHandle> IncludeInAnalyseAllProperty = DetailBuilder.GetProperty(
			GET_MEMBER_NAME_CHECKED(FBlendSample, bIncludeInAnalyseAll), FBlendSample::StaticStruct());
		CategoryBuilder.AddProperty(IncludeInAnalyseAllProperty);

		TSharedPtr<IPropertyHandle> MirrorProperty = DetailBuilder.GetProperty(
			GET_MEMBER_NAME_CHECKED(FBlendSample, bMirror), FBlendSample::StaticStruct());
		IDetailPropertyRow& MirrorPropertyRow = CategoryBuilder.AddProperty(MirrorProperty);

		TAttribute<bool> IsMirrorDataTableValid = TAttribute<bool>::CreateLambda([this]() { return BlendSpace->MirrorDataTable != nullptr; });
		MirrorPropertyRow.EditCondition(IsMirrorDataTableValid, nullptr);
		
		bool bShowAnalysis = false;
		bool bAnalyzed[3] = { false, false, false };
		// Note that the analyzed position won't change whilst this menu is open, but the original value might.
		FVector OrigValue = BlendSpace->GetBlendSample(SampleIndex).SampleValue;
		FVector NewValue = BlendSpaceAnalysis::CalculateSampleValue(
			*BlendSpace, *BlendSpace->GetBlendSample(SampleIndex).Animation, 
			BlendSpace->GetBlendSample(SampleIndex).RateScale, OrigValue, bAnalyzed);
		FText AnalysisTexts[2];
		FText ValueTexts[2];
		FText ToolTipTexts[2];

		for (int32 Index = 0 ; Index != 2 ; ++Index)
		{
			if (bAnalyzed[Index])
			{
				AnalysisTexts[Index] = FText::Format(FTextFormat(
					LOCTEXT("AnalysisSampleValue", "Set {0}")), 
					FText::FromString(BlendSpace->GetBlendParameter(Index).DisplayName));
				ToolTipTexts[Index] = FText::Format(FTextFormat(
					LOCTEXT("AnalysisSampleToolTip", "Set {0} to the analysed value {1}")), 
					FText::FromString(BlendSpace->GetBlendParameter(Index).DisplayName),
					NewValue[Index]);
				ValueTexts[Index] = FText::Format(FTextFormat(
					LOCTEXT("AnalysisValue", "{0}")), NewValue[Index]);
				bShowAnalysis = true;
			}
			else
			{
				AnalysisTexts[Index] = LOCTEXT("Unanalyzed", "Not analyzed");
				ToolTipTexts[Index] = FText::Format(FTextFormat(
					LOCTEXT("AnalysisSampleToolTipUnanalyzed", "Analysis has not been set for {0}")), 
					FText::FromString(BlendSpace->GetBlendParameter(Index).DisplayName));
			}
		}

		TAttribute<bool> WouldMoveConditionX = TAttribute<bool>::Create(
			[this, bAnalyzed, NewValue]()
			{
				FVector OrigValue = BlendSpace->GetBlendSample(SampleIndex).SampleValue;
				return bAnalyzed[0] && OrigValue.X != NewValue.X;
			});
		TAttribute<bool> WouldMoveConditionY = TAttribute<bool>::Create(
			[this, bAnalyzed, NewValue]()
			{
				FVector OrigValue = BlendSpace->GetBlendSample(SampleIndex).SampleValue;
				return bAnalyzed[1] && OrigValue.Y != NewValue.Y;
			});
		TAttribute<bool> WouldMoveCondition = TAttribute<bool>::Create(
			[this, NewValue]()
			{
				FVector OrigValue = BlendSpace->GetBlendSample(SampleIndex).SampleValue;
				return OrigValue != NewValue;
			});

		const bool b1DBlendSpace = BlendSpace->IsA<UBlendSpace1D>();

		if (bShowAnalysis)
		{
			// Move buttons
			CategoryBuilder
				.AddGroup(FName("MoveGroup"), FText::GetEmpty())
				.HeaderRow()
				.NameContent()
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					[
						SNew(SButton)
						.IsEnabled(WouldMoveCondition)
						.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
						.ToolTipText(LOCTEXT("MoveText", "Move this sample to the analyzed position"))
						.OnClicked(this, &FBlendSampleDetails::HandleAnalyzeAndMoveSample)
						[
							SNew(SHorizontalBox)
							+SHorizontalBox::Slot()
							.Padding(3.0f, 0.0f)
							.AutoWidth()
							[
								SNew(SImage)
								.Image(FAppStyle::GetBrush("Icons.ArrowRight"))
								.ColorAndOpacity(FSlateColor::UseForeground())
							]
							+SHorizontalBox::Slot()
							.Padding(3.0f, 0.0f)
							.VAlign(VAlign_Center)
							[
								SNew(STextBlock)
								.Font(DetailBuilder.GetDetailFont())
								.Text(LOCTEXT("MoveLabel", "Move"))
							]
						]
					]
				]
				.ValueContent()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Center)
				[
					SNew(SUniformGridPanel)
					+SUniformGridPanel::Slot(0,0)
					.HAlign(HAlign_Fill)
					[   // Analyze X button
						SNew(SButton)
						.IsEnabled(WouldMoveConditionX)
						.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
						.ToolTipText(ToolTipTexts[0])
						.OnClicked(this, &FBlendSampleDetails::HandleAnalyzeAndMoveSampleX)
						[
							SNew(STextBlock)
							.Justification(ETextJustify::Center)
							.Text(AnalysisTexts[0])
						]
					]
					+SUniformGridPanel::Slot(1,0)
					.HAlign(HAlign_Fill)
					[   // Analyze Y button
						SNew(SButton)
						.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
						.ToolTipText(ToolTipTexts[1])
						.Visibility(b1DBlendSpace ? EVisibility::Hidden : EVisibility::Visible)
						.IsEnabled(WouldMoveConditionY) // Needs to be after visibility
						.OnClicked(this, &FBlendSampleDetails::HandleAnalyzeAndMoveSampleY)
						[
							SNew(STextBlock)
							.Justification(ETextJustify::Center)
							.Text(AnalysisTexts[1])
						]
					]
				];

			// Button to duplicate analysis
			CategoryBuilder
				.AddGroup(FName("DuplicateGroup"), FText::GetEmpty())
				.HeaderRow()
				.NameContent()
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					[
						SNew(SButton)
						.IsEnabled(WouldMoveCondition)
						.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
						.ToolTipText(LOCTEXT("DuplicateText", "Duplicate this sample and place it at the analyzed position"))
						.OnClicked(this, &FBlendSampleDetails::HandleAnalyzeAndDuplicateSample)
						[
							SNew(SHorizontalBox)
							+SHorizontalBox::Slot()
							.Padding(3.0f, 0.0f)
							.AutoWidth()
							[
								SNew(SImage)
								.Image(FAppStyle::GetBrush("Icons.Duplicate"))
								.ColorAndOpacity(FSlateColor::UseForeground())
							]
							+SHorizontalBox::Slot()
							.Padding(3.0f, 0.0f)
							.VAlign(VAlign_Center)
							[
								SNew(STextBlock)
								.Font(DetailBuilder.GetDetailFont())
								.Text(LOCTEXT("DuplicateLabel", "Duplicate"))
							]
						]
					]
				]
				.ValueContent()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Center)
				[
					SNew(SUniformGridPanel)
					+SUniformGridPanel::Slot(0,0)
					.HAlign(HAlign_Center)
					[
						SNew(STextBlock)
						.Text(ValueTexts[0])
					]
					+SUniformGridPanel::Slot(1,0)
					.HAlign(HAlign_Center)
					[
						SNew(STextBlock)
						.Text(ValueTexts[1])
					]
				];

		}
	}
	else
	{
		// Expose mirror option
		TSharedPtr<IPropertyHandle> MirrorProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(FBlendSample, bMirror), FBlendSample::StaticStruct());
		IDetailPropertyRow& MirrorPropertyRow = CategoryBuilder.AddProperty(MirrorProperty);

		TAttribute<bool> IsMirrorDataTableValid = TAttribute<bool>::CreateLambda([this]() { return BlendSpace->MirrorDataTable != nullptr; });
		MirrorPropertyRow.EditCondition(IsMirrorDataTableValid, nullptr);
		
		// Edit graph
		check(BlendSpaceNode);
		FDetailWidgetRow& GraphRow = CategoryBuilder.AddCustomRow(FText::FromString(TEXT("Graph")));
		FBlendSampleDetails::GenerateSampleGraphWidget(GraphRow, BlendSpaceNode, SampleIndex);
	}
}

void FBlendSampleDetails::GenerateBlendSampleWidget(TFunction<IDetailPropertyRow& (void)> InFunctor, FOnSampleMoved OnSampleMoved, const UBlendSpace* BlendSpace, const int32 SampleIndex, bool bShowLabel)
{
	const int32 NumParameters = BlendSpace->IsA<UBlendSpace1D>() ? 1 : 2;
	for (int32 ParameterIndex = 0; ParameterIndex < NumParameters; ++ParameterIndex)
	{
		auto ValueChangedLambda = [BlendSpace, SampleIndex, ParameterIndex, OnSampleMoved](const float NewValue, bool bIsInteractive)
		{
			const FBlendParameter& BlendParameter = BlendSpace->GetBlendParameter(ParameterIndex);
			const FBlendSample& Sample = BlendSpace->GetBlendSample(SampleIndex);
			FVector SampleValue = Sample.SampleValue;
			SampleValue[ParameterIndex] = NewValue;
			OnSampleMoved.ExecuteIfBound(SampleIndex, SampleValue, bIsInteractive);		
		};
		
		IDetailPropertyRow& ParameterRow = InFunctor();

		ParameterRow.CustomWidget()
		.NameContent()
		[
			SNew(STextBlock)
			.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
			.Text_Lambda([BlendSpace, ParameterIndex]()
				{ 

					return FText::FromString(BlendSpace->GetBlendParameter(ParameterIndex).DisplayName);
				})
		]
		.ValueContent()
		[
			SNew(SNumericEntryBox<float>)
			.Font(FAppStyle::GetFontStyle("CurveEd.InfoFont"))
			.Value_Lambda(
			[BlendSpace, SampleIndex, ParameterIndex]() -> float
			{
				if (BlendSpace)
				{
					return BlendSpace->IsValidBlendSampleIndex(SampleIndex) ? static_cast<float>(BlendSpace->GetBlendSample(SampleIndex).SampleValue[ParameterIndex]) : 0.0f;
				}

				return 0.0f;
			})
			.UndeterminedString(LOCTEXT("MultipleValues", "Multiple Values"))
			.OnBeginSliderMovement_Lambda([]()
			{
				GEditor->BeginTransaction(LOCTEXT("MoveSample", "Moving Blend Grid Sample"));
			})
			.OnEndSliderMovement_Lambda([](const float NewValue)
			{
				GEditor->EndTransaction();
			})
			.OnValueCommitted_Lambda([ValueChangedLambda](const float NewValue, ETextCommit::Type CommitType)
			{
				ValueChangedLambda(NewValue, false);
			})
			.OnValueChanged_Lambda([ValueChangedLambda](const float NewValue)
			{
				ValueChangedLambda(NewValue, true);
			})
			.LabelVAlign(VAlign_Center)
			.AllowSpin(true)
			.MinValue_Lambda([BlendSpace, ParameterIndex]() -> float { return BlendSpace->GetBlendParameter(ParameterIndex).Min; })
			.MaxValue_Lambda([BlendSpace, ParameterIndex]() -> float { return BlendSpace->GetBlendParameter(ParameterIndex).Max; })
			.MinSliderValue_Lambda([BlendSpace, ParameterIndex]() -> float { return BlendSpace->GetBlendParameter(ParameterIndex).Min; })
			.MaxSliderValue_Lambda([BlendSpace, ParameterIndex]() -> float { return BlendSpace->GetBlendParameter(ParameterIndex).Max; })
			.MinDesiredValueWidth(60.0f)
			.Label()
			[
				SNew(STextBlock)
				.Visibility(bShowLabel ? EVisibility::Visible : EVisibility::Collapsed)
				.Text_Lambda([BlendSpace, ParameterIndex]() { return FText::FromString(BlendSpace->GetBlendParameter(ParameterIndex).DisplayName); })
			]
		];
	}
}

void FBlendSampleDetails::GenerateAnimationWidget(IDetailPropertyRow& PropertyRow, const UBlendSpace* BlendSpace, TSharedPtr<IPropertyHandle> AnimationProperty)
{
	PropertyRow.CustomWidget()
	.NameContent()
		[
			AnimationProperty->CreatePropertyNameWidget()
		]
	.ValueContent()
		.MinDesiredWidth(250.f)
		[
			SNew(SObjectPropertyEntryBox)
			.AllowedClass(UAnimSequence::StaticClass())
			.OnShouldFilterAsset(FOnShouldFilterAsset::CreateStatic(&FBlendSampleDetails::ShouldFilterAssetStatic, BlendSpace))
			.PropertyHandle(AnimationProperty)
		];
}

void FBlendSampleDetails::GenerateSingleFrameBlendingWidget(IDetailPropertyRow& PropertyRow, TSharedPtr<IPropertyHandle> SingleFrameIndexProperty, TSharedPtr<IPropertyHandle> AnimationProperty)
{
	auto ValueChangedLambda = [SingleFrameIndexProperty](uint32 NewValue, bool bIsInteractive)
	{
		if (SingleFrameIndexProperty.IsValid())
		{			
			SingleFrameIndexProperty->SetValue(NewValue, bIsInteractive ? EPropertyValueSetFlags::InteractiveChange : EPropertyValueSetFlags::DefaultFlags);
		}
	};
	
	auto MaxValueLambda = [AnimationProperty]() -> TOptional<uint32>
	{
		if (AnimationProperty.IsValid())
		{
			FAssetData AssetData;
			AnimationProperty->GetValue(AssetData);

			if (UAnimSequence* BlendSampleAnimation = Cast<UAnimSequence>(AssetData.GetAsset()))
			{
				return BlendSampleAnimation->GetNumberOfSampledKeys() - 1;
			}
		}
		
		return TOptional<uint32>();
	};

	TSharedPtr<SWidget> DefaultNameWidget;
	TSharedPtr<SWidget> DefaultValueWidget;
	PropertyRow.GetDefaultWidgets(DefaultNameWidget, DefaultValueWidget);
	
	PropertyRow.CustomWidget()
	.NameContent()
	[
		DefaultNameWidget.ToSharedRef()
	]
	.ValueContent()
	[
		SNew(SNumericEntryBox<uint32>)
		.Font(FAppStyle::GetFontStyle("CurveEd.InfoFont"))
		.Value_Lambda([SingleFrameIndexProperty]() -> TOptional<uint32>
		{
			if (SingleFrameIndexProperty.IsValid())
			{
				uint32 Value = 0;
				if (SingleFrameIndexProperty->GetValue(Value) == FPropertyAccess::Success)
				{
					return Value;
				}
			}
			
			return TOptional<uint32>();
		})
		.OnValueCommitted_Lambda([ValueChangedLambda](uint32 NewValue, ETextCommit::Type CommitType)
		{
			ValueChangedLambda(NewValue, false);
		})
		.OnValueChanged_Lambda([ValueChangedLambda](uint32 NewValue)
		{
			ValueChangedLambda(NewValue, true);
		})
		.LabelVAlign(VAlign_Center)
		.AllowSpin(true)
		.MinValue(0)
		.MaxValue_Lambda(MaxValueLambda)
		.MinSliderValue(0)
		.MaxSliderValue_Lambda(MaxValueLambda)
	];
}

void FBlendSampleDetails::GenerateSampleGraphWidget(FDetailWidgetRow& InRow, UAnimGraphNode_BlendSpaceGraphBase* InBlendSpaceNode, int32 InSampleIndex)
{
	InRow
		.NameContent()
		[
			SNew(STextBlock)
			.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
			.Text(LOCTEXT("GraphLabel", "Graph"))
		]
		.ValueContent()
		[
			SNew(SBox)
			.WidthOverride(125.0f)
			[
				SNew(SButton)
				.ToolTipText(LOCTEXT("BlendSampleGraphButtonToolTip", "Edit the graph associated with this sample point"))
				.OnClicked_Lambda([InSampleIndex, WeakBlendSpaceNode = TWeakObjectPtr<UAnimGraphNode_BlendSpaceGraphBase>(InBlendSpaceNode)]()
				{
					FSlateApplication::Get().DismissAllMenus();

					if(UAnimGraphNode_BlendSpaceGraphBase* BlendSpaceNode = WeakBlendSpaceNode.Get())
					{
						UAnimBlueprint* AnimBlueprint = BlendSpaceNode->GetAnimBlueprint();
						TSharedPtr<IToolkit> FoundAssetEditor = FToolkitManager::Get().FindEditorForAsset(AnimBlueprint);
						if (FoundAssetEditor.IsValid() && FoundAssetEditor->IsBlueprintEditor())
						{
							TSharedPtr<IBlueprintEditor> BlueprintEditor = StaticCastSharedPtr<IBlueprintEditor>(FoundAssetEditor);
							if(BlendSpaceNode->GetGraphs().IsValidIndex(InSampleIndex))
							{
								BlueprintEditor->JumpToHyperlink(BlendSpaceNode->GetGraphs()[InSampleIndex], false);
							}
						}
					}

					return FReply::Handled();
				})
				[
					SNew(SBox)
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Center)
					[
						SNew(STextBlock)
						.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
						.Text(LOCTEXT("BlendSampleGraphButtonLabel", "Edit Graph"))
					]
				]
			]
		];
}

bool FBlendSampleDetails::ShouldFilterAssetStatic(const FAssetData& AssetData, const UBlendSpace* BlendSpaceBase)
{
	/** Cached flags to check whether or not an additive animation type is compatible with the blend space*/
	TMap<FString, bool> bValidAdditiveTypes;
	// Retrieve the additive animation type enum
	const UEnum* AdditiveTypeEnum = StaticEnum<EAdditiveAnimationType>();
	// For each type check whether or not the blend space is compatible with it and cache the result
	for (int32 TypeValue = 0; TypeValue < (int32)EAdditiveAnimationType::AAT_MAX; ++TypeValue)
	{
		EAdditiveAnimationType Type = (EAdditiveAnimationType)TypeValue;
		// In case of non additive type make sure the blendspace is made up out of non additive samples only
		const bool bAdditiveFlag = (Type == EAdditiveAnimationType::AAT_None) ? !BlendSpaceBase->IsValidAdditive() : BlendSpaceBase->IsValidAdditive() && BlendSpaceBase->IsValidAdditiveType(Type);
		bValidAdditiveTypes.Add(AdditiveTypeEnum->GetNameByValue(TypeValue).ToString(), bAdditiveFlag);
	}

	bool bShouldFilter = true;

	// Skeleton is a private member so cannot use GET_MEMBER_NAME_CHECKED and friend class seemed unjustified to add
	const FName SkeletonTagName = "Skeleton";
	FString SkeletonName;
	if (AssetData.GetTagValue(SkeletonTagName, SkeletonName))
	{
		// Check whether or not the skeletons are compatible
		if (BlendSpaceBase->GetSkeleton()->IsCompatibleForEditor(AssetData))
		{
			// If so check if the additive animation type is compatible with the blend space
			const FName AdditiveTypeTagName = GET_MEMBER_NAME_CHECKED(UAnimSequence, AdditiveAnimType);
			FString AnimationTypeName;
			if (AssetData.GetTagValue(AdditiveTypeTagName, AnimationTypeName))
			{
				bShouldFilter = !bValidAdditiveTypes.FindChecked(AnimationTypeName);
			}
			else
			{
				// If the asset does not contain the required tag value retrieve the asset and validate it
				const UAnimSequence* AnimSequence = Cast<UAnimSequence>(AssetData.GetAsset());
				if (AnimSequence)
				{
					bShouldFilter = !(AnimSequence && BlendSpaceBase->ValidateAnimationSequence(AnimSequence));
				}
			}
		}
	}

	return bShouldFilter;
}

bool FBlendSampleDetails::ShouldFilterAsset(const FAssetData& AssetData) const
{
	bool bShouldFilter = true;

	// Skeleton is a private member so cannot use GET_MEMBER_NAME_CHECKED and friend class seemed unjustified to add
	const FName SkeletonTagName = "Skeleton";
	FString SkeletonName;
	if (AssetData.GetTagValue(SkeletonTagName, SkeletonName))
	{
		// Check whether or not the skeletons match
		if (SkeletonName == BlendSpace->GetSkeleton()->GetPathName())
		{
			// If so check if the additive animation tpye is compatible with the blend space
			const FName AdditiveTypeTagName = GET_MEMBER_NAME_CHECKED(UAnimSequence, AdditiveAnimType);
			FString AnimationTypeName;
			if (AssetData.GetTagValue(AdditiveTypeTagName, AnimationTypeName))
			{
				bShouldFilter = !bValidAdditiveTypes.FindChecked(AnimationTypeName);
			}
			else
			{
				// If the asset does not contain the required tag value retrieve the asset and validate it
				const UAnimSequence* AnimSequence = Cast<UAnimSequence>(AssetData.GetAsset());
				if (AnimSequence)
				{
					bShouldFilter = !(AnimSequence && BlendSpace->ValidateAnimationSequence(AnimSequence));
				}
			}
		}
	}
	
	return bShouldFilter;
}

#undef LOCTEXT_NAMESPACE
