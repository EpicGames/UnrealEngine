// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"

#include "Templates/Function.h"
#include "DetailWidgetRow.h"
#include "SAnimationBlendSpaceGridWidget.h"

struct FAssetData;
class FDetailWidgetRow;
class UBlendSpace;
class UAnimGraphNode_BlendSpaceGraphBase;

class FBlendSampleDetails : public IDetailCustomization
{
public:
	FBlendSampleDetails(const class UBlendSpace* InBlendSpace, class SBlendSpaceGridWidget* InGridWidget, int32 InSampleIndex);

	static TSharedRef<IDetailCustomization> MakeInstance(const class UBlendSpace* InBlendSpace, class SBlendSpaceGridWidget* InGridWidget, int32 InSampleIndex)
	{
		return MakeShareable( new FBlendSampleDetails(InBlendSpace, InGridWidget, InSampleIndex) );
	}

	// Begin IDetailCustomization interface
	virtual void CustomizeDetails(class IDetailLayoutBuilder& DetailBuilder) override;
	// End IDetailCustomization interface
	
	static void GenerateBlendSampleWidget(TFunction<IDetailPropertyRow& (void)>InFunctor, FOnSampleMoved OnSampleMoved, const class UBlendSpace* BlendSpace, const int32 SampleIndex, bool bShowLabel);

	static void GenerateAnimationWidget(IDetailPropertyRow& PropertyRow, const UBlendSpace* BlendSpace, TSharedPtr<IPropertyHandle> AnimationProperty);

	static void GenerateSingleFrameBlendingWidget(IDetailPropertyRow& PropertyRow, TSharedPtr<IPropertyHandle> SingleFrameIndexProperty, TSharedPtr<IPropertyHandle> AnimationProperty);

	static void GenerateSampleGraphWidget(FDetailWidgetRow& Row, UAnimGraphNode_BlendSpaceGraphBase* BlendSpaceNode, int32 SampleIndex);

	static bool ShouldFilterAssetStatic(const FAssetData& AssetData, const UBlendSpace* BlendSpaceBase);

protected:
	/** Checks whether or not the specified asset should not be shown in the mini content browser when changing the animation */
	bool ShouldFilterAsset(const FAssetData& AssetData) const;

	FReply HandleAnalyzeAndDuplicateSample();
	FReply HandleAnalyzeAndMoveSample();
	FReply HandleAnalyzeAndMoveSampleX();
	FReply HandleAnalyzeAndMoveSampleY();
private:
	/** Pointer to the current parent blend space for the customized blend sample*/
	const class UBlendSpace* BlendSpace;
	/** Parent grid widget object */
	SBlendSpaceGridWidget* GridWidget;
	/** Current sample index */
	int32 SampleIndex;
	/** Cached flags to check whether or not an additive animation type is compatible with the blend space*/	
	TMap<FString, bool> bValidAdditiveTypes;
};
