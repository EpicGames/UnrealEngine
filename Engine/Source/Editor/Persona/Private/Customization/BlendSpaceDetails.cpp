// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customization/BlendSpaceDetails.h"

#include "IDetailsView.h"
#include "IDetailGroup.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"

#include "Animation/BlendSpace.h"
#include "Animation/BlendSpace1D.h"

#include "BlendSampleDetails.h"

#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/STextComboBox.h"
#include "PropertyCustomizationHelpers.h"
#include "ScopedTransaction.h"
#include "BlendSpaceGraph.h"
#include "AnimGraphNode_BlendSpaceGraph.h"
#include "PersonaBlendSpaceAnalysis.h"

#include "DetailBuilderTypes.h"
#include "SWarningOrErrorBox.h"
#include "Animation/MirrorDataTable.h"

#define LOCTEXT_NAMESPACE "BlendSpaceDetails"

FBlendSpaceDetails::FBlendSpaceDetails()
{
	Builder = nullptr;
	BlendSpace = nullptr;
	BlendSpaceNode = nullptr;

	GEditor->RegisterForUndo(this);
}

FBlendSpaceDetails::~FBlendSpaceDetails()
{
	GEditor->UnregisterForUndo(this);
}

//======================================================================================================================
FReply FBlendSpaceDetails::HandleClearSamples()
{
	FSlateApplication::Get().DismissAllMenus();
	BlendSpace->Modify();
	BlendSpace->SampleData.Empty();

	FPropertyChangedEvent ChangedEvent(nullptr, EPropertyChangeType::ArrayClear);
	BlendSpace->PostEditChangeProperty(ChangedEvent);

	return FReply::Handled();
}

//======================================================================================================================
FReply FBlendSpaceDetails::HandleAnalyzeSamples()
{
	if(BlendSpace->IsAsset())
	{
		FScopedTransaction ScopedTransaction(LOCTEXT("AnalyzeBlendSpaceSamples", "Applying Blend Space sample analysis"));
		BlendSpace->Modify();

		FSlateApplication::Get().DismissAllMenus();
		bool bChangedOne = false;

		const int32 NumSamples = BlendSpace->SampleData.Num();
		for (int32 SampleIndex = 0 ; SampleIndex != NumSamples ; ++SampleIndex)
		{
			bool bAnalyzed[3] = { false, false, false };
			if (BlendSpace->SampleData[SampleIndex].bIncludeInAnalyseAll)
			{
				FVector NewValue = BlendSpaceAnalysis::CalculateSampleValue(
					*BlendSpace, *BlendSpace->SampleData[SampleIndex].Animation, 
					BlendSpace->SampleData[SampleIndex].RateScale, 
					BlendSpace->SampleData[SampleIndex].SampleValue, bAnalyzed);
				if (bAnalyzed[0] || bAnalyzed[1] || bAnalyzed[2])
				{
					BlendSpace->EditSampleValue(SampleIndex, NewValue);
					// Note that the sample might not move if the destination position is in use
					if (NewValue == BlendSpace->SampleData[SampleIndex].SampleValue)
					{
						bChangedOne = true;
					}
				}
			}
		}

		if (bChangedOne)
		{
			FPropertyChangedEvent ChangedEvent(nullptr, EPropertyChangeType::ArrayClear);
			BlendSpace->PostEditChangeProperty(ChangedEvent);
		}
		else
		{
			ScopedTransaction.Cancel();
		}
	}
	return FReply::Handled();
}

//======================================================================================================================
// Returns a bit mask of EAnalysisProperty
static EVisibility GetAnalyzeButtonVisibility(
	TSharedPtr<IPropertyHandle> AnalysisPropertiesHandle,
	int32                       HideIndex)
{
	for (int32 AxisIndex = 0; AxisIndex < 3; ++AxisIndex)
	{
		if (AxisIndex < HideIndex)
		{
			TSharedPtr<IPropertyHandle> AnalysisPropertyHandle = AnalysisPropertiesHandle->GetChildHandle(AxisIndex);
			UObject* Object;
			if (AnalysisPropertyHandle->GetValue(Object))
			{
				UAnalysisProperties* AnalysisProperties = Cast<UAnalysisProperties>(Object);
				if (AnalysisProperties && !AnalysisProperties->Function.IsEmpty() && AnalysisProperties->Function != TEXT("None"))
				{
					return EVisibility::Visible;
				}
			}
		}
	}
	return EVisibility::Collapsed;
}

//======================================================================================================================
// When the existing one gets replaced it will get garbage collected automatically as nothing will reference it. Note 
// that if "None" was selected then the new value will be null
void FBlendSpaceDetails::HandleAnalysisFunctionChanged(int32 AxisIndex, TSharedPtr<FString> NewFunctionName)
{
	const FScopedTransaction Transaction(LOCTEXT("BlendSpaceDetailsChangedAxisTransaction", "Changed Axis Function"));
	BlendSpace->Modify();

	UAnalysisProperties* NewAnalysisProperties = BlendSpaceAnalysis::MakeAnalysisProperties(BlendSpace, *NewFunctionName);
	// Preserve values where possible
	if (BlendSpace->AnalysisProperties[AxisIndex])
	{
		BlendSpace->AnalysisProperties[AxisIndex]->MakeCache(BlendSpace->CachedAnalysisProperties[AxisIndex], BlendSpace);
	}
	if (NewAnalysisProperties)
	{
		NewAnalysisProperties->InitializeFromCache(BlendSpace->CachedAnalysisProperties[AxisIndex]);
	}
	BlendSpace->AnalysisProperties[AxisIndex] = NewAnalysisProperties;
	Builder->ForceRefreshDetails();
}

//======================================================================================================================
void FBlendSpaceDetails::CustomizeDetails(class IDetailLayoutBuilder& DetailBuilder)
{
	TArray< TWeakObjectPtr<UObject> > Objects;
	DetailBuilder.GetObjectsBeingCustomized(Objects);

	Builder = &DetailBuilder;
	
	TWeakObjectPtr<UObject>* BlendSpacePtr = Objects.FindByPredicate([](const TWeakObjectPtr<UObject>& ObjectPtr) { return ObjectPtr->IsA<UBlendSpace>(); });
	if (BlendSpacePtr)
	{
		BlendSpace = Cast<UBlendSpace>(BlendSpacePtr->Get());

		if(!BlendSpace->IsAsset())
		{
			// Hide various properties when we are 'internal'
			DetailBuilder.HideCategory("MetaData");
			DetailBuilder.HideCategory("AnimationNotifies");
			DetailBuilder.HideCategory("Thumbnail");
			DetailBuilder.HideCategory("Animation");
			DetailBuilder.HideCategory("AdditiveSettings");
			DetailBuilder.HideCategory("Analysis");
			DetailBuilder.HideCategory("AnalysisProperties");
			DetailBuilder.HideCategory("Graph");
		}

		if(UBlendSpaceGraph* BlendSpaceGraph = Cast<UBlendSpaceGraph>(BlendSpace->GetOuter()))
		{
			check(BlendSpace == BlendSpaceGraph->BlendSpace);
			BlendSpaceNode = Cast<UAnimGraphNode_BlendSpaceGraphBase>(BlendSpaceGraph->GetOuter());
		}
		const bool b1DBlendSpace = BlendSpace->IsA<UBlendSpace1D>();

		if (b1DBlendSpace)
		{
			DetailBuilder.HideProperty(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UBlendSpace, AxisToScaleAnimation), UBlendSpace::StaticClass()));
			DetailBuilder.HideProperty(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UBlendSpace, PreferredTriangulationDirection), UBlendSpace::StaticClass()));
		}

		// How many samples are there?
		TSharedPtr<IPropertyHandleArray> BlendSamplesArrayProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UBlendSpace, SampleData), UBlendSpace::StaticClass())->AsArray();
		uint32 NumBlendSampleEntries = 0;
		BlendSamplesArrayProperty->GetNumElements(NumBlendSampleEntries);

		//==============================================================================================================
		// Axis Settings section
		//==============================================================================================================
		{
			IDetailCategoryBuilder& DetailCategoryBuilder = DetailBuilder.EditCategory(FName("Axis Settings"));
			IDetailGroup* Groups[2] =
			{
				&DetailCategoryBuilder.AddGroup(FName("Horizontal Axis"), LOCTEXT("HorizontalAxis", "Horizontal Axis")),
                b1DBlendSpace ? nullptr : &DetailCategoryBuilder.AddGroup(FName("Vertical Axis"), LOCTEXT("VerticalAxis", "Vertical Axis"))
            };

			// Hide the default blend and interpolation parameters
			TSharedPtr<IPropertyHandle> BlendParameters = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UBlendSpace, BlendParameters), UBlendSpace::StaticClass());
			TSharedPtr<IPropertyHandle> InterpolationParameters = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UBlendSpace, InterpolationParam), UBlendSpace::StaticClass());
			DetailBuilder.HideProperty(BlendParameters);
			DetailBuilder.HideProperty(InterpolationParameters);

			// Add the properties to the corresponding groups created above (third axis will always be hidden since it isn't used)
			int32 HideIndex = b1DBlendSpace ? 1 : 2;
			for (int32 AxisIndex = 0; AxisIndex < 3; ++AxisIndex)
			{
				TSharedPtr<IPropertyHandle> BlendParameter = BlendParameters->GetChildHandle(AxisIndex);
				TSharedPtr<IPropertyHandle> InterpolationParameter = InterpolationParameters->GetChildHandle(AxisIndex);

				if (AxisIndex < HideIndex)
				{
					Groups[AxisIndex]->AddPropertyRow(BlendParameter.ToSharedRef());
					// Don't add InterpolationParameter in the same way as BlendParameter, because it would add the
					// elements as customizations that we can't subsequently customize. We will add them individually
					// below.

					TSharedPtr<IPropertyHandle> InterpolationTime = InterpolationParameter->GetChildHandle(
						GET_MEMBER_NAME_CHECKED(FInterpolationParameter, InterpolationTime));
					TSharedPtr<IPropertyHandle> DampingRatio = InterpolationParameter->GetChildHandle(
						GET_MEMBER_NAME_CHECKED(FInterpolationParameter, DampingRatio));
					TSharedPtr<IPropertyHandle> MaxSpeed = InterpolationParameter->GetChildHandle(
						GET_MEMBER_NAME_CHECKED(FInterpolationParameter, MaxSpeed));
					TSharedPtr<IPropertyHandle> InterpolationType = InterpolationParameter->GetChildHandle(
						GET_MEMBER_NAME_CHECKED(FInterpolationParameter, InterpolationType));

					// Custom edit condition for MaxSpeed
					TAttribute<bool> MaxSpeedEditCondition = TAttribute<bool>::Create(
						[this, InterpolationTime, InterpolationType]()
						{
							uint8 IntType;
							InterpolationType->GetValue(IntType);
							EFilterInterpolationType Type = (EFilterInterpolationType)IntType;
							float Time;
							InterpolationTime->GetValue(Time);
							if (Time > 0.0f && (Type == EFilterInterpolationType::BSIT_SpringDamper ||
												Type == EFilterInterpolationType::BSIT_ExponentialDecay))
							{
								return true;
							}
							return false;
						});

					Groups[AxisIndex]->AddPropertyRow(InterpolationTime.ToSharedRef());
					Groups[AxisIndex]->AddPropertyRow(InterpolationType.ToSharedRef());
					Groups[AxisIndex]->AddPropertyRow(DampingRatio.ToSharedRef());
					IDetailPropertyRow& MaxSpeedProperty = Groups[AxisIndex]->AddPropertyRow(MaxSpeed.ToSharedRef());
					MaxSpeedProperty.EditCondition(MaxSpeedEditCondition, nullptr);
				}
				else
				{
					DetailBuilder.HideProperty(BlendParameter);
					DetailBuilder.HideProperty(InterpolationParameter);
				}
			}
		}
		
		//==============================================================================================================
		// Analysis section
		//==============================================================================================================
		if (BlendSpace->IsAsset())
		{
			IDetailCategoryBuilder& DetailCategoryBuilder = DetailBuilder.EditCategory(FName("Analysis"));

			TWeakObjectPtr<const UBlendSpace> WeakBlendspace = BlendSpace;
			auto ContainsSingleFrameBlendSampleVisibility = [WeakBlendspace](bool bExpectedValueForVisibility) -> EVisibility
			{
				if (const UBlendSpace* PinnedBlendspace = WeakBlendspace.Get())
				{
					const bool bValue = PinnedBlendspace->GetBlendSamples().ContainsByPredicate([](const FBlendSample& BlendSample) { return BlendSample.bUseSingleFrameForBlending; } );
					return bExpectedValueForVisibility == bValue ? EVisibility::Visible : EVisibility::Collapsed;
				}

				return EVisibility::Collapsed;
			};
					
			TSharedPtr<IPropertyHandle> AnalysisPropertiesHandle = DetailBuilder.GetProperty(
				GET_MEMBER_NAME_CHECKED(UBlendSpace, AnalysisProperties), UBlendSpace::StaticClass());

			int32 HideIndex = b1DBlendSpace ? 1 : 2;

			// Re-analyse button
			if (NumBlendSampleEntries)
			{
				// Use AddCustomRow rather than AddGroup because the latter fails to include the button when searching
				DetailCategoryBuilder.AddCustomRow(
					LOCTEXT("AnalyzeAllSamplesRow", "Analyse all samples"))
					.Visibility(TAttribute<EVisibility>::Create([AnalysisPropertiesHandle, HideIndex]() {
						return GetAnalyzeButtonVisibility(AnalysisPropertiesHandle, HideIndex); }))
					.NameContent()
					[
						SNew(STextBlock)
						.Font(DetailBuilder.GetDetailFont())
						.Text(FText::Format(LOCTEXT("AnalyzeSamples", "Analyze {0} Samples"), NumBlendSampleEntries))
					]
					.ValueContent()
					[
						SNew(SButton)
						.ButtonStyle(FAppStyle::Get(), "SimpleButton")
						.HAlign(EHorizontalAlignment::HAlign_Left)
						.ToolTipText(LOCTEXT("AnalyzeAllSamples", "Analyze all samples"))
						.OnClicked(this, &FBlendSpaceDetails::HandleAnalyzeSamples)
						.ContentPadding(1.f)
						[
							SNew(SImage)
							.Image(FAppStyle::GetBrush("Icons.Refresh"))
							.ColorAndOpacity(FSlateColor::UseForeground())
						]
					];
			}

			FText AxisTexts[2] = 
			{
				LOCTEXT("HorizontalAxisFunction", "Horizontal Axis Function"), 
				LOCTEXT("VerticalAxisFunction", "Vertical Axis Function") 
			};

			// Hide the default parameters
			DetailBuilder.HideProperty(AnalysisPropertiesHandle);

			// Add the properties to the corresponding groups created above (third axis will always be hidden since it isn't used)
			for (int32 AxisIndex = 0; AxisIndex < 3; ++AxisIndex)
			{
				TSharedPtr<IPropertyHandle> AnalysisProperty = AnalysisPropertiesHandle->GetChildHandle(AxisIndex);
				
				if (AxisIndex < HideIndex)
				{
					UObject* AnalysisPropertiesObject;
					FPropertyAccess::Result Result = AnalysisProperty->GetValue(AnalysisPropertiesObject);
					UAnalysisProperties* AnalysisProperties = Cast<UAnalysisProperties>(AnalysisPropertiesObject);

					// Note that we need to make sure the function drop-down is shown, even if there is no current
					// analysis property.

					// Prepare the drop-down. Note that it uses pointers to strings, and comparison is between
					// pointers, not string contents!
					AnalysisFunctionNames[AxisIndex].Empty();
					const TArray<FString> AnalysisFunctions = BlendSpaceAnalysis::GetAnalysisFunctions();
					TSharedPtr<FString> CurrentlySelectedFunction;
					for (const FString& AnalysisFunction : AnalysisFunctions)
					{
						AnalysisFunctionNames[AxisIndex].Add(MakeShared<FString>(AnalysisFunction));
						if (AnalysisProperties && AnalysisProperties->Function == *AnalysisFunctionNames[AxisIndex].Last())
						{
							CurrentlySelectedFunction = AnalysisFunctionNames[AxisIndex].Last();
						}
						// In the event that we don't find anything (possible if the list of available functions
						// changes), it's nice if we start off by at least selecting "None".
						if (!CurrentlySelectedFunction && *AnalysisFunctionNames[AxisIndex].Last() == "None")
						{
							CurrentlySelectedFunction = AnalysisFunctionNames[AxisIndex].Last();
						}
					}

					// Add the analysis properties. We want the function drop-down even if they don't exist
					IDetailPropertyRow* AnalysisPropertiesRow = nullptr;
					if (AnalysisProperties && 
						!AnalysisProperties->Function.IsEmpty() && 
						AnalysisProperties->Function != TEXT("None") && 
						AnalysisFunctions.Contains(AnalysisProperties->Function))
					{
						FAddPropertyParams Params;
						Params.UniqueId(FName(TEXT("AnalysisPropertiesCombo")));
						AnalysisPropertiesRow = DetailCategoryBuilder.AddExternalObjects( {AnalysisProperties}, EPropertyLocation::Default, Params);
					}

					FDetailWidgetRow *FunctionWidgetRow = nullptr;
					// Insert the function drop-down
					if (AnalysisPropertiesRow)
					{
						// Note that there is an extra/unwanted level of indirection here in the UI that is generated,
						// with the block being underneath an unnecessary "Analysis Properties" section.
						FunctionWidgetRow = &AnalysisPropertiesRow->CustomWidget();
					}
					else
					{
						IDetailPropertyRow& SourceAnimationRow = DetailCategoryBuilder.AddProperty(AnalysisProperty);
						FunctionWidgetRow = &SourceAnimationRow.CustomWidget();
					}

					FunctionWidgetRow->NameContent()
					[
						SNew(STextBlock)
						.Text(AxisTexts[AxisIndex])
						.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
					]
					.ValueContent()
					[
						SNew(STextComboBox)
						.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
						.OptionsSource(&AnalysisFunctionNames[AxisIndex])
						.InitiallySelectedItem(CurrentlySelectedFunction)
						.OnSelectionChanged_Lambda(
							[this, AxisIndex, CurrentlySelectedFunction]
							(TSharedPtr<FString> NewFunctionName, ESelectInfo::Type SelectInfo) 
							{
								if (NewFunctionName && NewFunctionName != CurrentlySelectedFunction)
								{
									this->HandleAnalysisFunctionChanged(AxisIndex, NewFunctionName);
								}
							})
					]
					.Visibility(TAttribute<EVisibility>::CreateLambda(ContainsSingleFrameBlendSampleVisibility, false));
				}
				else
				{
					DetailBuilder.HideProperty(AnalysisProperty);
				}
			}

			// show incompatibility warning
			const FText WarningText = LOCTEXT("AnalysisAndSingleFrameIncompatibilityWarning", "Analysis feature is not compatible with single frame blending. "
																					  "\n\nPlease disable single frame blending in each blend sample to use analysis.");
			FDetailWidgetRow& IncompatibilityWarningRow = DetailCategoryBuilder.AddCustomRow(WarningText)
				.WholeRowContent()
				[
					SNew(SWarningOrErrorBox)
						.MessageStyle(EMessageStyle::Warning)
						.Message(WarningText)
				]
				.Visibility(TAttribute<EVisibility>::CreateLambda(ContainsSingleFrameBlendSampleVisibility, true));
		}
		
		//==============================================================================================================
		// Blend Samples section
		//==============================================================================================================
		{
			IDetailCategoryBuilder& DetailCategoryBuilder = DetailBuilder.EditCategory(FName("BlendSamples"));
			TArray<TSharedRef<IPropertyHandle>> DefaultProperties;
			DetailCategoryBuilder.GetDefaultProperties(DefaultProperties);
			for (TSharedRef<IPropertyHandle> DefaultProperty : DefaultProperties)
			{
				 DefaultProperty->MarkHiddenByCustomization();
			}
			
			BlendSamplesArrayProperty->SetOnNumElementsChanged(FSimpleDelegate::CreateSP(this, &FBlendSpaceDetails::RefreshDetails));

			// Add a "Remove all" button if there are some samples. Only in the asset blendspace for now.
			if (NumBlendSampleEntries && BlendSpace->IsAsset())
			{
				DetailCategoryBuilder.AddCustomRow(
					LOCTEXT("RemoveSamplesRow", "Remove all samples"))
					.NameContent()
					[
						SNew(STextBlock)
						.Font(DetailBuilder.GetDetailFont())
						.Text(FText::Format(LOCTEXT("SamplesLabel", "{0} Samples"), NumBlendSampleEntries))
					]
					.ValueContent()
					[
						SNew(SButton)
						.ButtonStyle(FAppStyle::Get(), "SimpleButton")
						.HAlign(EHorizontalAlignment::HAlign_Left)
						.ToolTipText(LOCTEXT("RemoveAllSamples", "Remove all samples"))
						.OnClicked(this, &FBlendSpaceDetails::HandleClearSamples)
						.ContentPadding(1.f)
						[
							SNew(SImage)
							.Image(FAppStyle::GetBrush("Icons.Delete"))
							.ColorAndOpacity(FSlateColor::UseForeground())
						]
					];
			}

			for (uint32 SampleIndex = 0; SampleIndex < NumBlendSampleEntries; ++SampleIndex)
			{
				TSharedPtr<IPropertyHandle> BlendSampleProperty = BlendSamplesArrayProperty->GetElement(SampleIndex);
				TSharedPtr<IPropertyHandle> AnimationProperty = BlendSampleProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FBlendSample, Animation));
				TSharedPtr<IPropertyHandle> SampleValueProperty = BlendSampleProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FBlendSample, SampleValue));
				TSharedPtr<IPropertyHandle> RateScaleProperty = BlendSampleProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FBlendSample, RateScale));
				TSharedPtr<IPropertyHandle> IncludeInAnalyseAllProperty = BlendSampleProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FBlendSample, bIncludeInAnalyseAll));
				TSharedPtr<IPropertyHandle> UseSingleFrameBlendingProperty = BlendSampleProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FBlendSample, FrameIndexToSample));
				TSharedPtr<IPropertyHandle> MirrorProperty = BlendSampleProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(FBlendSample, bMirror));

				IDetailGroup& Group = DetailCategoryBuilder.AddGroup(FName("BlendSamples_Samples"), FText::GetEmpty());
				Group.HeaderRow()
				.NameContent()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.Padding(FMargin(0,2,2,2))
					.FillWidth(1.0f)
					.HAlign(HAlign_Right)
					[
						SNew(STextBlock)
						.Font(DetailBuilder.GetDetailFont())
						.Text_Lambda([this, AnimationProperty, SampleIndex]() -> FText
						{
							FAssetData AssetData;
							AnimationProperty->GetValue(AssetData);
							if(AssetData.IsValid())
							{
								return FText::Format(
									LOCTEXT("BlendSpaceAnimationNameLabel", "{0} ({1})"),
									FText::FromString(AssetData.GetAsset()->GetName()),
									FText::FromString(FString::FromInt(SampleIndex)));
							}
							else if(BlendSpaceNode.Get() && BlendSpaceNode->GetGraphs().IsValidIndex(SampleIndex))
							{
								return FText::Format(
									LOCTEXT("BlendSpaceAnimationNameLabel", "{0} ({1})"),
									FText::FromName(BlendSpaceNode->GetGraphs()[SampleIndex]->GetFName()),
									FText::FromString(FString::FromInt(SampleIndex)));
							}
							return LOCTEXT("NoAnimation", "No Animation");
						})
					]
				];

				FBlendSampleDetails::GenerateBlendSampleWidget(
					[&Group, &SampleValueProperty]() -> IDetailPropertyRow& 
						{ 
							return Group.AddPropertyRow(SampleValueProperty.ToSharedRef()); 
						}, 
					FOnSampleMoved::CreateLambda(
						[this](const uint32 Index, const FVector& SampleValue, bool bIsInteractive) 
						{
							if (BlendSpace->IsValidBlendSampleIndex(Index) && BlendSpace->GetBlendSample(Index).SampleValue !=
								SampleValue && !BlendSpace->IsTooCloseToExistingSamplePoint(SampleValue, Index))
							{
								BlendSpace->Modify();
								bool bMoveSuccesful = BlendSpace->EditSampleValue(Index, SampleValue);
								if (bMoveSuccesful)
								{
									BlendSpace->ValidateSampleData();
									FPropertyChangedEvent ChangedEvent(
										nullptr, bIsInteractive ? EPropertyChangeType::Interactive : EPropertyChangeType::ValueSet);
									BlendSpace->PostEditChangeProperty(ChangedEvent);
								}
							}
						}), 
					BlendSpace, SampleIndex, false);

				if(BlendSpace->IsAsset())
				{
					IDetailPropertyRow& AnimationRow = Group.AddPropertyRow(AnimationProperty.ToSharedRef());
					FBlendSampleDetails::GenerateAnimationWidget(AnimationRow, BlendSpace, AnimationProperty);
					Group.AddPropertyRow(RateScaleProperty.ToSharedRef());
					IDetailPropertyRow& UseSingleFrameBlendingPropertyRow = Group.AddPropertyRow(UseSingleFrameBlendingProperty.ToSharedRef());
					FBlendSampleDetails::GenerateSingleFrameBlendingWidget(UseSingleFrameBlendingPropertyRow, UseSingleFrameBlendingProperty.ToSharedRef(), AnimationProperty.ToSharedRef());
					Group.AddPropertyRow(IncludeInAnalyseAllProperty.ToSharedRef());

					IDetailPropertyRow& MirrorPropertyRow = Group.AddPropertyRow(MirrorProperty.ToSharedRef());
					TAttribute<bool> IsMirrorDataTableValid = TAttribute<bool>::CreateLambda([this]() { return BlendSpace->MirrorDataTable != nullptr; });
					MirrorPropertyRow.EditCondition(IsMirrorDataTableValid, nullptr);
				}
				else if(BlendSpaceNode.Get())
				{
					// Expose mirror option
					IDetailPropertyRow& MirrorPropertyRow = Group.AddPropertyRow(MirrorProperty.ToSharedRef());

					TAttribute<bool> IsMirrorDataTableValid = TAttribute<bool>::CreateLambda([this]() { return BlendSpace->MirrorDataTable != nullptr; });
					MirrorPropertyRow.EditCondition(IsMirrorDataTableValid, nullptr);

					// Edit graph
					FDetailWidgetRow& GraphRow = Group.AddWidgetRow();
					FBlendSampleDetails::GenerateSampleGraphWidget(GraphRow, BlendSpaceNode.Get(), SampleIndex);
				}
			}
		}

		/// Mirroring - only allow to set MDT that share the same skeleton.
		{
			TSharedPtr<IPropertyHandle> MDTPropertyHandle =  DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UBlendSpace, MirrorDataTable));

			if (!MDTPropertyHandle->IsValidHandle())
			{
				return;
			}
			
			IDetailCategoryBuilder& MirroringCategory = DetailBuilder.EditCategory(FName("Mirroring"));
			
			IDetailPropertyRow& MirroringPropertyRow = MirroringCategory.AddProperty(MDTPropertyHandle);
			FDetailWidgetRow& Row = MirroringPropertyRow.CustomWidget(true);
			
			TSharedPtr<SWidget> ValueWidget = SNew(SObjectPropertyEntryBox)
				.PropertyHandle(MDTPropertyHandle)
				.AllowedClass(UMirrorDataTable::StaticClass())
				.AllowClear(true)
				.OnShouldFilterAsset(FOnShouldFilterAsset::CreateLambda([Skeleton = BlendSpace->GetSkeleton(), bGraph = !BlendSpace->IsAsset() && BlendSpaceNode.IsValid(), BlendSpaceNode = BlendSpaceNode](const FAssetData& InAssetData)
				{
					if (!bGraph)
					{
						return Skeleton ? !Skeleton->IsCompatibleForEditor(InAssetData) : true;
					}
					else
					{
						if (UAnimBlueprint* AnimBlueprint = BlendSpaceNode->GetAnimBlueprint())
						{
							return AnimBlueprint->TargetSkeleton ? !AnimBlueprint->TargetSkeleton->IsCompatibleForEditor(InAssetData) : true;
						}

						return true;
					}
				}));

			Row.NameContent()
			[
				MDTPropertyHandle->CreatePropertyNameWidget()
			];

			Row.ValueContent()
			.MinDesiredWidth(Row.ValueWidget.MinWidth)
			.MaxDesiredWidth(Row.ValueWidget.MaxWidth)
			[
				ValueWidget.ToSharedRef()
			];
		}
	}	
}

void FBlendSpaceDetails::PostUndo(bool bSuccess)
{
	GEditor->GetTimerManager()->SetTimerForNextTick(FTimerDelegate::CreateSP(this, &FBlendSpaceDetails::RefreshDetails));
}

void FBlendSpaceDetails::RefreshDetails() const
{
	if (Builder)
	{
		Builder->ForceRefreshDetails();
	}
}

#undef LOCTEXT_NAMESPACE
