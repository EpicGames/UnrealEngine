// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"
#include "Input/Reply.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "EditorUndoClient.h"

class IDetailLayoutBuilder;
class UBlendSpace;
class UAnimGraphNode_BlendSpaceGraphBase;

class FBlendSpaceDetails : public IDetailCustomization, public FEditorUndoClient
{
public:
	FBlendSpaceDetails();
	~FBlendSpaceDetails();

	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShareable( new FBlendSpaceDetails() );
	}

	// IDetailCustomization interface
	virtual void CustomizeDetails(class IDetailLayoutBuilder& DetailBuilder) override;

	// FEditorUndoClient interface
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override { PostUndo(bSuccess); }
	// End of FEditorUndoClient interface.

private:
	FReply HandleClearSamples();
	FReply HandleAnalyzeSamples();
	void HandleAnalysisFunctionChanged(int32 AxisIndex, TSharedPtr<FString> NewItem);
	void RefreshDetails() const;

	IDetailLayoutBuilder* Builder;
	UBlendSpace* BlendSpace;
	TWeakObjectPtr<UAnimGraphNode_BlendSpaceGraphBase> BlendSpaceNode;
	TArray<TSharedPtr<FString>> AnalysisFunctionNames[3];
};
