// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customization/InterpolationParameterDetails.h"

#include "HAL/Platform.h"
#include "IDetailChildrenBuilder.h"
#include "PropertyHandle.h"

class IDetailPropertyRow;

#define LOCTEXT_NAMESPACE "InterpolationParameterDetails"

void FInterpolationParameterDetails::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	uint32 NumChildren;
	StructPropertyHandle->GetNumChildren(NumChildren);

	for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
	{
		TSharedRef<IPropertyHandle> ChildHandle = StructPropertyHandle->GetChildHandle(ChildIndex).ToSharedRef();
		IDetailPropertyRow& Property = ChildBuilder.AddProperty(ChildHandle);
	}
}


#undef LOCTEXT_NAMESPACE
