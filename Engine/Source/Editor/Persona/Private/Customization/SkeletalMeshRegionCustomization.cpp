// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customization/SkeletalMeshRegionCustomization.h"

#include "Engine/SkeletalMesh.h"
#include "Engine/SkeletalMeshSampling.h"
#include "Engine/SkinnedAssetCommon.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Class.h"
#include "IDetailsView.h"
#include "Styling/AppStyle.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "PropertyHandle.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"

#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SNumericEntryBox.h"

#include "Animation/AnimSequence.h"
#include "Animation/BlendSpace.h"
#include "Animation/BlendSpace1D.h"
#include "SAnimationBlendSpaceGridWidget.h"
#include "PropertyCustomizationHelpers.h"

#include "PackageTools.h"
#include "IDetailGroup.h"
#include "Editor.h"
#include "IDetailChildrenBuilder.h"
#include "Widgets/Input/SComboBox.h"

#define LOCTEXT_NAMESPACE "SkeletalMeshRegionCustomization"

void FNiagaraSkeletalMeshRegionBoneFilterDetails::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	HeaderRow
		.NameContent()
		[
			StructPropertyHandle->CreatePropertyNameWidget()
		];
}

void FNiagaraSkeletalMeshRegionBoneFilterDetails::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	TArray<TWeakObjectPtr<UObject>> SelectedObjects;
	ChildBuilder.GetParentCategory().GetParentLayout().GetObjectsBeingCustomized(SelectedObjects);
	check(SelectedObjects.Num() == 1);
	MeshObject = Cast<USkeletalMesh>(SelectedObjects[0].Get());
	check(MeshObject);
	
	uint32 NumChildren;
	StructPropertyHandle->GetNumChildren(NumChildren);

	for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
	{
		TSharedRef<IPropertyHandle> ChildHandle = StructPropertyHandle->GetChildHandle(ChildIndex).ToSharedRef();
		const FName ChildName = ChildHandle->GetProperty()->GetFName();

		// Pick out Min and Max range value properties
		if (ChildName == GET_MEMBER_NAME_CHECKED(FSkeletalMeshSamplingRegionBoneFilter, BoneName))
		{
			BoneNameHandle = ChildHandle;
			ChildBuilder.AddCustomRow(BoneNameHandle->GetPropertyDisplayName())
				.NameWidget
				[
					BoneNameHandle->CreatePropertyNameWidget()
				]
				.ValueWidget
				[
					SNew(SComboBox<FName>)
					.OptionsSource(&PossibleBoneNames)
					.ContentPadding(2.0f)
					.OnGenerateWidget(this, &FNiagaraSkeletalMeshRegionBoneFilterDetails::HandleBoneNameComboBoxGenerateWidget)
					.OnSelectionChanged(this, &FNiagaraSkeletalMeshRegionBoneFilterDetails::HandleBoneNameComboBoxSelectionChanged)
					.OnComboBoxOpening(this, &FNiagaraSkeletalMeshRegionBoneFilterDetails::OnComboOpening)
					[
						SNew(STextBlock)
						.Text(this, &FNiagaraSkeletalMeshRegionBoneFilterDetails::HandleBoneNameComboBoxContentText)
					]
				];
		}
		else
		{
			ChildBuilder.AddProperty(ChildHandle);
		}
	}
}

FText FNiagaraSkeletalMeshRegionBoneFilterDetails::HandleBoneNameComboBoxContentText() const
{
	FName OutName;
	BoneNameHandle->GetValue(OutName);
	return FText::FromName(OutName);
}

TSharedRef<SWidget> FNiagaraSkeletalMeshRegionBoneFilterDetails::HandleBoneNameComboBoxGenerateWidget(FName StringItem)
{
	return SNew(STextBlock)
		.Text(FText::FromName(StringItem));
}

void FNiagaraSkeletalMeshRegionBoneFilterDetails::HandleBoneNameComboBoxSelectionChanged(FName StringItem, ESelectInfo::Type SelectInfo)
{
	BoneNameHandle->SetValue(StringItem);
}


void FNiagaraSkeletalMeshRegionBoneFilterDetails::OnComboOpening()
{
	PossibleBoneNames.Reset();
	if (MeshObject->GetSkeleton() != nullptr)
	{
		//Populate PossibleBonesNames
		for (FMeshBoneInfo Bone : MeshObject->GetSkeleton()->GetReferenceSkeleton().GetRefBoneInfo())
		{
			PossibleBoneNames.Add(Bone.Name);
		}
	}
}

//////////////////////////////////////////////////////////////////////////


void FNiagaraSkeletalMeshRegionMaterialFilterDetails::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	HeaderRow
		.NameContent()
		[
			StructPropertyHandle->CreatePropertyNameWidget()
		];
}

void FNiagaraSkeletalMeshRegionMaterialFilterDetails::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	TArray<TWeakObjectPtr<UObject>> SelectedObjects;
	ChildBuilder.GetParentCategory().GetParentLayout().GetObjectsBeingCustomized(SelectedObjects);
	check(SelectedObjects.Num() == 1);
	MeshObject = Cast<USkeletalMesh>(SelectedObjects[0].Get());
	check(MeshObject);
	
	uint32 NumChildren;
	StructPropertyHandle->GetNumChildren(NumChildren);

	for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
	{
		TSharedRef<IPropertyHandle> ChildHandle = StructPropertyHandle->GetChildHandle(ChildIndex).ToSharedRef();
		const FName ChildName = ChildHandle->GetProperty()->GetFName();

		// Pick out Min and Max range value properties
		if (ChildName == GET_MEMBER_NAME_CHECKED(FSkeletalMeshSamplingRegionMaterialFilter, MaterialName))
		{
			MaterialNameHandle = ChildHandle;
			ChildBuilder.AddCustomRow(MaterialNameHandle->GetPropertyDisplayName())
				.NameWidget
				[
					MaterialNameHandle->CreatePropertyNameWidget()
				]
			.ValueWidget
				[
					SNew(SComboBox<TSharedPtr<FName>>)
					.OptionsSource(&PossibleMaterialNames)
					.ContentPadding(2.0f)
					.OnGenerateWidget(this, &FNiagaraSkeletalMeshRegionMaterialFilterDetails::HandleMaterialNameComboBoxGenerateWidget)
					.OnSelectionChanged(this, &FNiagaraSkeletalMeshRegionMaterialFilterDetails::HandleMaterialNameComboBoxSelectionChanged)
					.OnComboBoxOpening(this, &FNiagaraSkeletalMeshRegionMaterialFilterDetails::OnComboOpening)
					[
						SNew(STextBlock)
						.Text(this, &FNiagaraSkeletalMeshRegionMaterialFilterDetails::HandleMaterialNameComboBoxContentText)
					]
				];
		}
		else
		{
			ChildBuilder.AddProperty(ChildHandle);
		}
	}
}

FText FNiagaraSkeletalMeshRegionMaterialFilterDetails::HandleMaterialNameComboBoxContentText() const
{
	FName OutName;
	MaterialNameHandle->GetValue(OutName);
	return FText::FromName(OutName);
}

TSharedRef<SWidget> FNiagaraSkeletalMeshRegionMaterialFilterDetails::HandleMaterialNameComboBoxGenerateWidget(TSharedPtr<FName> StringItem)
{
	FName DefaultName = StringItem.IsValid() ? *StringItem : NAME_None;
	return SNew(STextBlock).Text(FText::FromName(DefaultName));
}

void FNiagaraSkeletalMeshRegionMaterialFilterDetails::HandleMaterialNameComboBoxSelectionChanged(TSharedPtr<FName> StringItem, ESelectInfo::Type SelectInfo)
{
	if (StringItem)
	{
		MaterialNameHandle->SetValue(*StringItem);
	}
}

void FNiagaraSkeletalMeshRegionMaterialFilterDetails::OnComboOpening()
{
	PossibleMaterialNames.Reset();
	if (MeshObject->GetSkeleton() != nullptr)
	{
		//Populate PossibleMaterialsNames
		for (FSkeletalMaterial Material : MeshObject->GetMaterials())
		{
			TSharedPtr<FName> MaterialName(new FName(Material.MaterialSlotName));
			PossibleMaterialNames.Add(MaterialName);
		}
	}
}

#undef LOCTEXT_NAMESPACE