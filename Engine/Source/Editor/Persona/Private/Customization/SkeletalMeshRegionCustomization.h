// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "PropertyHandle.h"
#include "IDetailCustomization.h"

class IDetailLayoutBuilder;
class IPropertyHandle;
class IDetailChildrenBuilder;
class USkeletalMesh;


class FNiagaraSkeletalMeshRegionBoneFilterDetails : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShareable(new FNiagaraSkeletalMeshRegionBoneFilterDetails());
	}

	// IDetailCustomization interface
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

private:
	FText HandleBoneNameComboBoxContentText() const;
	TSharedRef<SWidget> HandleBoneNameComboBoxGenerateWidget(FName StringItem);
	void HandleBoneNameComboBoxSelectionChanged(FName StringItem, ESelectInfo::Type SelectInfo);

	void OnComboOpening();

	/** The mesh object whose details we're customizing. */
	USkeletalMesh* MeshObject;

	TArray<FName> PossibleBoneNames;

	TSharedPtr<IPropertyHandle> BoneNameHandle;
};

class FNiagaraSkeletalMeshRegionMaterialFilterDetails : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShareable(new FNiagaraSkeletalMeshRegionMaterialFilterDetails());
	}

	// IDetailCustomization interface
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

private:
	FText HandleMaterialNameComboBoxContentText() const;
	TSharedRef<SWidget> HandleMaterialNameComboBoxGenerateWidget(TSharedPtr<FName> StringItem);
	void HandleMaterialNameComboBoxSelectionChanged(TSharedPtr<FName> StringItem, ESelectInfo::Type SelectInfo);

	void OnComboOpening();

	/** The mesh object whose details we're customizing. */
	USkeletalMesh * MeshObject;

	TArray<TSharedPtr<FName>> PossibleMaterialNames;

	TSharedPtr<IPropertyHandle> MaterialNameHandle;
};


