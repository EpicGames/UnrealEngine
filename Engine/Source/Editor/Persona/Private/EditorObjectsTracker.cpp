// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorObjectsTracker.h"

#include "HAL/PlatformCrt.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"

void FEditorObjectTracker::AddReferencedObjects( FReferenceCollector& Collector )
{
	Collector.AddReferencedObjects(EditorObjMap);
	Collector.AddReferencedObjects(EditorObjectArray);
}

UObject* FEditorObjectTracker::GetEditorObjectForClass( UClass* EdClass )
{
	UObject *Obj = (bAllowOnePerClass && EditorObjMap.Contains(EdClass) ? *EditorObjMap.Find(EdClass) : nullptr);

	if (Obj == NULL)
	{
		FString ObjName = MakeUniqueObjectName(GetTransientPackage(), EdClass).ToString();
		ObjName += "_EdObj";
		Obj = NewObject<UObject>(GetTransientPackage(), EdClass, FName(*ObjName), RF_Public | RF_Standalone | RF_Transient);
		if (bAllowOnePerClass)
		{
			EditorObjMap.Add(EdClass, Obj);
		}
		else
		{
			EditorObjectArray.Add(Obj);
		}
	}
	return Obj;
}
