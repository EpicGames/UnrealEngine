// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "UObject/GCObject.h"

class FReferenceCollector;
class UClass;
class UObject;

//////////////////////////////////////////////////////////////////////////
// FEditorObjectTracker

class FEditorObjectTracker : public FGCObject
{
public:
	FEditorObjectTracker(bool bInAllowOnePerClass = true)
		: bAllowOnePerClass(bInAllowOnePerClass)
	{}

	// FGCObject interface
	virtual void AddReferencedObjects( FReferenceCollector& Collector ) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FEditorObjectTracker");
	}
	// End of FGCObject interface

	/** Returns an existing editor object for the specified class or creates one
	    if none exist */
	UObject* GetEditorObjectForClass( UClass* EdClass );

	void SetAllowOnePerClass(bool bInAllowOnePerClass)
	{
		bAllowOnePerClass = bInAllowOnePerClass;
	}
private:

	/** If true, it uses TMap, otherwise, it just uses TArray */
	bool bAllowOnePerClass;

	/** Tracks editor objects created for details panel */
	TMap< TObjectPtr<UClass>, TObjectPtr<UObject> >	EditorObjMap;

	/** Tracks editor objects created for detail panel */
	TArray<TObjectPtr<UObject>> EditorObjectArray;
};
