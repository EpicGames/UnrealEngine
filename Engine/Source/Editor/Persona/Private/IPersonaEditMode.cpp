// Copyright Epic Games, Inc. All Rights Reserved.

#include "IPersonaEditMode.h"

IPersonaEditMode::IPersonaEditMode()
{
}

IPersonaEditMode::~IPersonaEditMode()
{
}

void IPersonaEditMode::Enter()
{
	FAnimationEditMode::Enter();
}

void IPersonaEditMode::Exit()
{
	FAnimationEditMode::Exit();
}

void IPersonaEditMode::AddReferencedObjects(FReferenceCollector& Collector)
{
	FAnimationEditMode::AddReferencedObjects(Collector);
}
