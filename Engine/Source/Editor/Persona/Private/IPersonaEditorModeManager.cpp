// Copyright Epic Games, Inc. All Rights Reserved.

#include "IPersonaEditorModeManager.h"

#include "ContextObjectStore.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "SEditorViewport.h"
#include "Slate/SceneViewport.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Application/SlateUser.h"
#include "Selection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IPersonaEditorModeManager)

IPersonaEditorModeManager* UPersonaEditorModeManagerContext::GetPersonaEditorModeManager() const
{
	return ModeManager;
}

bool UPersonaEditorModeManagerContext::GetCameraTarget(FSphere& OutTarget) const
{
	check(ModeManager);
	return ModeManager->GetCameraTarget(OutTarget);
}

void UPersonaEditorModeManagerContext::GetOnScreenDebugInfo(TArray<FText>& OutDebugText) const
{
	check(ModeManager);
	return ModeManager->GetOnScreenDebugInfo(OutDebugText);
}

void UPersonaEditorModeManagerContext::SetFocusInViewport()
{
	const IPersonaEditorModeManager* PersonaEditorModeManager = GetPersonaEditorModeManager();
	if (!PersonaEditorModeManager)
	{
		return;
	}

	const FEditorViewportClient* ViewportClient = PersonaEditorModeManager->GetHoveredViewportClient();
	if (!ViewportClient)
	{
		ViewportClient = PersonaEditorModeManager->GetFocusedViewportClient();
	}
	if (!ViewportClient)
	{
		return;
	}

	const TWeakPtr<SViewport> ViewportWidget = ViewportClient->GetEditorViewportWidget()->GetSceneViewport()->GetViewportWidget();
	if (!ViewportWidget.IsValid())
	{
		return;
	}
	
	// set focus back to viewport so that hotkeys are immediately detected
	TSharedPtr<SWidget> ViewportContents = ViewportWidget.Pin()->GetContent();
	FSlateApplication::Get().ForEachUser([&ViewportContents](FSlateUser& User) 
	{
		User.SetFocus(ViewportContents.ToSharedRef());
	});
}

IPersonaEditorModeManager::IPersonaEditorModeManager()
{
	UModeManagerInteractiveToolsContext* ToolsContext = GetInteractiveToolsContext();
	UContextObjectStore* ContextObjectStore = ToolsContext->ContextObjectStore;
	ContextObjectStore->AddContextObject(PersonaModeManagerContext.Get());

	SelectionSet = NewObject<UTypedElementSelectionSet>(GetTransientPackage(), NAME_None, RF_Transactional);

	// Support legacy USelections
	ActorSet->SetElementSelectionSet(SelectionSet);
	ComponentSet->SetElementSelectionSet(SelectionSet);
	ObjectSet->SetElementSelectionSet(SelectionSet);
}

IPersonaEditorModeManager::~IPersonaEditorModeManager()
{
	UContextObjectStore* ContextObjectStore = GetInteractiveToolsContext()->ContextObjectStore;
	ContextObjectStore->RemoveContextObject(PersonaModeManagerContext.Get());
}

void IPersonaEditorModeManager::AddReferencedObjects(FReferenceCollector& Collector)
{
	FAssetEditorModeManager::AddReferencedObjects(Collector);
	Collector.AddReferencedObject(PersonaModeManagerContext);
	Collector.AddReferencedObject(SelectionSet);
}

UTypedElementSelectionSet* IPersonaEditorModeManager::GetEditorSelectionSet() const
{
	return SelectionSet.Get();
}
