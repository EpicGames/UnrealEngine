// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "PoseWatchManagerFwd.h"
#include "PoseWatchManagerStandaloneTypes.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SHeaderRow.h"
#include "BlueprintEditor.h"
#include "Animation/AnimBlueprint.h"

class FPoseWatchManagerDefaultMode;
template<typename ItemType> class STreeView;

/**
 * The public interface for the Pose Watch Manager widget
 */
class IPoseWatchManager : public SCompoundWidget
{
public:

	/** Sends a requests to refresh itself the next chance it gets */
	virtual void Refresh() = 0;

	/** Sends a request to clear the entire tree and rebuild it from scratch */
	virtual void FullRefresh() = 0;

	/** @return Returns a string to use for highlighting results in the outliner list */
	virtual TAttribute<FText> GetFilterHighlightText() const = 0;

	/** Get a const reference to the actual tree hierarchy */
	virtual const STreeView<FPoseWatchManagerTreeItemPtr>& GetTree() const = 0;

	/** Set the keyboard focus to the outliner */
	virtual void SetKeyboardFocus() = 0;

	/** Return the sorting mode for the specified ColumnId */
	virtual EColumnSortMode::Type GetColumnSortMode(const FName ColumnId) const = 0;

	virtual uint32 GetTypeSortPriority(const IPoseWatchManagerTreeItem& Item) const = 0;

	/** Request that the tree be sorted at a convenient time */
	virtual void RequestSort() = 0;

	/** Executes rename. */
	virtual void Rename_Execute() = 0;

	/** Set the item selection of the outliner based on a selector function. Any items which return true will be added */
	virtual void SetSelection(const TFunctionRef<bool(IPoseWatchManagerTreeItem&)> Selector) = 0;

	UAnimBlueprint* AnimBlueprint;

	/** Get the active PoseWatchManagerMode */
	const TSharedPtr<FPoseWatchManagerDefaultMode> GetMode() const { return Mode; }

protected:
	TSharedPtr<FPoseWatchManagerDefaultMode> Mode;
};
