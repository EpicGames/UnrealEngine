// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PoseWatchManagerFwd.h"
#include "PoseWatchManagerStandaloneTypes.h"
#include "Widgets/SWidget.h"
#include "Widgets/Views/SHeaderRow.h"

template<typename ItemType> class STableRow;

class IPoseWatchManagerColumn : public TSharedFromThis<IPoseWatchManagerColumn>
{
public:
	virtual ~IPoseWatchManagerColumn() = default;

	virtual FName GetColumnID() = 0;

	virtual SHeaderRow::FColumn::FArguments ConstructHeaderRowColumn() = 0;

	virtual const TSharedRef<SWidget> ConstructRowWidget(FPoseWatchManagerTreeItemRef TreeItem, const STableRow<FPoseWatchManagerTreeItemPtr>& Row) = 0;

	virtual void PopulateSearchStrings(const IPoseWatchManagerTreeItem& Item, TArray<FString>& OutSearchStrings) const {}

	virtual bool SupportsSorting() const { return false; }

	virtual void SortItems(TArray<FPoseWatchManagerTreeItemPtr>& OutItems, const EColumnSortMode::Type SortMode) const {}
};

