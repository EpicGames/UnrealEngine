// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "Widgets/Views/STableRow.h"
#include "PoseWatchManagerStandaloneTypes.h"

#define UE_API PERSONA_API

class UToolMenu;
class SPoseWatchManager;

struct IPoseWatchManagerTreeItem : TSharedFromThis<IPoseWatchManagerTreeItem>
{
	friend SPoseWatchManager;

public:
	/** Delegate for hooking up an inline editable text block to be notified that a rename is requested. */
	DECLARE_DELEGATE(FOnRenameRequest);
	FOnRenameRequest RenameRequestEvent;

protected:
	/** Default constructor */
	IPoseWatchManagerTreeItem(EPoseWatchTreeItemType InType) : Parent(nullptr), TreeType(InType) {}
	virtual ~IPoseWatchManagerTreeItem() = default;

public:
	/** Get this item's parent. If nullptr then belongs to the root of the tree */
	FPoseWatchManagerTreeItemPtr GetParent() const
	{
		return Parent.Pin();
	}

	/** Returns true if this tree item is inside a folder */
	virtual bool IsAssignedFolder() const = 0;

	/** Sets this tree item's visibility */
	virtual void SetIsVisible(const bool bVisible) = 0;

	/** Sets whether or not this tree item is expanded  */
	virtual void SetIsExpanded(const bool bIsExpanded) {}

	/** Add a child to this tree item */
	void AddChild(FPoseWatchManagerTreeItemRef Child)
	{
		check(!Children.Contains(Child));
		Child->Parent = AsShared();
		Children.Add(MoveTemp(Child));
	}

	/** Get this item's children, if any. */
	FORCEINLINE const TSet<TWeakPtr<IPoseWatchManagerTreeItem>>& GetChildren() const
	{
		return Children;
	}

	/** Attempt to cast this item to another type if it is of that type. Returns nullptr if it fails */
	template <typename T>
	T* CastTo()
	{
		return TreeType == T::Type ? StaticCast<T*>(this) : nullptr;
	}
	/** Attempt to cast this item to another type if it is of that type. Returns nullptr if it fails */
	template <typename T>
	const T* CastTo() const
	{
		return TreeType == T::Type ? StaticCast<const T*>(this) : nullptr;
	}
	/** Returns true if this item is of the specified type */
	template <typename T>
	bool IsA() const
	{
		return TreeType == T::Type;
	}

	/** Returns true if the data the item references is valid */
	virtual bool IsValid() const = 0;

	/** Get the ID that represents this tree item. Used to reference this item in a map */
	virtual FObjectKey GetID() const = 0;

	/** Get the raw string to display for this tree item - used for sorting */
	virtual FString GetDisplayString() const = 0;

	/** Generate the label widget for this item */
	virtual TSharedRef<SWidget> GenerateLabelWidget(IPoseWatchManager& Outliner, const STableRow<FPoseWatchManagerTreeItemPtr>& InRow) { return SNullWidget::NullWidget; }

	/** Generate a context menu for this item. Only called if *only* this item is selected. */
	virtual TSharedPtr<SWidget> CreateContextMenu() = 0;

	/** Query this items visibility state. Only called if the item type has visibility info */
	virtual bool GetVisibility() const = 0;

	/** Returns true if this tree item has at least one child */
	virtual bool HasChildren() const { return false; }

	/** Returns true if this tree item is expanded */
	virtual bool IsExpanded() const { return false; }

	virtual bool IsEnabled() const { return true; }

	virtual void OnRemoved() = 0;

	/** Returns the view port render color */
	virtual FColor GetColor() const { return FColor(); }

	/** Sets the view port render color */
	virtual void SetColor(const FColor& InColor) {}

	/** Returns true if the view port render color for this item can be changed */
	virtual bool ShouldDisplayColorPicker() const { return false; }

protected:
	/** This item's parent tree item */
	TWeakPtr<IPoseWatchManagerTreeItem> Parent;

	/** Array of children contained underneath this item */
	TSet<TWeakPtr<IPoseWatchManagerTreeItem>> Children;

	/** Static type identifier for the base class tree item */
	static UE_API const EPoseWatchTreeItemType Type;

	/** Tree item type identifier */
	EPoseWatchTreeItemType TreeType;
};

template <> inline const IPoseWatchManagerTreeItem* IPoseWatchManagerTreeItem::CastTo<IPoseWatchManagerTreeItem>() const
{
	return this;
}

template <> inline IPoseWatchManagerTreeItem* IPoseWatchManagerTreeItem::CastTo<IPoseWatchManagerTreeItem>()
{
	return this;
}

#undef UE_API
