// Copyright Epic Games, Inc. All Rights Reserved.

#include "LODInfoUILayout.h"
#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "SkeletalMeshTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LODInfoUILayout)

ULODInfoUILayout::ULODInfoUILayout(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PersonaToolkit = nullptr;
	LODIndex = INDEX_NONE;
}

void ULODInfoUILayout::SetReferenceLODInfo(TWeakPtr<IPersonaToolkit> InPersonaToolkit, int32 InLODIndex)
{
	check(InPersonaToolkit.IsValid());
	PersonaToolkit = InPersonaToolkit;
	LODIndex = InLODIndex;
	USkeletalMesh* SkeletalMesh = PersonaToolkit.Pin()->GetPreviewMesh();
	const FSkeletalMeshLODInfo* SkeletalMeshLODInfo = SkeletalMesh->GetLODInfo(LODIndex);
	check(SkeletalMeshLODInfo != nullptr);
	//Copy the LODInfo Array to the temporary
	LODInfo = *SkeletalMeshLODInfo;
}

void ULODInfoUILayout::RefreshReferenceLODInfo()
{
	// Request to refresh can be queued from other threads and thus
	// by the time it tries to refresh, it is possible that we no longer have a valid ptr
	if (PersonaToolkit.IsValid())
	{
		if (USkeletalMesh* SkeletalMesh = PersonaToolkit.Pin()->GetPreviewMesh())
		{
			// If the LOD info is out of date, don't update. Most likely this object is going to
			// go away when Persona details panel for the skelmesh updates.
			if (const FSkeletalMeshLODInfo* SkeletalMeshLODInfo = SkeletalMesh->GetLODInfo(LODIndex))
			{
				//Copy the LODInfo Array to the temporary
				LODInfo = *SkeletalMeshLODInfo;	
			}
		}
	}
}

void ULODInfoUILayout::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if ((PropertyChangedEvent.ChangeType & EPropertyChangeType::Interactive) == 0)
	{
		USkeletalMesh* SkeletalMesh = PersonaToolkit.Pin()->GetPreviewMesh();
		FSkeletalMeshLODInfo* SkeletalMeshLODInfo = SkeletalMesh->GetLODInfo(LODIndex);
		check(SkeletalMeshLODInfo != nullptr);
		SkeletalMesh->Modify();
		const FGuid OldBuildGuid = SkeletalMeshLODInfo->BuildGUID; // Save the current Guid
		*SkeletalMeshLODInfo = LODInfo; //Copy the LODInfo into the real skeletal mesh LODInfo data
		SkeletalMeshLODInfo->BuildGUID = OldBuildGuid;	// Restore the saved Guid, as it is used to detect changes in the LODInfo data
		FScopedSkeletalMeshPostEditChange ScopeSkeletalmeshPostEdit(SkeletalMesh);
	}
}
