// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * Fbx Importer UI options.
 */

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "IPersonaToolkit.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkinnedAssetCommon.h"
#include "LODInfoUILayout.generated.h"

UCLASS(HideCategories=Object, MinimalAPI)
class ULODInfoUILayout : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	void SetReferenceLODInfo(TWeakPtr<IPersonaToolkit> InPersonaToolkit, int32 InLODIndex);
	void RefreshReferenceLODInfo();

	TSharedPtr<IPersonaToolkit> GetPersonaToolkit() const
	{
		check(PersonaToolkit.IsValid());
		return PersonaToolkit.Pin();
	}

	int32 GetLODIndex() const { return LODIndex; }

	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	
	/** Struct containing information for a LOD level, such as materials to use, and when use the LOD. */
	UPROPERTY(EditAnywhere, Category = LevelOfDetail)
	struct FSkeletalMeshLODInfo LODInfo;

private:
	TWeakPtr<IPersonaToolkit> PersonaToolkit;
	int32 LODIndex;
};


