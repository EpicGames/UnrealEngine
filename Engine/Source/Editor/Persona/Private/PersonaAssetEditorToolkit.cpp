// Copyright Epic Games, Inc. All Rights Reserved.

#include "PersonaAssetEditorToolkit.h"

#include "PersonaModule.h"
#include "AssetEditorModeManager.h"
#include "Elements/Framework/EngineElementsLibrary.h"
#include "Elements/Framework/TypedElementSelectionSet.h"
#include "Modules/ModuleManager.h"
#include "IPersonaEditorModeManager.h"
#include "ISkeletonTreeItem.h"
#include "PersonaTabs.h"

void FPersonaAssetEditorToolkit::CreateEditorModeManager()
{
	EditorModeManager = MakeShareable(FModuleManager::LoadModuleChecked<FPersonaModule>("Persona").CreatePersonaEditorModeManager());

	// Make sure we get told when the editor mode changes so we can switch to the appropriate tab
	// if there's a toolbox available.
	GetEditorModeManager().OnEditorModeIDChanged().AddSP(this, &FPersonaAssetEditorToolkit::OnEditorModeIdChanged);

	if (UTypedElementSelectionSet* SelectionSet = GetEditorModeManager().GetEditorSelectionSet())
	{
		SelectionSetChangedHandle = SelectionSet->OnChanged().AddSP(this, &FPersonaAssetEditorToolkit::OnSelectionSetChanged);
	}
}

void FPersonaAssetEditorToolkit::OnClose()
{
	if (SelectionSetChangedHandle.IsValid())
	{
		if (UTypedElementSelectionSet* SelectionSet = GetEditorModeManager().GetEditorSelectionSet())
		{
			SelectionSet->OnChanged().Remove(SelectionSetChangedHandle);
		}
		SelectionSetChangedHandle.Reset();
	}

	FWorkflowCentricApplication::OnClose();
}

void FPersonaAssetEditorToolkit::OnSelectionSetChanged(const UTypedElementSelectionSet* InSelectionSet)
{
	// Base implementation is empty. Subclasses override to update details views, etc.
}

void FPersonaAssetEditorToolkit::OnToolkitHostingStarted(const TSharedRef<IToolkit>& Toolkit)
{
	if (ensure(!HostedToolkit.IsValid()))
	{
		HostedToolkit = Toolkit;
		OnAttachToolkit.Broadcast(Toolkit);
	}
}

void FPersonaAssetEditorToolkit::OnToolkitHostingFinished(const TSharedRef<IToolkit>& Toolkit)
{
	if (ensure(Toolkit == HostedToolkit))
	{
		HostedToolkit.Reset();
		OnDetachToolkit.Broadcast(Toolkit);
	}
}

void FPersonaAssetEditorToolkit::OnEditorModeIdChanged(const FEditorModeID& ModeChangedID, bool bIsEnteringMode)
{
	if (GetEditorModeManager().IsDefaultMode(ModeChangedID))
	{
		return;
	}

	if (bIsEnteringMode)
	{
		if (HostedToolkit.IsValid())
		{
			TabManager->TryInvokeTab(FPersonaTabs::ToolboxID);
		}
	}
	else
	{
		const TSharedPtr<SDockTab> ToolboxTab = TabManager->FindExistingLiveTab(FPersonaTabs::ToolboxID);
		if (ToolboxTab.IsValid())
		{
			ToolboxTab->RequestCloseTab();
		}
	}
}

void FPersonaAssetEditorToolkit::RouteTreeSelectionToModeManager(const TConstArrayView<TSharedPtr<ISkeletonTreeItem>>& InItems)
{
	// @note: this is one-way. Selecting a TypedElement will not propagate to the SkeletonTree, it may be cleaner to implement an adapter
	// (see: USelection::SetElementSelectionSet and associated plumbing)

	UTypedElementSelectionSet* SelectionSet = GetEditorModeManager().GetEditorSelectionSet();
	if (!SelectionSet)
	{
		return;
	}

	FTypedElementSelectionOptions SelectionOptions;
	SelectionOptions.SetAllowLegacyNotifications(false);

	// Only deselect elements that we previously routed, rather than clearing the entire SelectionSet.
	for (const FTypedElementHandle& Handle : RoutedSelectionHandles)
	{
		SelectionSet->DeselectElement(Handle, SelectionOptions);
	}

	RoutedSelectionHandles.Reset();

	for (const TSharedPtr<ISkeletonTreeItem>& Item : InItems)
	{
		if (const UObject* Object = Item->GetObject())
		{
			FTypedElementHandle ElementHandle = UEngineElementsLibrary::AcquireEditorObjectElementHandle(Object);
			if (ElementHandle.IsSet())
			{
				SelectionSet->SelectElement(ElementHandle, SelectionOptions);
				RoutedSelectionHandles.Add(ElementHandle);
			}
		}
	}
}
