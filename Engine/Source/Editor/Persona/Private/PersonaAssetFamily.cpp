// Copyright Epic Games, Inc. All Rights Reserved.

#include "PersonaAssetFamily.h"
#include "Modules/ModuleManager.h"
#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Animation/AnimBlueprint.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "Styling/AppStyle.h"
#include "AssetToolsModule.h"
#include "Preferences/PersonaOptions.h"
#include "UObject/UObjectIterator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PersonaAssetFamily)

#define LOCTEXT_NAMESPACE "PersonaAssetFamily"

FPersonaAssetFamily::FPersonaAssetFamily(const UObject* InFromObject)
{
	for (TObjectIterator<UAnimationEditorsAssetFamilyExtension> ExtensionIterator(RF_NoFlags); ExtensionIterator; ++ExtensionIterator)
	{
		if (ExtensionIterator->GetClass() == UAnimationEditorsAssetFamilyExtension::StaticClass())
		{
			continue;
		}
		FExtenderObjects& Extender = Extenders.AddDefaulted_GetRef();
		Extender.Extension = *ExtensionIterator;
		Extender.Asset = nullptr;
	}

	// Construct a directed graph to sort extenders according to position rules
	{
		TMap<FName, TArray<FName>> Graph;
		TMap<FName, int32> InDegree;
		TMap<FName, FExtenderObjects*> ExtenderMap;

		for (FExtenderObjects& Extender : Extenders)
		{
			const FName AssetClassName = Extender.Extension->GetAssetClass()->GetFName();
			ExtenderMap.Add(AssetClassName, &Extender);
			InDegree.Add(AssetClassName, 0);
			Graph.Add(AssetClassName, TArray<FName>());
		}

		for (FExtenderObjects& Extender : Extenders)
		{
			const FName AssetClassName = Extender.Extension->GetAssetClass()->GetFName();

			FName BeforeName = NAME_None;
			FName AfterName = NAME_None;
			Extender.Extension->GetPosition(BeforeName, AfterName);

			if (BeforeName != NAME_None && Graph.Contains(AssetClassName) && InDegree.Contains(BeforeName))
			{
				Graph[AssetClassName].Add(BeforeName);
				++InDegree[BeforeName];
			}
			if (AfterName != NAME_None && Graph.Contains(AfterName) && InDegree.Contains(AssetClassName))
			{
				Graph[AfterName].Add(AssetClassName);
				++InDegree[AssetClassName];
			}
		}

		TQueue<FName> Queue;
		for (const auto& [ClassName, Degree] : InDegree)
		{
			if (Degree == 0)
			{
				Queue.Enqueue(ClassName);
			}
		}

		TArray<FExtenderObjects> SortedExtenders;

		while (!Queue.IsEmpty())
		{
			FName Current = NAME_None;
			Queue.Dequeue(Current);

			SortedExtenders.Add(*ExtenderMap[Current]);

			for (const auto& Neighbour : Graph[Current])
			{
				--InDegree[Neighbour];
				if (InDegree[Neighbour] == 0)
				{
					Queue.Enqueue(Neighbour);
				}
			}
		}

		if (SortedExtenders.Num() != Extenders.Num())
		{
			UE_LOGF(LogAnimation, Warning, "Unable to sort animation editor extenders");
		}
		else
		{
			Extenders = SortedExtenders;
		}
	}

	if (InFromObject)
	{
		for (FExtenderObjects& Extender : Extenders)
		{
			if (InFromObject->IsA(Extender.Extension->GetAssetClass()))
			{
				Extender.Asset = InFromObject;
				Extender.Extension->FindCounterpartAssets(InFromObject, *this);
			}
		}
	}
}

FPersonaAssetFamily::FPersonaAssetFamily(const UObject* InFromObject, const TSharedRef<FPersonaAssetFamily> InFromFamily)
	: Extenders(InFromFamily->Extenders)
{
	if (InFromObject)
	{
		for (FExtenderObjects& Extender : Extenders)
		{
			if (InFromObject->IsA(Extender.Extension->GetAssetClass()))
			{
				Extender.Asset = InFromObject;
				Extender.Extension->FindCounterpartAssets(InFromObject, *this);
			}
		}
	}
}

void FPersonaAssetFamily::Initialize()
{
	GetMutableDefault<UPersonaOptions>()->RegisterOnUpdateSettings(UPersonaOptions::FOnUpdateSettingsMulticaster::FDelegate::CreateSP(this, &FPersonaAssetFamily::OnSettingsChange));
}

void FPersonaAssetFamily::GetAssetTypes(TArray<UClass*>& OutAssetTypes) const
{
	OutAssetTypes.Reset();

	for (const FExtenderObjects& Extender : Extenders)
	{
		OutAssetTypes.Add(Extender.Extension->GetAssetClass());
	}
}

template<typename AssetType>
static void FindAssets(const USkeleton* InSkeleton, TArray<FAssetData>& OutAssetData, FName SkeletonTag)
{
	if (!InSkeleton)
	{
		return;
	}

	InSkeleton->GetCompatibleAssets(AssetType::StaticClass(), *SkeletonTag.ToString(), OutAssetData);
}

FAssetData FPersonaAssetFamily::FindAssetOfType(UClass* InAssetClass) const
{
	if (const FExtenderObjects* const Extender = GetExtensionForClass(InAssetClass))
	{
		if (Extender->Asset.IsValid())
		{
			return FAssetData(Extender->Asset.Get());
		}
		else
		{
			TArray<FAssetData> Assets;
			Extender->Extension->FindAssetsOfType(Assets, *this);

			if (Assets.Num() > 0)
			{
				return Assets[0];
			}
		}
	}
	return FAssetData();
}

void FPersonaAssetFamily::FindAssetsOfType(UClass* InAssetClass, TArray<FAssetData>& OutAssets) const
{
	if (const FExtenderObjects* const Extender = GetExtensionForClass(InAssetClass))
	{
		Extender->Extension->FindAssetsOfType(OutAssets, *this);
	}
}

FText FPersonaAssetFamily::GetAssetTypeDisplayName(UClass* InAssetClass) const
{
	if (const FExtenderObjects* const Extender = GetExtensionForClass(InAssetClass))
	{
		return Extender->Extension->GetAssetTypeDisplayName();
	}
	return FText();
}

const FSlateBrush* FPersonaAssetFamily::GetAssetTypeDisplayIcon(UClass* InAssetClass) const
{
	if (const FExtenderObjects* const Extender = GetExtensionForClass(InAssetClass))
	{
		return Extender->Extension->GetAssetTypeDisplayIcon();
	}
	return nullptr;
}	

FSlateColor FPersonaAssetFamily::GetAssetTypeDisplayTint(UClass* InAssetClass) const
{
	static const FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");

	if (InAssetClass)
	{
		if (const FExtenderObjects* const Extender = GetExtensionForClass(InAssetClass))
		{
			TWeakPtr<IAssetTypeActions> AssetTypeActions = AssetToolsModule.Get().GetAssetTypeActionsForClass(Extender->Extension->GetAssetClass());
			if (AssetTypeActions.IsValid())
			{
				return AssetTypeActions.Pin()->GetTypeColor();
			}
		}
	}

	return FSlateColor::UseForeground();
}

bool FPersonaAssetFamily::IsAssetCompatible(const FAssetData& InAssetData) const
{
	UClass* Class = InAssetData.GetClass();

	if (const FExtenderObjects* const Extender = GetExtensionForClass(Class))
	{
		return Extender->Extension->IsAssetCompatible(InAssetData, *this);
	}

	return false;
}

UClass* FPersonaAssetFamily::GetAssetFamilyClass(UClass* InClass) const
{
	if (const FExtenderObjects* const Extender = GetExtensionForClass(InClass))
	{
		return Extender->Extension->GetAssetClass();
	}
	return nullptr;
}

void FPersonaAssetFamily::RecordAssetOpened(const FAssetData& InAssetData)
{
	if (IsAssetCompatible(InAssetData))
	{
		UClass* Class = InAssetData.GetClass();
		if (Class)
		{
			if (FExtenderObjects* Extender = GetExtensionForClass(Class))
			{
				Extender->Asset = InAssetData.GetAsset();
			}
		}

		OnAssetOpened.Broadcast(InAssetData.GetAsset());
	}
}

bool FPersonaAssetFamily::IsAssetTypeInFamily(const TObjectPtr<UClass> InClass) const
{
	return GetExtensionForClass(InClass) != nullptr;
}

TWeakObjectPtr<const UObject> FPersonaAssetFamily::GetAssetOfType(const TObjectPtr<UClass> InClass) const
{
	if (const FExtenderObjects* const Extender = GetExtensionForClass(InClass))
	{
		return Extender->Asset;
	}
	return nullptr;
}

bool FPersonaAssetFamily::SetAssetOfType(const TObjectPtr<UClass> InClass, TWeakObjectPtr<const UObject> InObject)
{
	if (FExtenderObjects* Extender = GetExtensionForClass(InClass))
	{
		Extender->Asset = InObject;	
		return true;

	}
	return false;
}

void FPersonaAssetFamily::AddReferencedObjects(FReferenceCollector& Collector)
{
	for (FExtenderObjects& Extender : Extenders)
	{
		Collector.AddReferencedObject(Extender.Extension);
	}
}

FString FPersonaAssetFamily::GetReferencerName() const
{
	return TEXT("FPersonaAssetFamily");
}

void FPersonaAssetFamily::OnSettingsChange(const UPersonaOptions* InOptions, EPropertyChangeType::Type InChangeType)
{
	OnAssetFamilyChanged.Broadcast();
}

FPersonaAssetFamily::FExtenderObjects* FPersonaAssetFamily::GetExtensionForClass(const TObjectPtr<const UClass> InClass)
{
	if (InClass)
	{
		for (FExtenderObjects& Extender : Extenders)
		{
			if (InClass->IsChildOf(Extender.Extension->GetAssetClass()))
			{
				return &Extender;
			}
		}
	}
	return nullptr;
}

const FPersonaAssetFamily::FExtenderObjects* const FPersonaAssetFamily::GetExtensionForClass(const TObjectPtr<const UClass> InClass) const
{
	if (InClass)
	{
		for (const FExtenderObjects& Extender : Extenders)
		{
			if (InClass->IsChildOf(Extender.Extension->GetAssetClass()))
			{
				return &Extender;
			}
		}
	}
	return nullptr;
}

// UAnimationEditorsAssetFamilyExtension_SkeletonAsset

TObjectPtr<UClass> UAnimationEditorsAssetFamilyExtension_SkeletonAsset::GetAssetClass() const
{
	return USkeleton::StaticClass();
}

FText UAnimationEditorsAssetFamilyExtension_SkeletonAsset::GetAssetTypeDisplayName() const
{
	return LOCTEXT("SkeletonAssetDisplayName", "Skeleton");
}

const FSlateBrush* UAnimationEditorsAssetFamilyExtension_SkeletonAsset::GetAssetTypeDisplayIcon() const
{
	return FAppStyle::Get().GetBrush("Persona.AssetClass.Skeleton");
}

void UAnimationEditorsAssetFamilyExtension_SkeletonAsset::FindAssetsOfType(TArray<FAssetData>& OutAssets, const IAnimationEditorsAssetFamilyInterface& AssetFamilyInterface) const
{
	if (TObjectPtr<const USkeleton> SkeletonAsset = AssetFamilyInterface.GetAssetOfType<USkeleton>())
	{
		SkeletonAsset->GetCompatibleSkeletonAssets(OutAssets);
	}
}

bool UAnimationEditorsAssetFamilyExtension_SkeletonAsset::IsAssetCompatible(const FAssetData& InAssetData, const IAnimationEditorsAssetFamilyInterface& AssetFamilyInterface) const
{
	if (TObjectPtr<const USkeleton> SkeletonAsset = AssetFamilyInterface.GetAssetOfType<USkeleton>())
	{
		return SkeletonAsset.Get()->IsCompatibleForEditor(InAssetData);
	}
	return false;
}

void UAnimationEditorsAssetFamilyExtension_SkeletonAsset::FindCounterpartAssets(const UObject* InAsset, IAnimationEditorsAssetFamilyInterface& AssetFamilyInterface)
{
	const USkeleton* SkeletonAsset = CastChecked<const USkeleton>(InAsset);

	AssetFamilyInterface.SetAssetOfType<USkeletalMesh>(SkeletonAsset->GetPreviewMesh());
};

void UAnimationEditorsAssetFamilyExtension_SkeletonAsset::GetPosition(FName& OutBeforeClass, FName& OutAfterClass) const
{
	OutBeforeClass = NAME_None;
	OutAfterClass = NAME_None;
}

// UAnimationEditorsAssetFamilyExtension_SkeletalMeshAsset

TObjectPtr<UClass> UAnimationEditorsAssetFamilyExtension_SkeletalMeshAsset::GetAssetClass() const
{
	return USkeletalMesh::StaticClass();
}

FText UAnimationEditorsAssetFamilyExtension_SkeletalMeshAsset::GetAssetTypeDisplayName() const
{
	return LOCTEXT("SkeletalMeshAssetDisplayName", "Skeletal Mesh");
}

const FSlateBrush* UAnimationEditorsAssetFamilyExtension_SkeletalMeshAsset::GetAssetTypeDisplayIcon() const
{
	return FAppStyle::Get().GetBrush("Persona.AssetClass.SkeletalMesh");
}

void UAnimationEditorsAssetFamilyExtension_SkeletalMeshAsset::FindAssetsOfType(TArray<FAssetData>& OutAssets, const IAnimationEditorsAssetFamilyInterface& AssetFamilyInterface) const
{
	if (TObjectPtr<const USkeleton> SkeletonAsset = AssetFamilyInterface.GetAssetOfType<USkeleton>())
	{
		FindAssets<USkeletalMesh>(SkeletonAsset.Get(), OutAssets, "Skeleton");
	}
}

bool UAnimationEditorsAssetFamilyExtension_SkeletalMeshAsset::IsAssetCompatible(const FAssetData& InAssetData, const IAnimationEditorsAssetFamilyInterface& AssetFamilyInterface) const
{
	FAssetDataTagMapSharedView::FFindTagResult Result = InAssetData.TagsAndValues.FindTag("Skeleton");

	if (Result.IsSet())
	{
		if (TObjectPtr<const USkeleton> SkeletonAsset = AssetFamilyInterface.GetAssetOfType<USkeleton>())
		{
			return SkeletonAsset->IsCompatibleForEditor(Result.GetValue());
		}
	}
	return false;
}

void UAnimationEditorsAssetFamilyExtension_SkeletalMeshAsset::FindCounterpartAssets(const UObject* InAsset, IAnimationEditorsAssetFamilyInterface& AssetFamilyInterface)
{
	const USkeletalMesh* SkeletalMesh = CastChecked<const USkeletalMesh>(InAsset);

	AssetFamilyInterface.SetAssetOfType<USkeleton>(SkeletalMesh->GetSkeleton());

};

void UAnimationEditorsAssetFamilyExtension_SkeletalMeshAsset::GetPosition(FName& OutBeforeClass, FName& OutAfterClass) const
{
	OutBeforeClass = NAME_None;
	OutAfterClass = USkeleton::StaticClass()->GetFName();
}

// UAnimationEditorsAssetFamilyExtension_AnimationAsset

TObjectPtr<UClass> UAnimationEditorsAssetFamilyExtension_AnimationAsset::GetAssetClass() const
{
	return UAnimationAsset::StaticClass();
}

FText UAnimationEditorsAssetFamilyExtension_AnimationAsset::GetAssetTypeDisplayName() const
{
	return LOCTEXT("AnimationAssetDisplayName", "Animation");
}

const FSlateBrush* UAnimationEditorsAssetFamilyExtension_AnimationAsset::GetAssetTypeDisplayIcon() const
{
	return FAppStyle::Get().GetBrush("Persona.AssetClass.Animation");
}

void UAnimationEditorsAssetFamilyExtension_AnimationAsset::FindAssetsOfType(TArray<FAssetData>& OutAssets, const IAnimationEditorsAssetFamilyInterface& AssetFamilyInterface) const
{
	if (TObjectPtr<const USkeleton> SkeletonAsset = AssetFamilyInterface.GetAssetOfType<USkeleton>())
	{
		FindAssets<UAnimationAsset>(SkeletonAsset, OutAssets, "Skeleton");
	}
}

bool UAnimationEditorsAssetFamilyExtension_AnimationAsset::IsAssetCompatible(const FAssetData& InAssetData, const IAnimationEditorsAssetFamilyInterface& AssetFamilyInterface) const
{
	FAssetDataTagMapSharedView::FFindTagResult Result = InAssetData.TagsAndValues.FindTag("Skeleton");

	if (Result.IsSet())
	{
		if (TObjectPtr<const USkeleton> SkeletonAsset = AssetFamilyInterface.GetAssetOfType<USkeleton>())
		{
			return SkeletonAsset->IsCompatibleForEditor(Result.GetValue());
		}
	}

	return false;
}

void UAnimationEditorsAssetFamilyExtension_AnimationAsset::FindCounterpartAssets(const UObject* InAsset, IAnimationEditorsAssetFamilyInterface& AssetFamilyInterface)
{
	const UAnimationAsset* AnimationAsset = CastChecked<const UAnimationAsset>(InAsset);

	AssetFamilyInterface.SetAssetOfType<USkeleton>(AnimationAsset->GetSkeleton());

	AssetFamilyInterface.SetAssetOfType<USkeletalMesh>(AnimationAsset->GetPreviewMesh());

	if (AssetFamilyInterface.IsAssetTypeInFamilyAndUnassigned<USkeletalMesh>())
	{
		AssetFamilyInterface.SetAssetOfType<USkeletalMesh>(AnimationAsset->GetSkeleton()->GetPreviewMesh());
	}

	if (AssetFamilyInterface.IsAssetTypeInFamilyAndUnassigned<USkeletalMesh>())
	{
		AssetFamilyInterface.SetAssetOfType<USkeletalMesh>(AnimationAsset->GetSkeleton()->FindCompatibleMesh());
	}
};

void UAnimationEditorsAssetFamilyExtension_AnimationAsset::GetPosition(FName& OutBeforeClass, FName& OutAfterClass) const
{
	OutBeforeClass = NAME_None;
	OutAfterClass = USkeletalMesh::StaticClass()->GetFName();;
}

// UAnimationEditorsAssetFamilyExtension_AnimBlueprintAsset

TObjectPtr<UClass> UAnimationEditorsAssetFamilyExtension_AnimBlueprintAsset::GetAssetClass() const
{
	return UAnimBlueprint::StaticClass();
}

FText UAnimationEditorsAssetFamilyExtension_AnimBlueprintAsset::GetAssetTypeDisplayName() const
{
	return LOCTEXT("AnimBlueprintAssetDisplayName", "Animation Blueprint");
}

const FSlateBrush* UAnimationEditorsAssetFamilyExtension_AnimBlueprintAsset::GetAssetTypeDisplayIcon() const
{
	return FAppStyle::Get().GetBrush("Persona.AssetClass.Blueprint");
}

void UAnimationEditorsAssetFamilyExtension_AnimBlueprintAsset::FindAssetsOfType(TArray<FAssetData>& OutAssets, const IAnimationEditorsAssetFamilyInterface& AssetFamilyInterface) const
{
	if (TObjectPtr<const USkeleton> SkeletonAsset = AssetFamilyInterface.GetAssetOfType<USkeleton>())
	{
		FindAssets<UAnimBlueprint>(SkeletonAsset, OutAssets, "TargetSkeleton");
	}
}

bool UAnimationEditorsAssetFamilyExtension_AnimBlueprintAsset::IsAssetCompatible(const FAssetData& InAssetData, const IAnimationEditorsAssetFamilyInterface& AssetFamilyInterface) const
{
	FAssetDataTagMapSharedView::FFindTagResult Result = InAssetData.TagsAndValues.FindTag("TargetSkeleton");

	if (Result.IsSet())
	{
		if (TObjectPtr<const USkeleton> SkeletonAsset = AssetFamilyInterface.GetAssetOfType<USkeleton>())
		{
			return SkeletonAsset->IsCompatibleForEditor(Result.GetValue());
		}
	}

	return false;
}

void UAnimationEditorsAssetFamilyExtension_AnimBlueprintAsset::FindCounterpartAssets(const UObject* InAsset, IAnimationEditorsAssetFamilyInterface& AssetFamilyInterface)
{
	const UAnimBlueprint* AnimBlueprint = CastChecked<const UAnimBlueprint>(InAsset);

	AssetFamilyInterface.SetAssetOfType<USkeleton>(AnimBlueprint->TargetSkeleton);

	AssetFamilyInterface.SetAssetOfType<USkeletalMesh>(AnimBlueprint->GetPreviewMesh());

	check(AnimBlueprint->BlueprintType == BPTYPE_Interface || AnimBlueprint->bIsTemplate || AnimBlueprint->TargetSkeleton != nullptr);

	if (AssetFamilyInterface.IsAssetTypeInFamilyAndUnassigned<USkeletalMesh>() && AnimBlueprint->TargetSkeleton)
	{
		AssetFamilyInterface.SetAssetOfType<USkeletalMesh>(AnimBlueprint->TargetSkeleton->GetPreviewMesh());
	}

	if (AssetFamilyInterface.IsAssetTypeInFamilyAndUnassigned<USkeletalMesh>() && AnimBlueprint->TargetSkeleton)
	{
		AssetFamilyInterface.SetAssetOfType<USkeletalMesh>(AnimBlueprint->TargetSkeleton->FindCompatibleMesh());
	}
};

void UAnimationEditorsAssetFamilyExtension_AnimBlueprintAsset::GetPosition(FName& OutBeforeClass, FName& OutAfterClass) const
{
	OutBeforeClass = NAME_None;
	OutAfterClass = UAnimationAsset::StaticClass()->GetFName();;
}

// UAnimationEditorsAssetFamilyExtension_PhysicsAsset

TObjectPtr<UClass> UAnimationEditorsAssetFamilyExtension_PhysicsAsset::GetAssetClass() const
{
	return UPhysicsAsset::StaticClass();
}

FText UAnimationEditorsAssetFamilyExtension_PhysicsAsset::GetAssetTypeDisplayName() const
{
	return LOCTEXT("PhysicsAssetDisplayName", "Physics");
}

const FSlateBrush* UAnimationEditorsAssetFamilyExtension_PhysicsAsset::GetAssetTypeDisplayIcon() const
{
	return FAppStyle::Get().GetBrush("Persona.AssetClass.Physics");
}

void UAnimationEditorsAssetFamilyExtension_PhysicsAsset::FindAssetsOfType(TArray<FAssetData>& OutAssets, const IAnimationEditorsAssetFamilyInterface& AssetFamilyInterface) const
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	FARFilter Filter;
	Filter.bRecursiveClasses = true;
	Filter.ClassPaths.Add(UPhysicsAsset::StaticClass()->GetClassPathName());

	// If we have a mesh, look for a physics asset that has that mesh as its preview mesh
	if (TObjectPtr<const USkeletalMesh> SkeletalMeshAsset = AssetFamilyInterface.GetAssetOfType<USkeletalMesh>())
	{
		Filter.TagsAndValues.Add(GET_MEMBER_NAME_CHECKED(UPhysicsAsset, PreviewSkeletalMesh), FSoftObjectPath(SkeletalMeshAsset).ToString());
	}

	AssetRegistryModule.Get().GetAssets(Filter, OutAssets);

	// If we have a mesh and it has a physics asset, use it but only if its different from the one on the preview mesh
	if (TObjectPtr<const USkeletalMesh> SkeletalMeshAsset = AssetFamilyInterface.GetAssetOfType<USkeletalMesh>())
	{
		if (UPhysicsAsset* MeshPhysicsAsset = SkeletalMeshAsset->GetPhysicsAsset())
		{
			OutAssets.AddUnique(FAssetData(MeshPhysicsAsset));
		}
	}
}

bool UAnimationEditorsAssetFamilyExtension_PhysicsAsset::IsAssetCompatible(const FAssetData& InAssetData, const IAnimationEditorsAssetFamilyInterface& AssetFamilyInterface) const
{
	// If our mesh is valid and this is the physics asset used on it, we are compatible
	TObjectPtr<const USkeletalMesh> SkeletalMeshAsset = AssetFamilyInterface.GetAssetOfType<USkeletalMesh>();

	if (SkeletalMeshAsset && InAssetData.GetSoftObjectPath() == FSoftObjectPath(SkeletalMeshAsset->GetPhysicsAsset()))
	{
		return true;
	}

	// Otherwise check if our mesh is the preview mesh of the physics asset
	FAssetDataTagMapSharedView::FFindTagResult Result = InAssetData.TagsAndValues.FindTag(GET_MEMBER_NAME_CHECKED(UPhysicsAsset, PreviewSkeletalMesh));
	if (Result.IsSet() && SkeletalMeshAsset)
	{
		return Result.GetValue() == FSoftObjectPath(SkeletalMeshAsset).ToString();
	}

	return false;
}

void UAnimationEditorsAssetFamilyExtension_PhysicsAsset::FindCounterpartAssets(const UObject* InAsset, IAnimationEditorsAssetFamilyInterface& AssetFamilyInterface)
{
	const UPhysicsAsset* PhysicsAsset = CastChecked<const UPhysicsAsset>(InAsset);

	TObjectPtr<USkeletalMesh> SkeletalMesh = PhysicsAsset->PreviewSkeletalMesh.LoadSynchronous();
	if (SkeletalMesh)
	{
		AssetFamilyInterface.SetAssetOfType<USkeletalMesh>(SkeletalMesh);

		AssetFamilyInterface.SetAssetOfType<USkeleton>(SkeletalMesh->GetSkeleton());
	}
};

void UAnimationEditorsAssetFamilyExtension_PhysicsAsset::GetPosition(FName& OutBeforeClass, FName& OutAfterClass) const
{
	OutBeforeClass = NAME_None;
	OutAfterClass = UAnimBlueprint::StaticClass()->GetFName();;
}

#undef LOCTEXT_NAMESPACE
 
