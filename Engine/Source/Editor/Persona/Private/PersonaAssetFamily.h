// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Animation/Skeleton.h"
#include "Engine/SkeletalMesh.h"
#include "Animation/AnimationAsset.h"
#include "AssetRegistry/AssetData.h"
#include "IAssetFamily.h"
#include "AnimationEditorsAssetFamilyExtension.h"
#include "UObject/GCObject.h"

#include "PersonaAssetFamily.generated.h"

class UAnimBlueprint;
class UPhysicsAsset;
class UPersonaOptions;

UCLASS()
class UAnimationEditorsAssetFamilyExtension_SkeletonAsset : public UAnimationEditorsAssetFamilyExtension
{
	GENERATED_BODY()

public:
	// UAnimationEditorsAssetFamilyExtension
	virtual TObjectPtr<UClass> GetAssetClass() const override;
	virtual FText GetAssetTypeDisplayName() const override;
	virtual const FSlateBrush* GetAssetTypeDisplayIcon() const override;
	virtual void FindAssetsOfType(TArray<FAssetData>& OutAssets, const IAnimationEditorsAssetFamilyInterface& AssetFamilyInterface) const override;
	virtual bool IsAssetCompatible(const FAssetData& InAssetData, const IAnimationEditorsAssetFamilyInterface& AssetFamilyInterface) const override;
	virtual void FindCounterpartAssets(const UObject* InAsset, IAnimationEditorsAssetFamilyInterface& AssetFamilyInterface) override;
	virtual void GetPosition(FName& OutBeforeClass, FName& OutAfterClass) const override;
};

UCLASS()
class UAnimationEditorsAssetFamilyExtension_SkeletalMeshAsset : public UAnimationEditorsAssetFamilyExtension
{
	GENERATED_BODY()

public:
	// UAnimationEditorsAssetFamilyExtension
	virtual TObjectPtr<UClass> GetAssetClass() const override;
	virtual FText GetAssetTypeDisplayName() const override;
	virtual const FSlateBrush* GetAssetTypeDisplayIcon() const override;
	virtual void FindAssetsOfType(TArray<FAssetData>& OutAssets, const IAnimationEditorsAssetFamilyInterface& AssetFamilyInterface) const override;
	virtual bool IsAssetCompatible(const FAssetData& InAssetData, const IAnimationEditorsAssetFamilyInterface& AssetFamilyInterface) const override;
	virtual void FindCounterpartAssets(const UObject* InAsset, IAnimationEditorsAssetFamilyInterface& AssetFamilyInterface) override;
	virtual void GetPosition(FName& OutBeforeClass, FName& OutAfterClass) const override;
};

UCLASS()
class UAnimationEditorsAssetFamilyExtension_AnimationAsset : public UAnimationEditorsAssetFamilyExtension
{
	GENERATED_BODY()

public:
	// UAnimationEditorsAssetFamilyExtension
	virtual TObjectPtr<UClass> GetAssetClass() const override;
	virtual FText GetAssetTypeDisplayName() const override;
	virtual const FSlateBrush* GetAssetTypeDisplayIcon() const override;
	virtual void FindAssetsOfType(TArray<FAssetData>& OutAssets, const IAnimationEditorsAssetFamilyInterface& AssetFamilyInterface) const override;
	virtual bool IsAssetCompatible(const FAssetData& InAssetData, const IAnimationEditorsAssetFamilyInterface& AssetFamilyInterface) const override;
	virtual void FindCounterpartAssets(const UObject* InAsset, IAnimationEditorsAssetFamilyInterface& AssetFamilyInterface) override;
	virtual void GetPosition(FName& OutBeforeClass, FName& OutAfterClass) const override;
};

UCLASS()
class UAnimationEditorsAssetFamilyExtension_AnimBlueprintAsset : public UAnimationEditorsAssetFamilyExtension
{
	GENERATED_BODY()

public:
	// UAnimationEditorsAssetFamilyExtension
	virtual TObjectPtr<UClass> GetAssetClass() const override;
	virtual FText GetAssetTypeDisplayName() const override;
	virtual const FSlateBrush* GetAssetTypeDisplayIcon() const override;
	virtual void FindAssetsOfType(TArray<FAssetData>& OutAssets, const IAnimationEditorsAssetFamilyInterface& AssetFamilyInterface) const override;
	virtual bool IsAssetCompatible(const FAssetData& InAssetData, const IAnimationEditorsAssetFamilyInterface& AssetFamilyInterface) const override;
	virtual void FindCounterpartAssets(const UObject* InAsset, IAnimationEditorsAssetFamilyInterface& AssetFamilyInterface) override;
	virtual void GetPosition(FName& OutBeforeClass, FName& OutAfterClass) const override;
};

UCLASS()
class UAnimationEditorsAssetFamilyExtension_PhysicsAsset : public UAnimationEditorsAssetFamilyExtension
{
	GENERATED_BODY()

public:
	// UAnimationEditorsAssetFamilyExtension
	virtual TObjectPtr<UClass> GetAssetClass() const override;
	virtual FText GetAssetTypeDisplayName() const override;
	virtual const FSlateBrush* GetAssetTypeDisplayIcon() const override;
	virtual void FindAssetsOfType(TArray<FAssetData>& OutAssets, const IAnimationEditorsAssetFamilyInterface& AssetFamilyInterface) const override;
	virtual bool IsAssetCompatible(const FAssetData& InAssetData, const IAnimationEditorsAssetFamilyInterface& AssetFamilyInterface) const override;
	virtual void FindCounterpartAssets(const UObject* InAsset, IAnimationEditorsAssetFamilyInterface& AssetFamilyInterface) override;
	virtual void GetPosition(FName& OutBeforeClass, FName& OutAfterClass) const override;
};

class FPersonaAssetFamily : public IAssetFamily, public TSharedFromThis<FPersonaAssetFamily>, public IAnimationEditorsAssetFamilyInterface, public FGCObject
{
private:
	struct FExtenderObjects
	{
		TObjectPtr<UAnimationEditorsAssetFamilyExtension> Extension;

		TWeakObjectPtr<const UObject> Asset;
	};

public:
	FPersonaAssetFamily(const UObject* InFromObject);
	FPersonaAssetFamily(const UObject* InFromObject, const TSharedRef<FPersonaAssetFamily> InFromFamily);

	virtual ~FPersonaAssetFamily() = default;

	/** IAssetFamily interface */
	virtual void GetAssetTypes(TArray<UClass*>& OutAssetTypes) const override;
	virtual FAssetData FindAssetOfType(UClass* InAssetClass) const override;
	virtual void FindAssetsOfType(UClass* InAssetClass, TArray<FAssetData>& OutAssets) const override;
	virtual FText GetAssetTypeDisplayName(UClass* InAssetClass) const override;
	virtual const FSlateBrush* GetAssetTypeDisplayIcon(UClass* InAssetClass) const override;
	virtual FSlateColor GetAssetTypeDisplayTint(UClass* InAssetClass) const override;
	virtual bool IsAssetCompatible(const FAssetData& InAssetData) const override;
	virtual UClass* GetAssetFamilyClass(UClass* InClass) const override;
	void RecordAssetOpened(const FAssetData& InAssetData);
	DECLARE_DERIVED_EVENT(FPersonaAssetFamily, IAssetFamily::FOnAssetOpened, FOnAssetOpened)
	virtual FOnAssetOpened& GetOnAssetOpened() override { return OnAssetOpened;  }
	DECLARE_DERIVED_EVENT(FPersonaAssetFamily, IAssetFamily::FOnAssetFamilyChanged, FOnAssetFamilyChanged)
	virtual FOnAssetFamilyChanged& GetOnAssetFamilyChanged() override { return OnAssetFamilyChanged; }

	/* IAnimationEditorsAssetFamilyInterface */
	virtual bool IsAssetTypeInFamily(const TObjectPtr<UClass> InClass) const override;
	virtual TWeakObjectPtr<const UObject> GetAssetOfType(const TObjectPtr<UClass> InClass) const override;
	virtual bool SetAssetOfType(const TObjectPtr<UClass> InClass, TWeakObjectPtr<const UObject> InObject) override;
	
	/* FGCObject */
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override;

	template <typename T>
	TObjectPtr<const T> GetAssetOfType() const
	{
		return IAnimationEditorsAssetFamilyInterface::GetAssetOfType<T>();
	}

private:
	FExtenderObjects* GetExtensionForClass(const TObjectPtr<const UClass> InClass);
	const FExtenderObjects* const GetExtensionForClass(const TObjectPtr<const UClass> InClass) const;

	/** Initialization to avoid shared ptr access in constructor */
	void Initialize();
	
	/** Hande key persona settings changes (e.g. skeleton compatibilty) */
	void OnSettingsChange(const UPersonaOptions* InOptions, EPropertyChangeType::Type InChangeType);
	
private:
	friend class FPersonaAssetFamilyManager;
	
	TArray<FExtenderObjects> Extenders;

	/** Event fired when an asset is opened */
	FOnAssetOpened OnAssetOpened;

	/** Event fired when an asset family changes (e.g. relationships are altered) */
	FOnAssetFamilyChanged OnAssetFamilyChanged;
};
