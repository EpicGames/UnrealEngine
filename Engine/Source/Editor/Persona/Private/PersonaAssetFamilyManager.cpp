// Copyright Epic Games, Inc. All Rights Reserved.

#include "PersonaAssetFamilyManager.h"
#include "AssetRegistry/AssetData.h"
#include "IAssetFamily.h"
#include "PersonaAssetFamily.h"

FPersonaAssetFamilyManager& FPersonaAssetFamilyManager::Get()
{
	static FPersonaAssetFamilyManager TheManager;
	return TheManager;
}

TSharedRef<IAssetFamily> FPersonaAssetFamilyManager::CreatePersonaAssetFamily(const UObject* InAsset)
{
	// compact any invalid entries
	AssetFamilies.RemoveAll([](const TWeakPtr<FPersonaAssetFamily>& InAssetFamily) { return !InAssetFamily.IsValid(); });

	// look for an existing matching asset family
	TWeakPtr<FPersonaAssetFamily>* ExistingAssetFamily = nullptr;
	FAssetData AssetData(InAsset);
	for (TWeakPtr<FPersonaAssetFamily>& AssetFamily : AssetFamilies)
	{
		if (AssetFamily.Pin()->IsAssetCompatible(AssetData))
		{
			ExistingAssetFamily = &AssetFamily;
			break;
		}
	}

	// Create new asset family
	TSharedRef<FPersonaAssetFamily> NewAssetFamily = ExistingAssetFamily != nullptr ? MakeShared<FPersonaAssetFamily>(InAsset, ExistingAssetFamily->Pin().ToSharedRef()) : MakeShared<FPersonaAssetFamily>(InAsset);
	NewAssetFamily->Initialize();
	AssetFamilies.Add(NewAssetFamily);
	return NewAssetFamily;
}

void FPersonaAssetFamilyManager::BroadcastAssetFamilyChange()
{
	// Create copy as delegate can modify the AssetFamilies array
	TArray<TWeakPtr<FPersonaAssetFamily>> AssetFamiliesCopy = AssetFamilies;
	for (TWeakPtr<FPersonaAssetFamily>& AssetFamily : AssetFamiliesCopy)
	{
		if (TSharedPtr<FPersonaAssetFamily> PinnedAssetFamily = AssetFamily.Pin())
		{
			PinnedAssetFamily->GetOnAssetFamilyChanged().Broadcast();
		}
	}
}

void FPersonaAssetFamilyManager::RecordAssetOpened(const FAssetData& InAssetData) const
{
	for (const TWeakPtr<FPersonaAssetFamily>& AssetFamily : AssetFamilies)
	{
		if (TSharedPtr<FPersonaAssetFamily> PinnedAssetFamily = AssetFamily.Pin())
		{
			if(PinnedAssetFamily->IsAssetCompatible(InAssetData))
			{
				PinnedAssetFamily->RecordAssetOpened(InAssetData);
			}
		}
	}
}