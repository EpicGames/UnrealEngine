// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class IAssetFamily;
class FPersonaAssetFamily;
struct FAssetData;

/** Central registry of persona asset families */
class FPersonaAssetFamilyManager
{
public:
	/** Singleton access */
	static FPersonaAssetFamilyManager& Get();

	/** Create an asset family using the supplied asset */
	TSharedRef<IAssetFamily> CreatePersonaAssetFamily(const UObject* InAsset);

	/** All asset families have changed (e.g. universal compatibility) */
	void BroadcastAssetFamilyChange();

	/** Record an asset being opened - forward to all compatible asset families */
	void RecordAssetOpened(const FAssetData& InAssetData) const;
	
private:
	/** Hidden constructor */
	FPersonaAssetFamilyManager() {}

private:
	/** All current asset families */
	TArray<TWeakPtr<FPersonaAssetFamily>> AssetFamilies;
};
