// Copyright Epic Games, Inc. All Rights Reserved.
#include "PersonaBlendSpaceAnalysis.h"

#include "AnimPose.h"
#include "Engine/SkeletalMeshSocket.h"
#include "Animation/BlendSpace.h"
#include "Animation/AnimSequence.h"
#include "Animation/BoneSocketReference.h"
#include "Features/IModularFeatures.h"

#define LOCTEXT_NAMESPACE "BlendSpaceAnalysis"

//======================================================================================================================
void UCachedAnalysisProperties::CopyFrom(const UCachedAnalysisProperties& Other)
{
	LinearFunctionAxis = Other.LinearFunctionAxis;
	EulerFunctionAxis = Other.EulerFunctionAxis;
	BoneSocket1 = Other.BoneSocket1;
	BoneSocket2 = Other.BoneSocket2;
	BoneFacingAxis = Other.BoneFacingAxis;
	BoneRightAxis = Other.BoneRightAxis;
	Space = Other.Space;
	SpaceBoneSocket = Other.SpaceBoneSocket;
	CharacterFacingAxis = Other.CharacterFacingAxis;
	CharacterUpAxis = Other.CharacterUpAxis;
	StartTimeFraction = Other.StartTimeFraction;
	EndTimeFraction = Other.EndTimeFraction;
}

//======================================================================================================================
FVector BlendSpaceAnalysis::GetAxisFromTM(const FTransform& TM, EAnalysisLinearAxis Axis)
{
	switch (Axis)
	{
	case EAnalysisLinearAxis::PlusX: return TM.TransformVectorNoScale(FVector(1, 0, 0));
	case EAnalysisLinearAxis::PlusY: return TM.TransformVectorNoScale(FVector(0, 1, 0));
	case EAnalysisLinearAxis::PlusZ: return TM.TransformVectorNoScale(FVector(0, 0, 1));
	case EAnalysisLinearAxis::MinusX: return TM.TransformVectorNoScale(FVector(-1, 0, 0));
	case EAnalysisLinearAxis::MinusY: return TM.TransformVectorNoScale(FVector(0, -1, 0));
	case EAnalysisLinearAxis::MinusZ: return TM.TransformVectorNoScale(FVector(0, 0, -1));
	}
	return FVector(0, 0, 0);
}

bool BlendSpaceAnalysis::CalculatePosition(FVector& Result, const UBlendSpace& BlendSpace, const ULinearAnalysisProperties* AnalysisProperties,
	const UAnimSequence& Animation, const float RateScale)
{
	const int32 NumSampledKeys = Animation.GetNumberOfSampledKeys() - 1;
	if (!AnalysisProperties || NumSampledKeys <= 0)
	{
		return false;
	}

	FTransform BoneOffset;
	FName BoneName;
	if (!GetBoneInfo(Animation, AnalysisProperties->BoneSocket, BoneOffset, BoneName))
	{
		return false;
	}

	int32 FirstKey = FMath::Clamp(
		(int32) (float(NumSampledKeys) * AnalysisProperties->StartTimeFraction), 0, NumSampledKeys);
	int32 LastKey = FMath::Clamp(
		(int32) (float(NumSampledKeys) * AnalysisProperties->EndTimeFraction), FirstKey, NumSampledKeys);

	FTransform FrameTM;
	bool bNeedToUpdateFrameTM = true;
	CalculateFrameTM(bNeedToUpdateFrameTM, FrameTM, FirstKey, AnalysisProperties, Animation);

	Result.Set(0, 0, 0);
	for (int32 Key = FirstKey; Key != LastKey + 1; ++Key)
	{
		CalculateFrameTM(bNeedToUpdateFrameTM, FrameTM, Key, AnalysisProperties, Animation);
		FTransform BoneTM = GetBoneTransform(Animation, Key, BoneName);
		FTransform TM = BoneOffset * BoneTM;
		FVector RelativePos = FrameTM.InverseTransformPosition(TM.GetTranslation());
		Result += RelativePos;
	}
	Result /= (1 + LastKey - FirstKey);
	return true;
}

bool BlendSpaceAnalysis::CalculateDeltaPosition(FVector& Result, const UBlendSpace& BlendSpace, const ULinearAnalysisProperties* AnalysisProperties,
	const UAnimSequence& Animation, const float RateScale)
{
	const int32 NumSampledKeys = Animation.GetNumberOfSampledKeys() - 1;
	if (!AnalysisProperties || NumSampledKeys <= 0)
	{
		return false;
	}

	FTransform BoneOffset;
	FName BoneName;
	if (!GetBoneInfo(Animation, AnalysisProperties->BoneSocket, BoneOffset, BoneName))
	{
		return false;
	}

	int32 FirstKey = FMath::Clamp(
		(int32) (float(NumSampledKeys) * AnalysisProperties->StartTimeFraction), 0, NumSampledKeys);
	int32 LastKey = FMath::Clamp(
		(int32) (float(NumSampledKeys) * AnalysisProperties->EndTimeFraction), FirstKey, NumSampledKeys);

	FTransform FrameTM;
	bool bNeedToUpdateFrameTM = true;
	CalculateFrameTM(bNeedToUpdateFrameTM, FrameTM, FirstKey, AnalysisProperties, Animation);

	FTransform BoneTM1 = GetBoneTransform(Animation, FirstKey, BoneName);
	FTransform TM1 = BoneOffset * BoneTM1;
	FVector RelativePos1 = FrameTM.InverseTransformPosition(TM1.GetTranslation());

	CalculateFrameTM(bNeedToUpdateFrameTM, FrameTM, FirstKey, AnalysisProperties, Animation);
	FTransform BoneTM2 = GetBoneTransform(Animation, LastKey, BoneName);
	FTransform TM2 = BoneOffset * BoneTM2;
	FVector RelativePos2 = FrameTM.InverseTransformPosition(TM2.GetTranslation());

	Result = RelativePos2 - RelativePos1;
	return true;
}

bool BlendSpaceAnalysis::CalculateVelocity(FVector& Result, const UBlendSpace& BlendSpace, const ULinearAnalysisPropertiesBase* AnalysisProperties,
	const UAnimSequence& Animation, const float RateScale)
{
	const int32 NumSampledKeys = Animation.GetNumberOfSampledKeys() - 1;
	if (!AnalysisProperties || NumSampledKeys <= 0)
	{
		return false;
	}

	if (NumSampledKeys == 1)
	{
		Result.Set(0, 0, 0);
		return true;
	}

	FTransform BoneOffset;
	FName BoneName;
	if (!GetBoneInfo(Animation, AnalysisProperties->BoneSocket, BoneOffset, BoneName))
	{
		return false;
	}

	double DeltaTime = Animation.GetPlayLength() / double(NumSampledKeys);

	int32 FirstKey = FMath::Clamp(
		(int32) (float(NumSampledKeys) * AnalysisProperties->StartTimeFraction), 0, NumSampledKeys);
	int32 LastKey = FMath::Clamp(
		(int32) (float(NumSampledKeys) * AnalysisProperties->EndTimeFraction), FirstKey, NumSampledKeys-1);

	// First and Last key are for averaging. However, the finite differencing always goes from one frame to the next
	int32 NumKeys = FMath::Max(1 + LastKey - FirstKey, 1);

	FTransform FrameTM;
	bool bNeedToUpdateFrameTM = true;
	CalculateFrameTM(bNeedToUpdateFrameTM, FrameTM, FirstKey, AnalysisProperties, Animation);

	Result.Set(0, 0, 0);
	for (int32 iKey = 0; iKey != NumKeys; ++iKey)
	{
		int32 Key = (FirstKey + iKey) % (NumSampledKeys + 1);
		CalculateFrameTM(bNeedToUpdateFrameTM, FrameTM, Key, AnalysisProperties, Animation);

		FTransform BoneTM1 = GetBoneTransform(Animation, Key, BoneName);
		FTransform TM1 = BoneOffset * BoneTM1;
		FVector RelativePos1 = FrameTM.InverseTransformPosition(TM1.GetTranslation());

		int32 NextKey = (Key + 1) % (NumSampledKeys + 1);
		if (AnalysisProperties->Space == EAnalysisSpace::Moving)
		{
			CalculateFrameTM(bNeedToUpdateFrameTM, FrameTM, NextKey, AnalysisProperties, Animation);
		}

		FTransform BoneTM2 = GetBoneTransform(Animation, NextKey, BoneName);
		FTransform TM2 = BoneOffset * BoneTM2;
		FVector RelativePos2 = FrameTM.InverseTransformPosition(TM2.GetTranslation());
		FVector Velocity = (RelativePos2 - RelativePos1) / DeltaTime;

#ifdef ANALYSIS_VERBOSE_LOG
		UE_LOGF(LogAnimation, Log, "%d Velocity = %f %f %f Height = %f", 
			   Key, Velocity.X, Velocity.Y, Velocity.Z, 0.5f * (RelativePos1 + RelativePos2).Z);
#endif
		Result += Velocity;
	}
	Result /= (1 + LastKey - FirstKey);
	Result *= Animation.RateScale * RateScale;
	UE_LOGF(LogAnimation, Log, "%ls vel = %f %f %f", *BoneName.ToString(), Result.X, Result.Y, Result.Z);
	return true;
}

void BlendSpaceAnalysis::CalculateBoneOrientation(FVector& RollPitchYaw, const UAnimSequence& Animation, const int32 Key, const FName BoneName,
	const FTransform& BoneOffset, const UEulerAnalysisProperties* AnalysisProperties, const FVector& FrameFacingDir, const FVector& FrameRightDir,
	const FVector& FrameUpDir)
{
	const FTransform BoneTM = GetBoneTransform(Animation, Key, BoneName);

	const FTransform TM = BoneOffset * BoneTM;
	const FVector AimForwardDir = BlendSpaceAnalysis::GetAxisFromTM(TM, AnalysisProperties->BoneFacingAxis);
	const FVector AimRightDir = BlendSpaceAnalysis::GetAxisFromTM(TM, AnalysisProperties->BoneRightAxis);

	double Yaw;
	if (AnalysisProperties->EulerCalculationMethod == EEulerCalculationMethod::AimDirection)
	{
		// Yaw is taken from the AimRightDir to avoid problems when the gun is pointing up or down - especially if it 
		// goes beyond 90 degrees in pitch. However, if there is roll around the gun axis, then this can produce
		// incorrect/undesirable results.
		Yaw = FMath::RadiansToDegrees(FMath::Atan2(
			FVector::DotProduct(AimRightDir, -FrameFacingDir), FVector::DotProduct(AimRightDir, FrameRightDir)));
	}
	else
	{
		// This takes yaw directly from the forwards direction. Note that if the pose is really one with small yaw 
		// and pitch more than 90 degrees, then this will calculate a yaw that is nearer to 180 degrees.
		Yaw = FMath::RadiansToDegrees(FMath::Atan2(
			FVector::DotProduct(AimForwardDir, FrameRightDir), FVector::DotProduct(AimForwardDir, FrameFacingDir)));
	}

	// Undo the yaw to get pitch
	const FQuat YawQuat(FrameUpDir, FMath::DegreesToRadians(Yaw));
	const FVector UnYawedAimForwardDir = YawQuat.UnrotateVector(AimForwardDir);
	const double Up = UnYawedAimForwardDir | FrameUpDir;
	const double Forward = UnYawedAimForwardDir | FrameFacingDir;
	const double Pitch = FMath::RadiansToDegrees(FMath::Atan2(Up, Forward));

	// Undo the pitch to get roll
	const FVector UnYawedAimRightDir = YawQuat.UnrotateVector(AimRightDir);
	const FQuat PitchQuat(FrameRightDir, -FMath::DegreesToRadians(Pitch));

	const FVector UnYawedUnPitchedAimRightDir = PitchQuat.UnrotateVector(UnYawedAimRightDir);

	const double Roll = FMath::RadiansToDegrees(FMath::Atan2(
		FVector::DotProduct(UnYawedUnPitchedAimRightDir, -FrameUpDir), 
		FVector::DotProduct(UnYawedUnPitchedAimRightDir, FrameRightDir)));

	RollPitchYaw.Set(Roll, Pitch, Yaw);
}

bool BlendSpaceAnalysis::CalculateOrientation(FVector& Result, const UBlendSpace& BlendSpace, const UEulerAnalysisProperties* AnalysisProperties,
	const UAnimSequence& Animation, const float RateScale)
{
	const int32 NumSampledKeys = Animation.GetNumberOfSampledKeys() - 1;
	if (!AnalysisProperties || NumSampledKeys <= 0)
	{
		return false;
	}

	FTransform BoneOffset;
	FName BoneName;
	if (!GetBoneInfo(Animation, AnalysisProperties->BoneSocket, BoneOffset, BoneName))
	{
		return false;
	}

	int32 FirstKey = FMath::Clamp(
		(int32) (float(NumSampledKeys) * AnalysisProperties->StartTimeFraction), 0, NumSampledKeys);
	int32 LastKey = FMath::Clamp(
		(int32) (float(NumSampledKeys) * AnalysisProperties->EndTimeFraction), FirstKey, NumSampledKeys);

	FTransform FrameTM;
	bool bNeedToUpdateFrameTM = true;
	FVector FrameFacingDir, FrameUpDir, FrameRightDir;
	CalculateFrameTM(bNeedToUpdateFrameTM, FrameTM, FirstKey, AnalysisProperties, Animation);
	GetFrameDirs(FrameFacingDir, FrameUpDir, FrameRightDir, FrameTM, AnalysisProperties);

	Result.Set(0, 0, 0);
	for (int32 Key = FirstKey; Key != LastKey + 1; ++Key)
	{
		CalculateFrameTM(bNeedToUpdateFrameTM, FrameTM, Key, AnalysisProperties, Animation);
		GetFrameDirs(FrameFacingDir, FrameUpDir, FrameRightDir, FrameTM, AnalysisProperties);

		FVector RollPitchYaw;
		CalculateBoneOrientation(
			RollPitchYaw, Animation, Key, BoneName, BoneOffset, 
			AnalysisProperties, FrameFacingDir, FrameRightDir, FrameUpDir);
#ifdef ANALYSIS_VERBOSE_LOG
		UE_LOGF(LogAnimation, Log, "Roll/pitch/yaw = %f %f %f", RollPitchYaw.X, RollPitchYaw.Y, RollPitchYaw.Z);
#endif
		Result += RollPitchYaw;
	}
	Result /= (1 + LastKey - FirstKey);
	UE_LOGF(LogAnimation, Log, "%ls Orientation = %f %f %f", *BoneName.ToString(), Result.X, Result.Y, Result.Z);
	return true;
}

bool BlendSpaceAnalysis::CalculateDeltaOrientation(FVector& Result, const UBlendSpace& BlendSpace, const UEulerAnalysisProperties* AnalysisProperties,
	const UAnimSequence& Animation, const float RateScale)
{
	const int32 NumSampledKeys = Animation.GetNumberOfSampledKeys() - 1;
	if (!AnalysisProperties || NumSampledKeys <= 0)
	{
		return false;
	}

	FTransform BoneOffset;
	FName BoneName;
	if (!GetBoneInfo(Animation, AnalysisProperties->BoneSocket, BoneOffset, BoneName))
	{
		return false;
	}

	int32 FirstKey = FMath::Clamp(
		(int32) (float(NumSampledKeys) * AnalysisProperties->StartTimeFraction), 0, NumSampledKeys);
	int32 LastKey = FMath::Clamp(
		(int32) (float(NumSampledKeys) * AnalysisProperties->EndTimeFraction), FirstKey, NumSampledKeys);

	FTransform FrameTM;
	bool bNeedToUpdateFrameTM = true;
	FVector FrameFacingDir, FrameUpDir, FrameRightDir;
	CalculateFrameTM(bNeedToUpdateFrameTM, FrameTM, FirstKey, AnalysisProperties, Animation);
	GetFrameDirs(FrameFacingDir, FrameUpDir, FrameRightDir, FrameTM, AnalysisProperties);

	FVector RollPitchYaw1;
	CalculateBoneOrientation(
		RollPitchYaw1, Animation, FirstKey, BoneName, BoneOffset,
		AnalysisProperties, FrameFacingDir, FrameRightDir, FrameUpDir);

	CalculateFrameTM(bNeedToUpdateFrameTM, FrameTM, LastKey, AnalysisProperties, Animation);
	GetFrameDirs(FrameFacingDir, FrameUpDir, FrameRightDir, FrameTM, AnalysisProperties);

	FVector RollPitchYaw2;
	CalculateBoneOrientation(
		RollPitchYaw2, Animation, LastKey, BoneName, BoneOffset,
		AnalysisProperties, FrameFacingDir, FrameRightDir, FrameUpDir);

	Result = RollPitchYaw2 - RollPitchYaw1;
	return true;
}

bool BlendSpaceAnalysis::CalculateAngularVelocity(FVector& Result, const UBlendSpace& BlendSpace, const ULinearAnalysisProperties* AnalysisProperties,
	const UAnimSequence& Animation, const float RateScale)
{
	const int32 NumSampledKeys = Animation.GetNumberOfSampledKeys() - 1;
	if (!AnalysisProperties || NumSampledKeys <= 0)
	{
		return false;
	}

	if (NumSampledKeys == 1)
	{
		Result.Set(0, 0, 0);
		return true;
	}

	FTransform BoneOffset;
	FName BoneName;
	if (!GetBoneInfo(Animation, AnalysisProperties->BoneSocket, BoneOffset, BoneName))
	{
		return false;
	}

	double DeltaTime = Animation.GetPlayLength() / double(NumSampledKeys);

	int32 FirstKey = FMath::Clamp(
		(int32) (float(NumSampledKeys) * AnalysisProperties->StartTimeFraction), 0, NumSampledKeys);
	int32 LastKey = FMath::Clamp(
		(int32) (float(NumSampledKeys) * AnalysisProperties->EndTimeFraction), FirstKey, NumSampledKeys-1);

	// First and Last key are for averaging. However, the finite differencing always goes from one frame to the next
	int32 NumKeys = FMath::Max(1 + LastKey - FirstKey, 1);

	FTransform FrameTM;
	bool bNeedToUpdateFrameTM = true;
	CalculateFrameTM(bNeedToUpdateFrameTM, FrameTM, FirstKey, AnalysisProperties, Animation);

	Result.Set(0, 0, 0);
	for (int32 iKey = 0; iKey != NumKeys; ++iKey)
	{
		int32 Key = (FirstKey + iKey) % (NumSampledKeys + 1);
		CalculateFrameTM(bNeedToUpdateFrameTM, FrameTM, Key, AnalysisProperties, Animation);

		FTransform BoneTM1 = GetBoneTransform(Animation, Key, BoneName);
		FTransform TM1 = BoneOffset * BoneTM1;
		FQuat RelativeQuat1 = FrameTM.InverseTransformRotation(TM1.GetRotation());

		int32 NextKey = (Key + 1) % (NumSampledKeys + 1);
		if (AnalysisProperties->Space == EAnalysisSpace::Moving)
		{
			CalculateFrameTM(bNeedToUpdateFrameTM, FrameTM, NextKey, AnalysisProperties, Animation);
		}

		FTransform BoneTM2 = GetBoneTransform(Animation, NextKey, BoneName);
		FTransform TM2 = BoneOffset * BoneTM2;
		FQuat RelativeQuat2 = FrameTM.InverseTransformRotation(TM2.GetRotation());

		FQuat Rotation = RelativeQuat2 * RelativeQuat1.Inverse();
		FVector Axis;
		double Angle;
		Rotation.ToAxisAndAngle(Axis, Angle);
		FVector AngularVelocity = FMath::RadiansToDegrees(Axis * (Angle / DeltaTime));
#ifdef ANALYSIS_VERBOSE_LOG
		UE_LOGF(LogAnimation, Log, "Angular Velocity = %f %f %f", AngularVelocity.X, AngularVelocity.Y, AngularVelocity.Z);
#endif
		Result += AngularVelocity;
	}
	Result /= (1 + LastKey - FirstKey);
	Result *= Animation.RateScale * RateScale;
	UE_LOGF(LogAnimation, Log, "%ls angular velocity = %f %f %f", *BoneName.ToString(), Result.X, Result.Y, Result.Z);
	return true;
}

bool BlendSpaceAnalysis::CalculateOrientationRate(FVector& Result, const UBlendSpace& BlendSpace, const UEulerAnalysisProperties* AnalysisProperties,
	const UAnimSequence& Animation, const float RateScale)
{
	const int32 NumSampledKeys = Animation.GetNumberOfSampledKeys() - 1;
	if (!AnalysisProperties || NumSampledKeys <= 0)
	{
		return false;
	}

	if (NumSampledKeys == 1)
	{
		Result.Set(0, 0, 0);
		return true;
	}

	FTransform BoneOffset;
	FName BoneName;
	if (!GetBoneInfo(Animation, AnalysisProperties->BoneSocket, BoneOffset, BoneName))
	{
		return false;
	}

	double DeltaTime = Animation.GetPlayLength() / double(NumSampledKeys);

	int32 FirstKey = FMath::Clamp(
		(int32) (float(NumSampledKeys) * AnalysisProperties->StartTimeFraction), 0, NumSampledKeys);
	int32 LastKey = FMath::Clamp(
		(int32) (float(NumSampledKeys) * AnalysisProperties->EndTimeFraction), FirstKey, NumSampledKeys-1);

	// First and Last key are for averaging. However, the finite differencing always goes from one frame to the next
	int32 NumKeys = FMath::Max(1 + LastKey - FirstKey, 1);

	FTransform FrameTM;
	bool bNeedToUpdateFrameTM = true;
	FVector FrameFacingDir, FrameUpDir, FrameRightDir;
	CalculateFrameTM(bNeedToUpdateFrameTM, FrameTM, FirstKey, AnalysisProperties, Animation);
	GetFrameDirs(FrameFacingDir, FrameUpDir, FrameRightDir, FrameTM, AnalysisProperties);

	Result.Set(0, 0, 0);
	for (int32 iKey = 0; iKey != NumKeys; ++iKey)
	{
		int32 Key = (FirstKey + iKey) % (NumSampledKeys + 1);
		CalculateFrameTM(bNeedToUpdateFrameTM, FrameTM, Key, AnalysisProperties, Animation);
		GetFrameDirs(FrameFacingDir, FrameUpDir, FrameRightDir, FrameTM, AnalysisProperties);

		FVector RollPitchYaw1;
		CalculateBoneOrientation(
			RollPitchYaw1, Animation, Key, BoneName, BoneOffset, 
			AnalysisProperties, FrameFacingDir, FrameRightDir, FrameUpDir);

		int32 NextKey = (Key + 1) % (NumSampledKeys + 1);
		if (AnalysisProperties->Space == EAnalysisSpace::Moving)
		{
			CalculateFrameTM(bNeedToUpdateFrameTM, FrameTM, NextKey, AnalysisProperties, Animation);
			GetFrameDirs(FrameFacingDir, FrameUpDir, FrameRightDir, FrameTM, AnalysisProperties);
		}

		FVector RollPitchYaw2;
		CalculateBoneOrientation(
			RollPitchYaw2, Animation, NextKey, BoneName, BoneOffset, 
			AnalysisProperties, FrameFacingDir, FrameRightDir, FrameUpDir);

		const FVector OrientationRate = (RollPitchYaw2 - RollPitchYaw1) / DeltaTime;
#ifdef ANALYSIS_VERBOSE_LOG
		UE_LOGF(LogAnimation, Log, "Orientation rate = %f %f %f", OrientationRate.X, OrientationRate.Y, OrientationRate.Z);
#endif
		Result += OrientationRate;
	}
	Result /= (1 + LastKey - FirstKey);
	Result *= Animation.RateScale * RateScale;
	UE_LOGF(LogAnimation, Log, "%ls Orientation rate = %f %f %f", *BoneName.ToString(), Result.X, Result.Y, Result.Z);
	return true;
}

//======================================================================================================================
bool BlendSpaceAnalysis::GetBoneInfo(const UAnimSequence&     Animation, 
									 const FBoneSocketTarget& BoneSocket, 
									 FTransform&              BoneOffset, 
									 FName&                   BoneName)
{
	if (BoneSocket.bUseSocket)
	{
		const USkeleton* Skeleton = Animation.GetSkeleton();
		const USkeletalMeshSocket* Socket = Skeleton->FindSocket(BoneSocket.SocketReference.SocketName);
		if (Socket)
		{
			BoneOffset = Socket->GetSocketLocalTransform();
			BoneName = Socket->BoneName;
			return !BoneName.IsNone();
		}
	}
	else
	{
		BoneName = BoneSocket.BoneReference.BoneName;
		return !BoneName.IsNone();
	}
	return false;
}

//======================================================================================================================
FTransform BlendSpaceAnalysis::GetBoneTransform(const UAnimSequence& Animation, int32 Key, const FName& BoneName)
{
	FAnimPose AnimPose;
	UAnimPoseExtensions::GetAnimPoseAtFrame(&Animation, Key, FAnimPoseEvaluationOptions(), AnimPose);
	FTransform BoneTM = UAnimPoseExtensions::GetBonePose(AnimPose, BoneName, EAnimPoseSpaces::World);
	return BoneTM;
}


//======================================================================================================================
void ULinearAnalysisProperties::InitializeFromCache(TObjectPtr<UCachedAnalysisProperties> Cache)
{
	if (Cache)
	{
		UCachedAnalysisProperties* CachePtr = Cache.Get();
		FunctionAxis = CachePtr->LinearFunctionAxis;
		BoneSocket = CachePtr->BoneSocket1;
		Space = CachePtr->Space;
		SpaceBoneSocket = CachePtr->SpaceBoneSocket;
		StartTimeFraction = CachePtr->StartTimeFraction;
		EndTimeFraction = CachePtr->EndTimeFraction;
	}
}

//======================================================================================================================
void ULinearAnalysisProperties::MakeCache(TObjectPtr<UCachedAnalysisProperties>& Cache, UBlendSpace* BlendSpace)
{
	UCachedAnalysisProperties* CachePtr = Cache.Get();
	if (!Cache)
	{
		Cache = NewObject<UCachedAnalysisProperties>(BlendSpace);
		CachePtr = Cache.Get();
	}
	CachePtr->LinearFunctionAxis = FunctionAxis;
	CachePtr->BoneSocket1 = BoneSocket;
	CachePtr->Space = Space;
	CachePtr->SpaceBoneSocket = SpaceBoneSocket;
	CachePtr->StartTimeFraction = StartTimeFraction;
	CachePtr->EndTimeFraction = EndTimeFraction;
}

//======================================================================================================================
void UEulerAnalysisProperties::InitializeFromCache(TObjectPtr<UCachedAnalysisProperties> Cache)
{
	if (Cache)
	{
		UCachedAnalysisProperties* CachePtr = Cache.Get();
		FunctionAxis = CachePtr->EulerFunctionAxis;
		BoneSocket = CachePtr->BoneSocket1;
		BoneFacingAxis = CachePtr->BoneFacingAxis;
		BoneRightAxis = CachePtr->BoneRightAxis;
		Space = CachePtr->Space;
		SpaceBoneSocket = CachePtr->SpaceBoneSocket;
		CharacterFacingAxis = CachePtr->CharacterFacingAxis;
		CharacterUpAxis = CachePtr->CharacterUpAxis;
		StartTimeFraction = CachePtr->StartTimeFraction;
		EndTimeFraction = CachePtr->EndTimeFraction;
	}
}

//======================================================================================================================
void UEulerAnalysisProperties::MakeCache(TObjectPtr<UCachedAnalysisProperties>& Cache, UBlendSpace* BlendSpace)
{
	UCachedAnalysisProperties* CachePtr = Cache.Get();
	if (!Cache)
	{
		Cache = NewObject<UCachedAnalysisProperties>(BlendSpace);
		CachePtr = Cache.Get();
	}
	CachePtr->EulerFunctionAxis = FunctionAxis;
	CachePtr->BoneSocket1 = BoneSocket;
	CachePtr->BoneFacingAxis = BoneFacingAxis;
	CachePtr->BoneRightAxis = BoneRightAxis;
	CachePtr->Space = Space;
	CachePtr->SpaceBoneSocket = SpaceBoneSocket;
	CachePtr->CharacterFacingAxis = CharacterFacingAxis;
	CachePtr->CharacterUpAxis = CharacterUpAxis;
	CachePtr->StartTimeFraction = StartTimeFraction;
	CachePtr->EndTimeFraction = EndTimeFraction;
}

//======================================================================================================================
class FCoreBlendSpaceAnalysisFeature : public IBlendSpaceAnalysisFeature
{
public:
	// This should process the animation according to the analysis properties, or return false if that is not possible.
	bool CalculateSampleValue(float&                     Result,
							  const UBlendSpace&         BlendSpace,
							  const UAnalysisProperties* AnalysisProperties,
							  const UAnimSequence&       Animation,
							  const float                RateScale) const override;

	// This should return an instance derived from UAnalysisProperties that is suitable for the FunctionName
	UAnalysisProperties* MakeAnalysisProperties(UObject* Outer, const FString& FunctionName) const override;

	// This should return the names of the functions handled
	TArray<FString> GetAnalysisFunctions() const override;
};

static FCoreBlendSpaceAnalysisFeature CoreBlendSpaceAnalysisFeature;

//======================================================================================================================
TArray<FString> FCoreBlendSpaceAnalysisFeature::GetAnalysisFunctions() const
{
	TArray<FString> Functions = 
	{
		TEXT("None"),
		TEXT("Position"),
		TEXT("Velocity"),
		TEXT("DeltaPosition"),
		TEXT("Orientation"),
		TEXT("OrientationRate"),
		TEXT("DeltaOrientation"),
		TEXT("AngularVelocity")
	};
	return Functions;
}

//======================================================================================================================
UAnalysisProperties* FCoreBlendSpaceAnalysisFeature::MakeAnalysisProperties(UObject* Outer, const FString& FunctionName) const
{
	UAnalysisProperties* Result = nullptr;
	if (FunctionName.Equals(TEXT("Position")) ||
		FunctionName.Equals(TEXT("Velocity")) ||
		FunctionName.Equals(TEXT("DeltaPosition")) ||
		FunctionName.Equals(TEXT("AngularVelocity")))
	{
		Result = NewObject<ULinearAnalysisProperties>(Outer);
	}
	else if (FunctionName.Equals(TEXT("Orientation")) ||
			 FunctionName.Equals(TEXT("OrientationRate")) ||
			 FunctionName.Equals(TEXT("DeltaOrientation")))
	{
		Result = NewObject<UEulerAnalysisProperties>(Outer);
	}

	if (Result)
	{
		Result->Function = FunctionName;
	}
	return Result;
}

//======================================================================================================================
bool FCoreBlendSpaceAnalysisFeature::CalculateSampleValue(float&                     Result,
														  const UBlendSpace&         BlendSpace,
														  const UAnalysisProperties* AnalysisProperties,
														  const UAnimSequence&       Animation,
														  const float                RateScale) const
{
	if (!AnalysisProperties)
	{
		return false;
	}

	// Note we need to be explicit on function via static_cast until deprecated versions of the Calculate methods are removed otherwise the compiler cannot disambiguate the call

	const FString& FunctionName = AnalysisProperties->Function;
	if (FunctionName.Equals(TEXT("Position")))
	{
		return BlendSpaceAnalysis::CalculateComponentSampleValue(
			Result,
			static_cast<bool(*)(FVector&, const UBlendSpace&, const ULinearAnalysisProperties*, const UAnimSequence&, const float)>(&BlendSpaceAnalysis::CalculatePosition),
			BlendSpace,
			static_cast<const ULinearAnalysisProperties*>(AnalysisProperties),
			Animation,
			RateScale);
	}
	else if (FunctionName.Equals(TEXT("Velocity")))
	{
		return BlendSpaceAnalysis::CalculateComponentSampleValue(
			Result,
			static_cast<bool(*)(FVector&, const UBlendSpace&, const ULinearAnalysisPropertiesBase*, const UAnimSequence&, const float)>(&BlendSpaceAnalysis::CalculateVelocity),
			BlendSpace,
			Cast<ULinearAnalysisProperties>(AnalysisProperties),
			Animation,
			RateScale);
	}
	else if (FunctionName.Equals(TEXT("DeltaPosition")))
	{
		return BlendSpaceAnalysis::CalculateComponentSampleValue(
			Result,
			static_cast<bool(*)(FVector&, const UBlendSpace&, const ULinearAnalysisProperties*, const UAnimSequence&, const float)>(&BlendSpaceAnalysis::CalculateDeltaPosition),
			BlendSpace,
			Cast<ULinearAnalysisProperties>(AnalysisProperties), Animation, RateScale);
	}
	else if (FunctionName.Equals(TEXT("AngularVelocity")))
	{
		return BlendSpaceAnalysis::CalculateComponentSampleValue(
			Result,
			static_cast<bool(*)(FVector&, const UBlendSpace&, const ULinearAnalysisProperties*, const UAnimSequence&, const float)>(&BlendSpaceAnalysis::CalculateAngularVelocity),
			BlendSpace,
			Cast<ULinearAnalysisProperties>(AnalysisProperties),
			Animation,
			RateScale);
	}
	else if (FunctionName.Equals(TEXT("Orientation")))
	{
		return BlendSpaceAnalysis::CalculateComponentSampleValue(
			Result,
			static_cast<bool(*)(FVector&, const UBlendSpace&, const UEulerAnalysisProperties*, const UAnimSequence&, const float)>(&BlendSpaceAnalysis::CalculateOrientation),
			BlendSpace,
			Cast<UEulerAnalysisProperties>(AnalysisProperties),
			Animation,
			RateScale);
	}
	else if (FunctionName.Equals(TEXT("OrientationRate")))
	{
		return BlendSpaceAnalysis::CalculateComponentSampleValue(
			Result,
			static_cast<bool(*)(FVector&, const UBlendSpace&, const UEulerAnalysisProperties*, const UAnimSequence&, const float)>(&BlendSpaceAnalysis::CalculateOrientationRate),
			BlendSpace,
			Cast<UEulerAnalysisProperties>(AnalysisProperties),
			Animation,
			RateScale);
	}
	else if (FunctionName.Equals(TEXT("DeltaOrientation")))
	{
		return BlendSpaceAnalysis::CalculateComponentSampleValue(
			Result,
			static_cast<bool(*)(FVector&, const UBlendSpace&, const UEulerAnalysisProperties*,
				const UAnimSequence&, const float)>(&BlendSpaceAnalysis::CalculateDeltaOrientation),
			BlendSpace,
			Cast<UEulerAnalysisProperties>(AnalysisProperties),
			Animation,
			RateScale);
	}
	return false;
}

//======================================================================================================================
static TArray<IBlendSpaceAnalysisFeature*> GetAnalysisFeatures(bool bCoreFeaturesLast)
{
	TArray<IBlendSpaceAnalysisFeature*> ModularFeatures;

	if (!bCoreFeaturesLast)
	{
		ModularFeatures.Push(&CoreBlendSpaceAnalysisFeature);
	}

	TArray<IBlendSpaceAnalysisFeature*> ExtraModularFeatures = 
		IModularFeatures::Get().GetModularFeatureImplementations<IBlendSpaceAnalysisFeature>(
			IBlendSpaceAnalysisFeature::GetModuleFeatureName());
	ModularFeatures += ExtraModularFeatures;

	if (bCoreFeaturesLast)
	{
		ModularFeatures.Push(&CoreBlendSpaceAnalysisFeature);
	}
	return ModularFeatures;
}

//======================================================================================================================
FVector BlendSpaceAnalysis::CalculateSampleValue(const UBlendSpace& BlendSpace, const UAnimSequence& Animation,
                                                 const float RateScale, const FVector& OriginalPosition, bool bAnalyzed[3])
{
	FVector AdjustedPosition = OriginalPosition;
	TArray<IBlendSpaceAnalysisFeature*> ModularFeatures = GetAnalysisFeatures(true);
	for (int32 Index = 0; Index != 2; ++Index)
	{
		bAnalyzed[Index] = false;
		const UAnalysisProperties* AnalysisProperties = BlendSpace.AnalysisProperties[Index].Get();
		for (const IBlendSpaceAnalysisFeature* Feature : ModularFeatures)
		{
			float NewPosition = float(AdjustedPosition[Index]);
			bAnalyzed[Index] = Feature->CalculateSampleValue(
				NewPosition, BlendSpace, AnalysisProperties, Animation, RateScale);
			if (bAnalyzed[Index])
			{
				AdjustedPosition[Index] = NewPosition;
				break;
			}
		}
	}
	return AdjustedPosition;
}

//======================================================================================================================
UAnalysisProperties* BlendSpaceAnalysis::MakeAnalysisProperties(UObject* Outer, const FString& FunctionName)
{
	UAnalysisProperties* Result = nullptr;

	TArray<IBlendSpaceAnalysisFeature*> ModularFeatures = GetAnalysisFeatures(true);
	for (const IBlendSpaceAnalysisFeature* Feature : ModularFeatures)
	{
		Result = Feature->MakeAnalysisProperties(Outer, FunctionName);
		if (Result)
		{
			// Need to explicitly set flags to make undo work on the new object
			Result->SetFlags(RF_Transactional);
			return Result;
		}
	}
	return Result;
}

//======================================================================================================================
TArray<FString> BlendSpaceAnalysis::GetAnalysisFunctions()
{
	TArray<FString> FunctionNames;
	TArray<IBlendSpaceAnalysisFeature*> ModularFeatures = GetAnalysisFeatures(false);
	for (const IBlendSpaceAnalysisFeature* Feature : ModularFeatures)
	{
		TArray<FString> FeatureFunctionNames = Feature->GetAnalysisFunctions();
		for (const FString& FeatureFunctionName : FeatureFunctionNames)
		{
			FunctionNames.AddUnique(FeatureFunctionName);
		}
	}
	return FunctionNames;
}

#undef LOCTEXT_NAMESPACE
