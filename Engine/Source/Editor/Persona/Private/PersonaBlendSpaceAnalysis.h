// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "BlendSpaceAnalysis.h"

namespace BlendSpaceAnalysis
{

/**
 * Calculates the sample value/position and indicates if it needed to be analyzed. Note that the original position
 * is passed in, and its elements will be used in the output where analysis doesn't need to be done.
 */
FVector CalculateSampleValue(const UBlendSpace&   BlendSpace, 
							 const UAnimSequence& Animation, 
							 const float          RateScale, 
							 const FVector&       OriginalPosition, 
							 bool                 bAnalyzed[3]);


/**
 * This will return an instance derived from UAnalysisProperties that is suitable for the Function. The caller will
 * pass in a suitable owning object, outer, that the implementation should assign as owner of the newly created object.
 */
UAnalysisProperties* MakeAnalysisProperties(UObject* Outer, const FString& FunctionName);

/**
 * This will return the names of the functions handled
 */
TArray<FString> GetAnalysisFunctions();

};
