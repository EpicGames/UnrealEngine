// Copyright Epic Games, Inc. All Rights Reserved.

#include "PersonaEditorModeManager.h"

#include "IPersonaEditMode.h"
#include "IPersonaPreviewScene.h"
#include "Selection.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "ContextObjectStore.h"

bool FPersonaEditorModeManager::GetCameraTarget(FSphere& OutTarget) const
{
	for (const UEdMode* Mode : ActiveScriptableModes)
	{
		if (const UEditorInteractiveToolsContext* ModeInteractiveToolsContext = Mode->GetInteractiveToolsContext())
		{
			if (const IAnimationEditContext* PersonaContext = ModeInteractiveToolsContext->ContextObjectStore->FindContext<UAnimationEditModeContext>())
			{
				if (PersonaContext->GetCameraTarget(OutTarget))
				{
					return true;
				}
			}
		}
	}
	return false;
}

void FPersonaEditorModeManager::GetOnScreenDebugInfo(TArray<FText>& OutDebugText) const
{
	for (const UEdMode* Mode : ActiveScriptableModes)
	{
		if (const UEditorInteractiveToolsContext* ModeInteractiveToolsContext = Mode->GetInteractiveToolsContext())
		{
			if (const IAnimationEditContext* PersonaContext = ModeInteractiveToolsContext->ContextObjectStore->FindContext<UAnimationEditModeContext>())
			{
				PersonaContext->GetOnScreenDebugInfo(OutDebugText);
			}
		}
	}
}


void FPersonaEditorModeManager::SetPreviewScene(FPreviewScene* NewPreviewScene)
{
	const IPersonaPreviewScene *PersonaPreviewScene = static_cast<const IPersonaPreviewScene *>(NewPreviewScene);

	if (PersonaPreviewScene && PersonaPreviewScene->GetPreviewMeshComponent())
	{
		ComponentSet->BeginBatchSelectOperation();
		ComponentSet->DeselectAll();
		ComponentSet->Select(PersonaPreviewScene->GetPreviewMeshComponent(), true);
		ComponentSet->EndBatchSelectOperation();
	}

	FAssetEditorModeManager::SetPreviewScene(NewPreviewScene);
}
