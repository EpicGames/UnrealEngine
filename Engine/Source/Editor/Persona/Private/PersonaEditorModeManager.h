// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IPersonaEditorModeManager.h"

/** Persona-specific extensions to the asset editor mode manager */
class FPersonaEditorModeManager : public IPersonaEditorModeManager
{
public:
	/** IPersonaEditorModeManager interface  */
	virtual bool GetCameraTarget(FSphere& OutTarget) const override;
	virtual void GetOnScreenDebugInfo(TArray<FText>& OutDebugText) const override;

	// FAssetEditorModeManager overrides
	virtual void SetPreviewScene(FPreviewScene* NewPreviewScene) override;
};
