// Copyright Epic Games, Inc. All Rights Reserved.

#include "PersonaMeshDetails.h"
#include "StaticMeshResources.h"
#include "Widgets/Input/SCheckBox.h"
#include "Misc/MessageDialog.h"
#include "Modules/ModuleManager.h"
#include "UObject/UObjectHash.h"
#include "Framework/Commands/UIAction.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailPropertyRow.h"
#include "IDetailGroup.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "IDocumentation.h"
#include "Components/SkinnedMeshComponent.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "IPersonaPreviewScene.h"
#include "Animation/AnimBlueprint.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SVectorInputBox.h"
#include "Editor/UnrealEdEngine.h"
#include "UnrealEdGlobals.h"
#include "MaterialList.h"
#include "PropertyCustomizationHelpers.h"
#include "ScopedTransaction.h"
#include "Editor.h"
#include "Animation/AnimBlueprintGeneratedClass.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Rendering/SkeletalMeshModel.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "EditorFramework/AssetImportData.h"
#include "ClothingAsset.h"

#include "LODUtilities.h"
#include "MeshUtilities.h"
#include "FbxMeshUtils.h"
#include "NaniteLayout.h"

#include "Widgets/Input/STextComboBox.h"

#include "Engine/SkeletalMeshLODSettings.h"
#include "Animation/AnimInstance.h"
#include "Animation/AnimSequence.h"
#include "IPersonaToolkit.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "JsonObjectConverter.h"
#include "Widgets/Layout/SUniformWrapPanel.h"
#include "ClothingAssetFactoryInterface.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "SKismetInspector.h"
#include "PersonaModule.h"
#include "PropertyEditorDelegates.h"
#include "IEditableSkeleton.h"
#include "IMeshReductionManagerModule.h"
#include "SkinWeightProfileHelpers.h"
#include "Factories/FbxSkeletalMeshImportData.h"
#include "IPropertyUtilities.h"
#include "LODInfoUILayout.h"
#include "SWarningOrErrorBox.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Misc/CoreMisc.h"
#include "InterchangeManager.h"
#include "InterchangeAssetImportData.h"
#include "SPerQualityLevelPropertiesWidget.h"
#include "UObject/SavePackage.h"

#define LOCTEXT_NAMESPACE "PersonaMeshDetails"

DEFINE_LOG_CATEGORY(LogSkeletalMeshPersonaMeshDetail);


static const TCHAR* GSuffixDefaultMaterial = TEXT(" (Default)");

static bool GEnableSkeletalMeshMaterialSlotOverlayMaterialUI = false;
static FAutoConsoleVariableRef CVarEnableSkeletalMeshMaterialSlotOverlayMaterialUI(
	TEXT("SkeletalMesh.FeatureFlags.UI.ShowMaterialSlotOverlayMaterial"),
	GEnableSkeletalMeshMaterialSlotOverlayMaterialUI,
	TEXT("Whether skeletal mesh editor show material overlay per material slot UI."),
	ECVF_Default);


/*
* Custom data key
*/
enum SK_CustomDataKey
{
	CustomDataKey_LODVisibilityState = 0, //This is the key to know if a LOD is shown in custom mode. Do CustomDataKey_LODVisibilityState + LodIndex for a specific LOD
	CustomDataKey_LODEditMode = 100 //This is the key to know the state of the custom lod edit mode.
};

namespace PersonaMeshDetailsConstants
{
	/** Number to extend the num LODs slider beyond the currently existing number of LODs in the mesh */
	constexpr static int32 LodSliderExtension = 5;
}

static int32 GetDefaultMaterialIndex(const USkeletalMesh* SkeletalMesh, int32 LODIndex, int32 SectionIndex)
{
	int32 DefaultMaterialIndex = INDEX_NONE;
	if (!SkeletalMesh || !SkeletalMesh->GetImportedModel() || !SkeletalMesh->GetImportedModel()->LODModels.IsValidIndex(LODIndex))
	{
		return DefaultMaterialIndex;
	}

	const FSkeletalMeshLODModel& LODModel = SkeletalMesh->GetImportedModel()->LODModels[LODIndex];
	if (LODIndex > 0)
	{
		//Only base LOD is using the default/modified feature
		return INDEX_NONE;
	}

	if (LODModel.Sections.IsValidIndex(SectionIndex))
	{
		DefaultMaterialIndex = LODModel.Sections[SectionIndex].MaterialIndex;
	}
	
	return DefaultMaterialIndex;
}

/** Returns true if automatic mesh reduction is available. */
static bool IsAutoMeshReductionAvailable()
{
	static bool bAutoMeshReductionAvailable = FModuleManager::Get().LoadModuleChecked<IMeshReductionManagerModule>("MeshReductionInterface").GetSkeletalMeshReductionInterface() != nullptr;
	return bAutoMeshReductionAvailable;
}

void SetSkelMeshSourceSectionUserData(FSkeletalMeshLODModel& LODModel, const int32 SectionIndex, const int32 OriginalSectionIndex)
{
	FSkelMeshSourceSectionUserData& SourceSectionUserData = LODModel.UserSectionsData.FindOrAdd(OriginalSectionIndex);
	SourceSectionUserData.bDisabled = LODModel.Sections[SectionIndex].bDisabled;
	SourceSectionUserData.bCastShadow = LODModel.Sections[SectionIndex].bCastShadow;
	SourceSectionUserData.bVisibleInRayTracing = LODModel.Sections[SectionIndex].bVisibleInRayTracing;
	SourceSectionUserData.bRecomputeTangent = LODModel.Sections[SectionIndex].bRecomputeTangent;
	SourceSectionUserData.RecomputeTangentsVertexMaskChannel = LODModel.Sections[SectionIndex].RecomputeTangentsVertexMaskChannel;
	SourceSectionUserData.GenerateUpToLodIndex = LODModel.Sections[SectionIndex].GenerateUpToLodIndex;
	SourceSectionUserData.CorrespondClothAssetIndex = LODModel.Sections[SectionIndex].CorrespondClothAssetIndex;
	SourceSectionUserData.ClothingData = LODModel.Sections[SectionIndex].ClothingData;
}

IDetailCategoryBuilder&  GetLODIndexCategory(IDetailLayoutBuilder& DetailLayout, int32 LODIndex)
{
	FString CategoryName = FString(TEXT("LOD"));
	CategoryName.AppendInt(LODIndex);
	FText LODLevelLabel = FText::Format(LOCTEXT("LODIndexLabel", "LOD {0}"), FText::AsNumber(LODIndex));
	return DetailLayout.EditCategory(*CategoryName, LODLevelLabel, ECategoryPriority::Important);
}

enum class EButtonFlags
{
	/** No special property exporting flags */
	None = 0x00000000,

	/** Show generate/apply button */
	Generate = 0x00000001,

	/** Show reimport button */
	Reimport = 0x00000002,

	/** Show reimportnewfile button */
	ReimportNewFile = 0x00000004,

	/** Show remove button */
	RemoveLOD = 0x00000008,

	/** Show remove geometry button */
	RemoveGeometry = 0x00000010
};
ENUM_CLASS_FLAGS(EButtonFlags)

/**
 * Window for Nanite settings.
 */

bool GIsNaniteSkeletalMeshSettingsInitiallyCollapsed = 1;
static FAutoConsoleVariableRef CVarIsNaniteSkeletalMeshSettingsInitiallyCollapsed(
	TEXT("r.Nanite.IsNaniteSkeletalMeshSettingsInitiallyCollapsed"),
	GIsNaniteSkeletalMeshSettingsInitiallyCollapsed,
	TEXT("If the Nanite Settings are initially collapsed in the details panel in the Skeletal Mesh Editor Tool."),
	ECVF_ReadOnly
);

typedef Nanite::FSettingsLayout<USkeletalMesh, false /* ForceEnable */, false /* HighRes */> FNaniteSkeletalMeshLayoutImpl;

class FNaniteSkeletalMeshLayout : public TSharedFromThis<FNaniteSkeletalMeshLayout>
{
public:
	FNaniteSkeletalMeshLayout(FPersonaMeshDetails& InMeshDetails, TSharedRef<IPersonaToolkit> InPersonaToolkit)
	: MeshDetails(InMeshDetails)
	, PersonaToolkit(InPersonaToolkit)
	{
		LayoutImpl = MakeShareable(new FNaniteSkeletalMeshLayoutImpl());

		LayoutImpl->OnGetMesh = TDelegate<USkeletalMesh*()>::CreateLambda([this]
		{
			return PersonaToolkit->GetMesh();
		});

		LayoutImpl->OnRefreshTool = TDelegate<void()>::CreateLambda([this]
		{
			MeshDetails.RequestLayoutUpdate();
		});

		const USkeletalMesh* SkeletalMesh = LayoutImpl->GetMesh();
		check(SkeletalMesh);
		LayoutImpl->UpdateSettings(SkeletalMesh->NaniteSettings);
	}

	~FNaniteSkeletalMeshLayout()
	{
	}

	void AddToDetailsPanel(IDetailLayoutBuilder& DetailBuilder)
	{
		const bool bInitiallyCollapsed = GIsNaniteSkeletalMeshSettingsInitiallyCollapsed != 0;

		USkeletalMesh* SkeletalMesh = LayoutImpl->GetMesh();
		TWeakObjectPtr<USkeletalMesh> WeakSkeletalMesh = SkeletalMesh;
		const int32 SortOrder = 0;
		LayoutImpl->AddToDetailsPanel(WeakSkeletalMesh, DetailBuilder, SortOrder, bInitiallyCollapsed);
	}

	inline bool IsApplyNeeded() const
	{
		return LayoutImpl->IsApplyNeeded();
	}

	inline void ApplyChanges()
	{
		return LayoutImpl->ApplyChanges();
	}

private:
	FPersonaMeshDetails& MeshDetails;
	TSharedRef<IPersonaToolkit> PersonaToolkit;

	TSharedPtr<FNaniteSkeletalMeshLayoutImpl> LayoutImpl;
};

// Container widget for LOD buttons
class SSkeletalLODActions : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SSkeletalLODActions)
		: _LODIndex(INDEX_NONE)
		, _PersonaToolkit(nullptr)
		, _ButtonFlags(EButtonFlags::None)
	{}
	SLATE_ARGUMENT(int32, LODIndex)
	SLATE_ARGUMENT(TWeakPtr<IPersonaToolkit>, PersonaToolkit)
	SLATE_ARGUMENT(EButtonFlags, ButtonFlags)
	SLATE_ARGUMENT(FString, MeshDescriptionReferenceIDString)
	SLATE_ARGUMENT(bool, BuildAvailable)
	SLATE_EVENT(FOnClicked, OnApplyLODChangeClicked)
	SLATE_EVENT(FOnClicked, OnRemoveLODClicked)
	SLATE_EVENT(FOnClicked, OnRemoveGeometryClicked)
	SLATE_EVENT(FOnClicked, OnReimportClicked)
	SLATE_EVENT(FOnClicked, OnReimportNewFileClicked)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:

	EActiveTimerReturnType RefreshExistFlag(double InCurrentTime, float InDeltaSeconds)
	{
		bDoesSourceFileExist_Cached = false;

		TSharedPtr<IPersonaToolkit> SharedToolkit = PersonaToolkit.Pin();
		if(SharedToolkit.IsValid())
		{
			USkeletalMesh* SkelMesh = SharedToolkit->GetMesh();

			if(!SkelMesh || SkelMesh->IsCompiling())
			{
				return EActiveTimerReturnType::Continue;
			}

			if(SkelMesh->IsValidLODIndex(LODIndex))
			{
				FSkeletalMeshLODInfo& LODInfo = *(SkelMesh->GetLODInfo(LODIndex));

				bDoesSourceFileExist_Cached = !LODInfo.SourceImportFilename.IsEmpty() && FPaths::FileExists(UAssetImportData::ResolveImportFilename(LODInfo.SourceImportFilename, nullptr));
			}
		}
		return EActiveTimerReturnType::Continue;
	}

	FText GetReimportButtonToolTipText() const
	{
		TSharedPtr<IPersonaToolkit> SharedToolkit = PersonaToolkit.Pin();

		if(!CanReimportFromSource() || !SharedToolkit.IsValid())
		{
			return LOCTEXT("ReimportButton_NewFile_NoSource_ToolTip", "No source file available for reimport");
		}

		USkeletalMesh* SkelMesh = SharedToolkit->GetMesh();
		check(SkelMesh);
		
		if (SkelMesh->IsCompiling())
		{
			return LOCTEXT("ReimportButton_NewFile_SkeletalMeshIsCompiling_ToolTip", "Cannot re-import, skeletal mesh is compiling...");
		}
		
		if (!SkelMesh->IsValidLODIndex(LODIndex))
		{
			// Should be true for the button to exist except if we delete a LOD
			return LOCTEXT("ReimportButton_NewFile_LODNotValid_ToolTip", "Cannot reimport, LOD has been deleted.");
		}

		const FSkeletalMeshLODInfo& LODInfo = *(SkelMesh->GetLODInfo(LODIndex));
		const FString Filename = FPaths::GetCleanFilename(LODInfo.SourceImportFilename);

		return FText::Format(LOCTEXT("ReimportButton_NewFile_ToolTip", "Re-import LOD {0} using the current source file ({1})"), FText::AsNumber(LODIndex), FText::FromString(Filename));
	}

	FText GetReimportButtonNewFileToolTipText() const
	{
		return FText::Format(LOCTEXT("ReimportButton_ToolTip", "Choose a new file to reimport over LOD {0}"), FText::AsNumber(LODIndex));
	}

	bool CanReimportFromSource() const
	{
		return bDoesSourceFileExist_Cached;
	}

	bool IsNeedApplyLODChange() const
	{
		TSharedPtr<IPersonaToolkit> SharedToolkit = PersonaToolkit.Pin();
		if (SharedToolkit.IsValid())
		{
			USkeletalMesh* SkeletalMesh = SharedToolkit->GetMesh();

			return LODSettingsChanged(SkeletalMesh, LODIndex);
		}
		return false;
	}

	bool CanRemoveGeometry() const
	{
		TSharedPtr<IPersonaToolkit> SharedToolkit = PersonaToolkit.Pin();
		if (SharedToolkit.IsValid())
		{
			USkeletalMesh* SkeletalMesh = SharedToolkit->GetMesh();

			return SkeletalMesh->IsValidLODIndex(LODIndex) && SkeletalMesh->HasMeshDescription(LODIndex);
		}
		return false;
	}

public:
	static bool LODSettingsChanged(USkeletalMesh* SkeletalMesh, int32 LODIndex)
	{
		if (!SkeletalMesh || SkeletalMesh->IsCompiling())
		{
			return false;
		}
		FSkeletalMeshLODInfo* LODInfo = SkeletalMesh->GetLODInfo(LODIndex);
		if (LODInfo == nullptr)
		{
			return false;
		}
		bool bValidLODSettings = false;
		if (SkeletalMesh->GetLODSettings() != nullptr)
		{
			const int32 NumSettings = FMath::Min(SkeletalMesh->GetLODSettings()->GetNumberOfSettings(), SkeletalMesh->GetLODNum());
			if (LODIndex < NumSettings)
			{
				bValidLODSettings = true;
			}
		}

		const FSkeletalMeshLODGroupSettings* SkeletalMeshLODGroupSettings = bValidLODSettings ? &SkeletalMesh->GetLODSettings()->GetSettingsForLODLevel(LODIndex) : nullptr;

		FGuid BuildGUID = LODInfo->ComputeDeriveDataCacheKey(SkeletalMeshLODGroupSettings);
		if (LODInfo->BuildGUID != BuildGUID)
		{
			return true;
		}
		else if (!SkeletalMesh->GetImportedModel() || !(SkeletalMesh->GetImportedModel()->LODModels.IsValidIndex(LODIndex)))
		{
			//If there is no valid LODIndex imported model we want to return false to force a build to happen
			return false;
		}
		return SkeletalMesh->GetImportedModel()->LODModels[LODIndex].BuildStringID != SkeletalMesh->GetImportedModel()->LODModels[LODIndex].GetLODModelDeriveDataKey();
	}

private:
	// Incoming arg data
	int32 LODIndex = INDEX_NONE;
	TWeakPtr<IPersonaToolkit> PersonaToolkit;
	EButtonFlags ButtonFlags = EButtonFlags::None;
	bool BuildAvailable = false;

	FOnClicked OnApplyLODChangeClicked;
	FOnClicked OnRemoveLODClicked;
	FOnClicked OnRemoveGeometryClicked;
	FOnClicked OnReimportClicked;
	FOnClicked OnReimportNewFileClicked;

	// Cached exists flag so we don't constantly hit the file system
	bool bDoesSourceFileExist_Cached = false;

	FString MeshDescriptionReferenceIDString;
};

void SSkeletalLODActions::Construct(const FArguments& InArgs)
{
	LODIndex = InArgs._LODIndex;
	PersonaToolkit = InArgs._PersonaToolkit;
	ButtonFlags = InArgs._ButtonFlags;
	MeshDescriptionReferenceIDString = InArgs._MeshDescriptionReferenceIDString;
	OnApplyLODChangeClicked = InArgs._OnApplyLODChangeClicked;
	OnRemoveLODClicked = InArgs._OnRemoveLODClicked;
	OnRemoveGeometryClicked = InArgs._OnRemoveGeometryClicked;
	OnReimportClicked = InArgs._OnReimportClicked;
	OnReimportNewFileClicked = InArgs._OnReimportNewFileClicked;
	BuildAvailable = InArgs._BuildAvailable;

	TSharedPtr<SUniformWrapPanel> WrapBox;
	this->ChildSlot

		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			[
				SAssignNew(WrapBox, SUniformWrapPanel)
				//.UseAllottedSize(true)
			]
		];

	if (OnApplyLODChangeClicked.IsBound() && EnumHasAllFlags(ButtonFlags, EButtonFlags::Generate))
	{
		FText ButtonNameText = BuildAvailable ? LOCTEXT("ApplyLODChange", "Apply Changes") : LOCTEXT("RegenerateLOD", "Regenerate LOD");
		WrapBox->AddSlot()
		[
			SNew(SButton)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.OnClicked(OnApplyLODChangeClicked)
			.IsEnabled(this, &SSkeletalLODActions::IsNeedApplyLODChange)
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(ButtonNameText)
			]
		];
	}

	if (OnRemoveLODClicked.IsBound() && EnumHasAllFlags(ButtonFlags, EButtonFlags::RemoveLOD))
	{
		WrapBox->AddSlot()
		[
			SNew(SButton)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.OnClicked(OnRemoveLODClicked)
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(LOCTEXT("RemoveLOD", "Remove this LOD"))
			]
		];
	}

	if (OnRemoveGeometryClicked.IsBound() && EnumHasAllFlags(ButtonFlags, EButtonFlags::RemoveGeometry))
	{
		WrapBox->AddSlot()
		[
			SNew(SButton)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.ToolTipText(LOCTEXT("RemoveGeometryTooltip", "Remove the geometry stored on this LOD. The LOD will instead be automatically generated, based on another LOD."))
			.IsEnabled(this, &SSkeletalLODActions::CanRemoveGeometry)
			.OnClicked(OnRemoveGeometryClicked)
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(LOCTEXT("RemoveGeometry", "Remove Geometry"))
			]
		];
	}
	
	if (OnReimportClicked.IsBound() && EnumHasAllFlags(ButtonFlags, EButtonFlags::Reimport))
	{
		WrapBox->AddSlot()
		[
			SNew(SButton)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.ToolTipText(this, &SSkeletalLODActions::GetReimportButtonToolTipText)
			.IsEnabled(this, &SSkeletalLODActions::CanReimportFromSource)
			.OnClicked(OnReimportClicked)
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(LOCTEXT("ReimportLOD", "Reimport"))
			]
		];
	}

	if (OnReimportNewFileClicked.IsBound() && EnumHasAllFlags(ButtonFlags, EButtonFlags::ReimportNewFile))
	{
		WrapBox->AddSlot()
		[
			SNew(SButton)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.ToolTipText(this, &SSkeletalLODActions::GetReimportButtonNewFileToolTipText)
			.OnClicked(OnReimportNewFileClicked)
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(LOCTEXT("ReimportLOD_NewFile", "Reimport (New File)"))
			]
		];
	}

	// Register timer to refresh out exists flag periodically, with a bit added per LOD so we're not doing everything on the same frame
	const float LODTimeOffset = 1.0f / 30.0f;
	RegisterActiveTimer(1.0f + LODTimeOffset * LODIndex, FWidgetActiveTimerDelegate::CreateSP(this, &SSkeletalLODActions::RefreshExistFlag));
}


//////////////////////////////////////////////////////////////////////////
// FSkeletalMeshReductionSettingsLayout implementation

FSkeletalMeshReductionSettingsLayout::FSkeletalMeshReductionSettingsLayout(FSkeletalMeshOptimizationSettings& InReductionSettings, bool InbIsLODModelbuildDataAvailable, int32 InLODIndex, FIsLODSettingsEnabledDelegate InIsLODSettingsEnabledDelegate, FModifyMeshLODSettingsDelegate InModifyMeshLODSettingsDelegate)
	: ReductionSettings(InReductionSettings)
	, bIsLODModelbuildDataAvailable(InbIsLODModelbuildDataAvailable)
	, LODIndex(InLODIndex)
	, IsLODSettingsEnabledDelegate(InIsLODSettingsEnabledDelegate)
	, ModifyMeshLODSettingsDelegate(InModifyMeshLODSettingsDelegate)
	, EnumReductionMethod(nullptr)
	, EnumImportance(nullptr)
	, EnumTerminationCriterion(nullptr)
{
	//Make sure apply is bound, this class mean nothing if apply is not bound
	check(IsLODSettingsEnabledDelegate.IsBound());
}

/** IDetailCustomNodeBuilder Interface*/
void FSkeletalMeshReductionSettingsLayout::GenerateHeaderRowContent(FDetailWidgetRow& NodeRow)
{
	NodeRow.NameContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("SkeletalMeshReductionSettings", "Reduction Settings"))
		.Font(IDetailLayoutBuilder::GetDetailFont())
		];
}

void FSkeletalMeshReductionSettingsLayout::GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder)
{
	//Get the Enums
	if (EnumReductionMethod == nullptr)
	{
		EnumReductionMethod = FindObject<UEnum>(nullptr, TEXT("/Script/Engine.SkeletalMeshOptimizationType"), EFindObjectFlags::ExactClass);
	}
	if (EnumImportance == nullptr)
	{
		EnumImportance = FindObject<UEnum>(nullptr, TEXT("/Script/Engine.SkeletalMeshOptimizationImportance"), EFindObjectFlags::ExactClass);
	}
	if (EnumTerminationCriterion == nullptr)
	{
		EnumTerminationCriterion = FindObject<UEnum>(nullptr, TEXT("/Script/Engine.SkeletalMeshTerminationCriterion"), EFindObjectFlags::ExactClass);
	}

	bool bUseThirdPartyUI = !UseNativeReductionTool();

	if (bUseThirdPartyUI)
	{
		ChildrenBuilder.AddCustomRow( LOCTEXT("ReductionReductionMethod", "Reduction_ReductionMethod") )
		.RowTag(TEXT("ReductionMethod"))
		.NameContent()
		[
			SNew(STextBlock)
			.Font( IDetailLayoutBuilder::GetDetailFont() )
			.Text(LOCTEXT("ReductionMethod", "Reduction Method"))
			.IsEnabled(this, &FSkeletalMeshReductionSettingsLayout::IsReductionEnabled)
		
		]
		.ValueContent()
		[
			SNew(SComboButton)
			.IsEnabled(this, &FSkeletalMeshReductionSettingsLayout::IsReductionEnabled)
			.OnGetMenuContent(this, &FSkeletalMeshReductionSettingsLayout::FillReductionMethodMenu)
			.VAlign(VAlign_Center)
			.ContentPadding(2.f)
			.ButtonContent()
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(this, &FSkeletalMeshReductionSettingsLayout::GetReductionMethodText)
			]
		];

		AddFloatRow(ChildrenBuilder, 
			LOCTEXT("PercentTriangles_Row", "Triangle Percentage"),
			LOCTEXT("PercentTriangles", "Percent of Triangles"),
			LOCTEXT("PercentTriangles_DeviationToolTip", "The percentage of triangles to retain as a ratio, e.g. 0.1 indicates 10 percent."),
			TEXT("PercentTriangles"),
			0.0f,
			1.0f,
			FGetFloatDelegate::CreateRaw(this, &FSkeletalMeshReductionSettingsLayout::GetNumTrianglesPercentage),
			FSetFloatDelegate::CreateRaw(this, &FSkeletalMeshReductionSettingsLayout::SetNumTrianglesPercentage))
			.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FSkeletalMeshReductionSettingsLayout::GetVisibiltyIfCurrentReductionMethodIsNot, SMOT_MaxDeviation)));

		AddFloatRow(ChildrenBuilder,
			LOCTEXT("Accuracy_Row", "Accuracy Percentage"),
			LOCTEXT("PercentAccuracy", "Accuracy Percentage"),
			LOCTEXT("PercentAccuracy_ToolTip", "The simplification uses this as how much deviate from source mesh. Better works with hard surface meshes."),
			TEXT("PercentAccuracy"),
			0.0f,
			1.0f,
			FGetFloatDelegate::CreateRaw(this, &FSkeletalMeshReductionSettingsLayout::GetAccuracyPercentage),
			FSetFloatDelegate::CreateRaw(this, &FSkeletalMeshReductionSettingsLayout::SetAccuracyPercentage))
			.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FSkeletalMeshReductionSettingsLayout::GetVisibiltyIfCurrentReductionMethodIsNot, SMOT_NumOfTriangles)));


		auto AddImportanceRow = [this, &ChildrenBuilder](const FText RowTitleText, const FText RowNameContentText, FName RowTag, const EImportanceType ImportanceType)
		{
			ChildrenBuilder.AddCustomRow(RowTitleText)
			.RowTag(RowTag)
			.NameContent()
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(RowNameContentText)
				.IsEnabled(this, &FSkeletalMeshReductionSettingsLayout::IsReductionEnabled)
			]
			.ValueContent()
			[
				SNew(SComboButton)
				.IsEnabled(this, &FSkeletalMeshReductionSettingsLayout::IsReductionEnabled)
				.OnGetMenuContent(this, &FSkeletalMeshReductionSettingsLayout::FillReductionImportanceMenu, ImportanceType)
				.VAlign(VAlign_Center)
				.ContentPadding(2.f)
				.ButtonContent()
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(this, &FSkeletalMeshReductionSettingsLayout::GetReductionImportanceText, ImportanceType)
				]
			];
		};
		
		AddImportanceRow(LOCTEXT("ReductionSilhouetteImportance", "Reduction_SilhouetteImportance"), LOCTEXT("SilhouetteImportance", "Silhouette"), TEXT("SilhouetteImportance"), ID_Silhouette);
		AddImportanceRow(LOCTEXT("ReductionTextureImportance", "Reduction_TextureImportance"), LOCTEXT("TextureImportance", "Texture"), TEXT("TextureImportance"), ID_Texture);
		AddImportanceRow(LOCTEXT("ReductionShadingImportance", "Reduction_ShadingImportance"), LOCTEXT("ShadingImportance", "Shading"), TEXT("ShadingImportance"), ID_Shading);
		AddImportanceRow(LOCTEXT("ReductionSkinningImportance", "Reduction_SkinningImportance"), LOCTEXT("SkinningImportance", "Skinning"), TEXT("SkinningImportance"), ID_Skinning);

		AddBoolRow(ChildrenBuilder,
			LOCTEXT("RemapMorphTargets_Row", "RemapMorphTargets"),
			LOCTEXT("RemapMorphTargets_RowNameContent", "Remap Morph Targets"),
			LOCTEXT("RemapMorphTargets_RowNameContentTooltip", "Remap the morph targets from the base LOD onto the reduce LOD."),
			TEXT("RemapMorphTargets"),
			FGetCheckBoxStateDelegate::CreateRaw(this, &FSkeletalMeshReductionSettingsLayout::GetRemapMorphTargets),
			FSetCheckBoxStateDelegate::CreateRaw(this, &FSkeletalMeshReductionSettingsLayout::SetRemapMorphTargets));

		AddBoolRow(ChildrenBuilder,
			LOCTEXT("RecalcNormals_Row", "Recalculate Normals"),
			LOCTEXT("RecalcNormals_RowNameContent", "Recompute Normal"),
			LOCTEXT("RecalcNormals_RowNameContentTooltip", "Whether Normal smoothing groups should be preserved. If true then Hard Edge Angle (NormalsThreshold) is used."),
			TEXT("RecalcNormals"),
			FGetCheckBoxStateDelegate::CreateRaw(this, &FSkeletalMeshReductionSettingsLayout::ShouldRecomputeNormals),
			FSetCheckBoxStateDelegate::CreateRaw(this, &FSkeletalMeshReductionSettingsLayout::OnRecomputeNormalsChanged));

		AddFloatRow(ChildrenBuilder,
			LOCTEXT("NormalsThreshold_Row", "Normals Threshold"),
			LOCTEXT("NormalsThreshold_RowNameContent", "Hard Edge Angle"),
			LOCTEXT("NormalsThreshold_RowNameContentToolTip", "If the angle between two triangles are above this value, the normals will not be smooth over the edge between those two triangles. Set in degrees. This is only used when Recalculate Normals is set to true."),
			TEXT("NormalsThreshold"),
			0.0f,
			360.0f,
			FGetFloatDelegate::CreateRaw(this, &FSkeletalMeshReductionSettingsLayout::GetNormalsThreshold),
			FSetFloatDelegate::CreateRaw(this, &FSkeletalMeshReductionSettingsLayout::SetNormalsThreshold));

		AddFloatRow(ChildrenBuilder,
			LOCTEXT("WeldingThreshold_Row", "Welding Threshold"),
			LOCTEXT("WeldingThreshold_RowNameContent", "Welding Threshold"),
			LOCTEXT("WeldingThreshold_RowNameContentToolTip", "The welding threshold distance.Vertices under this distance will be welded."),
			TEXT("WeldingThreshold"),
			0.0f,
			1000.0f,
			FGetFloatDelegate::CreateRaw(this, &FSkeletalMeshReductionSettingsLayout::GetWeldingThreshold),
			FSetFloatDelegate::CreateRaw(this, &FSkeletalMeshReductionSettingsLayout::SetWeldingThreshold));

		AddIntegerRow(ChildrenBuilder,
			LOCTEXT("MaxBonesPerVertex_Row", "MaxBonesPerVertex"),
			LOCTEXT("MaxBonesPerVertex", "Max Bones Influence"),
			LOCTEXT("MaxBonesPerVertex_ToolTip", "Maximum number of bones that can be assigned to each vertex."),
			TEXT("MaxBonesPerVertex"),
			1,
			MAX_TOTAL_INFLUENCES,
			FGetIntegerDelegate::CreateRaw(this, &FSkeletalMeshReductionSettingsLayout::GetMaxBonesPerVertex),
			FSetIntegerDelegate::CreateRaw(this, &FSkeletalMeshReductionSettingsLayout::SetMaxBonesPerVertex));
	}
	else  // Not third party: Using our own skeletal simplifier.
	{
		ChildrenBuilder.AddCustomRow( LOCTEXT("ReductionTerminationCriterion", "Reduction_TerminationCriterion") )
		.NameContent()
		[
			SNew(STextBlock)
			.Font( IDetailLayoutBuilder::GetDetailFont() )
			.Text(LOCTEXT("TerminationCriterion", "Termination Criterion"))
			.IsEnabled(this, &FSkeletalMeshReductionSettingsLayout::IsReductionEnabled)
		
		]
		.ValueContent()
		[
			SNew(SComboButton)
			.IsEnabled(this, &FSkeletalMeshReductionSettingsLayout::IsReductionEnabled)
			.OnGetMenuContent(this, &FSkeletalMeshReductionSettingsLayout::FillReductionTerminationCriterionMenu)
			.VAlign(VAlign_Center)
			.ContentPadding(2.f)
			.ButtonContent()
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(this, &FSkeletalMeshReductionSettingsLayout::GetReductionTerminationCriterionText)
			]
		];

		{
			FDetailWidgetRow& TrianglePercentRow = AddFloatRow(ChildrenBuilder,
				LOCTEXT("PercentTriangles_Row", "Triangle Percentage"),
				LOCTEXT("PercentTriangles", "Percent of Triangles"),
				LOCTEXT("PercentTriangles_ToolTip", "The simplification uses this percentage of source mesh's triangle count as a target."),
				TEXT("PercentTriangles"),
				0.0f,
				1.0f,
				FGetFloatDelegate::CreateRaw(this, &FSkeletalMeshReductionSettingsLayout::GetNumTrianglesPercentage),
				FSetFloatDelegate::CreateRaw(this, &FSkeletalMeshReductionSettingsLayout::SetNumTrianglesPercentage));

			SetPercentAndAbsoluteVisibility(TrianglePercentRow, SMTC_NumOfTriangles, SMTC_TriangleOrVert);
		}

		{
			FDetailWidgetRow& VerticesPercentRow = AddFloatRow(ChildrenBuilder,
				LOCTEXT("Percentvertices_Row", "Vertices Percentage"),
				LOCTEXT("PercentVertices", "Percent of Vertices"),
				LOCTEXT("PercentVertices_ToolTip", "The percentage of vertices to retain as a ratio, e.g. 0.1 indicates 10 percent."),
				TEXT("PercentVertices"),
				0.0f,
				1.0f,
				FGetFloatDelegate::CreateRaw(this, &FSkeletalMeshReductionSettingsLayout::GetNumVerticesPercentage),
				FSetFloatDelegate::CreateRaw(this, &FSkeletalMeshReductionSettingsLayout::SetNumVerticesPercentage));
			
			SetPercentAndAbsoluteVisibility(VerticesPercentRow, SMTC_NumOfVerts, SMTC_TriangleOrVert);
		}

		{
			FDetailWidgetRow& MaxTrianglesRow = AddIntegerRow(ChildrenBuilder,
				LOCTEXT("MaxTriangles_Row", "Max Number of Triangles"),
				LOCTEXT("MaxTriangles", "Max Triangles Count"),
				LOCTEXT("MaxTriangles_ToolTip", "The maximum number of triangles to retain."),
				TEXT("MaxTriangles"),
				0,
				INT_MAX,
				FGetIntegerDelegate::CreateRaw(this, &FSkeletalMeshReductionSettingsLayout::GetNumMaxTrianglesCount),
				FSetIntegerDelegate::CreateRaw(this, &FSkeletalMeshReductionSettingsLayout::SetNumMaxTrianglesCount));

			const TArray< SkeletalMeshTerminationCriterion > VizList =
			{
				SMTC_AbsNumOfTriangles,
				SMTC_AbsTriangleOrVert
			};

			MaxTrianglesRow.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FSkeletalMeshReductionSettingsLayout::ShowIfCurrentCriterionIs, VizList)));
			
		}

		{
			FDetailWidgetRow& MaxVerticesRow = AddIntegerRow(ChildrenBuilder,
				LOCTEXT("MaxVertices_Row", "Max Number of Vertices"),
				LOCTEXT("MaxVertices", "Max Vertex Count"),
				LOCTEXT("MaxVertices_ToolTip", "The maximum number of vertices to retain."),
				TEXT("MaxVertices"),
				0,
				INT_MAX,
				FGetIntegerDelegate::CreateRaw(this, &FSkeletalMeshReductionSettingsLayout::GetNumMaxVerticesCount),
				FSetIntegerDelegate::CreateRaw(this, &FSkeletalMeshReductionSettingsLayout::SetNumMaxVerticesCount));
				
			const TArray< SkeletalMeshTerminationCriterion > VizList =
			{
				SMTC_AbsNumOfVerts,
				SMTC_AbsTriangleOrVert
			};

			MaxVerticesRow.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FSkeletalMeshReductionSettingsLayout::ShowIfCurrentCriterionIs, VizList)));
		}

		{
			FDetailWidgetRow& MaxTrianglesRow = AddUnsignedIntegerRow(ChildrenBuilder,
				LOCTEXT("MaxTrianglesPercentage_Row", "Max Number of Triangles"),
				LOCTEXT("MaxTrianglesPercentage", "Max Triangles Count"),
				LOCTEXT("MaxTrianglesPercentage_ToolTip", "The maximum number of triangles to retain when using percentage criterion."),
				TEXT("MaxTrianglesPercentage"),
				0,
				MAX_uint32,
				FGetUnsignedIntegerDelegate::CreateRaw(this, &FSkeletalMeshReductionSettingsLayout::GetNumMaxTrianglesPercentageCount),
				FSetUnsignedIntegerDelegate::CreateRaw(this, &FSkeletalMeshReductionSettingsLayout::SetNumMaxTrianglesPercentageCount));

			const TArray< SkeletalMeshTerminationCriterion > VizList =
			{
				SMTC_NumOfTriangles
			};

			MaxTrianglesRow.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FSkeletalMeshReductionSettingsLayout::ShowIfCurrentCriterionIs, VizList)));

		}

		{
			FDetailWidgetRow& MaxVerticesRow = AddUnsignedIntegerRow(ChildrenBuilder,
				LOCTEXT("MaxVerticesPercentage_Row", "Max Number of Vertices"),
				LOCTEXT("MaxVerticesPercentage", "Max Vertex Count"),
				LOCTEXT("MaxVerticesPercentage_ToolTip", "The maximum number of vertices to retain when using percentage criterion."),
				TEXT("MaxVerticesPercentage"),
				0,
				MAX_uint32,
				FGetUnsignedIntegerDelegate::CreateRaw(this, &FSkeletalMeshReductionSettingsLayout::GetNumMaxVerticesPercentageCount),
				FSetUnsignedIntegerDelegate::CreateRaw(this, &FSkeletalMeshReductionSettingsLayout::SetNumMaxVerticesPercentageCount));

			const TArray< SkeletalMeshTerminationCriterion > VizList =
			{
				SMTC_NumOfVerts
			};

			MaxVerticesRow.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FSkeletalMeshReductionSettingsLayout::ShowIfCurrentCriterionIs, VizList)));
		}

		AddBoolRow(ChildrenBuilder,
			LOCTEXT("RemapMorphTargets_Row", "RemapMorphTargets"),
			LOCTEXT("RemapMorphTargets_RowNameContent", "Remap Morph Targets"),
			LOCTEXT("RemapMorphTargets_RowNameContentTooltip", "Remap the morph targets from the base LOD onto the reduce LOD."),
			TEXT("RemapMorphTargets"),
			FGetCheckBoxStateDelegate::CreateRaw(this, &FSkeletalMeshReductionSettingsLayout::GetRemapMorphTargets),
			FSetCheckBoxStateDelegate::CreateRaw(this, &FSkeletalMeshReductionSettingsLayout::SetRemapMorphTargets));

		AddIntegerRow(ChildrenBuilder,
			LOCTEXT("MaxBonesPerVertex_Row", "MaxBonesPerVertex"),
			LOCTEXT("MaxBonesPerVertex", "Max Bones Influence"),
			LOCTEXT("MaxBonesPerVertex_ToolTip", "Maximum number of bones that can be assigned to each vertex."),
			TEXT("MaxBonesPerVertex"),
			1,
			INT_MAX,
			FGetIntegerDelegate::CreateRaw(this, &FSkeletalMeshReductionSettingsLayout::GetMaxBonesPerVertex),
			FSetIntegerDelegate::CreateRaw(this, &FSkeletalMeshReductionSettingsLayout::SetMaxBonesPerVertex));

		AddBoolRow(ChildrenBuilder,
			LOCTEXT("EnforceBoneBoundaries_Row", "EnforceBoneBoundaries"),
			LOCTEXT("EnforceBoneBoundaries_RowNameContent", "Enforce Bone Boundaries"),
			LOCTEXT("EnforceBoneBoundaries_RowNameContentTooltip", "Penalize edge collapse between vertices that have different major bones.  This will help articulated segments like tongues but can lead to undesirable results under extreme simplification."),
			TEXT("EnforceBoneBoundaries"),
			FGetCheckBoxStateDelegate::CreateRaw(this, &FSkeletalMeshReductionSettingsLayout::GetEnforceBoneBoundaries),
			FSetCheckBoxStateDelegate::CreateRaw(this, &FSkeletalMeshReductionSettingsLayout::SetEnforceBoneBoundaries));

		AddBoolRow(ChildrenBuilder,
			LOCTEXT("MergeCoincidentVertBones_Row", "MergeCoincidentVertBones"),
			LOCTEXT("MergeCoincidentVertBones_RowNameContent", "Merge Coincident Vertices Bones"),
			LOCTEXT("MergeCoincidentVertBones_RowNameContentTooltip", "If enabled this option make sure vertices that share the same location (e.g. UV boundaries) have the same bone weights. This can fix cracks when the characters animate."),
			TEXT("MergeCoincidentVertBones"),
			FGetCheckBoxStateDelegate::CreateRaw(this, &FSkeletalMeshReductionSettingsLayout::GetMergeCoincidentVertBones),
			FSetCheckBoxStateDelegate::CreateRaw(this, &FSkeletalMeshReductionSettingsLayout::SetMergeCoincidentVertBones));

		AddFloatRow(ChildrenBuilder,
			LOCTEXT("VolumeImportance_Row", "Volume Importance"),
			LOCTEXT("VolumeImportance", "Volumetric Correction"),
			LOCTEXT("VolumeImportance_ToolTip", "Default value of 1 attempts to preserve volume.  Smaller values will loose volume by flattening curved surfaces, and larger values will accentuate curved surfaces."),
			TEXT("VolumeImportance"),
			0.0f,
			2.0f,
			FGetFloatDelegate::CreateRaw(this, &FSkeletalMeshReductionSettingsLayout::GetVolumeImportance),
			FSetFloatDelegate::CreateRaw(this, &FSkeletalMeshReductionSettingsLayout::SetVolumeImportance));

		AddBoolRow(ChildrenBuilder,
			LOCTEXT("LockEdges_Row", "LockEdges"),
			LOCTEXT("LockEdges_RowNameContent", "Lock Mesh Edges"),
			LOCTEXT("LockEdges_RowNameContentTooltip", "Preserve cuts in the mesh surface by locking vertices in place.  Increases the quality of the simplified mesh at edges at the cost of more triangles."),
			TEXT("LockEdges"),
			FGetCheckBoxStateDelegate::CreateRaw(this, &FSkeletalMeshReductionSettingsLayout::GetLockEdges),
			FSetCheckBoxStateDelegate::CreateRaw(this, &FSkeletalMeshReductionSettingsLayout::SetLockEdges));

		AddBoolRow(ChildrenBuilder,
			LOCTEXT("LockColorBoundaries_Row", "LockColorBoundaries"),
			LOCTEXT("LockColorBoundaries_RowNameContent", "Lock Vertex Color Boundaries"),
			LOCTEXT("LockColorBoundaries_RowNameContentTooltip", "Locking edges that connect two vertex colors.  Increases the quality of the simplified mesh at edges at the cost of more triangles."),
			TEXT("LockColorBoundaries"),
			FGetCheckBoxStateDelegate::CreateRaw(this, &FSkeletalMeshReductionSettingsLayout::GetLockColorBounaries),
			FSetCheckBoxStateDelegate::CreateRaw(this, &FSkeletalMeshReductionSettingsLayout::SetLockColorBounaries));

		AddBoolRow(ChildrenBuilder,
			LOCTEXT("ImproveTrianglesForCloth_Row", "ImproveTrianglesForCloth"),
			LOCTEXT("ImproveTrianglesForCloth_RowNameContent", "Improve Triangles For Cloth"),
			LOCTEXT("ImproveTrianglesForCloth_RowNameContentTooltip", "Better distribution of triangles on 2d meshes, such as flat cloth, but at the cost of potentially worse UVs in those areas.  This generally has little or no effect for mesh regions that aren't laid out on a plane intersecting the origin such as the xy-plane. When this is disabled, the planar regions may simplify to fewer large triangles."),
			TEXT("ImproveTrianglesForCloth"),
			FGetCheckBoxStateDelegate::CreateRaw(this, &FSkeletalMeshReductionSettingsLayout::GetImproveTrianglesForCloth),
			FSetCheckBoxStateDelegate::CreateRaw(this, &FSkeletalMeshReductionSettingsLayout::SetImproveTrianglesForCloth));
	}

	AddBaseLODRow(ChildrenBuilder);
}

bool FSkeletalMeshReductionSettingsLayout::IsReductionEnabled() const
{
	return IsLODSettingsEnabledDelegate.IsBound() ? IsLODSettingsEnabledDelegate.Execute(LODIndex) : false;
}

FDetailWidgetRow& FSkeletalMeshReductionSettingsLayout::AddFloatRow(IDetailChildrenBuilder& ChildrenBuilder, const FText RowTitleText, const FText RowNameContentText, const FText RowNameContentTootlipText, FName RowTag, const float MinSliderValue, const float MaxSliderValue, FGetFloatDelegate GetterDelegate, FSetFloatDelegate SetterDelegate)
{
	int32 SliderDataIndex = SliderStateDataArray.Num();
	FSliderStateData& SliderData = SliderStateDataArray.AddDefaulted_GetRef();
	SliderData.bSliderActiveMode = false;
	
	auto BeginSliderMovementHelperFunc = [GetterDelegate, SliderDataIndex, this]()
	{
		check(SliderStateDataArray.IsValidIndex(SliderDataIndex));
		SliderStateDataArray[SliderDataIndex].bSliderActiveMode = true;
		SliderStateDataArray[SliderDataIndex].MovementValueFloat = GetterDelegate.IsBound() ? GetterDelegate.Execute() : 0.0f;
	};

	auto EndSliderMovementHelperFunc = [SetterDelegate, SliderDataIndex, this](float Value)
	{
		check(SliderStateDataArray.IsValidIndex(SliderDataIndex));
		SliderStateDataArray[SliderDataIndex].bSliderActiveMode = false;
		SliderStateDataArray[SliderDataIndex].MovementValueFloat = 0.0f;
		SetterDelegate.ExecuteIfBound(Value);
	};

	auto SetValueHelperFunc = [SetterDelegate, SliderDataIndex, this](float Value)
	{
		check(SliderStateDataArray.IsValidIndex(SliderDataIndex));
		if (SliderStateDataArray[SliderDataIndex].bSliderActiveMode)
		{
			SliderStateDataArray[SliderDataIndex].MovementValueFloat = Value;
		}
		else
		{
			SetterDelegate.ExecuteIfBound(Value);
		}
	};

	auto GetValueHelperFunc = [GetterDelegate, SliderDataIndex, this]()
	{
		check(SliderStateDataArray.IsValidIndex(SliderDataIndex));
		if (SliderStateDataArray[SliderDataIndex].bSliderActiveMode)
		{
			return SliderStateDataArray[SliderDataIndex].MovementValueFloat;
		}
		return GetterDelegate.IsBound() ? GetterDelegate.Execute() : 0.0f;
	};
	
	FDetailWidgetRow& Row = ChildrenBuilder.AddCustomRow(RowTitleText)
	.RowTag(RowTag)
	.NameContent()
	[
		SNew(STextBlock)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.Text(RowNameContentText)
		.ToolTipText(RowNameContentTootlipText)
	]
	.ValueContent()
	[
		SNew(SSpinBox<float>)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.MinValue(MinSliderValue)
		.MaxValue(MaxSliderValue)
		.Value_Lambda(GetValueHelperFunc)
		.OnValueChanged_Lambda(SetValueHelperFunc)
		.OnBeginSliderMovement_Lambda(BeginSliderMovementHelperFunc)
		.OnEndSliderMovement_Lambda(EndSliderMovementHelperFunc)
		.IsEnabled(this, &FSkeletalMeshReductionSettingsLayout::IsReductionEnabled)
	];
	return Row;
}

FDetailWidgetRow& FSkeletalMeshReductionSettingsLayout::AddBoolRow(IDetailChildrenBuilder& ChildrenBuilder, const FText RowTitleText, const FText RowNameContentText, const FText RowNameContentToolitipText, FName RowTag, FGetCheckBoxStateDelegate GetterDelegate, FSetCheckBoxStateDelegate SetterDelegate)
{
	FDetailWidgetRow& Row = ChildrenBuilder.AddCustomRow(RowTitleText)
	.RowTag(RowTag)
	.NameContent()
	[
		SNew(STextBlock)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.Text(RowNameContentText)
		.ToolTipText(RowNameContentToolitipText)
	]
	.ValueContent()
	[
		SNew(SCheckBox)
		.IsChecked_Lambda([GetterDelegate]() {return GetterDelegate.IsBound() ? GetterDelegate.Execute() : ECheckBoxState::Unchecked; })
		.OnCheckStateChanged_Lambda([SetterDelegate](ECheckBoxState Value) {SetterDelegate.ExecuteIfBound(Value); })
		.IsEnabled(this, &FSkeletalMeshReductionSettingsLayout::IsReductionEnabled)
	];
	return Row;
}

FDetailWidgetRow& FSkeletalMeshReductionSettingsLayout::AddIntegerRow(IDetailChildrenBuilder& ChildrenBuilder, const FText RowTitleText, const FText RowNameContentText, const FText RowNameContentTootlipText, FName RowTag, const int32 MinSliderValue, const int32 MaxSliderValue, FGetIntegerDelegate GetterDelegate, FSetIntegerDelegate SetterDelegate)
{
	int32 SliderDataIndex = SliderStateDataArray.Num();
	FSliderStateData& SliderData = SliderStateDataArray.AddDefaulted_GetRef();
	SliderData.bSliderActiveMode = false;

	auto BeginSliderMovementHelperFunc = [GetterDelegate, SliderDataIndex, this]()
	{
		check(SliderStateDataArray.IsValidIndex(SliderDataIndex));
		SliderStateDataArray[SliderDataIndex].bSliderActiveMode = true;
		SliderStateDataArray[SliderDataIndex].MovementValueInt = GetterDelegate.IsBound() ? GetterDelegate.Execute() : 0;
	};

	auto EndSliderMovementHelperFunc = [SetterDelegate, SliderDataIndex, this](int32 Value)
	{
		check(SliderStateDataArray.IsValidIndex(SliderDataIndex));
		SliderStateDataArray[SliderDataIndex].bSliderActiveMode = false;
		SliderStateDataArray[SliderDataIndex].MovementValueInt = 0;
		SetterDelegate.ExecuteIfBound(Value);
	};

	auto SetValueHelperFunc = [SetterDelegate, SliderDataIndex, this](int32 Value)
	{
		check(SliderStateDataArray.IsValidIndex(SliderDataIndex));
		if (SliderStateDataArray[SliderDataIndex].bSliderActiveMode)
		{
			SliderStateDataArray[SliderDataIndex].MovementValueInt = Value;
		}
		else
		{
			SetterDelegate.ExecuteIfBound(Value);
		}
	};

	auto GetValueHelperFunc = [GetterDelegate, SliderDataIndex, this]()
	{
		check(SliderStateDataArray.IsValidIndex(SliderDataIndex));
		if (SliderStateDataArray[SliderDataIndex].bSliderActiveMode)
		{
			return SliderStateDataArray[SliderDataIndex].MovementValueInt;
		}
		return GetterDelegate.IsBound() ? GetterDelegate.Execute() : 0;
	};

	FDetailWidgetRow& Row = ChildrenBuilder.AddCustomRow(RowTitleText)
	.RowTag(RowTag)
	.NameContent()
	[
		SNew(STextBlock)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.Text(RowNameContentText)
		.ToolTipText(RowNameContentTootlipText)
	]
	.ValueContent()
	[
		SNew(SSpinBox<int32>)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.MinValue(MinSliderValue)
		.MaxValue(MaxSliderValue)
		.Value_Lambda(GetValueHelperFunc)
		.OnValueChanged_Lambda(SetValueHelperFunc)
		.OnBeginSliderMovement_Lambda(BeginSliderMovementHelperFunc)
		.OnEndSliderMovement_Lambda(EndSliderMovementHelperFunc)
		.IsEnabled(this, &FSkeletalMeshReductionSettingsLayout::IsReductionEnabled)
	];
	return Row;
}

FDetailWidgetRow& FSkeletalMeshReductionSettingsLayout::AddUnsignedIntegerRow(IDetailChildrenBuilder& ChildrenBuilder, const FText RowTitleText, const FText RowNameContentText, const FText RowNameContentTootlipText, FName RowTag, const uint32 MinSliderValue, const uint32 MaxSliderValue, FGetUnsignedIntegerDelegate GetterDelegate, FSetUnsignedIntegerDelegate SetterDelegate)
{
	uint32 SliderDataIndex = SliderStateDataArray.Num();
	FSliderStateData& SliderData = SliderStateDataArray.AddDefaulted_GetRef();
	SliderData.bSliderActiveMode = false;

	auto BeginSliderMovementHelperFunc = [GetterDelegate, SliderDataIndex, this]()
	{
		check(SliderStateDataArray.IsValidIndex(SliderDataIndex));
		SliderStateDataArray[SliderDataIndex].bSliderActiveMode = true;
		SliderStateDataArray[SliderDataIndex].MovementValueUnsignedInt = GetterDelegate.IsBound() ? GetterDelegate.Execute() : 0;
	};

	auto EndSliderMovementHelperFunc = [SetterDelegate, SliderDataIndex, this](int32 Value)
	{
		check(SliderStateDataArray.IsValidIndex(SliderDataIndex));
		SliderStateDataArray[SliderDataIndex].bSliderActiveMode = false;
		SliderStateDataArray[SliderDataIndex].MovementValueUnsignedInt = 0;
		SetterDelegate.ExecuteIfBound(Value);
	};

	auto SetValueHelperFunc = [SetterDelegate, SliderDataIndex, this](uint32 Value)
	{
		check(SliderStateDataArray.IsValidIndex(SliderDataIndex));
		if (SliderStateDataArray[SliderDataIndex].bSliderActiveMode)
		{
			SliderStateDataArray[SliderDataIndex].MovementValueUnsignedInt = Value;
		}
		else
		{
			SetterDelegate.ExecuteIfBound(Value);
		}
	};

	auto GetValueHelperFunc = [GetterDelegate, SliderDataIndex, this]()
	{
		check(SliderStateDataArray.IsValidIndex(SliderDataIndex));
		if (SliderStateDataArray[SliderDataIndex].bSliderActiveMode)
		{
			return SliderStateDataArray[SliderDataIndex].MovementValueUnsignedInt;
		}
		return GetterDelegate.IsBound() ? GetterDelegate.Execute() : 0;
	};

	FDetailWidgetRow& Row = ChildrenBuilder.AddCustomRow(RowTitleText)
	.RowTag(RowTag)
	.NameContent()
	[
		SNew(STextBlock)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.Text(RowNameContentText)
		.ToolTipText(RowNameContentTootlipText)
	]
	.ValueContent()
	[
		SNew(SSpinBox<uint32>)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.MinValue(MinSliderValue)
		.MaxValue(MaxSliderValue)
		.Value_Lambda(GetValueHelperFunc)
		.OnValueChanged_Lambda(SetValueHelperFunc)
		.OnBeginSliderMovement_Lambda(BeginSliderMovementHelperFunc)
		.OnEndSliderMovement_Lambda(EndSliderMovementHelperFunc)
		.IsEnabled(this, &FSkeletalMeshReductionSettingsLayout::IsReductionEnabled)
	];
	return Row;
}

void FSkeletalMeshReductionSettingsLayout::AddBaseLODRow(IDetailChildrenBuilder& ChildrenBuilder)
{
	// Only able to do this for LOD2 and above, so only show the property if this is the case
	if (LODIndex == 0)
	{
		return;
	}
	//Old workflow do not allow inline reducing of custom LOD
	int32 MaxBaseLOD = bIsLODModelbuildDataAvailable ? LODIndex : LODIndex - 1;
	{
		AddIntegerRow(ChildrenBuilder,
			LOCTEXT("ReductionBaseLOD", "Reduction_BaseLOD"),
			LOCTEXT("BaseLOD", "Base LOD"),
			LOCTEXT("BaseLODTooltip", "Base LOD index to generate this LOD. By default, we generate from LOD 0"),
			TEXT("BaseLOD"),
			0,
			MaxBaseLOD,
			FGetIntegerDelegate::CreateRaw(this, &FSkeletalMeshReductionSettingsLayout::GetBaseLODValue),
			FSetIntegerDelegate::CreateRaw(this, &FSkeletalMeshReductionSettingsLayout::SetBaseLODValue));
	}
}

void FSkeletalMeshReductionSettingsLayout::SetPercentAndAbsoluteVisibility(FDetailWidgetRow& Row, SkeletalMeshTerminationCriterion FirstCriterion, SkeletalMeshTerminationCriterion SecondCriterion)
{
	TArray< SkeletalMeshTerminationCriterion > VizList;
	VizList.Add(FirstCriterion);
	VizList.Add(SecondCriterion);
	// Hide property if using vert percentage
	Row.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FSkeletalMeshReductionSettingsLayout::ShowIfCurrentCriterionIs, VizList)));
};

TSharedRef<SWidget> FSkeletalMeshReductionSettingsLayout::FillReductionMethodMenu()
{
	if (EnumReductionMethod == nullptr)
	{
		return SNullWidget::NullWidget;
	}
	FMenuBuilder MenuBuilder(true, nullptr);
	int32 EnumCount = EnumReductionMethod->NumEnums();
	//Skip the last enum since this is the _MAX
	for (int32 EnumIndex = 0; EnumIndex < EnumCount - 1; ++EnumIndex)
	{
		FText EnumName = EnumReductionMethod->GetDisplayNameTextByIndex(EnumIndex);
		FUIAction ReductionMethodAction(FExecuteAction::CreateLambda([this, EnumIndex]()
		{
			ReductionSettings.ReductionMethod = (SkeletalMeshOptimizationType)EnumReductionMethod->GetValueByIndex(EnumIndex);
		}));
		MenuBuilder.AddMenuEntry(EnumName, FText::GetEmpty(), FSlateIcon(), ReductionMethodAction);
	}
	return MenuBuilder.MakeWidget();
}

FText FSkeletalMeshReductionSettingsLayout::GetReductionMethodText() const
{
	if (EnumReductionMethod == nullptr)
	{
		return FText::GetEmpty();
	}
	return EnumReductionMethod->GetDisplayNameTextByValue(ReductionSettings.ReductionMethod);
}

TSharedRef<SWidget> FSkeletalMeshReductionSettingsLayout::FillReductionImportanceMenu(const EImportanceType Importance)
{
	if (EnumImportance == nullptr)
	{
		return SNullWidget::NullWidget;
	}
	FMenuBuilder MenuBuilder(true, nullptr);
	int32 EnumCount = EnumImportance->NumEnums();
	//Skip the last enum since this is the _MAX
	for (int32 EnumIndex = 0; EnumIndex < EnumCount - 1; ++EnumIndex)
	{
		FText EnumName = EnumImportance->GetDisplayNameTextByIndex(EnumIndex);
		FUIAction ReductionMethodAction(FExecuteAction::CreateLambda([this, EnumIndex, Importance]()
		{
			switch(Importance)
			{ 
			case ID_Silhouette:
				ReductionSettings.SilhouetteImportance = (SkeletalMeshOptimizationImportance)EnumImportance->GetValueByIndex(EnumIndex);
				break;
			case ID_Texture:
				ReductionSettings.TextureImportance = (SkeletalMeshOptimizationImportance)EnumImportance->GetValueByIndex(EnumIndex);
				break;
			case ID_Shading:
				ReductionSettings.ShadingImportance = (SkeletalMeshOptimizationImportance)EnumImportance->GetValueByIndex(EnumIndex);
				break;
			case ID_Skinning:
				ReductionSettings.SkinningImportance= (SkeletalMeshOptimizationImportance)EnumImportance->GetValueByIndex(EnumIndex);
				break;
			}
		}));
		MenuBuilder.AddMenuEntry(EnumName, FText::GetEmpty(), FSlateIcon(), ReductionMethodAction);
	}
	return MenuBuilder.MakeWidget();
}

FText FSkeletalMeshReductionSettingsLayout::GetReductionImportanceText(const EImportanceType Importance) const
{
	if (EnumImportance == nullptr)
	{
		return FText::GetEmpty();
	}
	switch (Importance)
	{
	case ID_Silhouette:
		return EnumImportance->GetDisplayNameTextByValue(ReductionSettings.SilhouetteImportance);
	case ID_Texture:
		return EnumImportance->GetDisplayNameTextByValue(ReductionSettings.TextureImportance);
	case ID_Shading:
		return EnumImportance->GetDisplayNameTextByValue(ReductionSettings.ShadingImportance);
	case ID_Skinning:
		return EnumImportance->GetDisplayNameTextByValue(ReductionSettings.SkinningImportance);
	}
	return FText::GetEmpty();
}

TSharedRef<class SWidget> FSkeletalMeshReductionSettingsLayout::FillReductionTerminationCriterionMenu()
{
	if (EnumTerminationCriterion == nullptr)
	{
		return SNullWidget::NullWidget;
	}
	FMenuBuilder MenuBuilder(true, nullptr);
	int32 EnumCount = EnumTerminationCriterion->NumEnums();
	//Skip the last enum since this is the _MAX
	for (int32 EnumIndex = 0; EnumIndex < EnumCount - 1; ++EnumIndex)
	{
		FText EnumName = EnumTerminationCriterion->GetDisplayNameTextByIndex(EnumIndex);
		FUIAction ReductionAction(FExecuteAction::CreateLambda([this, EnumIndex]()
		{
			FText TransactionText = FText::Format(LOCTEXT("PersonaReductionChangedSetTerminationCriterionLOD", "LOD{0} reduction settings: termination criterion changed"), LODIndex);
			FScopedTransaction Transaction(TransactionText);
			ModifyMeshLODSettingsDelegate.ExecuteIfBound(LODIndex);

			ReductionSettings.TerminationCriterion = (SkeletalMeshTerminationCriterion)EnumTerminationCriterion->GetValueByIndex(EnumIndex);
		}));
		MenuBuilder.AddMenuEntry(EnumName, FText::GetEmpty(), FSlateIcon(), ReductionAction);
	}
	return MenuBuilder.MakeWidget();
}

FText FSkeletalMeshReductionSettingsLayout::GetReductionTerminationCriterionText() const
{
	if (EnumTerminationCriterion == nullptr)
	{
		return FText::GetEmpty();
	}
	return EnumTerminationCriterion->GetDisplayNameTextByValue(ReductionSettings.TerminationCriterion);
}

bool FSkeletalMeshReductionSettingsLayout::UseNativeReductionTool() const
{
	if (IMeshReduction* SkeletalReductionModule = FModuleManager::Get().LoadModuleChecked<IMeshReductionManagerModule>("MeshReductionInterface").GetSkeletalMeshReductionInterface())
	{
		FString ModuleVersionString = SkeletalReductionModule->GetVersionString();

		TArray<FString> SplitVersionString;
		ModuleVersionString.ParseIntoArray(SplitVersionString, TEXT("_"), true);
		return SplitVersionString[0].Equals("QuadricSkeletalMeshReduction");
	}
	return false;
}

EVisibility FSkeletalMeshReductionSettingsLayout::GetVisibiltyIfCurrentReductionMethodIsNot(SkeletalMeshOptimizationType ReductionType) const
{
	if (ReductionSettings.ReductionMethod != ReductionType)
	{
		return EVisibility::Visible;
	}
	return EVisibility::Hidden;
}

EVisibility FSkeletalMeshReductionSettingsLayout::ShowIfCurrentCriterionIs(TArray<SkeletalMeshTerminationCriterion> TerminationCriterionArray) const
{
	if (TerminationCriterionArray.Contains(ReductionSettings.TerminationCriterion))
	{
		return EVisibility::Visible;
	}
	return EVisibility::Hidden;
}

float FSkeletalMeshReductionSettingsLayout::GetNumTrianglesPercentage() const
{
	return ReductionSettings.NumOfTrianglesPercentage;
}

void FSkeletalMeshReductionSettingsLayout::SetNumTrianglesPercentage(float Value)
{
	if (ReductionSettings.NumOfTrianglesPercentage != Value)
	{
		FText TransactionText = FText::Format(LOCTEXT("PersonaReductionChangedSetNumTrianglePercentLOD", "LOD{0} reduction settings: percent of triangles changed"), LODIndex);
		FScopedTransaction Transaction(TransactionText);
		ModifyMeshLODSettingsDelegate.ExecuteIfBound(LODIndex);

		ReductionSettings.NumOfTrianglesPercentage = Value;
	}
}

float FSkeletalMeshReductionSettingsLayout::GetNumVerticesPercentage() const
{
	return ReductionSettings.NumOfVertPercentage;
}

void FSkeletalMeshReductionSettingsLayout::SetNumVerticesPercentage(float Value)
{
	if (ReductionSettings.NumOfVertPercentage != Value)
	{
		FText TransactionText = FText::Format(LOCTEXT("PersonaReductionChangedSetNumVerticePercentLOD", "LOD{0} reduction settings: percent of vertices changed"), LODIndex);
		FScopedTransaction Transaction(TransactionText);
		ModifyMeshLODSettingsDelegate.ExecuteIfBound(LODIndex);

		ReductionSettings.NumOfVertPercentage = Value;
	}
}

int32 FSkeletalMeshReductionSettingsLayout::GetNumMaxTrianglesCount() const
{
	return ReductionSettings.MaxNumOfTriangles;
}

void FSkeletalMeshReductionSettingsLayout::SetNumMaxTrianglesCount(int32 Value)
{
	if (ReductionSettings.MaxNumOfTriangles != Value)
	{
		FText TransactionText = FText::Format(LOCTEXT("PersonaReductionChangedSetMaxTriangleCountLOD", "LOD{0} reduction settings: max triangles count changed"), LODIndex);
		FScopedTransaction Transaction(TransactionText);
		ModifyMeshLODSettingsDelegate.ExecuteIfBound(LODIndex);

		ReductionSettings.MaxNumOfTriangles = Value;
	}
}

int32 FSkeletalMeshReductionSettingsLayout::GetNumMaxVerticesCount() const
{
	return ReductionSettings.MaxNumOfVerts;
}

void FSkeletalMeshReductionSettingsLayout::SetNumMaxVerticesCount(int32 Value)
{
	if (ReductionSettings.MaxNumOfVerts != Value)
	{
		FText TransactionText = FText::Format(LOCTEXT("PersonaReductionChangedSetMaxVertexCountLOD", "LOD{0} reduction settings: max vertex count changed"), LODIndex);
		FScopedTransaction Transaction(TransactionText);
		ModifyMeshLODSettingsDelegate.ExecuteIfBound(LODIndex);

		ReductionSettings.MaxNumOfVerts = Value;
	}
}

uint32 FSkeletalMeshReductionSettingsLayout::GetNumMaxTrianglesPercentageCount() const
{
	return ReductionSettings.MaxNumOfTrianglesPercentage;
}

void FSkeletalMeshReductionSettingsLayout::SetNumMaxTrianglesPercentageCount(uint32 Value)
{
	if (ReductionSettings.MaxNumOfTrianglesPercentage != Value)
	{
		FText TransactionText = FText::Format(LOCTEXT("PersonaReductionChangedSetMaxTrianglePercentageCountLOD", "LOD{0} reduction settings: max triangles percentage count changed"), LODIndex);
		FScopedTransaction Transaction(TransactionText);
		ModifyMeshLODSettingsDelegate.ExecuteIfBound(LODIndex);

		ReductionSettings.MaxNumOfTrianglesPercentage = Value;
	}
}

uint32 FSkeletalMeshReductionSettingsLayout::GetNumMaxVerticesPercentageCount() const
{
	return ReductionSettings.MaxNumOfVertsPercentage;
}

void FSkeletalMeshReductionSettingsLayout::SetNumMaxVerticesPercentageCount(uint32 Value)
{
	if (ReductionSettings.MaxNumOfVertsPercentage != Value)
	{
		FText TransactionText = FText::Format(LOCTEXT("PersonaReductionChangedSetMaxVertexPercentageCountLOD", "LOD{0} reduction settings: max vertex percentage count changed"), LODIndex);
		FScopedTransaction Transaction(TransactionText);
		ModifyMeshLODSettingsDelegate.ExecuteIfBound(LODIndex);

		ReductionSettings.MaxNumOfVertsPercentage = Value;
	}
}

float FSkeletalMeshReductionSettingsLayout::GetAccuracyPercentage() const
{
	return ReductionSettings.MaxDeviationPercentage;
}
void FSkeletalMeshReductionSettingsLayout::SetAccuracyPercentage(float Value)
{
	if (ReductionSettings.MaxDeviationPercentage != Value)
	{
		FText TransactionText = FText::Format(LOCTEXT("PersonaReductionChangedSetAccuracyPercentageLOD", "LOD{0} reduction settings: accuracy percentage changed"), LODIndex);
		FScopedTransaction Transaction(TransactionText);
		ModifyMeshLODSettingsDelegate.ExecuteIfBound(LODIndex);

		ReductionSettings.MaxDeviationPercentage = Value;
	}
}

ECheckBoxState FSkeletalMeshReductionSettingsLayout::ShouldRecomputeNormals() const
{
	return ReductionSettings.bRecalcNormals ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void FSkeletalMeshReductionSettingsLayout::OnRecomputeNormalsChanged(ECheckBoxState NewState)
{
	FText TransactionText = FText::Format(LOCTEXT("PersonaReductionChangedOnComputeNormalsLOD", "LOD{0} reduction settings: recompute normals changed"), LODIndex);
	FScopedTransaction Transaction(TransactionText);
	ModifyMeshLODSettingsDelegate.ExecuteIfBound(LODIndex);

	ReductionSettings.bRecalcNormals = (NewState == ECheckBoxState::Checked) ? true : false;
}

float FSkeletalMeshReductionSettingsLayout::GetNormalsThreshold() const
{
	return ReductionSettings.NormalsThreshold;
}

void FSkeletalMeshReductionSettingsLayout::SetNormalsThreshold(float Value)
{
	if (ReductionSettings.NormalsThreshold != Value)
	{
		FText TransactionText = FText::Format(LOCTEXT("PersonaReductionChangedSetNormalsThresholdLOD", "LOD{0} reduction settings: normals threshold changed"), LODIndex);
		FScopedTransaction Transaction(TransactionText);
		ModifyMeshLODSettingsDelegate.ExecuteIfBound(LODIndex);

		ReductionSettings.NormalsThreshold = Value;
	}
}

float FSkeletalMeshReductionSettingsLayout::GetWeldingThreshold() const
{
	return ReductionSettings.WeldingThreshold;
}

void FSkeletalMeshReductionSettingsLayout::SetWeldingThreshold(float Value)
{
	if (ReductionSettings.WeldingThreshold != Value)
	{
		FText TransactionText = FText::Format(LOCTEXT("PersonaReductionChangedSetWeldingThresholdLOD", "LOD{0} reduction settings: Welding threshold changed"), LODIndex);
		FScopedTransaction Transaction(TransactionText);
		ModifyMeshLODSettingsDelegate.ExecuteIfBound(LODIndex);

		ReductionSettings.WeldingThreshold = Value;
	}
}

ECheckBoxState FSkeletalMeshReductionSettingsLayout::GetLockEdges() const
{
	return ReductionSettings.bLockEdges ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void FSkeletalMeshReductionSettingsLayout::SetLockEdges(ECheckBoxState NewState)
{
	FText TransactionText = FText::Format(LOCTEXT("PersonaReductionChangedSetLockEdgesLOD", "LOD{0} reduction settings: lock edges changed"), LODIndex);
	FScopedTransaction Transaction(TransactionText);
	ModifyMeshLODSettingsDelegate.ExecuteIfBound(LODIndex);

	ReductionSettings.bLockEdges = (NewState == ECheckBoxState::Checked) ? true : false;
}

ECheckBoxState FSkeletalMeshReductionSettingsLayout::GetLockColorBounaries() const
{
	return ReductionSettings.bLockColorBounaries ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void FSkeletalMeshReductionSettingsLayout::SetLockColorBounaries(ECheckBoxState NewState)
{
	FText TransactionText = FText::Format(LOCTEXT("PersonaReductionChangedSetLockColorBounariesLOD", "LOD{0} reduction settings: lock color boundaries changed"), LODIndex);
	FScopedTransaction Transaction(TransactionText);
	ModifyMeshLODSettingsDelegate.ExecuteIfBound(LODIndex);

	ReductionSettings.bLockColorBounaries = (NewState == ECheckBoxState::Checked) ? true : false;
}

ECheckBoxState FSkeletalMeshReductionSettingsLayout::GetImproveTrianglesForCloth() const
{
	return ReductionSettings.bImproveTrianglesForCloth ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void FSkeletalMeshReductionSettingsLayout::SetImproveTrianglesForCloth(ECheckBoxState NewState)
{
	FText TransactionText = FText::Format(LOCTEXT("PersonaReductionChangedSetUseLegacyVersionLOD", "LOD{0} reduction settings: improve triangles for cloth changed"), LODIndex);
	FScopedTransaction Transaction(TransactionText);
	ModifyMeshLODSettingsDelegate.ExecuteIfBound(LODIndex);

	ReductionSettings.bImproveTrianglesForCloth = (NewState == ECheckBoxState::Checked) ? true : false;
}

ECheckBoxState FSkeletalMeshReductionSettingsLayout::GetEnforceBoneBoundaries() const
{
	return ReductionSettings.bEnforceBoneBoundaries ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void FSkeletalMeshReductionSettingsLayout::SetEnforceBoneBoundaries(ECheckBoxState NewState)
{
	FText TransactionText = FText::Format(LOCTEXT("PersonaReductionChangedSetEnforceBoneBoundariesLOD", "LOD{0} reduction settings: enforce bone boundaries changed"), LODIndex);
	FScopedTransaction Transaction(TransactionText);
	ModifyMeshLODSettingsDelegate.ExecuteIfBound(LODIndex);

	ReductionSettings.bEnforceBoneBoundaries = (NewState == ECheckBoxState::Checked) ? true : false;
}

ECheckBoxState FSkeletalMeshReductionSettingsLayout::GetMergeCoincidentVertBones() const
{
	return ReductionSettings.bMergeCoincidentVertBones ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void FSkeletalMeshReductionSettingsLayout::SetMergeCoincidentVertBones(ECheckBoxState NewState)
{
	FText TransactionText = FText::Format(LOCTEXT("PersonaReductionChangedSetMergeCoincidentVertBones", "LOD{0} reduction settings: Merge Coincident Vertices Bones changed"), LODIndex);
	FScopedTransaction Transaction(TransactionText);
	ModifyMeshLODSettingsDelegate.ExecuteIfBound(LODIndex);

	ReductionSettings.bMergeCoincidentVertBones = (NewState == ECheckBoxState::Checked) ? true : false;
}

float FSkeletalMeshReductionSettingsLayout::GetVolumeImportance() const
{
	return ReductionSettings.VolumeImportance;
}

void FSkeletalMeshReductionSettingsLayout::SetVolumeImportance(float Value)
{
	if (ReductionSettings.VolumeImportance != Value)
	{
		FText TransactionText = FText::Format(LOCTEXT("PersonaReductionChangedSetVolumeImportanceLOD", "LOD{0} reduction settings: volume importance changed"), LODIndex);
		FScopedTransaction Transaction(TransactionText);
		ModifyMeshLODSettingsDelegate.ExecuteIfBound(LODIndex);

		ReductionSettings.VolumeImportance = Value;
	}
}

ECheckBoxState FSkeletalMeshReductionSettingsLayout::GetRemapMorphTargets() const
{
	return ReductionSettings.bRemapMorphTargets ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void FSkeletalMeshReductionSettingsLayout::SetRemapMorphTargets(ECheckBoxState NewState)
{
	FText TransactionText = FText::Format(LOCTEXT("PersonaReductionChangedSetRemapMorphTargetsLOD", "LOD{0} reduction settings: remap morph targets changed"), LODIndex);
	FScopedTransaction Transaction(TransactionText);
	ModifyMeshLODSettingsDelegate.ExecuteIfBound(LODIndex);

	ReductionSettings.bRemapMorphTargets = (NewState == ECheckBoxState::Checked) ? true : false;
}

int32 FSkeletalMeshReductionSettingsLayout::GetMaxBonesPerVertex() const
{
	return ReductionSettings.MaxBonesPerVertex;
}

void FSkeletalMeshReductionSettingsLayout::SetMaxBonesPerVertex(int32 Value)
{
	FText TransactionText = FText::Format(LOCTEXT("PersonaReductionChangedSetMaxBonesPerVertexLOD", "LOD{0} reduction settings: max bones per vertex changed"), LODIndex);
	FScopedTransaction Transaction(TransactionText);
	ModifyMeshLODSettingsDelegate.ExecuteIfBound(LODIndex);

	//Cannot set a value lower then 1
	ReductionSettings.MaxBonesPerVertex = FMath::Max(1, Value);
}

//////////////////////////////////////////////////////////////////////////
// FSkeletalMeshBuildSettingsLayout implementation

FSkeletalMeshBuildSettingsLayout::FSkeletalMeshBuildSettingsLayout(FSkeletalMeshBuildSettings& InBuildSettings, int32 InLODIndex, FIsLODSettingsEnabledDelegate InIsBuildSettingsEnabledDelegate, FModifyMeshLODSettingsDelegate InModifyMeshLODSettingsDelegate)
	: BuildSettings(InBuildSettings)
	, LODIndex(InLODIndex)
	, IsBuildSettingsEnabledDelegate(InIsBuildSettingsEnabledDelegate)
	, ModifyMeshLODSettingsDelegate(InModifyMeshLODSettingsDelegate)
{
	//Make sure apply is bound, this class mean nothing if apply is not bound
	check(IsBuildSettingsEnabledDelegate.IsBound());
}

/** IDetailCustomNodeBuilder Interface*/
void FSkeletalMeshBuildSettingsLayout::GenerateHeaderRowContent(FDetailWidgetRow& NodeRow)
{
	NodeRow.NameContent()
	[
		SNew( STextBlock )
		.Text( LOCTEXT("SkeletalMeshBuildSettings", "Build Settings") )
		.Font( IDetailLayoutBuilder::GetDetailFont() )
	];
}

void FSkeletalMeshBuildSettingsLayout::GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder)
{
	{
		ChildrenBuilder.AddCustomRow( LOCTEXT("RecomputeNormals", "Recompute Normals") )
		.NameContent()
		[
			SNew(STextBlock)
			.Font( IDetailLayoutBuilder::GetDetailFont() )
			.Text(LOCTEXT("RecomputeNormals", "Recompute Normals"))
			.IsEnabled(this, &FSkeletalMeshBuildSettingsLayout::IsBuildEnabled)
		
		]
		.ValueContent()
		[
			SNew(SCheckBox)
			.IsChecked(this, &FSkeletalMeshBuildSettingsLayout::ShouldRecomputeNormals)
			.OnCheckStateChanged(this, &FSkeletalMeshBuildSettingsLayout::OnRecomputeNormalsChanged)
			.IsEnabled(this, &FSkeletalMeshBuildSettingsLayout::IsBuildEnabled)
		];
	}

	{
		ChildrenBuilder.AddCustomRow( LOCTEXT("RecomputeTangents", "Recompute Tangents") )
		.NameContent()
		[
			SNew(STextBlock)
			.Font( IDetailLayoutBuilder::GetDetailFont() )
			.Text(LOCTEXT("RecomputeTangents", "Recompute Tangents"))
			.IsEnabled(this, &FSkeletalMeshBuildSettingsLayout::IsBuildEnabled)
		]
		.ValueContent()
		[
			SNew(SCheckBox)
			.IsChecked(this, &FSkeletalMeshBuildSettingsLayout::ShouldRecomputeTangents)
			.OnCheckStateChanged(this, &FSkeletalMeshBuildSettingsLayout::OnRecomputeTangentsChanged)
			.IsEnabled(this, &FSkeletalMeshBuildSettingsLayout::IsBuildEnabled)
		];
	}

	{
		ChildrenBuilder.AddCustomRow( LOCTEXT("UseMikkTSpace", "Use MikkTSpace Tangent Space") )
		.NameContent()
		[
			SNew(STextBlock)
			.Font( IDetailLayoutBuilder::GetDetailFont() )
			.Text(LOCTEXT("UseMikkTSpace", "Use MikkTSpace Tangent Space"))
			.IsEnabled(this, &FSkeletalMeshBuildSettingsLayout::IsBuildEnabled)
		]
		.ValueContent()
		[
			SNew(SCheckBox)
			.IsChecked(this, &FSkeletalMeshBuildSettingsLayout::ShouldUseMikkTSpace)
			.OnCheckStateChanged(this, &FSkeletalMeshBuildSettingsLayout::OnUseMikkTSpaceChanged)
			.IsEnabled(this, &FSkeletalMeshBuildSettingsLayout::IsBuildEnabled)
		];
	}

	{
		ChildrenBuilder.AddCustomRow( LOCTEXT("ComputeWeightedNormals", "Compute Weighted normals") )
		.NameContent()
		[
			SNew(STextBlock)
			.Font( IDetailLayoutBuilder::GetDetailFont() )
			.Text(LOCTEXT("ComputeWeightedNormals", "Compute Weighted normals"))
			.IsEnabled(this, &FSkeletalMeshBuildSettingsLayout::IsBuildEnabled)
		]
		.ValueContent()
		[
			SNew(SCheckBox)
			.IsChecked(this, &FSkeletalMeshBuildSettingsLayout::ShouldComputeWeightedNormals)
			.OnCheckStateChanged(this, &FSkeletalMeshBuildSettingsLayout::OnComputeWeightedNormalsChanged)
			.IsEnabled(this, &FSkeletalMeshBuildSettingsLayout::IsBuildEnabled)
		];
	}

	{
		ChildrenBuilder.AddCustomRow( LOCTEXT("RemoveDegenerates", "Remove Degenerates") )
		.NameContent()
		[
			SNew(STextBlock)
			.Font( IDetailLayoutBuilder::GetDetailFont() )
			.Text(LOCTEXT("RemoveDegenerates", "Remove Degenerates"))
			.IsEnabled(this, &FSkeletalMeshBuildSettingsLayout::IsBuildEnabled)
		]
		.ValueContent()
		[
			SNew(SCheckBox)
			.IsChecked(this, &FSkeletalMeshBuildSettingsLayout::ShouldRemoveDegenerates)
			.OnCheckStateChanged(this, &FSkeletalMeshBuildSettingsLayout::OnRemoveDegeneratesChanged)
			.IsEnabled(this, &FSkeletalMeshBuildSettingsLayout::IsBuildEnabled)
		];
	}

	{
		ChildrenBuilder.AddCustomRow(LOCTEXT("UseHighPrecisionTangentBasis", "Use High Precision Tangent Basis"))
		.NameContent()
		[
			SNew(STextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(LOCTEXT("UseHighPrecisionTangentBasis", "Use High Precision Tangent Basis"))
			.IsEnabled(this, &FSkeletalMeshBuildSettingsLayout::IsBuildEnabled)
		]
		.ValueContent()
		[
			SNew(SCheckBox)
			.IsChecked(this, &FSkeletalMeshBuildSettingsLayout::ShouldUseHighPrecisionTangentBasis)
			.OnCheckStateChanged(this, &FSkeletalMeshBuildSettingsLayout::OnUseHighPrecisionTangentBasisChanged)
			.IsEnabled(this, &FSkeletalMeshBuildSettingsLayout::IsBuildEnabled)
		];
	}

	{
		ChildrenBuilder.AddCustomRow(LOCTEXT("UseHighPrecisionSkinWeights", "Use High Precision Skin Weights"))
		.NameContent()
		[
			SNew(STextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(LOCTEXT("UseHighPrecisionSkinWeights", "Use High Precision Skin Weights"))
			.IsEnabled(this, &FSkeletalMeshBuildSettingsLayout::IsBuildEnabled)
		]
		.ValueContent()
		[
			SNew(SCheckBox)
			.IsChecked(this, &FSkeletalMeshBuildSettingsLayout::ShouldUseHighPrecisionSkinWeights)
			.OnCheckStateChanged(this, &FSkeletalMeshBuildSettingsLayout::OnUseHighPrecisionSkinWeightsChanged)
			.IsEnabled(this, &FSkeletalMeshBuildSettingsLayout::IsBuildEnabled)
		];
	}

	{
		ChildrenBuilder.AddCustomRow( LOCTEXT("UseFullPrecisionUVs", "Use Full Precision UVs") )
		.NameContent()
		[
			SNew(STextBlock)
			.Font( IDetailLayoutBuilder::GetDetailFont() )
			.Text(LOCTEXT("UseFullPrecisionUVs", "Use Full Precision UVs"))
			.IsEnabled(this, &FSkeletalMeshBuildSettingsLayout::IsBuildEnabled)
		]
		.ValueContent()
		[
			SNew(SCheckBox)
			.IsChecked(this, &FSkeletalMeshBuildSettingsLayout::ShouldUseFullPrecisionUVs)
			.OnCheckStateChanged(this, &FSkeletalMeshBuildSettingsLayout::OnUseFullPrecisionUVsChanged)
			.IsEnabled(this, &FSkeletalMeshBuildSettingsLayout::IsBuildEnabled)
		];
	}
	
	{
		ChildrenBuilder.AddCustomRow( LOCTEXT("UseBackwardsCompatibleF16TruncUVs", "UE4 Compatible UVs") )
		.NameContent()
		[
			SNew(STextBlock)
			.Font( IDetailLayoutBuilder::GetDetailFont() )
			.Text(LOCTEXT("UseBackwardsCompatibleF16TruncUVs", "UE4 Compatible UVs"))
			.IsEnabled(this, &FSkeletalMeshBuildSettingsLayout::IsBuildEnabled)
		]
		.ValueContent()
		[
			SNew(SCheckBox)
			.IsChecked(this, &FSkeletalMeshBuildSettingsLayout::ShouldUseBackwardsCompatibleF16TruncUVs)
			.OnCheckStateChanged(this, &FSkeletalMeshBuildSettingsLayout::OnUseBackwardsCompatibleF16TruncUVsChanged)
			.IsEnabled(this, &FSkeletalMeshBuildSettingsLayout::IsBuildEnabled)
		];
	}

	{
		ChildrenBuilder.AddCustomRow( LOCTEXT("OptimizeForInstancing", "Optimize Sections for Instancing (Required for GPU Instance LOD)") )
		.NameContent()
		[
			SNew(STextBlock)
			.Font( IDetailLayoutBuilder::GetDetailFont() )
			.Text(LOCTEXT("OptimizeForInstancing", "Optimize Sections for Instancing (Required for GPU Instance LOD)"))
			.IsEnabled(this, &FSkeletalMeshBuildSettingsLayout::IsBuildEnabled)
		]
		.ValueContent()
		[
			SNew(SCheckBox)
			.IsChecked(this, &FSkeletalMeshBuildSettingsLayout::ShouldOptimizeForInstancing)
			.OnCheckStateChanged(this, &FSkeletalMeshBuildSettingsLayout::OnOptimizeForInstancingChanged)
			.IsEnabled(this, &FSkeletalMeshBuildSettingsLayout::IsBuildEnabled)
		];
	}

	{
		AddFloatRow(ChildrenBuilder,
			LOCTEXT("PositionThreshold_Row", "ThresholdPositionRow"),
			LOCTEXT("PositionThreshold", "Threshold Position"),
			LOCTEXT("PositionThreshold_ToolTip", "Threshold to compare vertex position equality."),
			0.0f,
			10000.0f,
			FGetFloatDelegate::CreateRaw(this, &FSkeletalMeshBuildSettingsLayout::GetThresholdPosition),
			FSetFloatDelegate::CreateRaw(this, &FSkeletalMeshBuildSettingsLayout::SetThresholdPosition));
	}

	{
		AddFloatRow(ChildrenBuilder,
			LOCTEXT("TangentNormalThreshold_Row", "ThresholdTangentNormalRow"),
			LOCTEXT("TangentNormalThreshold", "Threshold Tangent and Normal"),
			LOCTEXT("TangentNormalThreshold_ToolTip", "Threshold to compare normal, tangent or bi-normal equality."),
			0.0f,
			1.0f,
			FGetFloatDelegate::CreateRaw(this, &FSkeletalMeshBuildSettingsLayout::GetThresholdTangentNormal),
			FSetFloatDelegate::CreateRaw(this, &FSkeletalMeshBuildSettingsLayout::SetThresholdTangentNormal));
	}

	{
		AddFloatRow(ChildrenBuilder,
			LOCTEXT("UVThreshold_Row", "ThresholdUVRow"),
			LOCTEXT("UVThreshold", "Threshold UV"),
			LOCTEXT("UVThreshold_ToolTip", "Threshold to compare vertex UV equality."),
			0.0f,
			1.0f,
			FGetFloatDelegate::CreateRaw(this, &FSkeletalMeshBuildSettingsLayout::GetThresholdUV),
			FSetFloatDelegate::CreateRaw(this, &FSkeletalMeshBuildSettingsLayout::SetThresholdUV));
	}

	{
		AddFloatRow(ChildrenBuilder,
			LOCTEXT("MorphPositionThreshold_Row", "MorphThresholdPositionRow"),
			LOCTEXT("MorphPositionThreshold", "Morph Threshold Position"),
			LOCTEXT("MorphPositionThreshold_ToolTip", "Threshold to compare vertex position equality when computing morph target deltas."),
			0.0f,
			1.0f,
			FGetFloatDelegate::CreateRaw(this, &FSkeletalMeshBuildSettingsLayout::GetMorphThresholdPosition),
			FSetFloatDelegate::CreateRaw(this, &FSkeletalMeshBuildSettingsLayout::SetMorphThresholdPosition));
	}

	{
		AddIntegerRow(ChildrenBuilder,
			LOCTEXT("BoneInfluenceLimit_Row", "BoneInfluenceLimitRow"),
			LOCTEXT("BoneInfluenceLimit", "Bone Influence Limit"),
			LOCTEXT("BoneInfluenceLimit_ToolTip", "Limit the number of bone influences a vertex can have. If 0, the Default Bone Influence Limit from the project settings will be used."),
			TEXT("BoneInfluenceLimit"),
			0,
			MAX_TOTAL_INFLUENCES,
			FGetIntegerDelegate::CreateRaw(this, &FSkeletalMeshBuildSettingsLayout::GetBoneInfluenceLimit),
			FSetIntegerDelegate::CreateRaw(this, &FSkeletalMeshBuildSettingsLayout::SetBoneInfluenceLimit));
	}
}

bool FSkeletalMeshBuildSettingsLayout::IsBuildEnabled() const
{
	return IsBuildSettingsEnabledDelegate.IsBound() ? IsBuildSettingsEnabledDelegate.Execute(LODIndex) : false;
}

FDetailWidgetRow& FSkeletalMeshBuildSettingsLayout::AddFloatRow(IDetailChildrenBuilder& ChildrenBuilder, const FText RowTitleText, const FText RowNameContentText, const FText RowNameContentTootlipText, const float MinSliderValue, const float MaxSliderValue, FGetFloatDelegate GetterDelegate, FSetFloatDelegate SetterDelegate)
{
	int32 SliderDataIndex = SliderStateDataArray.Num();
	FSliderStateData& SliderData = SliderStateDataArray.AddDefaulted_GetRef();
	SliderData.bSliderActiveMode = false;

	auto BeginSliderMovementHelperFunc = [GetterDelegate, SliderDataIndex, this]()
	{
		check(SliderStateDataArray.IsValidIndex(SliderDataIndex));
		SliderStateDataArray[SliderDataIndex].bSliderActiveMode = true;
		SliderStateDataArray[SliderDataIndex].MovementValueFloat = GetterDelegate.IsBound() ? GetterDelegate.Execute() : 0.0f;
	};

	auto EndSliderMovementHelperFunc = [SetterDelegate, SliderDataIndex, this](float Value)
	{
		check(SliderStateDataArray.IsValidIndex(SliderDataIndex));
		SliderStateDataArray[SliderDataIndex].bSliderActiveMode = false;
		SliderStateDataArray[SliderDataIndex].MovementValueFloat = 0.0f;
		SetterDelegate.ExecuteIfBound(Value);
	};

	auto SetValueHelperFunc = [SetterDelegate, SliderDataIndex, this](float Value)
	{
		check(SliderStateDataArray.IsValidIndex(SliderDataIndex));
		if (SliderStateDataArray[SliderDataIndex].bSliderActiveMode)
		{
			SliderStateDataArray[SliderDataIndex].MovementValueFloat = Value;
		}
		else
		{
			SetterDelegate.ExecuteIfBound(Value);
		}
	};

	auto GetValueHelperFunc = [GetterDelegate, SliderDataIndex, this]()
	{
		check(SliderStateDataArray.IsValidIndex(SliderDataIndex));
		if (SliderStateDataArray[SliderDataIndex].bSliderActiveMode)
		{
			return SliderStateDataArray[SliderDataIndex].MovementValueFloat;
		}
		return GetterDelegate.IsBound() ? GetterDelegate.Execute() : 0.0f;
	};

	FDetailWidgetRow& Row = ChildrenBuilder.AddCustomRow(RowTitleText)
	.NameContent()
	[
		SNew(STextBlock)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.Text(RowNameContentText)
		.ToolTipText(RowNameContentTootlipText)
	]
	.ValueContent()
	[
		SNew(SSpinBox<float>)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.MinValue(MinSliderValue)
		.MaxValue(MaxSliderValue)
		.Value_Lambda(GetValueHelperFunc)
		.OnValueChanged_Lambda(SetValueHelperFunc)
		.OnBeginSliderMovement_Lambda(BeginSliderMovementHelperFunc)
		.OnEndSliderMovement_Lambda(EndSliderMovementHelperFunc)
		.IsEnabled(this, &FSkeletalMeshBuildSettingsLayout::IsBuildEnabled)
	];
	return Row;
}

FDetailWidgetRow& FSkeletalMeshBuildSettingsLayout::AddIntegerRow(
	IDetailChildrenBuilder& ChildrenBuilder,
	const FText& RowTitleText,
	const FText& RowNameContentText,
	const FText& RowNameContentTooltipText,
	FName RowTag,
	const int32 MinSliderValue,
	const int32 MaxSliderValue,
	const FGetIntegerDelegate& GetterDelegate,
	const FSetIntegerDelegate& SetterDelegate)
{
	const int32 SliderDataIndex = SliderStateDataArray.Num();
	FSliderStateData& SliderData = SliderStateDataArray.AddDefaulted_GetRef();
	SliderData.bSliderActiveMode = false;

	auto BeginSliderMovementHelperFunc = [GetterDelegate, SliderDataIndex, this]()
	{
		check(SliderStateDataArray.IsValidIndex(SliderDataIndex));
		SliderStateDataArray[SliderDataIndex].bSliderActiveMode = true;
		SliderStateDataArray[SliderDataIndex].MovementValueInt = GetterDelegate.IsBound() ? GetterDelegate.Execute() : 0;
	};

	auto EndSliderMovementHelperFunc = [SetterDelegate, SliderDataIndex, this](int32 Value)
	{
		check(SliderStateDataArray.IsValidIndex(SliderDataIndex));
		SliderStateDataArray[SliderDataIndex].bSliderActiveMode = false;
		SliderStateDataArray[SliderDataIndex].MovementValueInt = 0;
		SetterDelegate.ExecuteIfBound(Value);
	};

	auto SetValueHelperFunc = [SetterDelegate, SliderDataIndex, this](int32 Value)
	{
		check(SliderStateDataArray.IsValidIndex(SliderDataIndex));
		if (SliderStateDataArray[SliderDataIndex].bSliderActiveMode)
		{
			SliderStateDataArray[SliderDataIndex].MovementValueInt = Value;
		}
		else
		{
			SetterDelegate.ExecuteIfBound(Value);
		}
	};

	auto GetValueHelperFunc = [GetterDelegate, SliderDataIndex, this]()
	{
		check(SliderStateDataArray.IsValidIndex(SliderDataIndex));
		if (SliderStateDataArray[SliderDataIndex].bSliderActiveMode)
		{
			return SliderStateDataArray[SliderDataIndex].MovementValueInt;
		}
		return GetterDelegate.IsBound() ? GetterDelegate.Execute() : 0;
	};

	FDetailWidgetRow& Row = ChildrenBuilder.AddCustomRow(RowTitleText)
	.RowTag(RowTag)
	.NameContent()
	[
		SNew(STextBlock)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.Text(RowNameContentText)
		.ToolTipText(RowNameContentTooltipText)
	]
	.ValueContent()
	[
		SNew(SSpinBox<int32>)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.MinValue(MinSliderValue)
		.MaxValue(MaxSliderValue)
		.Value_Lambda(GetValueHelperFunc)
		.OnValueChanged_Lambda(SetValueHelperFunc)
		.OnBeginSliderMovement_Lambda(BeginSliderMovementHelperFunc)
		.OnEndSliderMovement_Lambda(EndSliderMovementHelperFunc)
		.IsEnabled(this, &FSkeletalMeshBuildSettingsLayout::IsBuildEnabled)
	];
	return Row;
}

float FSkeletalMeshBuildSettingsLayout::GetThresholdPosition() const
{
	return BuildSettings.ThresholdPosition;
}

void FSkeletalMeshBuildSettingsLayout::SetThresholdPosition(float Value)
{
	if (BuildSettings.ThresholdPosition != Value)
	{
		FText TransactionText = FText::Format(LOCTEXT("PersonaSetThresholdPositionLOD", "LOD{0} build settings: threshold for position changed"), LODIndex);
		FScopedTransaction Transaction(TransactionText);
		ModifyMeshLODSettingsDelegate.ExecuteIfBound(LODIndex);

		BuildSettings.ThresholdPosition = Value;
	}
}

float FSkeletalMeshBuildSettingsLayout::GetThresholdTangentNormal() const
{
	return BuildSettings.ThresholdTangentNormal;
}

void FSkeletalMeshBuildSettingsLayout::SetThresholdTangentNormal(float Value)
{
	if (BuildSettings.ThresholdTangentNormal != Value)
	{
		FText TransactionText = FText::Format(LOCTEXT("PersonaSetThresholdTangentNormalLOD", "LOD{0} build settings: threshold for tangent and normal changed"), LODIndex);
		FScopedTransaction Transaction(TransactionText);
		ModifyMeshLODSettingsDelegate.ExecuteIfBound(LODIndex);

		BuildSettings.ThresholdTangentNormal = Value;
	}
}

float FSkeletalMeshBuildSettingsLayout::GetThresholdUV() const
{
	return BuildSettings.ThresholdUV;
}

void FSkeletalMeshBuildSettingsLayout::SetThresholdUV(float Value)
{
	if (BuildSettings.ThresholdUV != Value)
	{
		FText TransactionText = FText::Format(LOCTEXT("PersonaSetThresholdUVLOD", "LOD{0} build settings: threshold for UVs changed"), LODIndex);
		FScopedTransaction Transaction(TransactionText);
		ModifyMeshLODSettingsDelegate.ExecuteIfBound(LODIndex);

		BuildSettings.ThresholdUV = Value;
	}
}

float FSkeletalMeshBuildSettingsLayout::GetMorphThresholdPosition() const
{
	return BuildSettings.MorphThresholdPosition;
}

void FSkeletalMeshBuildSettingsLayout::SetMorphThresholdPosition(float Value)
{
	if (BuildSettings.MorphThresholdPosition != Value)
	{
		FText TransactionText = FText::Format(LOCTEXT("PersonaSetMorphThresholdPositionLOD", "LOD{0} build settings: threshold for morph position changed"), LODIndex);
		FScopedTransaction Transaction(TransactionText);
		ModifyMeshLODSettingsDelegate.ExecuteIfBound(LODIndex);

		BuildSettings.MorphThresholdPosition = Value;
	}
}

int32 FSkeletalMeshBuildSettingsLayout::GetBoneInfluenceLimit() const
{
	return BuildSettings.BoneInfluenceLimit;
}

void FSkeletalMeshBuildSettingsLayout::SetBoneInfluenceLimit(int32 Value)
{
	if (BuildSettings.BoneInfluenceLimit != Value)
	{
		FText TransactionText = FText::Format(LOCTEXT("PersonaSetBoneInfluenceLimitLOD", "LOD{0} build settings: bone influence limit changed"), LODIndex);
		FScopedTransaction Transaction(TransactionText);
		ModifyMeshLODSettingsDelegate.ExecuteIfBound(LODIndex);

		BuildSettings.BoneInfluenceLimit = Value;
	}
}

ECheckBoxState FSkeletalMeshBuildSettingsLayout::ShouldRecomputeNormals() const
{
	return BuildSettings.bRecomputeNormals ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}
ECheckBoxState FSkeletalMeshBuildSettingsLayout::ShouldRecomputeTangents() const
{
	return BuildSettings.bRecomputeTangents ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}
ECheckBoxState FSkeletalMeshBuildSettingsLayout::ShouldUseMikkTSpace() const
{
	return BuildSettings.bUseMikkTSpace ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}
ECheckBoxState FSkeletalMeshBuildSettingsLayout::ShouldComputeWeightedNormals() const
{
	return BuildSettings.bComputeWeightedNormals ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}
ECheckBoxState FSkeletalMeshBuildSettingsLayout::ShouldRemoveDegenerates() const
{
	return BuildSettings.bRemoveDegenerates ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}
ECheckBoxState FSkeletalMeshBuildSettingsLayout::ShouldUseHighPrecisionTangentBasis() const
{
	return BuildSettings.bUseHighPrecisionTangentBasis ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}
ECheckBoxState FSkeletalMeshBuildSettingsLayout::ShouldUseHighPrecisionSkinWeights() const
{
	return BuildSettings.bUseHighPrecisionSkinWeights ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}
ECheckBoxState FSkeletalMeshBuildSettingsLayout::ShouldUseFullPrecisionUVs() const
{
	return BuildSettings.bUseFullPrecisionUVs ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}
ECheckBoxState FSkeletalMeshBuildSettingsLayout::ShouldUseBackwardsCompatibleF16TruncUVs() const
{
	return BuildSettings.bUseBackwardsCompatibleF16TruncUVs ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}
ECheckBoxState FSkeletalMeshBuildSettingsLayout::ShouldOptimizeForInstancing() const
{
	return BuildSettings.bOptimizeForInstancing ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}


void FSkeletalMeshBuildSettingsLayout::OnRecomputeNormalsChanged(ECheckBoxState NewState)
{
	const bool bRecomputeNormals = (NewState == ECheckBoxState::Checked) ? true : false;
	if (BuildSettings.bRecomputeNormals != bRecomputeNormals)
	{
		FText TransactionText = FText::Format(LOCTEXT("PersonaChangedOnComputeNormalsLOD", "LOD{0} build settings: recompute normals changed"), LODIndex);
		FScopedTransaction Transaction(TransactionText);
		ModifyMeshLODSettingsDelegate.ExecuteIfBound(LODIndex);

		BuildSettings.bRecomputeNormals = bRecomputeNormals;
	}
}
void FSkeletalMeshBuildSettingsLayout::OnRecomputeTangentsChanged(ECheckBoxState NewState)
{
	const bool bRecomputeTangents = (NewState == ECheckBoxState::Checked) ? true : false;
	if (BuildSettings.bRecomputeTangents != bRecomputeTangents)
	{
		FText TransactionText = FText::Format(LOCTEXT("PersonaChangedOnComputeTangentsLOD", "LOD{0} build settings: recompute tangents changed"), LODIndex);
		FScopedTransaction Transaction(TransactionText);
		ModifyMeshLODSettingsDelegate.ExecuteIfBound(LODIndex);

		BuildSettings.bRecomputeTangents = bRecomputeTangents;
	}
}
void FSkeletalMeshBuildSettingsLayout::OnUseMikkTSpaceChanged(ECheckBoxState NewState)
{
	const bool bUseMikkTSpace = (NewState == ECheckBoxState::Checked) ? true : false;
	if (BuildSettings.bUseMikkTSpace != bUseMikkTSpace)
	{
		FText TransactionText = FText::Format(LOCTEXT("PersonaChangedOnuseMikktSpaceTangentLOD", "LOD{0} build settings: use mikkt space tangent changed"), LODIndex);
		FScopedTransaction Transaction(TransactionText);
		ModifyMeshLODSettingsDelegate.ExecuteIfBound(LODIndex);

		BuildSettings.bUseMikkTSpace = bUseMikkTSpace;
	}
}

void FSkeletalMeshBuildSettingsLayout::OnComputeWeightedNormalsChanged(ECheckBoxState NewState)
{
	const bool bComputeWeightedNormals = (NewState == ECheckBoxState::Checked) ? true : false;
	if (BuildSettings.bComputeWeightedNormals != bComputeWeightedNormals)
	{
		FText TransactionText = FText::Format(LOCTEXT("PersonaChangedOnComputeWeightedNormalsLOD", "LOD{0} build settings: compute weighted normals changed"), LODIndex);
		FScopedTransaction Transaction(TransactionText);
		ModifyMeshLODSettingsDelegate.ExecuteIfBound(LODIndex);

		BuildSettings.bComputeWeightedNormals = bComputeWeightedNormals;
	}
}
void FSkeletalMeshBuildSettingsLayout::OnRemoveDegeneratesChanged(ECheckBoxState NewState)
{
	const bool bRemoveDegenerates = (NewState == ECheckBoxState::Checked) ? true : false;
	if (BuildSettings.bRemoveDegenerates != bRemoveDegenerates)
	{
		FText TransactionText = FText::Format(LOCTEXT("PersonaChangedOnRemoveDegeneratesLOD", "LOD{0} build settings: remove degenerates changed"), LODIndex);
		FScopedTransaction Transaction(TransactionText);
		ModifyMeshLODSettingsDelegate.ExecuteIfBound(LODIndex);

		BuildSettings.bRemoveDegenerates = bRemoveDegenerates;
	}
}
void FSkeletalMeshBuildSettingsLayout::OnUseHighPrecisionTangentBasisChanged(ECheckBoxState NewState)
{
	const bool bUseHighPrecisionTangents = (NewState == ECheckBoxState::Checked) ? true : false;
	if (BuildSettings.bUseHighPrecisionTangentBasis != bUseHighPrecisionTangents)
	{
		FText TransactionText = FText::Format(LOCTEXT("PersonaChangedOnHighPrecisionTangentLOD", "LOD{0} build settings: use high precision tangent changed"), LODIndex);
		FScopedTransaction Transaction(TransactionText);
		ModifyMeshLODSettingsDelegate.ExecuteIfBound(LODIndex);

		BuildSettings.bUseHighPrecisionTangentBasis = bUseHighPrecisionTangents;
	}
}

void FSkeletalMeshBuildSettingsLayout::OnUseHighPrecisionSkinWeightsChanged(ECheckBoxState NewState)
{
	const bool bUseHighPrecisionSkinWeights = (NewState == ECheckBoxState::Checked) ? true : false;
	if (BuildSettings.bUseHighPrecisionSkinWeights != bUseHighPrecisionSkinWeights)
	{
		FText TransactionText = FText::Format(LOCTEXT("PersonaChangedOnHighPrecisionSkinWeightsLOD", "LOD{0} build settings: use high precision skin weights changed"), LODIndex);
		FScopedTransaction Transaction(TransactionText);
		ModifyMeshLODSettingsDelegate.ExecuteIfBound(LODIndex);

		BuildSettings.bUseHighPrecisionSkinWeights = bUseHighPrecisionSkinWeights;
	}
}

void FSkeletalMeshBuildSettingsLayout::OnUseFullPrecisionUVsChanged(ECheckBoxState NewState)
{
	const bool bUseFullPrecisionUVs = (NewState == ECheckBoxState::Checked) ? true : false;
	if (BuildSettings.bUseFullPrecisionUVs != bUseFullPrecisionUVs)
	{
		FText TransactionText = FText::Format(LOCTEXT("PersonaChangedOnFullPrecisionUVsLOD", "LOD{0} build settings: use full precision UVs changed"), LODIndex);
		FScopedTransaction Transaction(TransactionText);
		ModifyMeshLODSettingsDelegate.ExecuteIfBound(LODIndex);

		BuildSettings.bUseFullPrecisionUVs = bUseFullPrecisionUVs;
	}
}
void FSkeletalMeshBuildSettingsLayout::OnUseBackwardsCompatibleF16TruncUVsChanged(ECheckBoxState NewState)
{
	const bool bUseBackwardsCompatibleF16TruncUVs = (NewState == ECheckBoxState::Checked) ? true : false;
	if (BuildSettings.bUseBackwardsCompatibleF16TruncUVs != bUseBackwardsCompatibleF16TruncUVs)
	{
		FText TransactionText = FText::Format(LOCTEXT("PersonaChangedUseBackwardsCompatibleF16TruncUVsLOD", "LOD{0} build settings: bUseBackwardsCompatibleF16TruncUVs changed"), LODIndex);
		FScopedTransaction Transaction(TransactionText);
		ModifyMeshLODSettingsDelegate.ExecuteIfBound(LODIndex);

		BuildSettings.bUseBackwardsCompatibleF16TruncUVs = bUseBackwardsCompatibleF16TruncUVs;
	}
}
void FSkeletalMeshBuildSettingsLayout::OnOptimizeForInstancingChanged(ECheckBoxState NewState)
{
	const bool bOptimizeForInstancing = (NewState == ECheckBoxState::Checked) ? true : false;
	if (BuildSettings.bOptimizeForInstancing != bOptimizeForInstancing)
	{
		FText TransactionText = FText::Format(LOCTEXT("PersonaChangedOptimizeForInstancingLOD", "LOD{0} build settings: bOptimizeForInstancing changed"), LODIndex);
		FScopedTransaction Transaction(TransactionText);
		ModifyMeshLODSettingsDelegate.ExecuteIfBound(LODIndex);

		BuildSettings.bOptimizeForInstancing = bOptimizeForInstancing;
	}
}

/**
* FPersonaMeshDetails
*/
FPersonaMeshDetails::FPersonaMeshDetails(TSharedRef<class IPersonaToolkit> InPersonaToolkit) : PersonaToolkitPtr(InPersonaToolkit), MeshDetailLayout(nullptr)
{
	CustomLODEditMode = false;
	bDeleteWarningConsumed = false;

	GEditor->GetEditorSubsystem<UImportSubsystem>()->OnAssetPostLODImport.AddRaw(this, &FPersonaMeshDetails::OnAssetPostLODImported);
	GEditor->GetEditorSubsystem<UImportSubsystem>()->OnAssetReimport.AddRaw(this, &FPersonaMeshDetails::OnAssetReimport);
}

/**
* FPersonaMeshDetails
*/
FPersonaMeshDetails::~FPersonaMeshDetails()
{
	if (HasValidPersonaToolkit())
	{
		TSharedRef<IPersonaPreviewScene> PreviewScene = GetPersonaToolkit()->GetPreviewScene();
		PreviewScene->UnregisterOnPreviewMeshChanged(this);
	}

	if (SkeletalMeshPtr.Get())
	{
		SkeletalMeshPtr.Get()->OnPostMeshCached().RemoveAll(this);
	}

	GEditor->GetEditorSubsystem<UImportSubsystem>()->OnAssetPostLODImport.RemoveAll(this);
	GEditor->GetEditorSubsystem<UImportSubsystem>()->OnAssetReimport.RemoveAll(this);
}

TSharedRef<IDetailCustomization> FPersonaMeshDetails::MakeInstance(TWeakPtr<class IPersonaToolkit> InPersonaToolkit)
{
	return MakeShareable( new FPersonaMeshDetails(InPersonaToolkit.Pin().ToSharedRef()) );
}

void FPersonaMeshDetails::OnCopySectionList(int32 LODIndex)
{
	USkeletalMesh* Mesh = GetPersonaToolkit()->GetMesh();

	if (Mesh != nullptr)
	{
		FSkeletalMeshModel* ImportedResource = Mesh->GetImportedModel();

		if (ImportedResource != nullptr && ImportedResource->LODModels.IsValidIndex(LODIndex))
		{
			const FSkeletalMeshLODModel& Model = ImportedResource->LODModels[LODIndex];

			TSharedRef<FJsonObject> RootJsonObject = MakeShareable(new FJsonObject());

			for (int32 SectionIdx = 0; SectionIdx < Model.Sections.Num(); ++SectionIdx)
			{
				const FSkelMeshSection& ModelSection = Model.Sections[SectionIdx];

				TSharedPtr<FJsonObject> JSonSection = MakeShareable(new FJsonObject);

				JSonSection->SetNumberField(TEXT("MaterialIndex"), ModelSection.MaterialIndex);
				JSonSection->SetBoolField(TEXT("Disabled"), ModelSection.bDisabled);
				JSonSection->SetBoolField(TEXT("RecomputeTangent"), ModelSection.bRecomputeTangent);
				JSonSection->SetNumberField(TEXT("RecomputeTangentsVertexMaskChannel"), static_cast<uint8>(ModelSection.RecomputeTangentsVertexMaskChannel));
				JSonSection->SetBoolField(TEXT("CastShadow"), ModelSection.bCastShadow);
				JSonSection->SetBoolField(TEXT("VisibleInRayTracing"), ModelSection.bVisibleInRayTracing);
				JSonSection->SetNumberField(TEXT("GenerateUpToLodIndex"), ModelSection.GenerateUpToLodIndex);
				JSonSection->SetNumberField(TEXT("ChunkedParentSectionIndex"), ModelSection.ChunkedParentSectionIndex);
				JSonSection->SetStringField(TEXT("ClothingData.AssetGuid"), ModelSection.ClothingData.AssetGuid.ToString(EGuidFormats::Digits));
				JSonSection->SetNumberField(TEXT("ClothingData.AssetLodIndex"), ModelSection.ClothingData.AssetLodIndex);

				RootJsonObject->SetObjectField(FString::Printf(TEXT("Section_%d"), SectionIdx), JSonSection);
			}

			typedef TJsonWriter<TCHAR, TPrettyJsonPrintPolicy<TCHAR>> FStringWriter;
			typedef TJsonWriterFactory<TCHAR, TPrettyJsonPrintPolicy<TCHAR>> FStringWriterFactory;

			FString CopyStr;
			TSharedRef<FStringWriter> Writer = FStringWriterFactory::Create(&CopyStr);
			FJsonSerializer::Serialize(RootJsonObject, Writer);

			if (!CopyStr.IsEmpty())
			{
				FPlatformApplicationMisc::ClipboardCopy(*CopyStr);
			}
		}
	}
}

bool FPersonaMeshDetails::OnCanCopySectionList(int32 LODIndex) const
{
	USkeletalMesh* Mesh = GetPersonaToolkit()->GetMesh();

	if (Mesh != nullptr && !Mesh->IsCompiling())
	{
		FSkeletalMeshModel* ImportedResource = Mesh->GetImportedModel();

		if (ImportedResource != nullptr && ImportedResource->LODModels.IsValidIndex(LODIndex))
		{
			return ImportedResource->LODModels[LODIndex].Sections.Num() > 0;
		}
	}

	return false;
}

void FPersonaMeshDetails::OnPasteSectionList(int32 LODIndex)
{
	USkeletalMesh* Mesh = GetPersonaToolkit()->GetMesh();

	if (Mesh != nullptr)
	{
		FString PastedText;
		FPlatformApplicationMisc::ClipboardPaste(PastedText);

		TSharedPtr<FJsonObject> RootJsonObject;
		TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(PastedText);
		FJsonSerializer::Deserialize(Reader, RootJsonObject);

		if (RootJsonObject.IsValid())
		{
			FSkeletalMeshModel* ImportedResource = Mesh->GetImportedModel();

			if (ImportedResource != nullptr && ImportedResource->LODModels.IsValidIndex(LODIndex))
			{
				FScopedTransaction Transaction(LOCTEXT("PersonaChangedPasteSectionList", "Persona editor: Pasted section list"));
				Mesh->Modify();

				FSkeletalMeshLODModel& Model = ImportedResource->LODModels[LODIndex];

				for (int32 SectionIdx = 0; SectionIdx < Model.Sections.Num(); ++SectionIdx)
				{
					FSkelMeshSection& ModelSection = Model.Sections[SectionIdx];

					const TSharedPtr<FJsonObject>* JSonSection = nullptr;
					if (RootJsonObject->TryGetObjectField(FString::Printf(TEXT("Section_%d"), SectionIdx), JSonSection))
					{
						int32 Value;
						FString StringValue;
						if ((*JSonSection)->TryGetNumberField(TEXT("MaterialIndex"), Value))
						{
							ModelSection.MaterialIndex = (uint16)Value;
						}
						(*JSonSection)->TryGetBoolField(TEXT("Disabled"), ModelSection.bDisabled);
						(*JSonSection)->TryGetBoolField(TEXT("RecomputeTangent"), ModelSection.bRecomputeTangent);
						if ((*JSonSection)->TryGetNumberField(TEXT("RecomputeTangentsVertexMaskChannel"), Value))
						{
							ModelSection.RecomputeTangentsVertexMaskChannel = static_cast<ESkinVertexColorChannel>((uint8)(Value & 0xFF));
						}
						(*JSonSection)->TryGetBoolField(TEXT("CastShadow"), ModelSection.bCastShadow);
						(*JSonSection)->TryGetBoolField(TEXT("VisibleInRayTracing"), ModelSection.bVisibleInRayTracing);
						if ((*JSonSection)->TryGetNumberField(TEXT("GenerateUpToLodIndex"), Value))
						{
							ModelSection.GenerateUpToLodIndex = (int8)Value;
						}
						if ((*JSonSection)->TryGetNumberField(TEXT("ChunkedParentSectionIndex"), Value))
						{
							ModelSection.ChunkedParentSectionIndex = Value;
						}
						
						if ((*JSonSection)->TryGetStringField(TEXT("ClothingData.AssetGuid"), StringValue))
						{
							FGuid::ParseExact(StringValue, EGuidFormats::Digits, ModelSection.ClothingData.AssetGuid);
						}

						if ((*JSonSection)->TryGetNumberField(TEXT("ClothingData.AssetLodIndex"), Value))
						{
							ModelSection.ClothingData.AssetLodIndex = Value;
						}
					}
				}

				Mesh->PostEditChange();
			}
		}
	}
}

void FPersonaMeshDetails::OnCopySectionItem(int32 LODIndex, int32 SectionIndex)
{
	USkeletalMesh* Mesh = GetPersonaToolkit()->GetMesh();

	if (Mesh != nullptr)
	{
		FSkeletalMeshModel* ImportedResource = Mesh->GetImportedModel();

		if (ImportedResource != nullptr && ImportedResource->LODModels.IsValidIndex(LODIndex))
		{
			const FSkeletalMeshLODModel& Model = ImportedResource->LODModels[LODIndex];

			TSharedRef<FJsonObject> RootJsonObject = MakeShareable(new FJsonObject());

			if (Model.Sections.IsValidIndex(SectionIndex))
			{
				const FSkelMeshSection& ModelSection = Model.Sections[SectionIndex];

				RootJsonObject->SetNumberField(TEXT("MaterialIndex"), ModelSection.MaterialIndex);
				RootJsonObject->SetBoolField(TEXT("Disabled"), ModelSection.bDisabled);
				RootJsonObject->SetBoolField(TEXT("RecomputeTangent"), ModelSection.bRecomputeTangent);
				RootJsonObject->SetNumberField(TEXT("RecomputeTangentsVertexMaskChannel"), static_cast<uint8>(ModelSection.RecomputeTangentsVertexMaskChannel));
				RootJsonObject->SetBoolField(TEXT("CastShadow"), ModelSection.bCastShadow);
				RootJsonObject->SetBoolField(TEXT("VisibleInRayTracing"), ModelSection.bVisibleInRayTracing);
				RootJsonObject->SetNumberField(TEXT("GenerateUpToLodIndex"), ModelSection.GenerateUpToLodIndex);
				RootJsonObject->SetNumberField(TEXT("ChunkedParentSectionIndex"), ModelSection.ChunkedParentSectionIndex);
				RootJsonObject->SetStringField(TEXT("ClothingData.AssetGuid"), ModelSection.ClothingData.AssetGuid.ToString(EGuidFormats::Digits));
				RootJsonObject->SetNumberField(TEXT("ClothingData.AssetLodIndex"), ModelSection.ClothingData.AssetLodIndex);
			}

			typedef TJsonWriter<TCHAR, TPrettyJsonPrintPolicy<TCHAR>> FStringWriter;
			typedef TJsonWriterFactory<TCHAR, TPrettyJsonPrintPolicy<TCHAR>> FStringWriterFactory;

			FString CopyStr;
			TSharedRef<FStringWriter> Writer = FStringWriterFactory::Create(&CopyStr);
			FJsonSerializer::Serialize(RootJsonObject, Writer);

			if (!CopyStr.IsEmpty())
			{
				FPlatformApplicationMisc::ClipboardCopy(*CopyStr);
			}
		}
	}
}

bool FPersonaMeshDetails::OnCanCopySectionItem(int32 LODIndex, int32 SectionIndex) const
{
	USkeletalMesh* Mesh = GetPersonaToolkit()->GetMesh();

	if (Mesh != nullptr && !Mesh->IsCompiling())
	{
		FSkeletalMeshModel* ImportedResource = Mesh->GetImportedModel();

		if (ImportedResource != nullptr && ImportedResource->LODModels.IsValidIndex(LODIndex))
		{
			return ImportedResource->LODModels[LODIndex].Sections.IsValidIndex(SectionIndex);
		}
	}

	return false;
}

void FPersonaMeshDetails::OnPasteSectionItem(int32 LODIndex, int32 SectionIndex)
{
	USkeletalMesh* Mesh = GetPersonaToolkit()->GetMesh();

	if (Mesh != nullptr)
	{
		FString PastedText;
		FPlatformApplicationMisc::ClipboardPaste(PastedText);

		TSharedPtr<FJsonObject> RootJsonObject;
		TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(PastedText);
		FJsonSerializer::Deserialize(Reader, RootJsonObject);

		if (RootJsonObject.IsValid())
		{
			FSkeletalMeshModel* ImportedResource = Mesh->GetImportedModel();

			if (ImportedResource != nullptr && ImportedResource->LODModels.IsValidIndex(LODIndex))
			{
				FSkeletalMeshLODModel& Model = ImportedResource->LODModels[LODIndex];

				FScopedTransaction Transaction(LOCTEXT("PersonaChangedPasteSectionItem", "Persona editor: Pasted section item"));
				Mesh->Modify();

				if (Model.Sections.IsValidIndex(SectionIndex))
				{
					FSkelMeshSection& ModelSection = Model.Sections[SectionIndex];

					int32 Value;
					FString StringValue;
					if (RootJsonObject->TryGetNumberField(TEXT("MaterialIndex"), Value))
					{
						ModelSection.MaterialIndex = (uint16)Value;
					}
					RootJsonObject->TryGetBoolField(TEXT("Disabled"), ModelSection.bDisabled);
					RootJsonObject->TryGetBoolField(TEXT("RecomputeTangent"), ModelSection.bRecomputeTangent);
					if (RootJsonObject->TryGetNumberField(TEXT("RecomputeTangentsVertexMaskChannel"), Value))
					{
						ModelSection.RecomputeTangentsVertexMaskChannel = static_cast<ESkinVertexColorChannel>((uint8) (Value & 0xFF));
					}
					RootJsonObject->TryGetBoolField(TEXT("CastShadow"), ModelSection.bCastShadow);
					RootJsonObject->TryGetBoolField(TEXT("VisibleInRayTracing"), ModelSection.bVisibleInRayTracing);
					if (RootJsonObject->TryGetNumberField(TEXT("GenerateUpToLodIndex"), Value))
					{
						ModelSection.GenerateUpToLodIndex = (int8)Value;
					}
					if (RootJsonObject->TryGetNumberField(TEXT("ChunkedParentSectionIndex"), Value))
					{
						ModelSection.ChunkedParentSectionIndex = Value;
					}

					if (RootJsonObject->TryGetStringField(TEXT("ClothingData.AssetGuid"), StringValue))
					{
						FGuid::ParseExact(StringValue, EGuidFormats::Digits, ModelSection.ClothingData.AssetGuid);
					}

					if (RootJsonObject->TryGetNumberField(TEXT("ClothingData.AssetLodIndex"), Value))
					{
						ModelSection.ClothingData.AssetLodIndex = Value;
					}
				}

				Mesh->PostEditChange();
			}
		}
	}
}

void FPersonaMeshDetails::OnCopyMaterialList()
{
	USkeletalMesh* Mesh = GetPersonaToolkit()->GetMesh();

	if (Mesh != nullptr)
	{
		FProperty* Property = USkeletalMesh::StaticClass()->FindPropertyByName(USkeletalMesh::GetMaterialsMemberName());
		auto JsonValue = FJsonObjectConverter::UPropertyToJsonValue(Property, &Mesh->GetMaterials(), 0, 0);

		typedef TJsonWriter<TCHAR, TPrettyJsonPrintPolicy<TCHAR>> FStringWriter;
		typedef TJsonWriterFactory<TCHAR, TPrettyJsonPrintPolicy<TCHAR>> FStringWriterFactory;

		FString CopyStr;
		TSharedRef<FStringWriter> Writer = FStringWriterFactory::Create(&CopyStr);
		FJsonSerializer::Serialize(JsonValue.ToSharedRef(), TEXT(""), Writer);

		if (!CopyStr.IsEmpty())
		{
			FPlatformApplicationMisc::ClipboardCopy(*CopyStr);
		}
	}
}

bool FPersonaMeshDetails::OnCanCopyMaterialList() const
{
	USkeletalMesh* Mesh = GetPersonaToolkit()->GetMesh();

	if (Mesh != nullptr && !Mesh->IsCompiling())
	{
		return Mesh->GetMaterials().Num() > 0;
	}

	return false;
}

void FPersonaMeshDetails::OnPasteMaterialList()
{
	USkeletalMesh* Mesh = GetPersonaToolkit()->GetMesh();

	if (Mesh != nullptr)
	{
		FString PastedText;
		FPlatformApplicationMisc::ClipboardPaste(PastedText);

		TSharedPtr<FJsonValue> RootJsonValue;
		TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(PastedText);
		FJsonSerializer::Deserialize(Reader, RootJsonValue);

		if (RootJsonValue.IsValid())
		{
			FProperty* Property = USkeletalMesh::StaticClass()->FindPropertyByName(USkeletalMesh::GetMaterialsMemberName());

			Mesh->PreEditChange(Property);
			FScopedTransaction Transaction(LOCTEXT("PersonaChangedPasteMaterialList", "Persona editor: Pasted material list"));
			Mesh->Modify();
			TArray<FSkeletalMaterial> TempMaterials;
			FJsonObjectConverter::JsonValueToUProperty(RootJsonValue, Property, &TempMaterials, 0, 0);
			//Do not change the number of material in the array
			TArray<FSkeletalMaterial>& MeshMaterials = Mesh->GetMaterials();
			for (int32 MaterialIndex = 0; MaterialIndex < TempMaterials.Num(); ++MaterialIndex)
			{
				if (MeshMaterials.IsValidIndex(MaterialIndex))
				{
					MeshMaterials[MaterialIndex].MaterialInterface = TempMaterials[MaterialIndex].MaterialInterface;
					MeshMaterials[MaterialIndex].OverlayMaterialInterface = TempMaterials[MaterialIndex].OverlayMaterialInterface;
				}
			}
			

			Mesh->PostEditChange();
		}
	}
}

void FPersonaMeshDetails::OnCopyMaterialItem(int32 CurrentSlot)
{
	USkeletalMesh* Mesh = GetPersonaToolkit()->GetMesh();

	if (Mesh != nullptr)
	{
		TSharedRef<FJsonObject> RootJsonObject = MakeShareable(new FJsonObject());

		if (Mesh->GetMaterials().IsValidIndex(CurrentSlot))
		{
			const FSkeletalMaterial& Material = Mesh->GetMaterials()[CurrentSlot];

			FJsonObjectConverter::UStructToJsonObject(FSkeletalMaterial::StaticStruct(), &Material, RootJsonObject, 0, 0);
		}

		typedef TJsonWriter<TCHAR, TPrettyJsonPrintPolicy<TCHAR>> FStringWriter;
		typedef TJsonWriterFactory<TCHAR, TPrettyJsonPrintPolicy<TCHAR>> FStringWriterFactory;

		FString CopyStr;
		TSharedRef<FStringWriter> Writer = FStringWriterFactory::Create(&CopyStr);
		FJsonSerializer::Serialize(RootJsonObject, Writer);

		if (!CopyStr.IsEmpty())
		{
			FPlatformApplicationMisc::ClipboardCopy(*CopyStr);
		}
	}
}

bool FPersonaMeshDetails::OnCanCopyMaterialItem(int32 CurrentSlot) const
{
	USkeletalMesh* Mesh = GetPersonaToolkit()->GetMesh();

	if (Mesh != nullptr && !Mesh->IsCompiling())
	{
		return Mesh->GetMaterials().IsValidIndex(CurrentSlot);
	}

	return false;
}

void FPersonaMeshDetails::OnPasteMaterialItem(int32 CurrentSlot)
{
	USkeletalMesh* Mesh = GetPersonaToolkit()->GetMesh();

	if (Mesh != nullptr)
	{
		FString PastedText;
		FPlatformApplicationMisc::ClipboardPaste(PastedText);

		TSharedPtr<FJsonObject> RootJsonObject;
		TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(PastedText);
		FJsonSerializer::Deserialize(Reader, RootJsonObject);

		if (RootJsonObject.IsValid())
		{
			Mesh->PreEditChange(USkeletalMesh::StaticClass()->FindPropertyByName(USkeletalMesh::GetMaterialsMemberName()));
			FScopedTransaction Transaction(LOCTEXT("PersonaChangedPasteMaterialItem", "Persona editor: Pasted material item"));
			Mesh->Modify();

			if (Mesh->GetMaterials().IsValidIndex(CurrentSlot))
			{
				FSkeletalMaterial TmpSkeletalMaterial;
				FJsonObjectConverter::JsonObjectToUStruct(RootJsonObject.ToSharedRef(), FSkeletalMaterial::StaticStruct(), &TmpSkeletalMaterial, 0, 0);
				Mesh->GetMaterials()[CurrentSlot].MaterialInterface = TmpSkeletalMaterial.MaterialInterface;
				Mesh->GetMaterials()[CurrentSlot].OverlayMaterialInterface = TmpSkeletalMaterial.OverlayMaterialInterface;
			}

			Mesh->PostEditChange();
		}
	}
}

void FPersonaMeshDetails::OnAttributePreChangePreventPostEditChange(const int32 LODIndex, const FName LODInfoPropertyName)
{
	//Avoid PostEditChange call 
	USkeletalMesh* Mesh = GetPersonaToolkit()->GetMesh();

	if (Mesh == nullptr)
	{
		return;
	}
	
	constexpr bool bCallPostEditChange = false;
	constexpr bool bReregisterComponents = false;
	ChangeScopes.Push(MakeUnique<FScopedSkeletalMeshPostEditChange>(Mesh, bCallPostEditChange, bReregisterComponents));
}

void FPersonaMeshDetails::OnAttributeChangedPreventPostEditChange(const int32 LODIndex, const FName LODInfoPropertyName, const bool bForceComponentRefresh)
{
	USkeletalMesh* Mesh = GetPersonaToolkit()->GetMesh();

	if (Mesh == nullptr)
	{
		return;
	}
	ChangeScopes.Pop();
	if(bForceComponentRefresh)
	{
		const bool bCallPostEditChange = false;
		const bool bReregisterComponents = true;
		FScopedSkeletalMeshPostEditChange ScopedSkelMeshChange(Mesh, bCallPostEditChange, bReregisterComponents);
	}
}

void FPersonaMeshDetails::PreventAttributePostEditChange(TSharedPtr<IPropertyHandle> AttributeHandle, const int32 LODIndex, const FName PropertyName, const bool bForceComponentRefresh)
{
	AttributeHandle->SetOnPropertyValuePreChange(FSimpleDelegate::CreateSP(this, &FPersonaMeshDetails::OnAttributePreChangePreventPostEditChange, LODIndex, PropertyName));
	AttributeHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FPersonaMeshDetails::OnAttributeChangedPreventPostEditChange, LODIndex, PropertyName, bForceComponentRefresh));
	//In case we have some childs (array attribute)
	AttributeHandle->SetOnChildPropertyValuePreChange(FSimpleDelegate::CreateSP(this, &FPersonaMeshDetails::OnAttributePreChangePreventPostEditChange, LODIndex, PropertyName));
	AttributeHandle->SetOnChildPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FPersonaMeshDetails::OnAttributeChangedPreventPostEditChange, LODIndex, PropertyName, bForceComponentRefresh));
}

void FPersonaMeshDetails::CustomizeLODInfoSetingsDetails(IDetailLayoutBuilder& DetailLayout, ULODInfoUILayout* LODInfoUILayout, TSharedRef<IPropertyHandle> LODInfoProperty, IDetailCategoryBuilder& LODCategory)
{
	check(LODInfoUILayout);

	int32 LODIndex = LODInfoUILayout->GetLODIndex();
	USkeletalMesh* SkelMesh = LODInfoUILayout->GetPersonaToolkit()->GetPreviewMesh();
	check(SkelMesh);
	//Hide the original LODInfo handle
	TSharedPtr<IPropertyHandle> LODInfoIndexOriginal = LODInfoProperty->GetChildHandle(LODIndex);
	check(LODInfoIndexOriginal->IsValidHandle());
	DetailLayout.HideProperty(LODInfoIndexOriginal);

	//Add a property row pointing on our mockup UObject
	TArray<UObject*> ExternalObjects;
	ExternalObjects.Add(LODInfoUILayout);
	IDetailPropertyRow* LODInfoPropertyRow = LODCategory.AddExternalObjectProperty(ExternalObjects, "LODInfo");
	//Collapse the row, we do not want to see this content.
	LODInfoPropertyRow->Visibility(EVisibility::Collapsed);

	//Use the properties pointing on the mockup object
	TSharedPtr<IPropertyHandle> LODInfoChild = LODInfoPropertyRow->GetPropertyHandle();
	uint32 NumInfoChildren = 0;
	LODInfoChild->GetNumChildren(NumInfoChildren);
	DetailLayout.HideProperty(LODInfoChild);
	//Create the UI under a LODInfo group
	IDetailGroup& LODInfoGroup = LODCategory.AddGroup(TEXT("LOD Info"), LOCTEXT("LODInfoGroupLabel", "LOD Info"));

	TAttribute<bool> EnabledAttrib = TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateSP(this, &FPersonaMeshDetails::IsLODInfoEditingEnabled, LODIndex));

	constexpr bool bForceComponentRefreshTrue = true;
	constexpr bool bForceComponentRefreshFalse = false;
	// enable/disable handler - because we want to make sure not editable if LOD sharing is on
	TSharedPtr<IPropertyHandle> ScreenSizeHandle = LODInfoChild->GetChildHandle(GET_MEMBER_NAME_CHECKED(FSkeletalMeshLODInfo, ScreenSize));
	IDetailPropertyRow& ScreenSizeRow = LODInfoGroup.AddPropertyRow(ScreenSizeHandle->AsShared()).EditCondition(EnabledAttrib, nullptr);
	ScreenSizeRow.IsEnabled(EnabledAttrib);
	PreventAttributePostEditChange(ScreenSizeHandle, LODIndex, ScreenSizeHandle->GetProperty()->GetFName(), bForceComponentRefreshTrue);
	DetailLayout.HideProperty(ScreenSizeHandle);

	TSharedPtr<IPropertyHandle> LODHysteresisHandle = LODInfoChild->GetChildHandle(GET_MEMBER_NAME_CHECKED(FSkeletalMeshLODInfo, LODHysteresis));
	IDetailPropertyRow& LODHysteresisRow = LODInfoGroup.AddPropertyRow(LODHysteresisHandle->AsShared());
	LODHysteresisRow.IsEnabled(EnabledAttrib);
	PreventAttributePostEditChange(LODHysteresisHandle, LODIndex, LODHysteresisHandle->GetProperty()->GetFName(), bForceComponentRefreshTrue);
	DetailLayout.HideProperty(LODHysteresisHandle);

	TSharedPtr<IPropertyHandle> BonesToPrioritizeHandle = LODInfoChild->GetChildHandle(GET_MEMBER_NAME_CHECKED(FSkeletalMeshLODInfo, BonesToPrioritize));
	IDetailPropertyRow& BonesToPrioritizeRow = LODInfoGroup.AddPropertyRow(BonesToPrioritizeHandle->AsShared());
	BonesToPrioritizeRow.IsEnabled(EnabledAttrib);
	PreventAttributePostEditChange(BonesToPrioritizeHandle, LODIndex, BonesToPrioritizeHandle->GetProperty()->GetFName(), bForceComponentRefreshFalse);
	DetailLayout.HideProperty(BonesToPrioritizeHandle);

	TSharedPtr<IPropertyHandle> SectionsToPrioritizeHandle = LODInfoChild->GetChildHandle(GET_MEMBER_NAME_CHECKED(FSkeletalMeshLODInfo, SectionsToPrioritize));
	IDetailPropertyRow& SectionsToPrioritizeRow = LODInfoGroup.AddPropertyRow(SectionsToPrioritizeHandle->AsShared());
	SectionsToPrioritizeRow.IsEnabled(EnabledAttrib);
	PreventAttributePostEditChange(SectionsToPrioritizeHandle, LODIndex, SectionsToPrioritizeHandle->GetProperty()->GetFName(), bForceComponentRefreshFalse);
	DetailLayout.HideProperty(SectionsToPrioritizeHandle);

	TSharedPtr<IPropertyHandle> WeightToPriortizeHandle = LODInfoChild->GetChildHandle(GET_MEMBER_NAME_CHECKED(FSkeletalMeshLODInfo, WeightOfPrioritization));
	IDetailPropertyRow& WeightToPriortizeRow = LODInfoGroup.AddPropertyRow(WeightToPriortizeHandle->AsShared());
	WeightToPriortizeRow.IsEnabled(EnabledAttrib);
	PreventAttributePostEditChange(WeightToPriortizeHandle, LODIndex, WeightToPriortizeHandle->GetProperty()->GetFName(), bForceComponentRefreshFalse);
	DetailLayout.HideProperty(WeightToPriortizeHandle);

	TSharedPtr<IPropertyHandle> AllowMeshDeformerHandle = LODInfoChild->GetChildHandle(GET_MEMBER_NAME_CHECKED(FSkeletalMeshLODInfo, bAllowMeshDeformer));
	IDetailPropertyRow& AllowMeshDeformerRow = LODInfoGroup.AddPropertyRow(AllowMeshDeformerHandle->AsShared());
	AllowMeshDeformerRow.IsEnabled(EnabledAttrib);
	// PreventAttributePostEditChange is not called here because we do want PostEditChange to fire
	// if this is changed, so that any mesh components will be re-registered and refresh their
	// deformer setups.
	DetailLayout.HideProperty(AllowMeshDeformerHandle);

	const TArray<FName> HiddenProperties = { GET_MEMBER_NAME_CHECKED(FSkeletalMeshLODInfo, ReductionSettings), GET_MEMBER_NAME_CHECKED(FSkeletalMeshLODInfo, BakePose), GET_MEMBER_NAME_CHECKED(FSkeletalMeshLODInfo, BakePoseOverride), GET_MEMBER_NAME_CHECKED(FSkeletalMeshLODInfo, BonesToRemove),
		GET_MEMBER_NAME_CHECKED(FSkeletalMeshLODInfo, BonesToPrioritize), GET_MEMBER_NAME_CHECKED(FSkeletalMeshLODInfo, SectionsToPrioritize), GET_MEMBER_NAME_CHECKED(FSkeletalMeshLODInfo, WeightOfPrioritization), GET_MEMBER_NAME_CHECKED(FSkeletalMeshLODInfo, ScreenSize),
		GET_MEMBER_NAME_CHECKED(FSkeletalMeshLODInfo, LODHysteresis), GET_MEMBER_NAME_CHECKED(FSkeletalMeshLODInfo, BuildSettings), GET_MEMBER_NAME_CHECKED(FSkeletalMeshLODInfo, bAllowMeshDeformer) };
	for (uint32 ChildIndex = 0; ChildIndex < NumInfoChildren; ++ChildIndex)
	{
		TSharedRef<IPropertyHandle> LODInfoChildHandle = LODInfoChild->GetChildHandle(ChildIndex).ToSharedRef();
		if (!HiddenProperties.Contains(LODInfoChildHandle->GetProperty()->GetFName()))
		{
			LODInfoGroup.AddPropertyRow(LODInfoChildHandle);

			//Some property has to call post edit change, since they are not part of the DDC key, but they are use by directly by the skinned mesh component.
			bool bShouldPreventPostEditChange = GET_MEMBER_NAME_CHECKED(FSkeletalMeshLODInfo, SkinCacheUsage) != LODInfoChildHandle->GetProperty()->GetFName() &&
												GET_MEMBER_NAME_CHECKED(FSkeletalMeshLODInfo, MorphTargetPositionErrorTolerance) != LODInfoChildHandle->GetProperty()->GetFName();
			if(bShouldPreventPostEditChange)
			{
				PreventAttributePostEditChange(LODInfoChildHandle, LODIndex, LODInfoChildHandle->GetProperty()->GetFName(), bForceComponentRefreshFalse);
			}
		}
	}

	TSharedPtr<IPropertyHandle> BakePoseHandle = LODInfoChild->GetChildHandle(GET_MEMBER_NAME_CHECKED(FSkeletalMeshLODInfo, BakePose));
	DetailLayout.HideProperty(BakePoseHandle);
	LODInfoGroup.AddWidgetRow()
	.IsEnabled(EnabledAttrib)
	.NameContent()
	[
		SNew(STextBlock)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.Text(LOCTEXT("BakePoseTitle", "Bake Pose"))
	]
	.ValueContent()
	[
		SNew(SObjectPropertyEntryBox)
		.PropertyHandle(BakePoseHandle)
		.AllowedClass(UAnimSequence::StaticClass())
		.OnShouldFilterAsset(this, &FPersonaMeshDetails::FilterOutBakePose, SkelMesh->GetSkeleton())
	];

	TSharedPtr<IPropertyHandle> BakePoseOverrideHandle = LODInfoChild->GetChildHandle(GET_MEMBER_NAME_CHECKED(FSkeletalMeshLODInfo, BakePoseOverride));
	DetailLayout.HideProperty(BakePoseOverrideHandle);
	LODInfoGroup.AddWidgetRow()
	.NameContent()
	[
		SNew(STextBlock)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.Text(LOCTEXT("BakePoseOverrideTitle", "Bake Pose Override"))
		.ToolTipText(LOCTEXT("BakePoseOverrideToolTip", "This is to override BakePose, the source BakePose could be disabled if LOD Setting is used."))
	]
	.ValueContent()
	[
		SNew(SObjectPropertyEntryBox)
		.PropertyHandle(BakePoseOverrideHandle)
		.AllowedClass(UAnimSequence::StaticClass())
		.OnShouldFilterAsset(this, &FPersonaMeshDetails::FilterOutBakePose, SkelMesh->GetSkeleton())
	];

	TSharedPtr<IPropertyHandle> RemovedBonesHandle = LODInfoChild->GetChildHandle(GET_MEMBER_NAME_CHECKED(FSkeletalMeshLODInfo, BonesToRemove));
	IDetailPropertyRow& RemoveBonesRow = LODInfoGroup.AddPropertyRow(RemovedBonesHandle->AsShared());
	RemoveBonesRow.IsEnabled(EnabledAttrib);
	PreventAttributePostEditChange(RemovedBonesHandle, LODIndex, RemovedBonesHandle->GetProperty()->GetFName(), bForceComponentRefreshFalse);
}

void FPersonaMeshDetails::AddLODLevelCategories(IDetailLayoutBuilder& DetailLayout)
{
	USkeletalMesh* SkelMesh = GetPersonaToolkit()->GetMesh();

	if (SkelMesh)
	{
		const int32 SkelMeshLODCount = SkelMesh->GetLODNum();

		ClothComboBoxes.Reset();

		//Create material list panel to let users control the materials array
		{
			FString MaterialCategoryName = FString(TEXT("Material Slots"));
			IDetailCategoryBuilder& MaterialCategory = DetailLayout.EditCategory(*MaterialCategoryName, FText::GetEmpty(), ECategoryPriority::Important);
			MaterialCategory
				.AddCustomRow(LOCTEXT("AddLODLevelCategories_MaterialArrayOperationAdd", "Materials Operation Add Material Slot"))
				.RowTag(TEXT("MaterialSlots"))
				.CopyAction(FUIAction(FExecuteAction::CreateSP(this, &FPersonaMeshDetails::OnCopyMaterialList), FCanExecuteAction::CreateSP(this, &FPersonaMeshDetails::OnCanCopyMaterialList)))
				.PasteAction(FUIAction(FExecuteAction::CreateSP(this, &FPersonaMeshDetails::OnPasteMaterialList)))
				.NameContent()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(LOCTEXT("AddLODLevelCategories_MaterialArrayOperations", "Material Slots"))
				]
				.ValueContent()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.FillWidth(1.0f)
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.Font(IDetailLayoutBuilder::GetDetailFont())
							.Text(this, &FPersonaMeshDetails::GetMaterialArrayText)
						]
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						.Padding(2.0f, 1.0f)
						[
							SNew(SButton)
							.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
							.Text(LOCTEXT("AddLODLevelCategories_MaterialArrayOpAdd", "Add Material Slot"))
							.ToolTipText(LOCTEXT("AddLODLevelCategories_MaterialArrayOpAdd_Tooltip", "Add Material Slot at the end of the Material slot array. Those Material slots can be used to override a LODs section, (not the base LOD)"))
							.ContentPadding(4.0f)
							.ForegroundColor(FSlateColor::UseForeground())
							.OnClicked(this, &FPersonaMeshDetails::AddMaterialSlot)
							.IsEnabled(true)
							.IsFocusable(false)
							[
								SNew(SImage)
								.Image(FAppStyle::GetBrush("Icons.PlusCircle"))
								.ColorAndOpacity(FSlateColor::UseForeground())
							]
						]
					]
				];
			{
				FMaterialListDelegates MaterialListDelegates;

				MaterialListDelegates.OnGetMaterials.BindSP(this, &FPersonaMeshDetails::OnGetMaterialsForArray, 0);
				MaterialListDelegates.OnMaterialChanged.BindSP(this, &FPersonaMeshDetails::OnMaterialArrayChanged, 0);
				MaterialListDelegates.OnGenerateCustomNameWidgets.BindSP(this, &FPersonaMeshDetails::OnGenerateCustomNameWidgetsForMaterialArray);
				MaterialListDelegates.OnGenerateCustomMaterialWidgets.BindSP(this, &FPersonaMeshDetails::OnGenerateCustomMaterialWidgetsForMaterialArray, 0);
				MaterialListDelegates.OnGenerateExtraBottomWidget.BindSP(this, &FPersonaMeshDetails::OnGenerateMaterialListExtraBottomWidget, 0);
				MaterialListDelegates.OnMaterialListDirty.BindSP(this, &FPersonaMeshDetails::OnMaterialListDirty);

				MaterialListDelegates.OnCopyMaterialItem.BindSP(this, &FPersonaMeshDetails::OnCopyMaterialItem);
				MaterialListDelegates.OnCanCopyMaterialItem.BindSP(this, &FPersonaMeshDetails::OnCanCopyMaterialItem);
				MaterialListDelegates.OnPasteMaterialItem.BindSP(this, &FPersonaMeshDetails::OnPasteMaterialItem);

				//Pass an empty material list owner (owner can be use by the asset picker filter. In this case we do not need it)
				TArray<FAssetData> MaterialListOwner;
				MaterialListOwner.Add(SkelMesh);
				MaterialCategory.AddCustomBuilder(MakeShareable(new FMaterialList(MaterialCategory.GetParentLayout(), MaterialListDelegates, MaterialListOwner, false, true)));
			}
		}

		int32 CurrentLodIndex = 0;
		if (GetPersonaToolkit()->GetPreviewMeshComponent() != nullptr)
		{
			CurrentLodIndex = GetPersonaToolkit()->GetPreviewMeshComponent()->GetForcedLOD();
		}

		FString LODControllerCategoryName = FString(TEXT("LODCustomMode"));
		FText LODControllerString = LOCTEXT("LODCustomModeCategoryName", "LOD Picker");

		IDetailCategoryBuilder& LODCustomModeCategory = DetailLayout.EditCategory(*LODControllerCategoryName, LODControllerString, ECategoryPriority::Important);
		LodCustomCategory = &LODCustomModeCategory;

		LODCustomModeCategory.AddCustomRow((LOCTEXT("LODCustomModeSelect", "Select LOD")))
		.RowTag("SelectLOD")
		.NameContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("LODCustomModeSelectTitle", "LOD"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.IsEnabled(this, &FPersonaMeshDetails::IsLodComboBoxEnabledForLodPicker)
		]
		.ValueContent()
		[
			OnGenerateLodComboBoxForLodPicker()
		];

		LODCustomModeCategory.AddCustomRow((LOCTEXT("LODCustomModeFirstRowName", "LODCustomMode")))
		.RowTag("LODCustomMode")
		.NameContent()
		[
			SNew(STextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(this, &FPersonaMeshDetails::GetLODCustomModeNameContent, (int32)INDEX_NONE)
			.ToolTipText(LOCTEXT("LODCustomModeFirstRowTooltip", "Custom Mode shows multiple LOD's properties at the same time for easier editing."))
		]
		.ValueContent()
		[
			SNew(SCheckBox)
			.IsChecked(this, &FPersonaMeshDetails::IsLODCustomModeCheck, (int32)INDEX_NONE)
			.OnCheckStateChanged(this, &FPersonaMeshDetails::SetLODCustomModeCheck, (int32)INDEX_NONE)
			.ToolTipText(LOCTEXT("LODCustomModeFirstRowTooltip", "Custom Mode shows multiple LOD's properties at the same time for easier editing."))
		];

		LodCategories.Empty(SkelMeshLODCount);
		DetailDisplayLODs.Reset();
		auto ClearLODInfoLayouts = [this, &SkelMeshLODCount, SkelMesh]()
		{
			for (ULODInfoUILayout* LODInfoUILayout : LODInfoUILayouts)
			{
				SkelMesh->GetOnVertexAttributesArrayChanged().RemoveAll(LODInfoUILayout);
				LODInfoUILayout->RemoveFromRoot();
				LODInfoUILayout->MarkAsGarbage();
				LODInfoUILayout = nullptr;
			}
			LODInfoUILayouts.Reset(SkelMeshLODCount);
		};

		ClearLODInfoLayouts();

		// Create information panel for each LOD level.
		for (int32 LODIndex = 0; LODIndex < SkelMeshLODCount; ++LODIndex)
		{
			//Construct temporary LODInfo editor object
			ULODInfoUILayout* LODInfoUILayout = NewObject<ULODInfoUILayout>(GetTransientPackage(), FName(*(FGuid::NewGuid().ToString())), RF_Standalone);
			LODInfoUILayout->AddToRoot();
			FSkeletalMeshLODInfo* LODInfoPtr = SkelMesh->GetLODInfo(LODIndex);
			if (!LODInfoPtr || !SkelMesh->GetImportedModel()->LODModels.IsValidIndex(LODIndex))
			{
				//Trip an ensure so user is aware of this
				UE_ASSET_LOG(LogSkeletalMeshPersonaMeshDetail, Error, SkelMesh, TEXT("Missing LOD %d data, cannot build persona mesh UI"), LODIndex);
				continue;
			}
			const FSkeletalMeshLODModel& LODModel = SkelMesh->GetImportedModel()->LODModels[LODIndex];
			LODInfoUILayout->SetReferenceLODInfo(GetPersonaToolkit(), LODIndex);
			SkelMesh->GetOnVertexAttributesArrayChanged().AddUObject(LODInfoUILayout, &ULODInfoUILayout::RefreshReferenceLODInfo);
			LODInfoUILayouts.Add(LODInfoUILayout);

			//Show the viewport LOD at start
			bool IsViewportLOD = (CurrentLodIndex == 0 ? 0 : CurrentLodIndex - 1) == LODIndex;
			DetailDisplayLODs.Add(true); //Enable all LOD in custum mode
			LODCustomModeCategory.AddCustomRow(( LOCTEXT("LODCustomModeRowName", "LODCheckBoxRowName")), true)
			.RowTag("LODCheckBoxRowName")
			.NameContent()
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(this, &FPersonaMeshDetails::GetLODCustomModeNameContent, LODIndex)
				.IsEnabled(this, &FPersonaMeshDetails::IsLODCustomModeEnable, LODIndex)
			]
			.ValueContent()
			[
				SNew(SCheckBox)
				.IsChecked(this, &FPersonaMeshDetails::IsLODCustomModeCheck, LODIndex)
				.OnCheckStateChanged(this, &FPersonaMeshDetails::SetLODCustomModeCheck, LODIndex)
				.IsEnabled(this, &FPersonaMeshDetails::IsLODCustomModeEnable, LODIndex)
			];

			TSharedRef<IPropertyHandle> LODInfoProperty = DetailLayout.GetProperty(FName("SourceModels"), USkeletalMesh::StaticClass());
			
			uint32 NumChildren = 0;
			LODInfoProperty->GetNumChildren(NumChildren);
			//If the skeleton change during a re-import the skleleton preview mesh will be force reload even if we are in the middle of a re-import
			//But when the re-import will be done the UI will be refresh, so skipping LOD ui until we have valid data is necessary to avoid crash.
			if (NumChildren <= (uint32)LODIndex)
			{
				ClearLODInfoLayouts();
				break;
			}

			IDetailCategoryBuilder& LODCategory = GetLODIndexCategory(DetailLayout, LODIndex);
			LodCategories.Add(&LODCategory);
			TSharedRef<SWidget> LODCategoryWidget =

				SNew(SBox)
				.Padding(FMargin(4.0f, 0.0f))
				[
					SNew(STextBlock)
					.Text_Raw(this, &FPersonaMeshDetails::GetLODImportedText, LODIndex)
					.Font(IDetailLayoutBuilder::GetDetailFontItalic())
				];

			// want to make sure if this data has imported or not
			LODCategory.HeaderContent(LODCategoryWidget);
			{
				FSectionListDelegates SectionListDelegates;

				SectionListDelegates.OnGetSections.BindSP(this, &FPersonaMeshDetails::OnGetSectionsForView, LODIndex);
				SectionListDelegates.OnSectionChanged.BindSP(this, &FPersonaMeshDetails::OnSectionChanged);
				SectionListDelegates.OnGenerateCustomNameWidgets.BindSP(this, &FPersonaMeshDetails::OnGenerateCustomNameWidgetsForSection);
				SectionListDelegates.OnGenerateCustomSectionWidgets.BindSP(this, &FPersonaMeshDetails::OnGenerateCustomSectionWidgetsForSection);

				SectionListDelegates.OnCopySectionList.BindSP(this, &FPersonaMeshDetails::OnCopySectionList, LODIndex);
				SectionListDelegates.OnCanCopySectionList.BindSP(this, &FPersonaMeshDetails::OnCanCopySectionList, LODIndex);
				SectionListDelegates.OnPasteSectionList.BindSP(this, &FPersonaMeshDetails::OnPasteSectionList, LODIndex);
				SectionListDelegates.OnCopySectionItem.BindSP(this, &FPersonaMeshDetails::OnCopySectionItem);
				SectionListDelegates.OnCanCopySectionItem.BindSP(this, &FPersonaMeshDetails::OnCanCopySectionItem);
				SectionListDelegates.OnPasteSectionItem.BindSP(this, &FPersonaMeshDetails::OnPasteSectionItem);
				SectionListDelegates.OnEnableSectionItem.BindSP(this, &FPersonaMeshDetails::OnSectionEnabledChanged);

				FName SkeletalMeshSectionListName = FName(*(FString(TEXT("SkeletalMeshSectionListNameLOD_")) + FString::FromInt(LODIndex)));
				LODCategory.AddCustomBuilder(MakeShareable(new FSectionList(LODCategory.GetParentLayout(), SectionListDelegates, false, 48, LODIndex, SkeletalMeshSectionListName)));

				GetPersonaToolkit()->GetPreviewScene()->RegisterOnSelectedLODChanged(FOnSelectedLODChanged::CreateSP(this, &FPersonaMeshDetails::UpdateLODCategoryVisibility));
			}

			if (LODInfoProperty->IsValidHandle())
			{
				//Display the LODInfo settings
				CustomizeLODInfoSetingsDetails(DetailLayout, LODInfoUILayout, LODInfoProperty, LODCategory);
				
				bool bIsbuildAvailable = SkelMesh->HasMeshDescription(LODIndex);

				
				//Avoid offering re-generate if the LOD is reduced on itself and does not have the original data. The user in this case has to re-import the asset to generate the data 
				bool LodCannotRegenerate = (SkelMesh->GetLODInfo(LODIndex) != nullptr
					&& LODIndex == SkelMesh->GetLODInfo(LODIndex)->ReductionSettings.BaseLOD
					&& SkelMesh->GetLODInfo(LODIndex)->bHasBeenSimplified
					&& !bIsbuildAvailable);

				bool bShowGenerateButtons = !LodCannotRegenerate;
				//LOD 0 never show Reimport and remove buttons
				bool bShowReimportButtons = LODIndex != 0;
				bool bShowRemoveButtons = LODIndex != 0;

				// Add reduction settings
				if (IsAutoMeshReductionAvailable() && bShowGenerateButtons)
				{
					//Create the build setting UI Layout
					ReductionSettingsWidgetsPerLOD.Add(LODIndex, MakeShareable(new FSkeletalMeshReductionSettingsLayout(SkelMesh->GetLODInfo(LODIndex)->ReductionSettings
						, bIsbuildAvailable
						, LODIndex
						, FIsLODSettingsEnabledDelegate::CreateSP(this, &FPersonaMeshDetails::IsLODInfoEditingEnabled)
						, FModifyMeshLODSettingsDelegate::CreateSP(this, &FPersonaMeshDetails::ModifyMeshLODSettings))));

					LODCategory.AddCustomBuilder(ReductionSettingsWidgetsPerLOD[LODIndex].ToSharedRef());
				}

				//Add build settings, we want those at the end of the LOD Info
				//Show them if we are not simplified or if we use ourself as the simplification base
				if (bIsbuildAvailable && (!SkelMesh->GetLODInfo(LODIndex)->bHasBeenSimplified || SkelMesh->GetLODInfo(LODIndex)->ReductionSettings.BaseLOD == LODIndex))
				{
					//Create the build setting UI Layout
					BuildSettingsWidgetsPerLOD.Add(LODIndex, MakeShareable(new FSkeletalMeshBuildSettingsLayout(SkelMesh->GetLODInfo(LODIndex)->BuildSettings
						, LODIndex
						, FIsLODSettingsEnabledDelegate::CreateLambda([](int32 InLODIndex)->bool {return true;})
						, FModifyMeshLODSettingsDelegate::CreateSP(this, &FPersonaMeshDetails::ModifyMeshLODSettings))));

					LODCategory.AddCustomBuilder(BuildSettingsWidgetsPerLOD[LODIndex].ToSharedRef());
				}

				EButtonFlags ButtonFlags = EButtonFlags::None;
				if (bShowGenerateButtons)
				{
					ButtonFlags |= EButtonFlags::Generate;
				}
				if (bShowReimportButtons)
				{
					ButtonFlags |= EButtonFlags::Reimport | EButtonFlags::ReimportNewFile;
				}
				if (bShowRemoveButtons)
				{
					ButtonFlags |= EButtonFlags::RemoveLOD | EButtonFlags::RemoveGeometry;
				}
				
				if (ButtonFlags != EButtonFlags::None)
				{
					FString MeshDescriptionReferenceIDString = LODModel.GetLODModelDeriveDataKey();
					LODCategory.AddCustomRow(LOCTEXT("LODButtonsRow", "LOD Buttons"))
						.RowTag(TEXT("LODButtons"))
						.ValueContent()
						.HAlign(HAlign_Fill)
						[
							SNew(SSkeletalLODActions)
							.LODIndex(LODIndex)
							.PersonaToolkit(GetPersonaToolkit())
							.ButtonFlags(ButtonFlags)
							.MeshDescriptionReferenceIDString(MeshDescriptionReferenceIDString)
							.BuildAvailable(bIsbuildAvailable)
							.OnApplyLODChangeClicked(this, &FPersonaMeshDetails::ApplyLODChanges, LODIndex)
							.OnRemoveLODClicked(this, &FPersonaMeshDetails::RemoveOneLOD, LODIndex)
							.OnRemoveGeometryClicked(this, &FPersonaMeshDetails::RemoveGeometryFromLOD, LODIndex)
							.OnReimportClicked(this, &FPersonaMeshDetails::OnReimportLodClicked, EReimportButtonType::Reimport, LODIndex)
							.OnReimportNewFileClicked(this, &FPersonaMeshDetails::OnReimportLodClicked, EReimportButtonType::ReimportWithNewFile, LODIndex)
						];
				}
			}

			LODCategory.SetCategoryVisibility(IsViewportLOD);
		}

		//Show the LOD custom category 
		if (SkelMeshLODCount > 1)
		{
			LODCustomModeCategory.SetCategoryVisibility(true);
			LODCustomModeCategory.SetShowAdvanced(false);
		}

		//Restore the state of the custom check LOD
		for (int32 DetailLODIndex = 0; DetailLODIndex < SkelMeshLODCount; ++DetailLODIndex)
		{
			int32 LodCheckValue = GetPersonaToolkit()->GetCustomData(CustomDataKey_LODVisibilityState + DetailLODIndex);
			if (LodCheckValue != INDEX_NONE && DetailDisplayLODs.IsValidIndex(DetailLODIndex))
			{
				DetailDisplayLODs[DetailLODIndex] = LodCheckValue > 0;
			}
		}

		//Restore the state of the custom LOD mode if its true (greater then 0)
		bool bCustomLodEditMode = GetPersonaToolkit()->GetCustomData(CustomDataKey_LODEditMode) > 0;
		if (bCustomLodEditMode)
		{
			for (int32 DetailLODIndex = 0; DetailLODIndex < SkelMeshLODCount; ++DetailLODIndex)
			{
				if (!LodCategories.IsValidIndex(DetailLODIndex))
				{
					break;
				}
				LodCategories[DetailLODIndex]->SetCategoryVisibility(DetailDisplayLODs[DetailLODIndex]);
			}
		}

		if (LodCustomCategory != nullptr)
		{
			LodCustomCategory->SetShowAdvanced(bCustomLodEditMode);
		}
	}
}

FText FPersonaMeshDetails::GetLODCustomModeNameContent(int32 LODIndex) const
{
	if (LODIndex == INDEX_NONE)
	{
		return LOCTEXT("GetLODCustomModeNameContent_None", "Custom");
	}
	return FText::Format(LOCTEXT("GetLODCustomModeNameContent", "LOD {0}"), LODIndex);
}

ECheckBoxState FPersonaMeshDetails::IsLODCustomModeCheck(int32 LODIndex) const
{
	if (LODIndex == INDEX_NONE)
	{
		return (GetPersonaToolkit()->GetCustomData(CustomDataKey_LODEditMode) > 0) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}
	return DetailDisplayLODs[LODIndex] ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void FPersonaMeshDetails::SetLODCustomModeCheck(ECheckBoxState NewState, int32 LODIndex)
{
	int32 CurrentLodIndex = 0;
	if (GetPersonaToolkit()->GetPreviewMeshComponent() != nullptr)
	{
		CurrentLodIndex = GetPersonaToolkit()->GetPreviewMeshComponent()->GetForcedLOD();
	}
	if (LODIndex == INDEX_NONE)
	{
		if (NewState == ECheckBoxState::Unchecked)
		{
			GetPersonaToolkit()->SetCustomData(CustomDataKey_LODEditMode, 0);
			SetCurrentLOD(CurrentLodIndex);
			for (int32 DetailLODIndex = 0; DetailLODIndex < LODCount; ++DetailLODIndex)
			{
				if (!LodCategories.IsValidIndex(DetailLODIndex))
				{
					break;
				}
				LodCategories[DetailLODIndex]->SetCategoryVisibility(DetailLODIndex == (CurrentLodIndex == 0 ? 0 : CurrentLodIndex - 1));
			}
		}
		else
		{
			GetPersonaToolkit()->SetCustomData(CustomDataKey_LODEditMode, 1);
			SetCurrentLOD(0);
		}
	}
	else if ((GetPersonaToolkit()->GetCustomData(CustomDataKey_LODEditMode) > 0))
	{
		DetailDisplayLODs[LODIndex] = NewState == ECheckBoxState::Checked;
		GetPersonaToolkit()->SetCustomData(CustomDataKey_LODVisibilityState + LODIndex, DetailDisplayLODs[LODIndex] ? 1 : 0);
	}

	if ((GetPersonaToolkit()->GetCustomData(CustomDataKey_LODEditMode) > 0))
	{
		for (int32 DetailLODIndex = 0; DetailLODIndex < LODCount; ++DetailLODIndex)
		{
			if (!LodCategories.IsValidIndex(DetailLODIndex))
			{
				break;
			}
			LodCategories[DetailLODIndex]->SetCategoryVisibility(DetailDisplayLODs[DetailLODIndex]);
		}
	}

	if (LodCustomCategory != nullptr)
	{
		LodCustomCategory->SetShowAdvanced((GetPersonaToolkit()->GetCustomData(CustomDataKey_LODEditMode) > 0));
	}
}

bool FPersonaMeshDetails::IsLODCustomModeEnable(int32 LODIndex) const
{
	if (LODIndex == INDEX_NONE)
	{
		// Custom checkbox is always enable
		return true;
	}
	return (GetPersonaToolkit()->GetCustomData(CustomDataKey_LODEditMode) > 0);
}

TOptional<int32> FPersonaMeshDetails::GetLodSliderMaxValue() const
{
	USkeletalMesh* SkelMesh = GetPersonaToolkit()->GetMesh();

	if(SkelMesh && !SkelMesh->IsCompiling())
	{
		return SkelMesh->GetLODNum() + PersonaMeshDetailsConstants::LodSliderExtension;
	}

	return 0;
}

void FPersonaMeshDetails::CustomizeSkinWeightProfiles(IDetailLayoutBuilder& DetailLayout)
{
	TSharedRef<IPropertyHandle> SkinWeightProfilesProperty = DetailLayout.GetProperty(FName("SkinWeightProfiles"), USkeletalMesh::StaticClass());
	DetailLayout.EditCategory("SkinWeights", LOCTEXT("SkinWeightsCategory", "Skin Weights"));

	IDetailPropertyRow& Row = DetailLayout.AddPropertyToCategory(SkinWeightProfilesProperty);	
	Row.CustomWidget(true)
	.NameContent()
	[
		SkinWeightProfilesProperty->CreatePropertyNameWidget()
	]	
	.ValueContent()
	[		
		SNew(SHorizontalBox)		
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(0.0f, 0.0f, 4.0f, 0.0f)
		[
			SkinWeightProfilesProperty->CreatePropertyValueWidget()
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(0.0f, 0.0f, 4.0f, 0.0f)
		[
			SNew(SComboButton)
			.VAlign(EVerticalAlignment::VAlign_Bottom)
			.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
			.ContentPadding(4.0f)
			.ForegroundColor(FSlateColor::UseForeground())
			.HasDownArrow(false)
			.ButtonContent()
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("Icons.PlusCircle"))
			]
			.OnGetMenuContent(this, &FPersonaMeshDetails::CreateSkinWeightProfileMenuContent)
			.ToolTipText(LOCTEXT("ImportSkinWeightButtonToolTip", "Import a new Skin Weight Profile"))
		]
	];	
}

TSharedRef<SWidget> FPersonaMeshDetails::CreateSkinWeightProfileMenuContent()
{
	FMenuBuilder AddProfileMenuBuilder(true, nullptr, nullptr, true);

	// Menu entry for importing skin weights from an FBX file
	AddProfileMenuBuilder.AddMenuEntry(LOCTEXT("ImportOverrideLabel", "Import Skin Weight Profile"), LOCTEXT("ImportOverrideToolTip", "Import a new Skin Weight Profile"),
		FSlateIcon(), FUIAction(FExecuteAction::CreateLambda([this, WeakSkeletalMeshPtr = SkeletalMeshPtr]()
	{
		if (USkeletalMesh* SkeletalMesh = WeakSkeletalMeshPtr.Get())
		{
			FScopedTransaction ScopedTransaction(LOCTEXT("ImportSkinWeightProfile", "Import Skin Weight Profile from FBX"));
			SkeletalMesh->Modify();

			FSkinWeightProfileHelpers::ImportSkinWeightProfile(SkeletalMesh);
			RequestLayoutUpdate();
		}
	})));
	
	// Add extra (sub)-menus for previously added Skin Weight Profiles
	if (USkeletalMesh* Mesh = SkeletalMeshPtr.Get())
	{
		const int32 NumLODs = Mesh->GetLODNum();
		const int32 NumProfiles = Mesh->GetNumSkinWeightProfiles();

		// In case there are already profiles stored and the current mesh has more than one LOD
		if (NumProfiles > 0 && NumLODs > 1)
		{
			// Delay adding of a separator, otherwise it'll be a random/lost separator if no submenus are generated
			bool bSeparatorAdded = false;
			
			// Add a sub menu for each profile allowing for importing skin weights for a specific (imported) LOD
			const TArray<FSkinWeightProfileInfo>& ProfilesInfo = Mesh->GetSkinWeightProfiles();
			for (int32 Index = 0; Index < NumProfiles; ++Index)
			{
				if (ProfilesInfo[Index].PerLODSourceFiles.Num() < NumLODs)
				{
					// Only add menu if there is any imported LOD beside LOD0
					bool bHaveImportedNonBaseLODs = false;
					for (int32 LODIndex = 1; LODIndex < NumLODs; ++LODIndex)
					{
						if (!Mesh->GetLODInfo(LODIndex)->bHasBeenSimplified)
						{
							bHaveImportedNonBaseLODs = true;
							break;
						}
					}
					
					if (bHaveImportedNonBaseLODs)
					{						
						if (!bSeparatorAdded)
						{
							AddProfileMenuBuilder.AddMenuSeparator();
							bSeparatorAdded = true;
						}

						AddProfileMenuBuilder.AddSubMenu(FText::FromName(ProfilesInfo[Index].Name), LOCTEXT("ProfileOptions", "Skin Weight Profile specific options"), 
							FNewMenuDelegate::CreateLambda([this, NumLODs](FMenuBuilder& MenuBuilder, const FSkinWeightProfileInfo& Info)
							{	
								for (int32 LODIndex = 0; LODIndex < NumLODs; ++LODIndex)
								{
									USkeletalMesh* SkeletalMesh = SkeletalMeshPtr.Get();

									// If we have not yet imported weights for this LOD, and if the Mesh LOD is imported (not generated)
									const FSkeletalMeshLODInfo* LODInfo = SkeletalMesh->GetLODInfo(LODIndex);

									if (!Info.PerLODSourceFiles.Contains(LODIndex) && (SkeletalMesh && LODInfo && !LODInfo->bHasBeenSimplified))
									{
										const FText Label = FText::Format(LOCTEXT("ImportOverrideText", "Import Weights for LOD {0}"), FText::AsNumber(LODIndex));
										MenuBuilder.AddMenuEntry(Label, Label,
											FSlateIcon(), FUIAction(FExecuteAction::CreateLambda([this, WeakSkeletalMeshPtr = SkeletalMeshPtr, ProfileName = Info.Name, LODIndex]()
										{
											if (USkeletalMesh* SkeletalMesh = WeakSkeletalMeshPtr.Get())
											{
												FScopedTransaction ScopedTransaction(LOCTEXT("ImportSkinWeightProfileLOD", "Import Skin Weight Profile LOD from FBX"));
												SkeletalMesh->Modify();

												FSkinWeightProfileHelpers::ImportSkinWeightProfileLOD(SkeletalMesh, ProfileName, LODIndex);
												RequestLayoutUpdate();
											}
										})));
									}
								}
							}, ProfilesInfo[Index]));
					}
				}
			}
		}
	}

	return AddProfileMenuBuilder.MakeWidget();
}

void FPersonaMeshDetails::CustomizeLODSettingsCategories(IDetailLayoutBuilder& DetailLayout)
{
	USkeletalMesh* SkelMesh = GetPersonaToolkit()->GetMesh();
	LODCount = SkelMesh->GetLODNum();

	UpdateLODNames();

	IDetailCategoryBuilder& LODSettingsCategory = DetailLayout.EditCategory("LodSettings", LOCTEXT("LodSettingsCategory", "LOD Settings"), ECategoryPriority::TypeSpecific);

	TSharedPtr<SWidget> LodTextPtr;

	LODSettingsCategory.AddCustomRow(LOCTEXT("LODImport", "LOD Import"))
	.RowTag("LODImport")
	.NameContent()
	[
		SAssignNew(LodTextPtr, STextBlock)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.Text(LOCTEXT("LODImport", "LOD Import"))
	]
	.ValueContent()
	[
		SNew(STextComboBox)
		.ContentPadding(0.f)
		.OptionsSource(&LODNames)
		.InitiallySelectedItem(LODNames[0])
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.OnSelectionChanged(this, &FPersonaMeshDetails::OnImportLOD, &DetailLayout)
	];

	// Add Number of LODs slider.
	const int32 MinAllowedLOD = 1;
	LODSettingsCategory.AddCustomRow(LOCTEXT("NumberOfLODs", "Number of LODs"))
	.RowTag("NumberOfLODs")
	.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateLambda([]()->EVisibility { return IsAutoMeshReductionAvailable()? EVisibility::Visible : EVisibility::Hidden; })))
	.NameContent()
	[
		SNew(STextBlock)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.Text(LOCTEXT("NumberOfLODs", "Number of LODs"))
	]
	.ValueContent()
	[
		SNew(SSpinBox<int32>)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.Value(this, &FPersonaMeshDetails::GetLODCount)
		.OnValueChanged(this, &FPersonaMeshDetails::OnLODCountChanged)
		.OnValueCommitted(this, &FPersonaMeshDetails::OnLODCountCommitted)
		.MinValue(MinAllowedLOD)
		.MaxValue(this, &FPersonaMeshDetails::GetLodSliderMaxValue)
		.ToolTipText(this, &FPersonaMeshDetails::GetLODCountTooltip)
		.IsEnabled(IsAutoMeshReductionAvailable())
	];

	LODSettingsCategory.AddCustomRow(LOCTEXT("ApplyChanges", "Apply Changes"))
	.RowTag("ApplyChanges")
	.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateLambda([]()->EVisibility { return IsAutoMeshReductionAvailable() ? EVisibility::Visible : EVisibility::Hidden; })))
	.ValueContent()
	.HAlign(HAlign_Left)
	[
		SNew(SButton)
		.OnClicked(this, &FPersonaMeshDetails::OnApplyChanges)
		.IsEnabled(this, &FPersonaMeshDetails::IsGenerateAvailable)
		[
			SNew(STextBlock)
			.Text(this, &FPersonaMeshDetails::GetApplyButtonText)
			.Font(DetailLayout.GetDetailFont())
		]
	];

	// add lod setting assets
	TSharedPtr<IPropertyHandle> LODSettingAssetPropertyHandle = DetailLayout.GetProperty(USkeletalMesh::GetLODSettingsMemberName(), USkeletalMesh::StaticClass());
	DetailLayout.HideProperty(LODSettingAssetPropertyHandle);
	LODSettingsCategory.AddCustomRow(LODSettingAssetPropertyHandle->GetPropertyDisplayName())
	.RowTag(TEXT("LodSettingsAsset"))
	.NameContent()
	[
		LODSettingAssetPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MinDesiredWidth(150)
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SObjectPropertyEntryBox)
			.AllowedClass(USkeletalMeshLODSettings::StaticClass())
			.PropertyHandle(LODSettingAssetPropertyHandle)
			.ThumbnailPool(DetailLayout.GetThumbnailPool())
			.OnObjectChanged(this, &FPersonaMeshDetails::OnLODSettingsSelected)
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SButton)
			.ToolTipText(LOCTEXT("GenerateAsset_Tooltip", "Save current LOD info to new or existing asset and use it"))
			.OnClicked(this, &FPersonaMeshDetails::OnSaveLODSettings)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("GenerateAsset_Lable", "Generate Asset..."))
				.Font(DetailLayout.GetDetailFont())
			]
		]
	];

	LODSettingsCategory.AddCustomRow(LOCTEXT("NoRefStreamingLODBias", "NoRef Streaming LOD Bias"))
	.RowTag("NoRefStreamingLODBias")
	.IsEnabled(TAttribute<bool>::CreateLambda([this]() { return GetLODCount() > 1; }))
	.NameContent()
	[
		SNew(STextBlock)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.Text(LOCTEXT("NoRefStreamingLODBias", "NoRef Streaming LOD Bias"))
	]
	.ValueContent()
	.MinDesiredWidth(float(SkelMesh->GetNoRefStreamingLODBias().PerQuality.Num() + 1) * 125.f)
	.MaxDesiredWidth(float((int32)EPerQualityLevels::Num + 1) * 125.f)
	[
		SNew(SPerQualityLevelPropertiesWidget)
		.OnGenerateWidget(this, &FPersonaMeshDetails::GetNoRefStreamingLODBiasWidget)
		.OnAddEntry(this, &FPersonaMeshDetails::AddNoRefStreamingLODBiasOverride)
		.OnRemoveEntry(this, &FPersonaMeshDetails::RemoveNoRefStreamingLODBiasOverride)
		.EntryNames(this, &FPersonaMeshDetails::GetNoRefStreamingLODBiasOverrideNames)
	];

	TAttribute<bool> IsQualityLevelLodEnabled = TAttribute<bool>::CreateLambda([this]() { return FPersonaMeshDetails::IsQualityLevelMinLodEnable(); });
	TAttribute<bool> IsPerPlatformMinLodEnabled = TAttribute<bool>::CreateLambda([this]() { return FPersonaMeshDetails::IsMinLodEnable(); });

	TSharedPtr<IPropertyHandle> MinLODPropertyHandle = DetailLayout.GetProperty(USkeletalMesh::GetMinLodMemberName(), USkeletalMesh::StaticClass());
	IDetailPropertyRow& MinLODRow = LODSettingsCategory.AddProperty(MinLODPropertyHandle);
	MinLODRow.EditCondition(IsPerPlatformMinLodEnabled, nullptr);
	DetailLayout.HideProperty(MinLODPropertyHandle);

	TSharedRef<IPropertyHandle> LODInfoProperty = DetailLayout.GetProperty(FName("SourceModels"), USkeletalMesh::StaticClass());
	DetailLayout.HideProperty(LODInfoProperty);

	TSharedPtr<IPropertyHandle> QualityLevelMinLODPropertyHandle = DetailLayout.GetProperty(USkeletalMesh::GetQualityLevelMinLodMemberName(), USkeletalMesh::StaticClass());
	DetailLayout.HideProperty(QualityLevelMinLODPropertyHandle);

	LODSettingsCategory.AddCustomRow(LOCTEXT("QualityLevelMinLOD", "Quality Level Min LOD"))
		.RowTag("QualityLevelMinLOD")
		.IsEnabled(IsQualityLevelLodEnabled)
		.EditCondition(IsQualityLevelLodEnabled, nullptr)
		.NameContent()
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.Padding(0.0f, 4.0f)
			.HAlign(HAlign_Left)
			.AutoWidth()
			[
			SNew(STextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(LOCTEXT("QualityLevelMinLOD", "Quality Level Min LOD"))
			]
		+ SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			.Padding(50.0f, 0.0f)
			.AutoWidth()
			[
				SNew(SButton)
				.OnClicked(this, &FPersonaMeshDetails::ResetToDefault)
				.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
				.ToolTipText(LOCTEXT("QualityLevelMinLodToolTip", "Clear MinLOD conversion data"))
				.ForegroundColor(FSlateColor::UseForeground())
				.IsEnabled(TAttribute<bool>::CreateLambda([this]()
					{
						return GetMinLod().PerPlatform.Num() != 0 || GetMinLod().Default != 0;
					}))
				.Content()
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("Icons.Delete"))
				]
			]
		]
		.ValueContent()
		.MinDesiredWidth((float)(SkelMesh->GetQualityLevelMinLod().PerQuality.Num() + 1) * 125.0f)
		.MaxDesiredWidth((float)((int32)EPerQualityLevels::Num + 1) * 125.0f)
		[
			SNew(SPerQualityLevelPropertiesWidget)
			.OnGenerateWidget(this, &FPersonaMeshDetails::GetMinQualityLevelLodWidget)
			.OnAddEntry(this, &FPersonaMeshDetails::AddMinLodQualityLevelOverride)
			.OnRemoveEntry(this, &FPersonaMeshDetails::RemoveMinLodQualityLevelOverride)
			.EntryNames(this, &FPersonaMeshDetails::GetMinQualityLevelLodOverrideNames)
		];

	TSharedPtr<IPropertyHandle> DisableBelowMinLodStrippingPropertyHandle = DetailLayout.GetProperty(USkeletalMesh::GetDisableBelowMinLodStrippingMemberName(), USkeletalMesh::StaticClass());
	IDetailPropertyRow& DisableBelowMinLodStrippingRow = LODSettingsCategory.AddProperty(DisableBelowMinLodStrippingPropertyHandle);
	DisableBelowMinLodStrippingRow.IsEnabled(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateSP(this, &FPersonaMeshDetails::IsLODInfoEditingEnabled, -1)));
	DetailLayout.HideProperty(DisableBelowMinLodStrippingPropertyHandle);

	TSharedPtr<IPropertyHandle> bOverrideLODStreamingSettingsHandle = DetailLayout.GetProperty(USkeletalMesh::GetOverrideLODStreamingSettingsMemberName(), USkeletalMesh::StaticClass());
	IDetailPropertyRow& bOverrideLODStreamingSettingsRow = LODSettingsCategory.AddProperty(bOverrideLODStreamingSettingsHandle);
	bOverrideLODStreamingSettingsRow.IsEnabled(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateSP(this, &FPersonaMeshDetails::IsLODInfoEditingEnabled, -1)));
	DetailLayout.HideProperty(bOverrideLODStreamingSettingsHandle);

	TSharedPtr<IPropertyHandle> bSupportLODStreamingPropertyHandle = DetailLayout.GetProperty(USkeletalMesh::GetSupportLODStreamingMemberName(), USkeletalMesh::StaticClass());
	IDetailPropertyRow& bSupportLODStreamingRow = LODSettingsCategory.AddProperty(bSupportLODStreamingPropertyHandle);
	bSupportLODStreamingRow.IsEnabled(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateSP(this, &FPersonaMeshDetails::IsLODInfoEditingEnabled, -1)));
	DetailLayout.HideProperty(bSupportLODStreamingPropertyHandle);

	TSharedPtr<IPropertyHandle> MaxNumStreamedLODsPropertyHandle = DetailLayout.GetProperty(USkeletalMesh::GetMaxNumStreamedLODsMemberName(), USkeletalMesh::StaticClass());
	IDetailPropertyRow& MaxNumStreamedLODsRow = LODSettingsCategory.AddProperty(MaxNumStreamedLODsPropertyHandle);
	MaxNumStreamedLODsRow.IsEnabled(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateSP(this, &FPersonaMeshDetails::IsLODInfoEditingEnabled, -1)));
	DetailLayout.HideProperty(MaxNumStreamedLODsPropertyHandle);

	TSharedPtr<IPropertyHandle> MaxNumOptionalLODsPropertyHandle = DetailLayout.GetProperty(USkeletalMesh::GetMaxNumOptionalLODsMemberName(), USkeletalMesh::StaticClass());
	IDetailPropertyRow& MaxNumOptionalLODsRow = LODSettingsCategory.AddProperty(MaxNumOptionalLODsPropertyHandle);
	MaxNumOptionalLODsRow.IsEnabled(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateSP(this, &FPersonaMeshDetails::IsLODInfoEditingEnabled, -1)));
	DetailLayout.HideProperty(MaxNumOptionalLODsPropertyHandle);
}

// save LOD settings
FReply FPersonaMeshDetails::OnSaveLODSettings()
{
	USkeletalMesh* SkelMesh = GetPersonaToolkit()->GetMesh();

	if (SkelMesh)
	{
		const FString DefaultPackageName = SkelMesh->GetPathName();
		const FString DefaultPath = FPackageName::GetLongPackagePath(DefaultPackageName);
		const FString DefaultName = SkelMesh->GetName() + TEXT("_LODSettings");

		// Initialize SaveAssetDialog config
		FSaveAssetDialogConfig SaveAssetDialogConfig;
		SaveAssetDialogConfig.DialogTitleOverride = LOCTEXT("CreateLODSettings", "Create LOD Settings from existing settings");
		SaveAssetDialogConfig.DefaultPath = DefaultPath;
		SaveAssetDialogConfig.DefaultAssetName = DefaultName;
		SaveAssetDialogConfig.ExistingAssetPolicy = ESaveAssetDialogExistingAssetPolicy::AllowButWarn;
		SaveAssetDialogConfig.AssetClassNames.Add(USkeletalMeshLODSettings::StaticClass()->GetClassPathName());

		FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
		FString SaveObjectPath = ContentBrowserModule.Get().CreateModalSaveAssetDialog(SaveAssetDialogConfig);
		if (!SaveObjectPath.IsEmpty())
		{
			const FString SavePackageName = FPackageName::ObjectPathToPackageName(SaveObjectPath);
			const FString SavePackagePath = FPaths::GetPath(SavePackageName);
			const FString SaveAssetName = FPaths::GetBaseFilename(SavePackageName);

			// create package and create object
			UPackage* Package = CreatePackage( *SavePackageName);
			USkeletalMeshLODSettings* NewLODSettingAsset = NewObject<USkeletalMeshLODSettings>(Package, *SaveAssetName, RF_Public | RF_Standalone);
			if (NewLODSettingAsset && SkelMesh->GetLODNum() > 0)
			{
				// update mapping information on the class
				NewLODSettingAsset->SetLODSettingsFromMesh(SkelMesh);

				// save mapper class
				FString const PackageName = Package->GetName();
				FString const PackageFileName = FPackageName::LongPackageNameToFilename(PackageName, FPackageName::GetAssetPackageExtension());

				FSavePackageArgs SaveArgs;
				SaveArgs.TopLevelFlags = RF_Standalone;
				SaveArgs.SaveFlags = SAVE_NoError;
				UPackage::SavePackage(Package, nullptr, *PackageFileName, SaveArgs);

				// set the property back to SkelMesh;
				SkelMesh->SetLODSettings(NewLODSettingAsset);
			}
		}
	}

	return FReply::Handled();
}

void FPersonaMeshDetails::OnLODSettingsSelected(const FAssetData& AssetData)
{
	USkeletalMesh* SkelMesh = GetPersonaToolkit()->GetMesh();
	if (SkelMesh)
	{
		USkeletalMeshLODSettings* SelectedSettingsAsset = Cast<USkeletalMeshLODSettings>(AssetData.GetAsset());
		if (SelectedSettingsAsset)
		{
			SelectedSettingsAsset->SetLODSettingsToMesh(SkelMesh);
		}
	}
}

bool FPersonaMeshDetails::IsQualityLevelMinLodEnable() const
{
	return GEngine->UseSkeletalMeshMinLODPerQualityLevels;
}

bool FPersonaMeshDetails::IsMinLodEnable() const
{
	return !GEngine->UseSkeletalMeshMinLODPerQualityLevels;
}

bool FPersonaMeshDetails::IsLODInfoEditingEnabled(int32 LODIndex) const
{
	USkeletalMesh* SkelMesh = GetPersonaToolkit()->GetMesh();
	if (SkelMesh && !SkelMesh->IsCompiling())
	{
		if (SkelMesh->GetLODSettings())
		{
			// if LODIndex == -1, we don't care about lod index
			if (LODIndex == -1)
			{
				return false;
			}

			if (SkelMesh->GetLODSettings()->GetNumberOfSettings() > LODIndex)
			{
				return false;
			}
		}
	}

	return true;
}

void FPersonaMeshDetails::ModifyMeshLODSettings(int32 LODIndex)
{
	USkeletalMesh* SkelMesh = GetPersonaToolkit()->GetMesh();
	if (SkelMesh)
	{
		SkelMesh->Modify();
	}
}

void FPersonaMeshDetails::OnAssetPostLODImported(UObject* InObject, int32 InLODIndex)
{
	if (InObject == GetPersonaToolkit()->GetMesh())
	{
		ForceLayoutRebuild();
	}
}

void FPersonaMeshDetails::OnAssetReimport(UObject* InObject)
{
	if (InObject == GetPersonaToolkit()->GetMesh())
	{
		ForceLayoutRebuild();
	}
}

void FPersonaMeshDetails::OnImportLOD(TSharedPtr<FString> NewValue, ESelectInfo::Type SelectInfo, IDetailLayoutBuilder* DetailLayout)
{
	int32 LODIndex = 0;
	if (LODNames.Find(NewValue, LODIndex) && LODIndex > 0)
	{
		USkeletalMesh* SkelMesh = GetPersonaToolkit()->GetMesh();
		check(SkelMesh);

		FbxMeshUtils::ImportMeshLODDialog(SkelMesh, LODIndex);
	}
}

int32 FPersonaMeshDetails::GetLODCount() const
{
	return LODCount;
}

void FPersonaMeshDetails::OnLODCountChanged(int32 NewValue)
{
	LODCount = FMath::Max<int32>(NewValue, 1);

	UpdateLODNames();
}

void FPersonaMeshDetails::OnLODCountCommitted(int32 InValue, ETextCommit::Type CommitInfo)
{
	OnLODCountChanged(InValue);
}

void FPersonaMeshDetails::ForceLayoutRebuild()
{
	// Force a full rebuild of the layout.
	
	//Unbound all UI delegate in case the user move the cursor during the refresh details	
	for (TPair<int32, TSharedPtr<FSkeletalMeshBuildSettingsLayout>>& BuildSettingWidgetPair : BuildSettingsWidgetsPerLOD)
	{
		TSharedPtr<FSkeletalMeshBuildSettingsLayout>& LodBuildSettings = BuildSettingWidgetPair.Value;
		LodBuildSettings->UnbindBuildSettings();
	}
	for (TPair<int32, TSharedPtr<FSkeletalMeshReductionSettingsLayout>>& ReductionSettingWidgetPair : ReductionSettingsWidgetsPerLOD)
	{
		TSharedPtr<FSkeletalMeshReductionSettingsLayout>& LodReductionSettings = ReductionSettingWidgetPair.Value;
		LodReductionSettings->UnbindReductionSettings();
	}
	
	MeshDetailLayout->ForceRefreshDetails();
}

void FPersonaMeshDetails::RequestLayoutUpdate()
{
	// Update when possible but don't fully rebuild the layout. Useful to ensure that complex widgets, such as FPerPlatformX specializations, get
	// time to update properly.
	MeshDetailLayout->GetPropertyUtilities()->RequestRefresh();
}

void FPersonaMeshDetails::OnNoRefStreamingLODBiasChanged(int32 NewValue, FName QualityLevel)
{
	USkeletalMesh* SkelMesh = GetPersonaToolkit()->GetMesh();
	check(SkelMesh);

	{
		FSkinnedMeshComponentRecreateRenderStateContext ReregisterContext(SkelMesh);
		NewValue = FMath::Clamp<int32>(NewValue, -1, SkelMesh->GetLODNum() - 1);
		FPerQualityLevelInt NoRefStreamingLODBias = SkelMesh->GetNoRefStreamingLODBias();
		int32 QLKey = QualityLevelProperty::FNameToQualityLevel(QualityLevel);
		if (QualityLevel == NAME_None || QLKey == INDEX_NONE)
		{
			NoRefStreamingLODBias.Default = NewValue;
		}
		else
		{
			int32* ValuePtr = NoRefStreamingLODBias.PerQuality.Find(QLKey);
			if (ValuePtr != nullptr)
			{
				*ValuePtr = NewValue;
			}
		}
		SkelMesh->SetNoRefStreamingLODBias(MoveTemp(NoRefStreamingLODBias));
		SkelMesh->Modify();
	}
	RequestLayoutUpdate();
}

void FPersonaMeshDetails::OnNoRefStreamingLODBiasCommitted(int32 InValue, ETextCommit::Type CommitInfo, FName QualityLevel)
{
	OnNoRefStreamingLODBiasChanged(InValue, QualityLevel);
}

int32 FPersonaMeshDetails::GetNoRefStreamingLODBias(FName QualityLevel) const
{
	USkeletalMesh* SkelMesh = GetPersonaToolkit()->GetMesh();
	check(SkelMesh);
	if (SkelMesh->IsCompiling())
	{
		return 0;
	}

	int32 QLKey = QualityLevelProperty::FNameToQualityLevel(QualityLevel);
	const int32* ValuePtr = (QualityLevel == NAME_None) ? nullptr : SkelMesh->GetNoRefStreamingLODBias().PerQuality.Find(QLKey);
	return (ValuePtr != nullptr) ? *ValuePtr : SkelMesh->GetNoRefStreamingLODBias().Default;
}

TSharedRef<SWidget> FPersonaMeshDetails::GetNoRefStreamingLODBiasWidget(FName QualityLevelName) const
{
	return SNew(SSpinBox<int32>)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.Value(this, &FPersonaMeshDetails::GetNoRefStreamingLODBias, QualityLevelName)
		.OnValueChanged(const_cast<FPersonaMeshDetails*>(this), &FPersonaMeshDetails::OnNoRefStreamingLODBiasChanged, QualityLevelName)
		.OnValueCommitted(const_cast<FPersonaMeshDetails*>(this), &FPersonaMeshDetails::OnNoRefStreamingLODBiasCommitted, QualityLevelName)
		.MinValue(-1)
		.MaxValue(GetPersonaToolkit()->GetMesh()->GetLODNum() - 1)
		.ToolTipText(this, &FPersonaMeshDetails::GetNoRefStreamingLODBiasTooltip)
		.IsEnabled(FPersonaMeshDetails::GetLODCount() > 1);
}

bool FPersonaMeshDetails::AddNoRefStreamingLODBiasOverride(FName QualityLevelName)
{
	FScopedTransaction Transaction(LOCTEXT("AddNoRefStreamingLODBiasOverride", "Add NoRef Streaming LOD Bias Override"));
	USkeletalMesh* SkelMesh = GetPersonaToolkit()->GetMesh();
	check(SkelMesh);
	SkelMesh->Modify();
	int32 QLKey = QualityLevelProperty::FNameToQualityLevel(QualityLevelName);
	if (SkelMesh->GetNoRefStreamingLODBias().PerQuality.Find(QLKey) == nullptr)
	{
		FPerQualityLevelInt NoRefStreamingLODBias = SkelMesh->GetNoRefStreamingLODBias();
		int32 Value = NoRefStreamingLODBias.Default;
		NoRefStreamingLODBias.PerQuality.Add(QLKey, Value);
		SkelMesh->SetNoRefStreamingLODBias(MoveTemp(NoRefStreamingLODBias));
		OnNoRefStreamingLODBiasChanged(Value, QualityLevelName);
		return true;
	}
	return false;
}

bool FPersonaMeshDetails::RemoveNoRefStreamingLODBiasOverride(FName QualityLevelName)
{
	FScopedTransaction Transaction(LOCTEXT("RemoveNoRefStreamingLODBiasOverride", "Remove NoRef Streaming LOD Bias Override"));
	USkeletalMesh* SkelMesh = GetPersonaToolkit()->GetMesh();
	check(SkelMesh);
	SkelMesh->Modify();

	FPerQualityLevelInt NoRefStreamingLODBias = SkelMesh->GetNoRefStreamingLODBias();
	int32 QL = QualityLevelProperty::FNameToQualityLevel(QualityLevelName);
	if (QL != INDEX_NONE && NoRefStreamingLODBias.PerQuality.Remove(QL) != 0)
	{
		int32 Value = NoRefStreamingLODBias.Default;
		SkelMesh->SetNoRefStreamingLODBias(MoveTemp(NoRefStreamingLODBias));
		OnNoRefStreamingLODBiasChanged(Value, QualityLevelName);
		return true;
	}
	return false;
}

TArray<FName> FPersonaMeshDetails::GetNoRefStreamingLODBiasOverrideNames() const
{
	USkeletalMesh* SkelMesh = GetPersonaToolkit()->GetMesh();
	check(SkelMesh);
	TArray<FName> OverrideNames;
	if (SkelMesh->IsCompiling())
	{
		return OverrideNames;
	}
	
	for (const TPair<int32, int32>& Pair : SkelMesh->GetNoRefStreamingLODBias().PerQuality)
	{
		OverrideNames.Add(QualityLevelProperty::QualityLevelToFName(Pair.Key));
	}
	OverrideNames.Sort(FNameLexicalLess());
	return OverrideNames;
}

FText FPersonaMeshDetails::GetNoRefStreamingLODBiasTooltip() const
{
	return LOCTEXT("NoRefStreamingLODBiasTooltip", "LOD bias for preloading no-ref mesh LODs. To use platform default, set to -1.");
}

FPerPlatformInt FPersonaMeshDetails::GetMinLod() 
{
	USkeletalMesh* SkelMesh = GetPersonaToolkit()->GetMesh();
	check(SkelMesh);

	return SkelMesh->IsCompiling() ? 0 : SkelMesh->GetMinLod();
}

int32 FPersonaMeshDetails::GetMinQualityLevelLod(FName QualityLevel) const
{
	USkeletalMesh* SkelMesh = GetPersonaToolkit()->GetMesh();
	check(SkelMesh);

	if (SkelMesh->IsCompiling())
	{
		return 0;
	}

	int32 QLKey = QualityLevelProperty::FNameToQualityLevel(QualityLevel);
	const int32* ValuePtr = (QualityLevel == NAME_None) ? nullptr : SkelMesh->GetQualityLevelMinLod().PerQuality.Find(QLKey);
	return (ValuePtr != nullptr) ? *ValuePtr : SkelMesh->GetQualityLevelMinLod().Default;
}

void FPersonaMeshDetails::OnMinQualityLevelLodChanged(int32 NewValue, FName QualityLevel)
{
	USkeletalMesh* SkelMesh = GetPersonaToolkit()->GetMesh();
	check(SkelMesh);

	{
		FSkinnedMeshComponentRecreateRenderStateContext ReregisterContext(SkelMesh);
		NewValue = FMath::Clamp<int32>(NewValue, 0, MAX_STATIC_MESH_LODS - 1);
		FPerQualityLevelInt MinLOD = SkelMesh->GetQualityLevelMinLod();
		int32 QLKey = QualityLevelProperty::FNameToQualityLevel(QualityLevel);
		if (QualityLevel == NAME_None || QLKey == INDEX_NONE)
		{
			MinLOD.Default = NewValue;
		}
		else
		{
			int32* ValuePtr = MinLOD.PerQuality.Find(QLKey);
			if (ValuePtr != nullptr)
			{
				*ValuePtr = NewValue;
			}
		}
		SkelMesh->SetQualityLevelMinLod(MoveTemp(MinLOD));
		SkelMesh->Modify();
	}
	RequestLayoutUpdate();
}

void FPersonaMeshDetails::OnMinQualityLevelLodCommitted(int32 InValue, ETextCommit::Type CommitInfo, FName QualityLevel)
{
	OnMinQualityLevelLodChanged(InValue, QualityLevel);
}

TSharedRef<SWidget> FPersonaMeshDetails::GetMinQualityLevelLodWidget(FName QualityLevelName) const
{
	return SNew(SSpinBox<int32>)
		.Font(IDetailLayoutBuilder::GetDetailFont())
		.Value(this, &FPersonaMeshDetails::GetMinQualityLevelLod, QualityLevelName)
		.OnValueChanged(const_cast<FPersonaMeshDetails*>(this), &FPersonaMeshDetails::OnMinQualityLevelLodChanged, QualityLevelName)
		.OnValueCommitted(const_cast<FPersonaMeshDetails*>(this), &FPersonaMeshDetails::OnMinQualityLevelLodCommitted, QualityLevelName)
		.MinValue(0)
		.MaxValue(MAX_STATIC_MESH_LODS)
		.ToolTipText(LOCTEXT("QualityLevelMinLodTooltip", "The minimum quality level LOD to use for rendering.  This can be overridden in components."))
		.IsEnabled(FPersonaMeshDetails::GetLODCount() > 1);
}

bool FPersonaMeshDetails::AddMinLodQualityLevelOverride(FName QualityLevelName)
{
	FScopedTransaction Transaction(LOCTEXT("AddMinLODQualityLevelOverride", "Add Min LOD Quality Level Override"));
	
	USkeletalMesh* SkelMesh = GetPersonaToolkit()->GetMesh();
	check(SkelMesh);
	SkelMesh->Modify();
	const int32 QLKey = QualityLevelProperty::FNameToQualityLevel(QualityLevelName);
	if (SkelMesh->GetQualityLevelMinLod().PerQuality.Find(QLKey) == nullptr)
	{
		FPerQualityLevelInt MinLOD = SkelMesh->GetQualityLevelMinLod();
		const int32 Value = MinLOD.Default;
		MinLOD.PerQuality.Add(QLKey, Value);
		SkelMesh->SetQualityLevelMinLod(MoveTemp(MinLOD));
		OnMinQualityLevelLodChanged(Value, QualityLevelName);
		return true;
	}
	return false;
}

bool FPersonaMeshDetails::RemoveMinLodQualityLevelOverride(FName QualityLevelName)
{
	FScopedTransaction Transaction(LOCTEXT("RemoveMinLODQualityLevelOverride", "Remove Min LOD Quality Level Override"));
	
	USkeletalMesh* SkelMesh = GetPersonaToolkit()->GetMesh();
	check(SkelMesh);
	SkelMesh->Modify();

	FPerQualityLevelInt MinLOD = SkelMesh->GetQualityLevelMinLod();
	const int32 QL = QualityLevelProperty::FNameToQualityLevel(QualityLevelName);
	if (QL != INDEX_NONE && MinLOD.PerQuality.Remove(QL) != 0)
	{
		const int32 Value = MinLOD.Default;
		SkelMesh->SetQualityLevelMinLod(MoveTemp(MinLOD));
		OnMinQualityLevelLodChanged(Value, QualityLevelName);
		return true;
	}
	return false;
}

TArray<FName> FPersonaMeshDetails::GetMinQualityLevelLodOverrideNames() const
{
	USkeletalMesh* SkelMesh = GetPersonaToolkit()->GetMesh();
	check(SkelMesh);
	TArray<FName> OverrideNames;
	if (SkelMesh->IsCompiling())
	{
		return OverrideNames;
	}
	
	for (const TPair<int32, int32>& Pair : SkelMesh->GetQualityLevelMinLod().PerQuality)
	{
		OverrideNames.Add(QualityLevelProperty::QualityLevelToFName(Pair.Key));
	}
	OverrideNames.Sort(FNameLexicalLess());
	return OverrideNames;
}

FReply FPersonaMeshDetails::ResetToDefault()
{
	if (FPersonaMeshDetails::IsQualityLevelMinLodEnable())
	{
		USkeletalMesh* SkelMesh = GetPersonaToolkit()->GetMesh();
		check(SkelMesh);

		FPerPlatformInt PlatformMinLOD;
		SkelMesh->SetMinLod(MoveTemp(PlatformMinLOD));
		SkelMesh->Modify();

		RequestLayoutUpdate();
	}
	return FReply::Handled();
}

FReply FPersonaMeshDetails::OnApplyChanges()
{
	ApplyChanges();
	return FReply::Handled();
}

void FPersonaMeshDetails::RestoreNonReducedLOD(int32 LODIndex)
{
	USkeletalMesh* SkelMesh = GetPersonaToolkit()->GetMesh();
	check(SkelMesh);

	FSkeletalMeshLODInfo* CurrentLODInfo = SkelMesh->GetLODInfo(LODIndex);
	const bool bIsReductionActive = SkelMesh->IsReductionActive(LODIndex);
	const bool bIsLODModelbuildDataAvailable = SkelMesh->HasMeshDescription(LODIndex);

	if (CurrentLODInfo->bHasBeenSimplified
		&& !bIsReductionActive
		&& bIsLODModelbuildDataAvailable)
	{
		CurrentLODInfo->bHasBeenSimplified = false;
	}
}

FReply FPersonaMeshDetails::ApplyLODChanges(int32 LODIndex)
{
	USkeletalMesh* SkelMesh = GetPersonaToolkit()->GetMesh();
	check(SkelMesh);

	if (SkelMesh->GetLODInfo(LODIndex) == nullptr)
	{
		return FReply::Handled();
	}
	
	// inject a transaction and mark the mesh as modified so we can undo this state
	FText TransactionText = FText::Format(LOCTEXT("PersonaReductionApplyLODChanges", "LOD{0} generation"), LODIndex);
	FScopedTransaction Transaction(TransactionText);
	SkelMesh->Modify();
	

	FScopedSuspendAlternateSkinWeightPreview ScopedSuspendAlternateSkinnWeightPreview(SkelMesh);
	{
		FScopedSkeletalMeshPostEditChange ScopedPostEditChange(SkelMesh);
		check(SkelMesh);
		FSkeletalMeshLODInfo* LODInfo = SkelMesh->GetLODInfo(LODIndex);
		check(LODInfo);
		int32 SourceLODIndex = LODIndex;
		bool bHasBeenSimplified = LODInfo ? LODInfo->bHasBeenSimplified : false;
		if(bHasBeenSimplified)
		{
			SourceLODIndex = LODInfo->ReductionSettings.BaseLOD;
		}
		bool bSrcBuildDataAvailable = SkelMesh->HasMeshDescription(SourceLODIndex);
		if (!bSrcBuildDataAvailable)
		{
			SkelMesh->InvalidateDeriveDataCacheGUID();
			RegenerateLOD(LODIndex);
		}
		else
		{
			if (LODIndex == 0) //Base LOD must update the asset import data
			{
				//Update the Asset Import Data
				UFbxSkeletalMeshImportData* SKImportData = Cast<UFbxSkeletalMeshImportData>(SkelMesh->GetAssetImportData());
				if (!SKImportData)
				{
					if (UInterchangeAssetImportData* InterchangeAssetImportData = Cast<UInterchangeAssetImportData>(SkelMesh->GetAssetImportData()))
					{
						if (UInterchangeManager::GetInterchangeManager().ConvertImportData(InterchangeAssetImportData, UFbxSkeletalMeshImportData::StaticClass(), reinterpret_cast<UObject**>(&SKImportData)))
						{
							SKImportData->SourceData = InterchangeAssetImportData->SourceData;
						}
					}
				}

				if (SKImportData)
				{
					check(bSrcBuildDataAvailable);
					{
						if (!LODInfo->BuildSettings.bRecomputeNormals && !LODInfo->BuildSettings.bRecomputeTangents)
						{
							SKImportData->NormalImportMethod = EFBXNormalImportMethod::FBXNIM_ImportNormalsAndTangents;
						}
						else
						{
							SKImportData->NormalImportMethod = LODInfo->BuildSettings.bRecomputeNormals ? EFBXNormalImportMethod::FBXNIM_ComputeNormals : EFBXNormalImportMethod::FBXNIM_ImportNormals;
							SKImportData->NormalGenerationMethod = LODInfo->BuildSettings.bUseMikkTSpace ? EFBXNormalGenerationMethod::MikkTSpace : EFBXNormalGenerationMethod::BuiltIn;
						}
						SKImportData->bComputeWeightedNormals = LODInfo->BuildSettings.bComputeWeightedNormals;
						SKImportData->ThresholdPosition = LODInfo->BuildSettings.ThresholdPosition;
						SKImportData->ThresholdTangentNormal = LODInfo->BuildSettings.ThresholdTangentNormal;
						SKImportData->ThresholdUV = LODInfo->BuildSettings.ThresholdUV;
						SKImportData->MorphThresholdPosition = LODInfo->BuildSettings.MorphThresholdPosition;
					}
				}

				UInterchangeAssetImportData* NewInterchangeAssetImportData = nullptr;
				UInterchangeManager::GetInterchangeManager().ConvertImportData(SKImportData, UInterchangeAssetImportData::StaticClass(), reinterpret_cast<UObject**>(&NewInterchangeAssetImportData));
				if (IsValid(NewInterchangeAssetImportData))
				{
					if (SKImportData)
					{
						NewInterchangeAssetImportData->SourceData = SKImportData->SourceData;
					}
					SkelMesh->SetAssetImportData(NewInterchangeAssetImportData);
				}
			}

			RestoreNonReducedLOD(LODIndex);
		}
		(void)SkelMesh->MarkPackageDirty();
	}

	RequestLayoutUpdate();
	
	if (SkelMesh->GetMorphTargets().Num() > 0)
	{
		GetPersonaToolkit()->GetPreviewScene()->BroadcastOnMorphTargetsChanged();
	}

	// Update mesh required bones to match the new LOD settings
	GetPersonaToolkit()->GetPreviewScene()->GetPreviewMeshComponent()->RecalcRequiredBones(LODIndex);
	// Force an update (this will update the Skeleton Tree to match the bones in the preview skeletal mesh required bones)
	GetPersonaToolkit()->GetPreviewScene()->BroadcastOnLODChanged();
	
	return FReply::Handled();
}

void FPersonaMeshDetails::RegenerateOneLOD(int32 LODIndex)
{
	USkeletalMesh* SkelMesh = GetPersonaToolkit()->GetMesh();
	check(SkelMesh);
	FScopedSuspendAlternateSkinWeightPreview ScopedSuspendAlternateSkinnWeightPreview(SkelMesh);

	if (SkelMesh->IsValidLODIndex(LODIndex))
	{
		FScopedSkeletalMeshPostEditChange ScopedPostEditChange(SkelMesh);
		FSkeletalMeshLODInfo& CurrentLODInfo = *(SkelMesh->GetLODInfo(LODIndex));
		
		bool bIsLODModelbuildDataAvailable = SkelMesh->GetImportedModel()->LODModels.IsValidIndex(LODIndex) && SkelMesh->HasMeshDescription(LODIndex);
		if (LODIndex == CurrentLODInfo.ReductionSettings.BaseLOD
			&& CurrentLODInfo.bHasBeenSimplified
			&& !SkelMesh->IsReductionActive(LODIndex)
			&& (bIsLODModelbuildDataAvailable))
		{
			//The build of the skeletalmesh will restore the data properly
			CurrentLODInfo.bHasBeenSimplified = false;
			return;
		}
		else if (!CurrentLODInfo.bHasBeenSimplified
			&& !SkelMesh->IsReductionActive(LODIndex))
		{
			//Nothing to reduce
			return;
		}

		FSkeletalMeshUpdateContext UpdateContext;
		UpdateContext.SkeletalMesh = SkelMesh;
		UpdateContext.AssociatedComponents.Push(GetPersonaToolkit()->GetPreviewMeshComponent());

		FLODUtilities::SimplifySkeletalMeshLOD(UpdateContext, LODIndex, GetTargetPlatformManagerRef().GetRunningTargetPlatform());

	}
	return;
}

//Regenerate dependent LODs if we re-import LOD X any LOD Z using X has source must be regenerated
//Also just generate already simplified mesh
void FPersonaMeshDetails::RegenerateDependentLODs(int32 LODIndex)
{
	USkeletalMesh* SkelMesh = GetPersonaToolkit()->GetMesh();
	check(SkelMesh);

	IMeshReductionModule& ReductionModule = FModuleManager::Get().LoadModuleChecked<IMeshReductionModule>("MeshReductionInterface");
	IMeshReduction* MeshReduction = ReductionModule.GetSkeletalMeshReductionInterface();
	FScopedSuspendAlternateSkinWeightPreview ScopedSuspendAlternateSkinnWeightPreview(SkelMesh);
	if (MeshReduction && MeshReduction->IsSupported())
	{
		FScopedSkeletalMeshPostEditChange ScopedPostEditChange(SkelMesh);
		TArray<bool> DependentLODs;
		DependentLODs.AddZeroed(SkelMesh->GetLODNum());
		DependentLODs[LODIndex] = true;
		for (int32 CurrentLODIndex = LODIndex + 1; CurrentLODIndex < DependentLODs.Num(); ++CurrentLODIndex)
		{
			FSkeletalMeshLODInfo& CurrentLODInfo = *(SkelMesh->GetLODInfo(CurrentLODIndex));
			FSkeletalMeshOptimizationSettings& Settings = CurrentLODInfo.ReductionSettings;
			if (CurrentLODInfo.bHasBeenSimplified && DependentLODs[Settings.BaseLOD])
			{
				DependentLODs[CurrentLODIndex] = true;
				//Regenerate this LOD
				RegenerateOneLOD(CurrentLODIndex);
			}
		}
	}
}

FReply FPersonaMeshDetails::RegenerateLOD(int32 LODIndex)
{
	USkeletalMesh* SkelMesh = GetPersonaToolkit()->GetMesh();
	check(SkelMesh);

	if (SkelMesh->IsValidLODIndex(LODIndex))
	{
		const FSkeletalMeshLODInfo& CurrentLODInfo = *(SkelMesh->GetLODInfo(LODIndex));
		const bool bIsReductionActive = SkelMesh->IsReductionActive(LODIndex);
		if (CurrentLODInfo.bHasBeenSimplified == false && (LODIndex > 0 || bIsReductionActive))
		{
			if (LODIndex > 0)
			{
				const FText Text = FText::Format(LOCTEXT("Warning_SimplygonApplyingToImportedMesh", "LOD {0} has been imported. Are you sure you'd like to apply mesh reduction?"), FText::AsNumber(LODIndex));
				EAppReturnType::Type Ret = FMessageDialog::Open(EAppMsgType::YesNo, Text);
				if (Ret == EAppReturnType::No)
				{
					return FReply::Handled();
				}
			}
			else if (bIsReductionActive)
			{
				//Ask user a special permission when the base LOD can be reduced 
				const FText Text(LOCTEXT("Warning_ReductionApplyingToImportedMesh_ReduceNonGenBaseLOD", "Are you sure you'd like to apply mesh reduction to the non-generated base LOD?"));
				EAppReturnType::Type Ret = FMessageDialog::Open(EAppMsgType::YesNo, Text);
				if (Ret == EAppReturnType::No)
				{
					return FReply::Handled();
				}
			}
		}

		if (bIsReductionActive && !SkelMesh->HasMeshDescription(LODIndex))
		{
			if (const FSkeletalMeshLODInfo* LodInfoPtr = SkelMesh->GetLODInfo(LODIndex))
			{
				if (LodInfoPtr->ReductionSettings.BaseLOD == LODIndex)
				{
					IMeshUtilities& MeshUtilities = FModuleManager::Get().LoadModuleChecked<IMeshUtilities>("MeshUtilities");
					//We create the import data for all LOD that do not have import data except for the generated LODs.
					MeshUtilities.CreateImportDataFromLODModel(SkelMesh);
				}
			}
		}
	}
	
	FScopedSuspendAlternateSkinWeightPreview ScopedSuspendAlternateSkinnWeightPreview(SkelMesh);
	//Reregister scope
	{
		FScopedSkeletalMeshPostEditChange ScopedPostEditChange(SkelMesh);
		SkelMesh->PreEditChange(nullptr);
		SkelMesh->Modify();

		RegenerateOneLOD(LODIndex);
		RegenerateDependentLODs(LODIndex);
	}

	return FReply::Handled();
}

FReply FPersonaMeshDetails::RemoveOneLOD(int32 LODIndex)
{
	USkeletalMesh* SkelMesh = GetPersonaToolkit()->GetMesh();
	check(SkelMesh);
	check(SkelMesh->IsValidLODIndex(LODIndex));

	if (LODIndex > 0)
	{
		FText ConfirmRemoveLODText = FText::Format( LOCTEXT("PersonaRemoveLOD_Confirmation", "Are you sure you want to remove LOD {0} from {1}?"), LODIndex, FText::FromString(SkelMesh->GetName()) );

		// if we have lod settings, and then 
		if (SkelMesh->GetLODSettings() != nullptr)
		{
			// if I have more LODs, and if LODSettings will be copied back over, 
			// all LODs have to be regenerated
			// warn users about it
			if (SkelMesh->IsValidLODIndex(LODIndex + 1) && SkelMesh->GetLODSettings()->GetNumberOfSettings() > LODIndex)
			{
				// now the information will get copied over after removing this LOD
				ConfirmRemoveLODText = FText::Format(LOCTEXT("PersonaRemoveLODOverriding_Confirmation", "You're currently using LOD Setting Asset \'{2}\' that will override the next LODs with current setting. This will require to regenerate the next LODs after removing this LOD. If you do not want this, clear the LOD Setting Asset before removing LODs. \n\n Are you sure you want to remove LOD {0} from {1}?"), LODIndex, FText::FromString(SkelMesh->GetName()), FText::FromString(SkelMesh->GetLODSettings()->GetName()));
			}
		}

		if ( FMessageDialog::Open(EAppMsgType::YesNo, ConfirmRemoveLODText) == EAppReturnType::Yes )
		{
			FText RemoveLODText = FText::Format(LOCTEXT("OnPersonaRemoveLOD", "Persona editor: Remove LOD {0}"), LODIndex);
			FScopedTransaction Transaction(TEXT(""), RemoveLODText, SkelMesh);
			SkelMesh->Modify();

			FScopedSuspendAlternateSkinWeightPreview ScopedSuspendAlternateSkinnWeightPreview(SkelMesh);
			//PostEditChange scope
			{
				FScopedSkeletalMeshPostEditChange ScopedPostEditChange(SkelMesh);
				
				FSkeletalMeshUpdateContext UpdateContext;
				UpdateContext.SkeletalMesh = SkelMesh;
				UpdateContext.AssociatedComponents.Push(GetPersonaToolkit()->GetPreviewMeshComponent());

				FLODUtilities::RemoveLOD(UpdateContext, LODIndex);

				if (SkelMesh->GetLODSettings())
				{
					SkelMesh->GetLODSettings()->SetLODSettingsToMesh(SkelMesh);
				}
			}

			int32 CurrentDisplayLOD = GetPersonaToolkit()->GetPreviewMeshComponent()->GetForcedLOD();
			if (CurrentDisplayLOD == LODIndex)
			{
				SetCurrentLOD(FMath::Min(CurrentDisplayLOD, SkelMesh->GetLODNum() - 1));
			}
			
			// We need to rebuild the layout to ensure all the LOD details are up-to-date.
			ForceLayoutRebuild();
		}
	}
	return FReply::Handled();
}


FReply FPersonaMeshDetails::RemoveGeometryFromLOD(int32 LODIndex)
{
	USkeletalMesh* SkelMesh = GetPersonaToolkit()->GetMesh();
	check(SkelMesh);
	check(SkelMesh->IsValidLODIndex(LODIndex));

	if (LODIndex <= 0)
	{
		return FReply::Handled();
	}
	
	FText ConfirmRemoveLODText = FText::Format( LOCTEXT("PersonaRemoveGeometry_Confirmation", "Are you sure you want to remove all geometry from LOD {0} on {1}?"), LODIndex, FText::FromString(SkelMesh->GetName()) );

	if ( FMessageDialog::Open(EAppMsgType::YesNo, ConfirmRemoveLODText) != EAppReturnType::Yes )
	{
		return FReply::Handled();
	}
	
	FText RemoveLODText = FText::Format(LOCTEXT("OnPersonaRemoveGeometry", "Remove geometry from LOD {0}"), LODIndex);
	FScopedTransaction Transaction(TEXT(""), RemoveLODText, SkelMesh);

	FScopedSuspendAlternateSkinWeightPreview ScopedSuspendAlternateSkinnWeightPreview(SkelMesh);
	//PostEditChange scope
	{
		FScopedSkeletalMeshPostEditChange ScopedPostEditChange(SkelMesh);
		SkelMesh->PreEditChange(nullptr);
		SkelMesh->Modify();

		SkelMesh->ModifyMeshDescription(LODIndex);
		SkelMesh->ClearMeshDescriptionAndBulkData(LODIndex);

		FSkeletalMeshLODInfo* LODInfo = SkelMesh->GetLODInfo(LODIndex);
		if (LODInfo->ReductionSettings.BaseLOD == LODIndex)
		{
			LODInfo->ReductionSettings.BaseLOD--;
		}
		
		// Make sure the reduction settings are kosher.
		if (SkelMesh->GetLODSettings())
		{
			SkelMesh->GetLODSettings()->SetLODSettingsToMesh(SkelMesh);
		}

		RegenerateOneLOD(LODIndex);
		RegenerateDependentLODs(LODIndex);
	}

	return FReply::Handled();
}


FText FPersonaMeshDetails::GetApplyButtonText() const
{
	if (IsApplyNeeded() || !IsGenerateAvailable())	
	{
		return LOCTEXT("ApplyChanges", "Apply Changes");
	}
	else
	{
		return LOCTEXT("Regenerate", "Regenerate");
	}
}

void FPersonaMeshDetails::ApplyChanges()
{
	USkeletalMesh* SkelMesh = GetPersonaToolkit()->GetMesh();
	check(SkelMesh);

	const int32 CurrentDisplayLOD = GetCurrentDisplayLODIndex(GetPersonaToolkit()->GetPreviewMeshComponent());
	// Check if there are any LOD changes before we regenerate, as we will have to send a LOD update event to fully refresh the bone tree state
	const bool bRequiresLODChangeUpdate = SSkeletalLODActions::LODSettingsChanged(SkelMesh, CurrentDisplayLOD);

	if (NaniteSettings.IsValid())
	{
		NaniteSettings->ApplyChanges();
	}

	FScopedSuspendAlternateSkinWeightPreview ScopedSuspendAlternateSkinnWeightPreview(SkelMesh);
	//Control the scope of the PostEditChange
	{
		FScopedSkeletalMeshPostEditChange ScopedPostEditChange(SkelMesh);
		// see if there is 
		bool bRegenerateEvenIfImported = false;
		bool bGenerateBaseLOD = false;
		int32 CurrentNumLODs = SkelMesh->GetLODNum();
		if (CurrentNumLODs == LODCount)
		{
			bool bImportedLODs = false;
			// check if anything is imported and ask if users wants to still regenerate it
			for (int32 LODIdx = 0; LODIdx < LODCount; LODIdx++)
			{
				FSkeletalMeshLODInfo& CurrentLODInfo = *(SkelMesh->GetLODInfo(LODIdx));
				bool bIsReductionActive = SkelMesh->IsReductionActive(LODIdx);
				
				if (CurrentLODInfo.bHasBeenSimplified == false && bIsReductionActive)
				{
					if (LODIdx > 0)
					{
						bImportedLODs = true;
					}
					else
					{
						bGenerateBaseLOD = true;
					}
				}
				else
				{
					RestoreNonReducedLOD(LODIdx);
				}

				//Make sure the editable skeleton is refresh
				GetPersonaToolkit()->GetEditableSkeleton()->RefreshBoneTree();
			}

			// if LOD is imported, ask users if they want to regenerate or just leave it
			if (bImportedLODs)
			{
				bRegenerateEvenIfImported = true;
			}
		}

		FLODUtilities::RegenerateLOD(SkelMesh, GetTargetPlatformManagerRef().GetRunningTargetPlatform(), LODCount, bRegenerateEvenIfImported, bGenerateBaseLOD);

		//PostEditChange will be call when going out of scope
	}
	//Update the morph target list since we have rebuild the asset
	if (SkelMesh->GetMorphTargets().Num() > 0)
	{
		GetPersonaToolkit()->GetPreviewScene()->BroadcastOnMorphTargetsChanged();
	}

	// Update preview mesh required bones, in case LOD settings have changed
	if (bRequiresLODChangeUpdate)
	{
		GetPersonaToolkit()->GetPreviewScene()->GetPreviewMeshComponent()->RecalcRequiredBones(CurrentDisplayLOD);
		// Force an update (this will update the Skeleton Tree to match the bones in the preview skeletal mesh required bones)
		GetPersonaToolkit()->GetPreviewScene()->BroadcastOnLODChanged();
	}

	// We need to rebuild the layout to ensure all the LOD details are up-to-date.
	ForceLayoutRebuild();
}

void FPersonaMeshDetails::UpdateLODNames()
{
	LODNames.Empty();
	LODNames.Add(MakeShareable(new FString(LOCTEXT("BaseLOD", "Base LOD").ToString())));
	for (int32 LODLevelID = 1; LODLevelID < LODCount; ++LODLevelID)
	{
		LODNames.Add(MakeShareable(new FString(FText::Format(NSLOCTEXT("LODSettingsLayout", "LODLevel_Reimport", "Reimport LOD Level {0}"), FText::AsNumber(LODLevelID)).ToString())));
	}
	LODNames.Add(MakeShareable(new FString(FText::Format(NSLOCTEXT("LODSettingsLayout", "LODLevel_Import", "Import LOD Level {0}"), FText::AsNumber(LODCount)).ToString())));
}

bool FPersonaMeshDetails::IsGenerateAvailable() const
{
	return IsAutoMeshReductionAvailable() && (IsApplyNeeded() || (LODCount > 1));
}

bool FPersonaMeshDetails::IsApplyNeeded() const
{
	USkeletalMesh* SkelMesh = GetPersonaToolkit()->GetMesh();
	check(SkelMesh);

	if (!SkelMesh->IsCompiling() && SkelMesh->GetLODNum() != LODCount)
	{
		return true;
	}

	if (NaniteSettings.IsValid() && NaniteSettings->IsApplyNeeded())
	{
		return true;
	}

	return false;
}

FText FPersonaMeshDetails::GetLODCountTooltip() const
{
	if (IsAutoMeshReductionAvailable())
	{
		return LOCTEXT("LODCountTooltip", "The number of LODs for this skeletal mesh. If auto mesh reduction is available, setting this number will determine the number of LOD levels to auto generate.");
	}

	return LOCTEXT("LODCountTooltip_Disabled", "Auto mesh reduction is unavailable! Please provide a mesh reduction interface such as Simplygon to use this feature or manually import LOD levels.");
}

FText FPersonaMeshDetails::GetLODImportedText(int32 LODIndex) const
{
	USkeletalMesh* Mesh = GetPersonaToolkit()->GetMesh();
	if (Mesh && !Mesh->IsCompiling() && Mesh->IsValidLODIndex(LODIndex))
	{
		if (Mesh->GetLODInfo(LODIndex)->bHasBeenSimplified)
		{
			if (Mesh->HasMeshDescription(LODIndex))
			{
				return LOCTEXT("LODMeshReductionText_Label_Inline", "[inline reduced]");
			}
			else
			{
				return LOCTEXT("LODMeshReductionText_Label_Reduced", "[generated]");
			}
		}
	}

	return FText();
}

FText FPersonaMeshDetails::GetMaterialSlotNameText(int32 MaterialIndex) const
{
	USkeletalMesh* Mesh = GetPersonaToolkit()->GetMesh();
	if (Mesh && !Mesh->IsCompiling() && Mesh->GetMaterials().IsValidIndex(MaterialIndex))
	{
		return FText::FromName(Mesh->GetMaterials()[MaterialIndex].MaterialSlotName);
	}

	return LOCTEXT("SkeletalMeshMaterial_InvalidIndex", "Invalid Material Index");
}

void FPersonaMeshDetails::CustomizeDetails( IDetailLayoutBuilder& DetailLayout )
{
	const TArray<TWeakObjectPtr<UObject>>& SelectedObjects = DetailLayout.GetSelectedObjects();
	check(SelectedObjects.Num()<=1); // The OnGenerateCustomWidgets delegate will not be useful if we try to process more than one object.

	TSharedRef<IPersonaPreviewScene> PreviewScene = GetPersonaToolkit()->GetPreviewScene();

	// Ensure that we only have one callback for this object registered
	PreviewScene->RegisterOnPreviewMeshChanged(FOnPreviewMeshChanged::CreateSP(this, &FPersonaMeshDetails::OnPreviewMeshChanged));

	SkeletalMeshPtr = SelectedObjects.Num() > 0 ? Cast<USkeletalMesh>(SelectedObjects[0].Get()) : nullptr;

	// copy temporarily to refresh Mesh details tab from the LOD settings window
	MeshDetailLayout = &DetailLayout;
	// add multiple LOD levels to LOD category
	AddLODLevelCategories(DetailLayout);

	CustomizeLODSettingsCategories(DetailLayout);

	IDetailCategoryBuilder& ClothingCategory = DetailLayout.EditCategory("Clothing", FText::GetEmpty(), ECategoryPriority::TypeSpecific);
	CustomizeClothingProperties(DetailLayout,ClothingCategory);

	// Get notified when the mesh has finished building
	if (SkeletalMeshPtr.Get())
	{
		SkeletalMeshPtr.Get()->OnPostMeshCached().AddSP(this, &FPersonaMeshDetails::OnMeshRebuildCompleted);
	}

	// Post process selector
	IDetailCategoryBuilder& SkelMeshCategory = DetailLayout.EditCategory("SkeletalMesh");
	TSharedRef<IPropertyHandle> PostProcessHandle = DetailLayout.GetProperty(USkeletalMesh::GetPostProcessAnimBlueprintMemberName(), USkeletalMesh::StaticClass());
	PostProcessHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FPersonaMeshDetails::OnPostProcessBlueprintChanged, &DetailLayout));
	PostProcessHandle->MarkHiddenByCustomization();

	// Hide the existing NaniteSettings property so we can use the customization instead
	DetailLayout.EditCategory("Mesh");
	TSharedRef<IPropertyHandle> NaniteSettingsProperty = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(USkeletalMesh, NaniteSettings), USkeletalMesh::StaticClass());
	NaniteSettingsProperty->MarkHiddenByCustomization();

	NaniteSettings = MakeShareable(new FNaniteSkeletalMeshLayout(*this, GetPersonaToolkit()));
	NaniteSettings->AddToDetailsPanel(DetailLayout);

	IDetailCategoryBuilder& AnimationCategory = DetailLayout.EditCategory("Animation");

	FDetailWidgetRow& PostProcessRow = AnimationCategory.AddCustomRow(LOCTEXT("PostProcessFilterString", "Post Process Blueprint"));
	PostProcessRow.RowTag(TEXT("PostProcessAnimBlueprint"));
	PostProcessRow.NameContent()
	[
		PostProcessHandle->CreatePropertyNameWidget()
	];

	PostProcessRow.ValueContent()
	[
		SNew(SObjectPropertyEntryBox)
		.ObjectPath(this, &FPersonaMeshDetails::GetCurrentPostProcessBlueprintPath)
		.AllowedClass(UAnimBlueprint::StaticClass())
		.NewAssetFactories(TArray<UFactory*>())
		.OnShouldFilterAsset(FOnShouldFilterAsset::CreateSP(this, &FPersonaMeshDetails::OnShouldFilterPostProcessBlueprint))
		.OnObjectChanged(FOnSetObject::CreateSP(this, &FPersonaMeshDetails::OnSetPostProcessBlueprint, PostProcessHandle))
	];

	// Add warning if the post process BP is using an incompatible skeleton
	FDetailWidgetRow& PostProcessWarningRow = SkelMeshCategory.AddCustomRow(LOCTEXT("PostProcessWarningFilterString", "Post Process Blueprint Warning"));
	PostProcessWarningRow.RowTag(TEXT("PostProcessAnimBlueprint"));

	PostProcessWarningRow
	.Visibility(MakeAttributeLambda([this]()
	{
		if(SkeletalMeshPtr.IsValid() && !SkeletalMeshPtr->IsCompiling())
		{
			TSubclassOf<UAnimInstance> PostProcessAnimBlueprintClass = SkeletalMeshPtr->GetPostProcessAnimBlueprint();
			if(PostProcessAnimBlueprintClass.Get())
			{
				if(UAnimBlueprint* AnimBlueprint = Cast<UAnimBlueprint>(PostProcessAnimBlueprintClass->ClassGeneratedBy))
				{
					return !SkeletalMeshPtr->GetSkeleton()->IsCompatibleForEditor(AnimBlueprint->TargetSkeleton) ? EVisibility::Visible : EVisibility::Collapsed;
				}
			}
		}
		return EVisibility::Collapsed;
	}))
	.WholeRowContent()
	[
		SNew(SWarningOrErrorBox)
		.MessageStyle(EMessageStyle::Error)
		.Message(LOCTEXT("IncompatibleSkeletonError", "Post Process Anim Blueprint for this mesh uses an incompatible skeleton"))
	];
	
	IDetailCategoryBuilder& ImportSettingsCategory = DetailLayout.EditCategory("ImportSettings");
	TSharedRef<IPropertyHandle> AssetImportProperty = DetailLayout.GetProperty(USkeletalMesh::GetAssetImportDataMemberName(), USkeletalMesh::StaticClass());
	if (!SkeletalMeshPtr.IsValid() || !IsValid(SkeletalMeshPtr->GetAssetImportData()) || !SkeletalMeshPtr->GetAssetImportData()->IsA<UFbxSkeletalMeshImportData>())
	{
		// Hide the ability to change the import settings object
		IDetailPropertyRow& Row = ImportSettingsCategory.AddProperty(AssetImportProperty);
		Row.CustomWidget(true)
			.NameContent()
			[
				AssetImportProperty->CreatePropertyNameWidget()
			];
	}
	else
	{
		// If the AssetImportData is an instance of UFbxSkeletalMeshImportData we create a custom UI.
		// Since DetailCustomization UI is not supported on instanced properties and because IDetailLayoutBuilder does not work well inside instanced objects scopes,
		// we need to manually recreate the whole FbxSkeletalMeshImportData UI in order to customize it.
		AssetImportProperty->MarkHiddenByCustomization();
		VertexColorImportOptionHandle = AssetImportProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(UFbxSkeletalMeshImportData, VertexColorImportOption));
		VertexColorImportOverrideHandle = AssetImportProperty->GetChildHandle(GET_MEMBER_NAME_CHECKED(UFbxSkeletalMeshImportData, VertexOverrideColor));
		TMap<FName, IDetailGroup*> ExistingGroup;
		PropertyCustomizationHelpers::MakeInstancedPropertyCustomUI(ExistingGroup, ImportSettingsCategory, AssetImportProperty, FOnInstancedPropertyIteration::CreateSP(this, &FPersonaMeshDetails::OnInstancedFbxSkeletalMeshImportDataPropertyIteration));
	}


	CustomizeSkinWeightProfiles(DetailLayout);

	HideUnnecessaryProperties(DetailLayout);
	
	// Run the customize mesh details delegates.
	FPersonaModule& PersonaModule = FModuleManager::GetModuleChecked<FPersonaModule>("Persona");

	const TArray<FPersonaModule::FOnCustomizeMeshDetails>& CustomizeMeshDetailsDelegates = PersonaModule.GetCustomizeMeshDetailsDelegates();
	for (const FPersonaModule::FOnCustomizeMeshDetails& Delegate : CustomizeMeshDetailsDelegates)
	{
		Delegate.ExecuteIfBound(DetailLayout, SkeletalMeshPtr);
	}

	USkeletalMesh* SkelMesh = GetPersonaToolkit()->GetMesh();
	const int32 SkelMeshLODCount = SkelMesh ? SkelMesh->GetLODNum() : 0;

	auto CategorySorter = [SkelMeshLODCount](const TMap<FName, IDetailCategoryBuilder*>& Categories)
	{
		int32 Order = 0;
		auto SafeSetOrder = [&Categories, &Order](const FName& CategoryName)
		{
			if (IDetailCategoryBuilder* const* Builder = Categories.Find(CategoryName))
			{
				(*Builder)->SetSortOrder(Order++);
			}
		};
		
		SafeSetOrder(FName("Material Slots"));

		SafeSetOrder(FName("LODCustomMode"));

		for (int32 LODIndex = 0; LODIndex < SkelMeshLODCount; ++LODIndex)
		{
			FString LODCategoryName = FString(TEXT("LOD"));
			LODCategoryName.AppendInt(LODIndex);
			SafeSetOrder(*LODCategoryName);
		}

		SafeSetOrder(FName("LodSettings"));
		
		SafeSetOrder(FName("Clothing"));
		
		SafeSetOrder(FName("Animation"));
		SafeSetOrder(FName("SkeletalMesh"));
		SafeSetOrder(FName("Mirroring"));
		SafeSetOrder(FName("Mesh"));
		SafeSetOrder(FName("SkinWeights"));
		SafeSetOrder(FName("NaniteSettings"));
		SafeSetOrder(FName("ImportSettings"));
		
	};
	
	DetailLayout.SortCategories(CategorySorter);
}

void FPersonaMeshDetails::OnInstancedFbxSkeletalMeshImportDataPropertyIteration(IDetailCategoryBuilder& BaseCategory, IDetailGroup* PropertyGroup, TSharedRef<IPropertyHandle>& Property) const
{
	IDetailPropertyRow* Row = nullptr;
	
	if (PropertyGroup)
	{
		Row = &PropertyGroup->AddPropertyRow(Property);
	}
	else
	{
		Row = &BaseCategory.AddProperty(Property);
	}

	if (Row)
	{
		//Vertex Override Color property should be disabled if we are not in override mode.
		if (Property->IsValidHandle() && Property->GetProperty() == VertexColorImportOverrideHandle->GetProperty())
		{
			Row->IsEnabled(TAttribute<bool>(this, &FPersonaMeshDetails::GetVertexOverrideColorEnabledState));
		}
	}
}

bool FPersonaMeshDetails::GetVertexOverrideColorEnabledState() const
{
	uint8 VertexColorImportOption;
	check(VertexColorImportOptionHandle.IsValid());
	ensure(VertexColorImportOptionHandle->GetValue(VertexColorImportOption) == FPropertyAccess::Success);

	return (VertexColorImportOption == EVertexColorImportOption::Override);
}


void FPersonaMeshDetails::OnMeshRebuildCompleted(USkeletalMesh* InMesh)
{
	if (InMesh == SkeletalMeshPtr.Get())
	{
		RequestLayoutUpdate();
	}
}

void FPersonaMeshDetails::HideUnnecessaryProperties(IDetailLayoutBuilder& DetailLayout)
{
	// LODInfo doesn't need to be showed anymore because it was moved to each LOD category
	TSharedRef<IPropertyHandle> LODInfoProperty = DetailLayout.GetProperty(FName("SourceModels"), USkeletalMesh::StaticClass());
	DetailLayout.HideProperty(LODInfoProperty);
	uint32 NumChildren = 0;
	LODInfoProperty->GetNumChildren(NumChildren);
	// Hide reduction settings property because it is duplicated with Reduction settings layout created by ReductionSettingsWidgets
	for (uint32 ChildIdx = 0; ChildIdx < NumChildren; ChildIdx++)
	{
		TSharedPtr<IPropertyHandle> ChildHandle = LODInfoProperty->GetChildHandle(ChildIdx);
		if (ChildHandle.IsValid())
		{
			TSharedPtr<IPropertyHandle> ReductionHandle = ChildHandle->GetChildHandle(FName("ReductionSettings"));
			DetailLayout.HideProperty(ReductionHandle);
		}
	}

	TSharedRef<IPropertyHandle> MaterialsProperty = DetailLayout.GetProperty(USkeletalMesh::GetMaterialsMemberName(), USkeletalMesh::StaticClass());
	DetailLayout.HideProperty(MaterialsProperty);

	// hide all properties in Mirroring category to hide Mirroring category itself
	IDetailCategoryBuilder& MirroringCategory = DetailLayout.EditCategory("Mirroring", FText::GetEmpty(), ECategoryPriority::Default);
	TArray<TSharedRef<IPropertyHandle>> MirroringProperties;
	MirroringCategory.GetDefaultProperties(MirroringProperties);
	for (int32 MirrorPropertyIdx = 0; MirrorPropertyIdx < MirroringProperties.Num(); MirrorPropertyIdx++)
	{
		DetailLayout.HideProperty(MirroringProperties[MirrorPropertyIdx]);
	}
}

void FPersonaMeshDetails::OnPostProcessBlueprintChanged(IDetailLayoutBuilder* DetailBuilder)
{
	RequestLayoutUpdate();
}

FString FPersonaMeshDetails::GetCurrentPostProcessBlueprintPath() const
{
	USkeletalMesh* SkelMesh = GetPersonaToolkit()->GetMesh();
	if (!SkelMesh->IsCompiling())
	{
		if (UClass* PostProcessClass = *SkelMesh->GetPostProcessAnimBlueprint())
		{

			return PostProcessClass->GetPathName();
		}
	}

	return FString();
}

bool FPersonaMeshDetails::OnShouldFilterPostProcessBlueprint(const FAssetData& AssetData) const
{
	if(USkeletalMesh* SkelMesh = GetPersonaToolkit()->GetMesh())
	{
		if (!SkelMesh->IsCompiling())
		{
			const FString SkeletonName = AssetData.GetTagValueRef<FString>("TargetSkeleton");
			return !SkelMesh->GetSkeleton()->IsCompatibleForEditor(SkeletonName);
		}
	}

	return true;
}

void FPersonaMeshDetails::OnSetPostProcessBlueprint(const FAssetData& AssetData, TSharedRef<IPropertyHandle> BlueprintProperty)
{
	if(UAnimBlueprint* SelectedBlueprint = Cast<UAnimBlueprint>(AssetData.GetAsset()))
	{
		BlueprintProperty->SetValue(SelectedBlueprint->GetAnimBlueprintGeneratedClass());
	}
	else if(!AssetData.IsValid())
	{
		// Asset data is not valid so clear the result
		UObject* Value = nullptr;
		BlueprintProperty->SetValue(Value);
	}
}

FReply FPersonaMeshDetails::OnReimportLodClicked(EReimportButtonType InReimportType, int32 InLODIndex)
{
	if(USkeletalMesh* SkelMesh = GetPersonaToolkit()->GetMesh())
	{
		if(!SkelMesh->IsValidLODIndex(InLODIndex))
		{
			return FReply::Unhandled();
		}

		FString SourceFilenameBackup("");
		
		//If we alter the reduction setting and the user cancel the import we must set them back
		bool bRestoreReductionOnfail = false;
		FSkeletalMeshOptimizationSettings ReductionSettingsBackup;
		FSkeletalMeshLODInfo* LODInfo = SkelMesh->GetLODInfo(InLODIndex);
		if(InReimportType == EReimportButtonType::ReimportWithNewFile)
		{
			// Back up current source filename and empty it so the importer asks for a new one.
			SourceFilenameBackup = LODInfo->SourceImportFilename;
			LODInfo->SourceImportFilename.Empty();
			
			//Avoid changing the settings if the skeletal mesh is using a LODSettings asset valid for this LOD
			bool bUseLODSettingAsset = SkelMesh->GetLODSettings() != nullptr && SkelMesh->GetLODSettings()->GetNumberOfSettings() > InLODIndex;
			//Make the reduction settings change according to the context
			if (!bUseLODSettingAsset && SkelMesh->IsReductionActive(InLODIndex) && LODInfo->bHasBeenSimplified && !SkelMesh->HasMeshDescription(InLODIndex))
			{
				FSkeletalMeshOptimizationSettings& ReductionSettings = LODInfo->ReductionSettings;
				//Backup the reduction settings
				ReductionSettingsBackup = ReductionSettings;
				//In case we have a vert/tri percent we just put the percent to 100% and avoid reduction
				//If we have a maximum criterion we change the BaseLOD to reduce the imported fbx instead of other LOD
				switch (ReductionSettings.TerminationCriterion)
				{
					case SkeletalMeshTerminationCriterion::SMTC_NumOfTriangles:
						ReductionSettings.NumOfTrianglesPercentage = 1.0f;
						break;
					case SkeletalMeshTerminationCriterion::SMTC_NumOfVerts:
						ReductionSettings.NumOfVertPercentage = 1.0f;
						break;
					case SkeletalMeshTerminationCriterion::SMTC_TriangleOrVert:
						ReductionSettings.NumOfTrianglesPercentage = 1.0f;
						ReductionSettings.NumOfVertPercentage = 1.0f;
						break;
					case SkeletalMeshTerminationCriterion::SMTC_AbsNumOfTriangles:
					case SkeletalMeshTerminationCriterion::SMTC_AbsNumOfVerts:
					case SkeletalMeshTerminationCriterion::SMTC_AbsTriangleOrVert:
						ReductionSettings.BaseLOD = InLODIndex;
						break;
				}
				bRestoreReductionOnfail = true;
			}
		}

		FbxMeshUtils::ImportMeshLODDialog(SkelMesh, InLODIndex).Then([this, InReimportType, SkelMesh, InLODIndex, SourceFilenameBackup, bRestoreReductionOnfail, ReductionSettingsBackup](TFuture<bool> Result)
			{
				bool bImportSucceeded = Result.Get();
				if (InReimportType == EReimportButtonType::ReimportWithNewFile && !bImportSucceeded)
				{
					FSkeletalMeshLODInfo* LODInfo = SkelMesh->GetLODInfo(InLODIndex);
					// Copy old source file back, as this one failed
					LODInfo->SourceImportFilename = SourceFilenameBackup;
					if (bRestoreReductionOnfail)
					{
						LODInfo->ReductionSettings = ReductionSettingsBackup;
					}
				}
				else if (InReimportType == EReimportButtonType::ReimportWithNewFile)
				{
					//Refresh the layout so the BaseLOD min max get recompute
					RequestLayoutUpdate();
				}
			});
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void FPersonaMeshDetails::OnGetMaterialsForArray(class IMaterialListBuilder& OutMaterials, int32 LODIndex)
{
	USkeletalMesh* SkelMesh = GetPersonaToolkit()->GetMesh();

	if (!SkelMesh || SkelMesh->IsCompiling())
		return;

	TArray<FSkeletalMaterial>& SkelMeshMaterials = SkelMesh->GetMaterials();
	for (int32 MaterialIndex = 0; MaterialIndex < SkelMeshMaterials.Num(); ++MaterialIndex)
	{
		OutMaterials.AddMaterial(MaterialIndex, SkelMeshMaterials[MaterialIndex].MaterialInterface, true);
	}
}

void FPersonaMeshDetails::OnMaterialArrayChanged(UMaterialInterface* NewMaterial, UMaterialInterface* PrevMaterial, int32 SlotIndex, bool bReplaceAll, int32 LODIndex)
{
	USkeletalMesh* Mesh = GetPersonaToolkit()->GetMesh();
	if (Mesh)
	{
		// Whether or not we made a transaction and need to end it
		bool bMadeTransaction = false;

		FProperty* MaterialProperty = FindFProperty<FProperty>(USkeletalMesh::StaticClass(), "Materials");
		check(MaterialProperty);
		Mesh->PreEditChange(MaterialProperty);
		TArray<FSkeletalMaterial>& MeshMaterials = Mesh->GetMaterials();
		check(MeshMaterials.Num() > SlotIndex);

		if (NewMaterial != PrevMaterial)
		{
			GEditor->BeginTransaction(LOCTEXT("PersonaEditorMaterialChanged", "Persona editor: material changed"));
			bMadeTransaction = true;
			Mesh->Modify();
			MeshMaterials[SlotIndex].MaterialInterface = NewMaterial;

			//Add a default name to the material slot if this slot was manually add and there is no name yet
			if (NewMaterial != nullptr && (MeshMaterials[SlotIndex].ImportedMaterialSlotName == NAME_None || MeshMaterials[SlotIndex].MaterialSlotName == NAME_None))
			{
				if (MeshMaterials[SlotIndex].MaterialSlotName == NAME_None)
				{
					MeshMaterials[SlotIndex].MaterialSlotName = NewMaterial->GetFName();
				}

				//Ensure the imported material slot name is unique
				if (MeshMaterials[SlotIndex].ImportedMaterialSlotName == NAME_None)
				{
					auto IsMaterialNameUnique = [&Mesh, &MeshMaterials, SlotIndex](const FName TestName)
					{
						for (int32 MaterialIndex = 0; MaterialIndex < MeshMaterials.Num(); ++MaterialIndex)
						{
							if (MaterialIndex == SlotIndex)
							{
								continue;
							}
							if (MeshMaterials[MaterialIndex].ImportedMaterialSlotName == TestName)
							{
								return false;
							}
						}
						return true;
					};
					int32 MatchNameCounter = 0;
					//Make sure the name is unique for imported material slot name
					bool bUniqueName = false;
					FString MaterialSlotName = NewMaterial->GetName();
					while (!bUniqueName)
					{
						bUniqueName = true;
						if (!IsMaterialNameUnique(FName(*MaterialSlotName)))
						{
							bUniqueName = false;
							MatchNameCounter++;
							MaterialSlotName = NewMaterial->GetName() + TEXT("_") + FString::FromInt(MatchNameCounter);
						}
					}
					MeshMaterials[SlotIndex].ImportedMaterialSlotName = FName(*MaterialSlotName);
				}
			}
		}

		FPropertyChangedEvent PropertyChangedEvent(MaterialProperty);
		Mesh->PostEditChangeProperty(PropertyChangedEvent);

		if (bMadeTransaction)
		{
			// End the transation if we created one
			GEditor->EndTransaction();
			// Redraw viewports to reflect the material changes 
			GUnrealEd->RedrawLevelEditingViewports();
		}
	}
}

FReply FPersonaMeshDetails::AddMaterialSlot()
{
	if (!SkeletalMeshPtr.IsValid())
	{
		return FReply::Handled();
	}

	FScopedTransaction Transaction(LOCTEXT("PersonaAddMaterialSlotTransaction", "Persona editor: Add material slot"));
	SkeletalMeshPtr->Modify();
	SkeletalMeshPtr->GetMaterials().Add(FSkeletalMaterial());

	SkeletalMeshPtr->PostEditChange();

	return FReply::Handled();
}

FText FPersonaMeshDetails::GetMaterialArrayText() const
{
	int32 SlotNumber = 0;
	if (SkeletalMeshPtr.IsValid() && !SkeletalMeshPtr->IsCompiling())
	{
		SlotNumber = SkeletalMeshPtr->GetMaterials().Num();
	}
	return FText::Format(LOCTEXT("MaterialSlots", "{0} {0}|plural(one=Material Slot,other=Material Slots)"), SlotNumber);
}

void FPersonaMeshDetails::OnGetSectionsForView(ISectionListBuilder& OutSections, int32 LODIndex)
{
	USkeletalMesh* SkelMesh = GetPersonaToolkit()->GetMesh();
	if (!SkelMesh || SkelMesh->IsCompiling())
	{
		return;
	}

	FSkeletalMeshModel* ImportedResource = SkelMesh->GetImportedModel();

	if (ImportedResource && ImportedResource->LODModels.IsValidIndex(LODIndex))
	{
		FSkeletalMeshLODModel& Model = ImportedResource->LODModels[LODIndex];

		TArray<int32>& MaterialMap = SkelMesh->GetLODInfo(LODIndex)->LODMaterialMap;
		const TArray<FSkeletalMaterial>& SkelMeshMaterials = SkelMesh->GetMaterials();
		int32 NumSections = Model.Sections.Num();
		for (int32 SectionIdx = 0; SectionIdx < NumSections; SectionIdx++)
		{
			int32 DefaultSectionMaterialIndex = GetDefaultMaterialIndex(SkelMesh, LODIndex, SectionIdx);
			int32 MaterialIndex = Model.Sections[SectionIdx].MaterialIndex;;
			if (MaterialMap.IsValidIndex(SectionIdx) && SkelMeshMaterials.IsValidIndex(MaterialMap[SectionIdx]))
			{
				MaterialIndex = MaterialMap[SectionIdx];
			}

			if (SkelMeshMaterials.IsValidIndex(MaterialIndex))
			{
				FName CurrentSectionMaterialSlotName = SkelMeshMaterials[MaterialIndex].MaterialSlotName;
				FName CurrentSectionOriginalImportedMaterialName = SkelMeshMaterials[MaterialIndex].ImportedMaterialSlotName;
				TMap<int32, FName> AvailableSectionName;
				int32 CurrentIterMaterialIndex = 0;
				for (const FSkeletalMaterial &SkeletalMaterial : SkelMeshMaterials)
				{
					if (MaterialIndex != CurrentIterMaterialIndex)
					{
						if (DefaultSectionMaterialIndex == CurrentIterMaterialIndex)
						{
							FString BuildDefaultName = SkeletalMaterial.MaterialSlotName.ToString() + GSuffixDefaultMaterial;
							AvailableSectionName.Add(CurrentIterMaterialIndex, FName(*BuildDefaultName));
						}
						else
						{
							AvailableSectionName.Add(CurrentIterMaterialIndex, SkeletalMaterial.MaterialSlotName);
						}
					}
					CurrentIterMaterialIndex++;
				}
				bool bClothSection = Model.Sections[SectionIdx].HasClothingData();
				bool bIsChunkSection = Model.Sections[SectionIdx].ChunkedParentSectionIndex != INDEX_NONE;
				OutSections.AddSection(LODIndex, SectionIdx, CurrentSectionMaterialSlotName, MaterialIndex, CurrentSectionOriginalImportedMaterialName, AvailableSectionName, SkelMeshMaterials[MaterialIndex].MaterialInterface, bClothSection, bIsChunkSection, DefaultSectionMaterialIndex);
			}
		}
	}
}

FText FPersonaMeshDetails::GetMaterialNameText(int32 MaterialIndex) const
{
	if (SkeletalMeshPtr.IsValid() && !SkeletalMeshPtr->IsCompiling() && SkeletalMeshPtr->GetMaterials().IsValidIndex(MaterialIndex))
	{
		return FText::FromName(SkeletalMeshPtr->GetMaterials()[MaterialIndex].MaterialSlotName);
	}
	return FText::FromName(NAME_None);
}

FText FPersonaMeshDetails::GetOriginalImportMaterialNameText(int32 MaterialIndex) const
{
	if (SkeletalMeshPtr.IsValid() && !SkeletalMeshPtr->IsCompiling() && SkeletalMeshPtr->GetMaterials().IsValidIndex(MaterialIndex))
	{
		FString OriginalImportMaterialName;
		SkeletalMeshPtr->GetMaterials()[MaterialIndex].ImportedMaterialSlotName.ToString(OriginalImportMaterialName);
		OriginalImportMaterialName = TEXT("Original Imported Material Name: ") + OriginalImportMaterialName;
		return FText::FromString(OriginalImportMaterialName);
			}
	return FText::FromName(NAME_None);
		}

void FPersonaMeshDetails::OnMaterialNameCommitted(const FText& InValue, ETextCommit::Type CommitType, int32 MaterialIndex)
{
	FName InValueName = FName(*(InValue.ToString()));
	if (SkeletalMeshPtr.IsValid() && SkeletalMeshPtr->GetMaterials().IsValidIndex(MaterialIndex) && InValueName != SkeletalMeshPtr->GetMaterials()[MaterialIndex].MaterialSlotName)
	{
		FScopedTransaction ScopeTransaction(LOCTEXT("PersonaMaterialSlotNameChanged", "Persona editor: Material slot name change"));

		FProperty* ChangedProperty = FindFProperty<FProperty>(USkeletalMesh::StaticClass(), "Materials");
		check(ChangedProperty);
		SkeletalMeshPtr->PreEditChange(ChangedProperty);

		SkeletalMeshPtr->GetMaterials()[MaterialIndex].MaterialSlotName = InValueName;
		
		FPropertyChangedEvent PropertyUpdateStruct(ChangedProperty);
		SkeletalMeshPtr->PostEditChangeProperty(PropertyUpdateStruct);
	}
}

TSharedRef<SWidget> FPersonaMeshDetails::OnGenerateCustomNameWidgetsForMaterialArray(UMaterialInterface* Material, int32 MaterialIndex)
{
	return SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SCheckBox)
			.IsChecked(this, &FPersonaMeshDetails::IsMaterialSelected, MaterialIndex)
			.OnCheckStateChanged(this, &FPersonaMeshDetails::OnMaterialSelectedChanged, MaterialIndex)
			.ToolTipText(LOCTEXT("Highlight_CustomMaterialName_ToolTip", "Highlights this material in the viewport"))
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.ColorAndOpacity(FLinearColor(0.4f, 0.4f, 0.4f, 1.0f))
				.Text(LOCTEXT("Highlight", "Highlight"))
			]
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0, 2, 0, 0)
		[
			SNew(SCheckBox)
			.IsChecked(this, &FPersonaMeshDetails::IsIsolateMaterialEnabled, MaterialIndex)
			.OnCheckStateChanged(this, &FPersonaMeshDetails::OnMaterialIsolatedChanged, MaterialIndex)
			.ToolTipText(LOCTEXT("Isolate_CustomMaterialName_ToolTip", "Isolates this material in the viewport"))
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.ColorAndOpacity(FLinearColor(0.4f, 0.4f, 0.4f, 1.0f))
				.Text(LOCTEXT("Isolate", "Isolate"))
			]
		];
}

TSharedRef<SWidget> FPersonaMeshDetails::OnGenerateCustomMaterialWidgetsForMaterialArray(UMaterialInterface* Material, int32 MaterialIndex, int32 LODIndex)
{
	bool bMaterialIsUsed = false;
	if(SkeletalMeshPtr.IsValid() && MaterialUsedMap.Contains(MaterialIndex))
	{
		bMaterialIsUsed = MaterialUsedMap.Find(MaterialIndex)->Num() > 0;
	}

	return
		SNew(SMaterialSlotWidget, MaterialIndex, bMaterialIsUsed)
		.MaterialName(this, &FPersonaMeshDetails::GetMaterialNameText, MaterialIndex)
		.OnMaterialNameCommitted(this, &FPersonaMeshDetails::OnMaterialNameCommitted, MaterialIndex)
		.CanDeleteMaterialSlot(this, &FPersonaMeshDetails::CanDeleteMaterialSlot, MaterialIndex)
		.OnDeleteMaterialSlot(this, &FPersonaMeshDetails::OnDeleteMaterialSlot, MaterialIndex)
		.ToolTipText(this, &FPersonaMeshDetails::GetOriginalImportMaterialNameText, MaterialIndex);
}

TSharedRef<SWidget> FPersonaMeshDetails::OnGenerateMaterialListExtraBottomWidget(UMaterialInterface* Material, int32 MaterialIndex, int32 LODIndex)
{
	if (!SkeletalMeshPtr.IsValid() || !SkeletalMeshPtr->GetMaterials().IsValidIndex(MaterialIndex))
	{
		return SNullWidget::NullWidget;
	}
	TSharedRef<SVerticalBox> VerticalWidget = SNew(SVerticalBox);
	const FSkeletalMaterial& SkeletalMaterial = SkeletalMeshPtr->GetMaterials()[MaterialIndex];
	if(GEnableSkeletalMeshMaterialSlotOverlayMaterialUI || SkeletalMaterial.OverlayMaterialInterface)
	{
		VerticalWidget->AddSlot()
		.AutoHeight()
		.Padding(0, 2, 0, 0)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SBox)
				.HAlign(HAlign_Right)
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(LOCTEXT("OverlayMaterial", "Overlay Material"))
				]
			]
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.Padding(5, 0, 0, 0)
			[
				SNew(SObjectPropertyEntryBox)
				.ObjectPath_Lambda([this, MaterialIndex]()
					{
						UMaterialInterface* OverlayMaterial = nullptr;
						if (SkeletalMeshPtr.IsValid() && SkeletalMeshPtr->GetMaterials().IsValidIndex(MaterialIndex))
						{
							OverlayMaterial = SkeletalMeshPtr->GetMaterials()[MaterialIndex].OverlayMaterialInterface;
						}
						if (OverlayMaterial)
						{
							return OverlayMaterial->GetPathName();
						}
						return FString();
					})
				.AllowClear(true)
				.AllowedClass(UMaterialInterface::StaticClass())
				.OnObjectChanged_Lambda([this, MaterialIndex](const FAssetData& AssetData)
					{
					
						UMaterialInterface* MaterialInterface = AssetData.IsValid() ? Cast<UMaterialInterface>(AssetData.GetAsset()) : nullptr;
						OnMaterialSlotOverlayMaterialChanged(MaterialInterface, MaterialIndex);
					})
				.ThumbnailPool(UThumbnailManager::Get().GetSharedThumbnailPool())
				.DisplayCompactSize(true)
			]
		];
	}

	return VerticalWidget;
}

bool FPersonaMeshDetails::CanDeleteMaterialSlot(int32 MaterialIndex) const
{
	if (!SkeletalMeshPtr.IsValid() || SkeletalMeshPtr->IsCompiling())
	{
		return false;
	}

	return SkeletalMeshPtr->GetMaterials().IsValidIndex(MaterialIndex);
}
	
void FPersonaMeshDetails::OnDeleteMaterialSlot(int32 MaterialIndex)
{
	if (!SkeletalMeshPtr.IsValid() || !CanDeleteMaterialSlot(MaterialIndex))
	{
		return;
	}

	if (!bDeleteWarningConsumed)
	{
		EAppReturnType::Type Answer = FMessageDialog::Open(EAppMsgType::OkCancel, LOCTEXT("FPersonaMeshDetails_DeleteMaterialSlot", "WARNING - Deleting a material slot can break the game play blueprint or the game play code. All indexes after the delete slot will change"));
		if (Answer == EAppReturnType::Cancel)
		{
			return;
		}
		bDeleteWarningConsumed = true;
	}

	FScopedTransaction Transaction(LOCTEXT("PersonaOnDeleteMaterialSlotTransaction", "Persona editor: Delete material slot"));
	SkeletalMeshPtr->Modify();
	{
		FScopedSkeletalMeshPostEditChange ScopedPostEditChange(SkeletalMeshPtr.Get());
		//When we delete a material slot we must invalidate the DDC because material index is not part of the DDC key by design
		SkeletalMeshPtr->InvalidateDeriveDataCacheGUID();

		SkeletalMeshPtr->GetMaterials().RemoveAt(MaterialIndex);
		FSkeletalMeshModel* Model = SkeletalMeshPtr->GetImportedModel();

		int32 NumLODInfos = SkeletalMeshPtr->GetLODNum();

		//When we delete a material slot we need to fix all MaterialIndex after the deleted index
		for (int32 LODInfoIdx = 0; LODInfoIdx < NumLODInfos; LODInfoIdx++)
		{
			TArray<FSkelMeshSection>& Sections = Model->LODModels[LODInfoIdx].Sections;
			TArray<int32>& LODMaterialMap = SkeletalMeshPtr->GetLODInfo(LODInfoIdx)->LODMaterialMap;
			for (int32 SectionIndex = 0; SectionIndex < Sections.Num(); ++SectionIndex)
			{
				int32 SectionMaterialIndex = Sections[SectionIndex].MaterialIndex;
				if (LODMaterialMap.IsValidIndex(SectionIndex) && LODMaterialMap[SectionIndex] != INDEX_NONE)
				{
					SectionMaterialIndex = LODMaterialMap[SectionIndex];
				}
				if (SectionMaterialIndex > MaterialIndex)
				{
					SectionMaterialIndex--;
					//Patch the lod material map
					while (!LODMaterialMap.IsValidIndex(SectionIndex))
					{
						LODMaterialMap.Add(INDEX_NONE);
					}
					LODMaterialMap[SectionIndex] = SectionMaterialIndex;
					Sections[SectionIndex].MaterialIndex--;
				}
			}
		}
	}
}

bool FPersonaMeshDetails::OnMaterialListDirty()
{
	bool ForceMaterialListRefresh = false;
	TMap<int32, TArray<FSectionLocalizer>> TempMaterialUsedMap;
	if (SkeletalMeshPtr.IsValid() && !SkeletalMeshPtr->IsCompiling())
	{
		const int32 MaterialCount = SkeletalMeshPtr->GetMaterials().Num();
		for (int32 MaterialIndex = 0; MaterialIndex < MaterialCount; ++MaterialIndex)
		{
			TArray<FSectionLocalizer> SectionLocalizers;
			FSkeletalMeshModel* ImportedResource = SkeletalMeshPtr->GetImportedModel();
			check(ImportedResource);
			for (int32 LODIndex = 0; LODIndex < ImportedResource->LODModels.Num(); ++LODIndex)
			{
				FSkeletalMeshLODInfo& Info = *(SkeletalMeshPtr->GetLODInfo(LODIndex));

				for (int32 SectionIndex = 0; SectionIndex < ImportedResource->LODModels[LODIndex].Sections.Num(); ++SectionIndex)
				{
					if (GetMaterialIndex(LODIndex, SectionIndex) == MaterialIndex)
					{
						SectionLocalizers.Add(FSectionLocalizer(LODIndex, SectionIndex));
					}
				}
			}
			TempMaterialUsedMap.Add(MaterialIndex, SectionLocalizers);
		}
	}
	if (TempMaterialUsedMap.Num() != MaterialUsedMap.Num())
	{
		ForceMaterialListRefresh = true;
	}
	else if (!ForceMaterialListRefresh)
	{
		for (auto KvpOld : MaterialUsedMap)
		{
			if (!TempMaterialUsedMap.Contains(KvpOld.Key))
			{
				ForceMaterialListRefresh = true;
				break;
			}
			const TArray<FSectionLocalizer> &TempSectionLocalizers = (*(TempMaterialUsedMap.Find(KvpOld.Key)));
			const TArray<FSectionLocalizer> &OldSectionLocalizers = KvpOld.Value;
			if (TempSectionLocalizers.Num() != OldSectionLocalizers.Num())
			{
				ForceMaterialListRefresh = true;
				break;
			}
			for (int32 SectionLocalizerIndex = 0; SectionLocalizerIndex < OldSectionLocalizers.Num(); ++SectionLocalizerIndex)
			{
				if (OldSectionLocalizers[SectionLocalizerIndex] != TempSectionLocalizers[SectionLocalizerIndex])
				{
					ForceMaterialListRefresh = true;
					break;
				}
			}
			if (ForceMaterialListRefresh)
	{
				break;
			}
		}
	}
	MaterialUsedMap = TempMaterialUsedMap;

	return ForceMaterialListRefresh;
}

void FPersonaMeshDetails::OnMaterialSlotOverlayMaterialChanged(UMaterialInterface* NewOverlayMaterial, int32 MaterialIndex)
{
	USkeletalMesh* Mesh = GetPersonaToolkit()->GetMesh();
	if (Mesh == nullptr)
		return;

	if (!ensure(Mesh->GetMaterials().IsValidIndex(MaterialIndex)))
	{
		return;
	}

	FSkeletalMaterial& SkeletalMaterial = Mesh->GetMaterials()[MaterialIndex];

	if (SkeletalMaterial.OverlayMaterialInterface == NewOverlayMaterial)
	{
		//Nothing change
		return;
	}

	//Make a transactional change
	{
		const FScopedTransaction Transaction(LOCTEXT("PersonaSetSectionOverlayMaterial", "Persona editor: Set Overlay Material For Material Slot"));
		Mesh->Modify();
		FScopedSuspendAlternateSkinWeightPreview ScopedSuspendAlternateSkinnWeightPreview(Mesh);
		{
			FScopedSkeletalMeshPostEditChange ScopedPostEditChange(Mesh);
			SkeletalMaterial.OverlayMaterialInterface = NewOverlayMaterial;
		}
	}
}

TSharedRef<SWidget> FPersonaMeshDetails::OnGenerateCustomNameWidgetsForSection(int32 LodIndex, int32 SectionIndex)
{
	bool IsSectionChunked = false;
	if (SkeletalMeshPtr.IsValid() && SkeletalMeshPtr->GetImportedModel() && SkeletalMeshPtr->GetImportedModel()->LODModels.IsValidIndex(LodIndex) && SkeletalMeshPtr->GetImportedModel()->LODModels[LodIndex].Sections.IsValidIndex(SectionIndex))
	{
		IsSectionChunked = SkeletalMeshPtr->GetImportedModel()->LODModels[LodIndex].Sections[SectionIndex].ChunkedParentSectionIndex != INDEX_NONE;
	}

	return SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SVerticalBox)
			.Visibility(this, &FPersonaMeshDetails::ShowEnabledSectionDetail, LodIndex, SectionIndex)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SCheckBox)
				.IsChecked(this, &FPersonaMeshDetails::IsSectionSelected, SectionIndex)
				.OnCheckStateChanged(this, &FPersonaMeshDetails::OnSectionSelectedChanged, SectionIndex)
				.ToolTipText(LOCTEXT("Highlight_ToolTip", "Highlights this section in the viewport"))
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.ColorAndOpacity(FLinearColor(0.4f, 0.4f, 0.4f, 1.0f))
					.Text(LOCTEXT("Highlight", "Highlight"))
				]
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0, 2, 0, 0)
			[
				SNew(SCheckBox)
				.IsChecked(this, &FPersonaMeshDetails::IsIsolateSectionEnabled, SectionIndex)
				.OnCheckStateChanged(this, &FPersonaMeshDetails::OnSectionIsolatedChanged, SectionIndex)
				.ToolTipText(LOCTEXT("Isolate_ToolTip", "Isolates this section in the viewport"))
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.ColorAndOpacity(FLinearColor(0.4f, 0.4f, 0.4f, 1.0f))
					.Text(LOCTEXT("Isolate", "Isolate"))
				]
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0, 2, 0, 0)
			[
				SNew(SBox)
				.Visibility(LodIndex == 0 && !IsSectionChunked ? EVisibility::Visible : EVisibility::Collapsed)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.FillWidth(1.0f)
					[
						SNew(SCheckBox)
						.IsChecked(this, &FPersonaMeshDetails::IsGenerateUpToSectionEnabled, LodIndex, SectionIndex)
						.OnCheckStateChanged(this, &FPersonaMeshDetails::OnSectionGenerateUpToChanged, LodIndex, SectionIndex)
						.ToolTipText(FText::Format(LOCTEXT("GenerateUpTo_ToolTip", "Generated LODs will use section {0} up to the specified value, and ignore it for lower quality LODs"), SectionIndex))
						[
							SNew(STextBlock)
							.Font(IDetailLayoutBuilder::GetDetailFont())
							.ColorAndOpacity(FLinearColor(0.4f, 0.4f, 0.4f, 1.0f))
							.Text(LOCTEXT("GenerateUpTo", "Generate Up To"))
						]
					]
					+SHorizontalBox::Slot()
					.Padding(5.f, 2.f, 5.f, 0.f)
					.AutoWidth()
					[
						SNew(SNumericEntryBox<int8>)
						.Visibility(this, &FPersonaMeshDetails::ShowSectionGenerateUpToSlider, LodIndex, SectionIndex)
						.Font(IDetailLayoutBuilder::GetDetailFont())
						.MinDesiredValueWidth(40.0f)
						.MinValue(static_cast<int8>(LodIndex))
						.MinSliderValue(static_cast<int8>(LodIndex))
						.MaxSliderValue(static_cast<int8>(FMath::Max(8, LODCount)))
						.AllowSpin(true)
						.Value(this, &FPersonaMeshDetails::GetSectionGenerateUpToValue, LodIndex, SectionIndex)
						.OnValueChanged(this, &FPersonaMeshDetails::SetSectionGenerateUpToValue, LodIndex, SectionIndex)
						.OnValueCommitted(this, &FPersonaMeshDetails::SetSectionGenerateUpToValueCommitted, LodIndex, SectionIndex)
					]
				]
			]
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(STextBlock)
			.Visibility(this, &FPersonaMeshDetails::ShowDisabledSectionDetail, LodIndex, SectionIndex)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.ColorAndOpacity(FLinearColor(0.4f, 0.4f, 0.4f, 1.0f))
			.Text(LOCTEXT("SectionDisabled", "Disabled"))
			.ToolTipText(LOCTEXT("SectionDisable_ToolTip", "The section will not be rendered."))
		];
}

TSharedRef<SWidget> FPersonaMeshDetails::OnGenerateCustomSectionWidgetsForSection(int32 LODIndex, int32 SectionIndex)
{
	TSharedRef<SVerticalBox> SectionWidget = SNew(SVerticalBox);
	
	//If we have a chunk section, prevent editing of cloth cast shadow and recompute tangent
	if (SkeletalMeshPtr.IsValid() && SkeletalMeshPtr->GetImportedModel() && SkeletalMeshPtr->GetImportedModel()->LODModels.IsValidIndex(LODIndex) && SkeletalMeshPtr->GetImportedModel()->LODModels[LODIndex].Sections.IsValidIndex(SectionIndex))
	{
		if (SkeletalMeshPtr->GetImportedModel()->LODModels[LODIndex].Sections[SectionIndex].ChunkedParentSectionIndex != INDEX_NONE)
		{
			return SectionWidget;
		}
	}

	UpdateClothingEntries();

	ClothComboBoxes.AddDefaulted();

	SectionWidget->AddSlot()
	.AutoHeight()
	.Padding(0, 2, 0, 0)
	.HAlign(HAlign_Fill)
	[
	SNew(SHorizontalBox)
	+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(SBox)
			.HAlign(HAlign_Right)
			.MinDesiredWidth(65.0f)
		[
			SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(LOCTEXT("Clothing", "Clothing"))
		]
		]
		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.Padding(5, 2, 0, 0)
		[
			SAssignNew(ClothComboBoxes.Last(), SClothComboBox)
			.OnGenerateWidget(this, &FPersonaMeshDetails::OnGenerateWidgetForClothingEntry)
			.OnSelectionChanged(this, &FPersonaMeshDetails::OnClothingSelectionChanged, ClothComboBoxes.Num() - 1, LODIndex, SectionIndex)
			.OnComboBoxOpening(this, &FPersonaMeshDetails::OnClothingComboBoxOpening)
			.OptionsSource(&NewClothingAssetEntries)
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(this, &FPersonaMeshDetails::OnGetClothingComboText, LODIndex, SectionIndex)
			]
		]
	];

	SectionWidget->AddSlot()
	.AutoHeight()
	.Padding(0, 2, 0, 0)
	.HAlign(HAlign_Fill)
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(SBox)
			.HAlign(HAlign_Right)
			.MinDesiredWidth(65.0f)
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(LOCTEXT("RecomputeTangents", "Recompute Tangents"))
			]
		]
		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.Padding(5, 2, 0, 0)
		[
			SNew(SComboButton)
			.OnGetMenuContent(this, &FPersonaMeshDetails::OnGenerateRecomputeTangentsSetting, LODIndex, SectionIndex)
			.VAlign(VAlign_Center)
			.ContentPadding(2.f)
			.ButtonContent()
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(this, &FPersonaMeshDetails::GetCurrentRecomputeTangentsSetting, LODIndex, SectionIndex)
				.ToolTipText(LOCTEXT("RecomputeTangentsToolTip", "Specifies how tangents are recomputed after skinning, at runtime. Requires skin cache to be enabled for this mesh."))
			]
		]
	];

	SectionWidget->AddSlot()
	.AutoHeight()
	.Padding(0, 2, 0, 0)
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2, 0, 2, 0)
		[
			SNew(SCheckBox)
			.IsChecked(this, &FPersonaMeshDetails::IsSectionShadowCastingEnabled, LODIndex, SectionIndex)
			.OnCheckStateChanged(this, &FPersonaMeshDetails::OnSectionShadowCastingChanged, LODIndex, SectionIndex)
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(LOCTEXT("Cast Shadows", "Cast Shadows"))
			]
		]
	];

	SectionWidget->AddSlot()
	.AutoHeight()
	.Padding(0, 2, 0, 0)
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2, 0, 2, 0)
		[
			SNew(SCheckBox)
			.IsChecked(this, &FPersonaMeshDetails::IsSectionVisibleInRayTracingEnabled, LODIndex, SectionIndex)
			.OnCheckStateChanged(this, &FPersonaMeshDetails::OnSectionVisibleInRayTracingChanged, LODIndex, SectionIndex)
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(LOCTEXT("Visible In RayTracing", "Visible In RayTracing"))
			]
		]
	];
	return SectionWidget;
}

bool FPersonaMeshDetails::IsSectionEnabled(int32 LodIndex, int32 SectionIndex) const
{
	if(SkeletalMeshPtr.IsValid() && !SkeletalMeshPtr->IsCompiling())
	{
		FSkeletalMeshModel* SourceModel = SkeletalMeshPtr->GetImportedModel();

		if(SourceModel->LODModels.IsValidIndex(LodIndex))
		{
			FSkeletalMeshLODModel& LodModel = SourceModel->LODModels[LodIndex];

			if(LodModel.Sections.IsValidIndex(SectionIndex))
			{
				return !LodModel.Sections[SectionIndex].bDisabled;
			}
		}
	}

	return false;
}

EVisibility FPersonaMeshDetails::ShowEnabledSectionDetail(int32 LodIndex, int32 SectionIndex) const
{
	return IsSectionEnabled(LodIndex, SectionIndex) ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility FPersonaMeshDetails::ShowDisabledSectionDetail(int32 LodIndex, int32 SectionIndex) const
{
	return IsSectionEnabled(LodIndex, SectionIndex) ? EVisibility::Collapsed : EVisibility::Visible;
}

void FPersonaMeshDetails::OnSectionEnabledChanged(int32 LodIndex, int32 SectionIndex, bool bEnable)
{
	if(SkeletalMeshPtr.IsValid())
	{
		FSkeletalMeshModel* SourceModel = SkeletalMeshPtr->GetImportedModel();

		if(SourceModel->LODModels.IsValidIndex(LodIndex))
		{
			FSkeletalMeshLODModel& LodModel = SourceModel->LODModels[LodIndex];

			if(LodModel.Sections.IsValidIndex(SectionIndex))
			{
				FSkelMeshSection& Section = LodModel.Sections[SectionIndex];

				if(Section.bDisabled != !bEnable)
				{
					FScopedSuspendAlternateSkinWeightPreview ScopedSuspendAlternateSkinnWeightPreview(SkeletalMeshPtr.Get());
					{
						FScopedSkeletalMeshPostEditChange ScopedPostEditChange(SkeletalMeshPtr.Get());
						FScopedTransaction Transaction(LOCTEXT("ChangeSectionEnabled", "Set section disabled flag."));

						SkeletalMeshPtr->Modify();
						SkeletalMeshPtr->PreEditChange(nullptr);

						Section.bDisabled = !bEnable;
						for (int32 AfterSectionIndex = SectionIndex + 1; AfterSectionIndex < LodModel.Sections.Num(); ++AfterSectionIndex)
						{
							if (LodModel.Sections[AfterSectionIndex].ChunkedParentSectionIndex == SectionIndex)
							{
								LodModel.Sections[AfterSectionIndex].bDisabled = Section.bDisabled;
							}
							else
							{
								break;
							}
						}
						//We display only the parent chunk
						check(Section.ChunkedParentSectionIndex == INDEX_NONE);

						SetSkelMeshSourceSectionUserData(LodModel, SectionIndex, Section.OriginalDataSectionIndex);

						// Disable highlight and isolate flags
						UDebugSkelMeshComponent * MeshComponent = GetPersonaToolkit()->GetPreviewScene()->GetPreviewMeshComponent();
						if (MeshComponent)
						{
							MeshComponent->SetSelectedEditorSection(INDEX_NONE);
							MeshComponent->SetSelectedEditorMaterial(INDEX_NONE);
							MeshComponent->SetMaterialPreview(INDEX_NONE);
							MeshComponent->SetSectionPreview(INDEX_NONE);
						}
					}
				}
			}
		}
	}
}

TOptional<int8> FPersonaMeshDetails::GetSectionGenerateUpToValue(int32 LodIndex, int32 SectionIndex) const
{
	if (!SkeletalMeshPtr.IsValid() ||
		SkeletalMeshPtr->IsCompiling() ||
		!SkeletalMeshPtr->GetImportedModel()->LODModels.IsValidIndex(LodIndex) ||
		!SkeletalMeshPtr->GetImportedModel()->LODModels[LodIndex].Sections.IsValidIndex(SectionIndex) )
	{
		return TOptional<int8>(-1);
	}
	const int8 SpecifiedLodIndex = static_cast<int8>(SkeletalMeshPtr->GetImportedModel()->LODModels[LodIndex].Sections[SectionIndex].GenerateUpToLodIndex);
	check(SpecifiedLodIndex == -1 || SpecifiedLodIndex >= LodIndex);
	return TOptional<int8>(SpecifiedLodIndex);
}

void FPersonaMeshDetails::SetSectionGenerateUpToValue(int8 Value, int32 LodIndex, int32 SectionIndex)
{
	if (!SkeletalMeshPtr.IsValid() ||
		!SkeletalMeshPtr->GetImportedModel()->LODModels.IsValidIndex(LodIndex) ||
		!SkeletalMeshPtr->GetImportedModel()->LODModels[LodIndex].Sections.IsValidIndex(SectionIndex))
	{
		return;
	}
	const int64 ValueKey = ((int64)LodIndex << 32) | (int64)SectionIndex;
	if (!OldGenerateUpToSliderValues.Contains(ValueKey))
	{
		OldGenerateUpToSliderValues.Add(ValueKey, static_cast<int8>(SkeletalMeshPtr->GetImportedModel()->LODModels[LodIndex].Sections[SectionIndex].GenerateUpToLodIndex));
	}
	SkeletalMeshPtr->GetImportedModel()->LODModels[LodIndex].Sections[SectionIndex].GenerateUpToLodIndex = Value;
}

void FPersonaMeshDetails::SetSectionGenerateUpToValueCommitted(int8 Value, ETextCommit::Type CommitInfo, int32 LodIndex, int32 SectionIndex)
{
	int64 ValueKey = ((int64)LodIndex << 32) | (int64)SectionIndex;
	int8 OldValue;
	bool bHasOldValue = OldGenerateUpToSliderValues.RemoveAndCopyValue(ValueKey, OldValue);
	if (!SkeletalMeshPtr.IsValid() ||
		!SkeletalMeshPtr->GetImportedModel()->LODModels.IsValidIndex(LodIndex) ||
		!SkeletalMeshPtr->GetImportedModel()->LODModels[LodIndex].Sections.IsValidIndex(SectionIndex))
	{
		return;
	}
	
	if (bHasOldValue)
	{
		//Put back the original value before registering the undo transaction
		SkeletalMeshPtr->GetImportedModel()->LODModels[LodIndex].Sections[SectionIndex].GenerateUpToLodIndex = OldValue;
	}

	if (CommitInfo == ETextCommit::OnCleared)
	{
		//If the user cancel is change early exit while the value is the same as the original
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("ChangeGenerateUpTo", "Set Generate Up To"));

	SkeletalMeshPtr->Modify();
	FSkeletalMeshLODModel& LODModel = SkeletalMeshPtr->GetImportedModel()->LODModels[LodIndex];
	FSkelMeshSection& Section = LODModel.Sections[SectionIndex];
	Section.GenerateUpToLodIndex = Value;
	for (int32 AfterSectionIndex = SectionIndex + 1; AfterSectionIndex < LODModel.Sections.Num(); ++AfterSectionIndex)
	{
		if (LODModel.Sections[AfterSectionIndex].ChunkedParentSectionIndex == SectionIndex)
		{
			LODModel.Sections[AfterSectionIndex].GenerateUpToLodIndex = Value;
		}
		else
		{
			break;
		}
	}
	//We display only the parent chunk
	check(Section.ChunkedParentSectionIndex == INDEX_NONE);

	SetSkelMeshSourceSectionUserData(LODModel, SectionIndex, Section.OriginalDataSectionIndex);

}

EVisibility FPersonaMeshDetails::ShowSectionGenerateUpToSlider(int32 LodIndex, int32 SectionIndex) const
{
	if (!SkeletalMeshPtr.IsValid() ||
		SkeletalMeshPtr->IsCompiling() ||
		!SkeletalMeshPtr->GetImportedModel()->LODModels.IsValidIndex(LodIndex) ||
		!SkeletalMeshPtr->GetImportedModel()->LODModels[LodIndex].Sections.IsValidIndex(SectionIndex))
	{
		return EVisibility::Collapsed;
	}
	return SkeletalMeshPtr->GetImportedModel()->LODModels[LodIndex].Sections[SectionIndex].GenerateUpToLodIndex == -1 ? EVisibility::Collapsed : EVisibility::Visible;
}

ECheckBoxState FPersonaMeshDetails::IsGenerateUpToSectionEnabled(int32 LodIndex, int32 SectionIndex) const
{
	if (!SkeletalMeshPtr.IsValid() ||
		SkeletalMeshPtr->IsCompiling() ||
		!SkeletalMeshPtr->GetImportedModel()->LODModels.IsValidIndex(LodIndex) ||
		!SkeletalMeshPtr->GetImportedModel()->LODModels[LodIndex].Sections.IsValidIndex(SectionIndex))
	{
		return ECheckBoxState::Unchecked;
	}
	return SkeletalMeshPtr->GetImportedModel()->LODModels[LodIndex].Sections[SectionIndex].GenerateUpToLodIndex != -1 ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void FPersonaMeshDetails::OnSectionGenerateUpToChanged(ECheckBoxState NewState, int32 LodIndex, int32 SectionIndex)
{
	SetSectionGenerateUpToValueCommitted(NewState == ECheckBoxState::Checked ? static_cast<int8>(LodIndex) : INDEX_NONE, ETextCommit::Type::Default , LodIndex, SectionIndex);
}

void FPersonaMeshDetails::SetCurrentLOD(int32 NewLodIndex)
{
	if (GetPersonaToolkit()->GetPreviewMeshComponent() == nullptr)
	{
		return;
	}
	int32 CurrentDisplayLOD = GetPersonaToolkit()->GetPreviewMeshComponent()->GetForcedLOD();
	int32 RealNewLOD = NewLodIndex == 0 ? 0 : NewLodIndex - 1;
	if (CurrentDisplayLOD == NewLodIndex || !LodCategories.IsValidIndex(RealNewLOD))
	{
		return;
	}
	GetPersonaToolkit()->GetPreviewMeshComponent()->SetForcedLOD(NewLodIndex);
	
	//Reset the preview section since we do not edit the same LOD
	GetPersonaToolkit()->GetPreviewMeshComponent()->SetSectionPreview(INDEX_NONE);
	GetPersonaToolkit()->GetPreviewMeshComponent()->SetSelectedEditorSection(INDEX_NONE);

	GetPersonaToolkit()->GetPreviewScene()->BroadcastOnSelectedLODChanged();
}

void FPersonaMeshDetails::UpdateLODCategoryVisibility() const
{
	if (GetPersonaToolkit()->GetCustomData(CustomDataKey_LODEditMode) > 0)
	{
		//Do not change the Category visibility if we are in custom mode
		return;
	}
	bool bAutoLod = false;
	if (GetPersonaToolkit()->GetPreviewMeshComponent() != nullptr)
	{
		bAutoLod = GetPersonaToolkit()->GetPreviewMeshComponent()->GetForcedLOD() == 0;
	}
	int32 CurrentDisplayLOD = bAutoLod ? 0 : GetPersonaToolkit()->GetPreviewMeshComponent()->GetForcedLOD() - 1;
	if (LodCategories.IsValidIndex(CurrentDisplayLOD) && GetPersonaToolkit()->GetMesh())
	{
		int32 SkeletalMeshLodNumber = GetPersonaToolkit()->GetMesh()->GetLODNum();
		for (int32 LodCategoryIndex = 0; LodCategoryIndex < SkeletalMeshLodNumber; ++LodCategoryIndex)
		{
			LodCategories[LodCategoryIndex]->SetCategoryVisibility(CurrentDisplayLOD == LodCategoryIndex);
		}
	}

	//Reset the preview section since we do not edit the same LOD
	GetPersonaToolkit()->GetPreviewMeshComponent()->SetSectionPreview(INDEX_NONE);
	GetPersonaToolkit()->GetPreviewMeshComponent()->SetSelectedEditorSection(INDEX_NONE);
}

FText FPersonaMeshDetails::GetCurrentLodName() const
{
	if (GetAutoLod(GetPersonaToolkit()->GetPreviewMeshComponent()))
	{
		return LOCTEXT("LODAutoLabel", "Auto (LOD 0)");
	}

	const int32 CurrentDisplayLOD = GetCurrentDisplayLODIndex(GetPersonaToolkit()->GetPreviewMeshComponent());
	return FText::Format(LOCTEXT("LODIndexLabel", "LOD {0}"), FText::AsNumber(CurrentDisplayLOD));
}

bool FPersonaMeshDetails::GetAutoLod(USkeletalMeshComponent* InMeshComponent)
{
	return (InMeshComponent != nullptr) ? InMeshComponent->GetForcedLOD() == 0 : false;
}

int32 FPersonaMeshDetails::GetCurrentDisplayLODIndex(USkeletalMeshComponent* InMeshComponent)
{
	int32 CurrentDisplayLOD = 0;
	if (InMeshComponent != nullptr)
	{
		CurrentDisplayLOD = GetAutoLod(InMeshComponent) ? 0 : InMeshComponent->GetForcedLOD() - 1;
	}
	return CurrentDisplayLOD;
}

FText FPersonaMeshDetails::GetCurrentLodTooltip() const
{
	if (GetPersonaToolkit()->GetPreviewMeshComponent() != nullptr && GetPersonaToolkit()->GetPreviewMeshComponent()->GetForcedLOD() == 0)
	{
		return LOCTEXT("PersonaLODPickerCurrentLODTooltip", "With Auto LOD selected, LOD 0's properties are shown by default");
	}
	return FText::GetEmpty();
}

TSharedRef<SWidget> FPersonaMeshDetails::OnGenerateLodComboBoxForLodPicker()
{
	return SNew(SComboButton)
		.IsEnabled(this, &FPersonaMeshDetails::IsLodComboBoxEnabledForLodPicker)
		.OnGetMenuContent(this, &FPersonaMeshDetails::OnGenerateLodMenuForLodPicker)
		.VAlign(VAlign_Center)
		.ContentPadding(2.f)
		.ButtonContent()
		[
			SNew(STextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(this, &FPersonaMeshDetails::GetCurrentLodName)
			.ToolTipText(this, &FPersonaMeshDetails::GetCurrentLodTooltip)
		];
}

EVisibility FPersonaMeshDetails::LodComboBoxVisibilityForLodPicker() const
{
	//No combo box when in Custom mode
	if (GetPersonaToolkit()->GetCustomData(CustomDataKey_LODEditMode) > 0)
	{
		return EVisibility::Hidden;
	}
	return EVisibility::Visible;
}

bool FPersonaMeshDetails::IsLodComboBoxEnabledForLodPicker() const
{
	//No combo box when in Custom mode
	return !(GetPersonaToolkit()->GetCustomData(CustomDataKey_LODEditMode) > 0);
}

TSharedRef<SWidget> FPersonaMeshDetails::OnGenerateLodMenuForLodPicker()
{
	USkeletalMesh* SkelMesh = GetPersonaToolkit()->GetMesh();

	if (SkelMesh == nullptr || SkelMesh->IsCompiling())
	{
		return SNullWidget::NullWidget;
	}

	bool bAutoLod = false;
	if (GetPersonaToolkit()->GetPreviewMeshComponent() != nullptr)
	{
		bAutoLod = GetPersonaToolkit()->GetPreviewMeshComponent()->GetForcedLOD() == 0;
	}
	const int32 SkelMeshLODCount = SkelMesh->GetLODNum();
	if(SkelMeshLODCount < 2)
	{
		return SNullWidget::NullWidget;
	}
	FMenuBuilder MenuBuilder(true, nullptr);

	FUIAction AutoLodAction(FExecuteAction::CreateSP(this, &FPersonaMeshDetails::SetCurrentLOD, 0));
	MenuBuilder.AddMenuEntry(LOCTEXT("LODAutoLabel", "Auto (LOD 0)"), LOCTEXT("OnGenerateLodMenuForSectionList_Auto_ToolTip", "With Auto LOD selected, LOD 0's properties are visible for editing."), FSlateIcon(), AutoLodAction);
	// Add a menu item for each texture.  Clicking on the texture will display it in the content browser
	for (int32 AllLodIndex = 0; AllLodIndex < SkelMeshLODCount; ++AllLodIndex)
	{
		FText LODLevelString = FText::Format(LOCTEXT("LODIndexLabel", "LOD {0}"), FText::AsNumber(AllLodIndex));
		FUIAction Action(FExecuteAction::CreateSP(this, &FPersonaMeshDetails::SetCurrentLOD, AllLodIndex+1));
		MenuBuilder.AddMenuEntry(LODLevelString, FText::GetEmpty(), FSlateIcon(), Action);
	}

	return MenuBuilder.MakeWidget();
}

ECheckBoxState FPersonaMeshDetails::IsMaterialSelected(int32 MaterialIndex) const
{
	ECheckBoxState State = ECheckBoxState::Unchecked;
	UDebugSkelMeshComponent* MeshComponent = GetPersonaToolkit()->GetPreviewMeshComponent();
	if (MeshComponent)
	{
		State = MeshComponent->GetSelectedEditorMaterial() == MaterialIndex ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}
	return State;
}

void FPersonaMeshDetails::OnMaterialSelectedChanged(ECheckBoxState NewState, int32 MaterialIndex)
{
	// Currently assumes that we only ever have one preview mesh in Persona.
	UDebugSkelMeshComponent* MeshComponent = GetPersonaToolkit()->GetPreviewMeshComponent();
	if (MeshComponent)
	{
		if (NewState == ECheckBoxState::Checked)
		{
			MeshComponent->SetSelectedEditorMaterial(MaterialIndex);
			if (MeshComponent->GetMaterialPreview() != MaterialIndex)
			{
				// Unhide all mesh sections
				MeshComponent->SetMaterialPreview(INDEX_NONE);
			}
			//Remove any section isolate or highlight
			MeshComponent->SetSelectedEditorSection(INDEX_NONE);
			MeshComponent->SetSectionPreview(INDEX_NONE);
		}
		else if (NewState == ECheckBoxState::Unchecked)
		{
			MeshComponent->SetSelectedEditorMaterial(INDEX_NONE);
		}
		MeshComponent->PushSelectionToProxy();
		GetPersonaToolkit()->GetPreviewScene()->InvalidateViews();
	}
}

ECheckBoxState FPersonaMeshDetails::IsIsolateMaterialEnabled(int32 MaterialIndex) const
{
	ECheckBoxState State = ECheckBoxState::Unchecked;
	const UDebugSkelMeshComponent* MeshComponent = GetPersonaToolkit()->GetPreviewMeshComponent();
	if (MeshComponent)
	{
		State = MeshComponent->GetMaterialPreview() == MaterialIndex ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}
	return State;
}

void FPersonaMeshDetails::OnMaterialIsolatedChanged(ECheckBoxState NewState, int32 MaterialIndex)
{
	UDebugSkelMeshComponent * MeshComponent = GetPersonaToolkit()->GetPreviewMeshComponent();
	if (MeshComponent)
	{
		if (NewState == ECheckBoxState::Checked)
		{
			MeshComponent->SetMaterialPreview(MaterialIndex);
			if (MeshComponent->GetSelectedEditorMaterial() != MaterialIndex)
			{
				MeshComponent->SetSelectedEditorMaterial(INDEX_NONE);
			}
			//Remove any section isolate or highlight
			MeshComponent->SetSelectedEditorSection(INDEX_NONE);
			MeshComponent->SetSectionPreview(INDEX_NONE);
		}
		else if (NewState == ECheckBoxState::Unchecked)
		{
			MeshComponent->SetMaterialPreview(INDEX_NONE);
		}
		GetPersonaToolkit()->GetPreviewScene()->InvalidateViews();
	}
}

ECheckBoxState FPersonaMeshDetails::IsSectionSelected(int32 SectionIndex) const
{
	ECheckBoxState State = ECheckBoxState::Unchecked;
	UDebugSkelMeshComponent * MeshComponent = GetPersonaToolkit()->GetPreviewMeshComponent();
	if (MeshComponent)
	{
		State = MeshComponent->GetSelectedEditorSection() == SectionIndex ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	return State;
}

void FPersonaMeshDetails::OnSectionSelectedChanged(ECheckBoxState NewState, int32 SectionIndex)
{
	// Currently assumes that we only ever have one preview mesh in Persona.
	UDebugSkelMeshComponent* MeshComponent = GetPersonaToolkit()->GetPreviewMeshComponent();

	if (MeshComponent)
	{
		if (NewState == ECheckBoxState::Checked)
		{
			MeshComponent->SetSelectedEditorSection(SectionIndex);
			if (MeshComponent->GetSectionPreview() != SectionIndex)
			{
				// Unhide all mesh sections
				MeshComponent->SetSectionPreview(INDEX_NONE);
			}
			MeshComponent->SetSelectedEditorMaterial(INDEX_NONE);
			MeshComponent->SetMaterialPreview(INDEX_NONE);
		}
		else if (NewState == ECheckBoxState::Unchecked)
		{
			MeshComponent->SetSelectedEditorSection(INDEX_NONE);
		}
		MeshComponent->PushSelectionToProxy();
		GetPersonaToolkit()->GetPreviewScene()->InvalidateViews();
	}
}

ECheckBoxState FPersonaMeshDetails::IsIsolateSectionEnabled(int32 SectionIndex) const
{
	ECheckBoxState State = ECheckBoxState::Unchecked;
	const UDebugSkelMeshComponent* MeshComponent = GetPersonaToolkit()->GetPreviewMeshComponent();
	if (MeshComponent)
	{
		State = MeshComponent->GetSectionPreview() == SectionIndex ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}
	return State;
}

void FPersonaMeshDetails::OnSectionIsolatedChanged(ECheckBoxState NewState, int32 SectionIndex)
{
	USkeletalMesh* Mesh = GetPersonaToolkit()->GetMesh();
	UDebugSkelMeshComponent * MeshComponent = GetPersonaToolkit()->GetPreviewMeshComponent();
	if (Mesh && MeshComponent)
	{
		if (NewState == ECheckBoxState::Checked)
		{
			MeshComponent->SetSectionPreview(SectionIndex);
			if (MeshComponent->GetSelectedEditorSection() != SectionIndex)
			{
				MeshComponent->SetSelectedEditorSection(INDEX_NONE);
			}
			MeshComponent->SetMaterialPreview(INDEX_NONE);
			MeshComponent->SetSelectedEditorMaterial(INDEX_NONE);
		}
		else if (NewState == ECheckBoxState::Unchecked)
		{
			MeshComponent->SetSectionPreview(INDEX_NONE);
		}
		GetPersonaToolkit()->GetPreviewScene()->InvalidateViews();
	}
}

ECheckBoxState FPersonaMeshDetails::IsSectionShadowCastingEnabled(int32 LODIndex, int32 SectionIndex) const
{
	ECheckBoxState State = ECheckBoxState::Unchecked;
	const USkeletalMesh* Mesh = GetPersonaToolkit()->GetMesh();
	if (Mesh == nullptr || Mesh->IsCompiling())
		return State;
	
	check(Mesh->GetImportedModel());

	if (!Mesh->GetImportedModel()->LODModels.IsValidIndex(LODIndex))
		return State;

	const FSkeletalMeshLODModel& LODModel = Mesh->GetImportedModel()->LODModels[LODIndex];

	if (!LODModel.Sections.IsValidIndex(SectionIndex))
		return State;

	const FSkelMeshSection& Section = LODModel.Sections[SectionIndex];

	State = Section.bCastShadow ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	return State;
}

void FPersonaMeshDetails::OnSectionShadowCastingChanged(ECheckBoxState NewState, int32 LODIndex, int32 SectionIndex)
{
	ECheckBoxState State = ECheckBoxState::Unchecked;
	USkeletalMesh* Mesh = GetPersonaToolkit()->GetMesh();
	if (Mesh == nullptr)
		return;

	check(Mesh->GetImportedModel());

	if (!Mesh->GetImportedModel()->LODModels.IsValidIndex(LODIndex))
		return;

	FSkeletalMeshLODModel& LODModel = Mesh->GetImportedModel()->LODModels[LODIndex];

	if (!LODModel.Sections.IsValidIndex(SectionIndex))
		return;

	FSkelMeshSection& Section = LODModel.Sections[SectionIndex];

	//Update Original PolygonGroup
	auto UpdatePolygonGroupCastShadow = [&Mesh, &LODModel, &Section, &SectionIndex](bool bCastShadow)
	{
		FScopedSuspendAlternateSkinWeightPreview ScopedSuspendAlternateSkinnWeightPreview(Mesh);
		{
			FScopedSkeletalMeshPostEditChange ScopedPostEditChange(Mesh);
			Section.bCastShadow = bCastShadow;
			//We change only the parent chunk data
			check(Section.ChunkedParentSectionIndex == INDEX_NONE);

			//The post edit change will kick a build
			SetSkelMeshSourceSectionUserData(LODModel, SectionIndex, Section.OriginalDataSectionIndex);
		}
	};

	if (NewState == ECheckBoxState::Checked)
	{
		const FScopedTransaction Transaction(LOCTEXT("PersonaSetSectionShadowCastingFlag", "Persona editor: Set Shadow Casting For Section"));
		Mesh->Modify();
		UpdatePolygonGroupCastShadow(true);
	}
	else if (NewState == ECheckBoxState::Unchecked)
	{
		const FScopedTransaction Transaction(LOCTEXT("PersonaClearSectionShadowCastingFlag", "Persona editor: Clear Shadow Casting For Section"));
		Mesh->Modify();
		UpdatePolygonGroupCastShadow(false);
	}
}

ECheckBoxState FPersonaMeshDetails::IsSectionVisibleInRayTracingEnabled(int32 LODIndex, int32 SectionIndex) const
{
	ECheckBoxState State = ECheckBoxState::Unchecked;
	const USkeletalMesh* Mesh = GetPersonaToolkit()->GetMesh();
	if (Mesh == nullptr || Mesh->IsCompiling())
		return State;

	check(Mesh->GetImportedModel());

	if (!Mesh->GetImportedModel()->LODModels.IsValidIndex(LODIndex))
		return State;

	const FSkeletalMeshLODModel& LODModel = Mesh->GetImportedModel()->LODModels[LODIndex];

	if (!LODModel.Sections.IsValidIndex(SectionIndex))
		return State;

	const FSkelMeshSection& Section = LODModel.Sections[SectionIndex];

	State = Section.bVisibleInRayTracing ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	return State;
}

void FPersonaMeshDetails::OnSectionVisibleInRayTracingChanged(ECheckBoxState NewState, int32 LODIndex, int32 SectionIndex)
{
	ECheckBoxState State = ECheckBoxState::Unchecked;
	USkeletalMesh* Mesh = GetPersonaToolkit()->GetMesh();
	if (Mesh == nullptr)
		return;

	check(Mesh->GetImportedModel());

	if (!Mesh->GetImportedModel()->LODModels.IsValidIndex(LODIndex))
		return;

	FSkeletalMeshLODModel& LODModel = Mesh->GetImportedModel()->LODModels[LODIndex];

	if (!LODModel.Sections.IsValidIndex(SectionIndex))
		return;

	FSkelMeshSection& Section = LODModel.Sections[SectionIndex];

	//Update Original PolygonGroup
	auto UpdatePolygonGroupVisibleInRayTracing = [&Mesh, &LODModel, &Section, &SectionIndex](bool bVisibleInRayTracing)
	{
		FScopedSuspendAlternateSkinWeightPreview ScopedSuspendAlternateSkinnWeightPreview(Mesh);
		{
			FScopedSkeletalMeshPostEditChange ScopedPostEditChange(Mesh);
			Section.bVisibleInRayTracing = bVisibleInRayTracing;
			//We change only the parent chunk data
			check(Section.ChunkedParentSectionIndex == INDEX_NONE);

			//The post edit change will kick a build
			SetSkelMeshSourceSectionUserData(LODModel, SectionIndex, Section.OriginalDataSectionIndex);
		}
	};

	if (NewState == ECheckBoxState::Checked)
	{
		const FScopedTransaction Transaction(LOCTEXT("PersonaSetSectionVisibleInRayTracingFlag", "Persona editor: Set Visible In RayTracing For Section"));
		Mesh->Modify();
		UpdatePolygonGroupVisibleInRayTracing(true);
	}
	else if (NewState == ECheckBoxState::Unchecked)
	{
		const FScopedTransaction Transaction(LOCTEXT("PersonaClearSectionVisibleInRayTracinggFlag", "Persona editor: Clear Visible In RayTracing For Section"));
		Mesh->Modify();
		UpdatePolygonGroupVisibleInRayTracing(false);
	}
}

static const FText RecomputeTangentsChannelLabels[][2] = {
	{
		LOCTEXT("RecomputeTangentChannel_Disabled", "Disabled"),
		LOCTEXT("RecomputeTangentChannelTooltip_Disabled", "Tangents will not be recomputed after skinning for this section."),
	},
	{LOCTEXT("RecomputeTangentChannel_Enabled", "Enabled with no mask"),
		LOCTEXT("RecomputeTangentChannelTooltip_Enabled", "Tangents will be recomputed after skinning for this section."),
	},
	{LOCTEXT("RecomputeTangentChannel_Red", "Enabled using red mask"), 
		LOCTEXT("RecomputeTangentChannelTooltip_Red", "Tangents will be recomputed after skinning, for this section, and blended with the base pose normal using the red vertex color channel as a mask to blend in the recomputed tangents."),
	},
	{
		LOCTEXT("RecomputeTangentChannel_Green", "Enabled using green mask"),
		LOCTEXT("RecomputeTangentChannelTooltip_Green", "Tangents will be recomputed after skinning, for this section, and blended with the base pose normal using the green vertex color channel as a mask to blend in the recomputed tangents."),
	},
	{
		LOCTEXT("RecomputeTangentChannel_Blue", "Enabled using blue mask"),
		LOCTEXT("RecomputeTangentChannelTooltip_Blue", "Tangents will be recomputed after skinning, for this section, and blended with the base pose normal using the blue vertex color channel as a mask to blend in the recomputed tangents."),
	},
};

FText FPersonaMeshDetails::GetCurrentRecomputeTangentsSetting(int32 LODIndex, int32 SectionIndex) const
{
	const USkeletalMesh* Mesh = GetPersonaToolkit()->GetMesh();
	if (Mesh == nullptr || Mesh->IsCompiling())
		return FText::GetEmpty();

	check(Mesh->GetImportedModel());

	if (!Mesh->GetImportedModel()->LODModels.IsValidIndex(LODIndex))
		return FText::GetEmpty();

	FSkeletalMeshLODModel& LODModel = Mesh->GetImportedModel()->LODModels[LODIndex];

	if (!LODModel.Sections.IsValidIndex(SectionIndex))
		return FText::GetEmpty();

	FSkelMeshSection& Section = LODModel.Sections[SectionIndex];

	// Index_0 None:  bRecomputeTangent == false
	// Index_1 All:   bRecomputeTangent == true & RecomputeTangentsVertexMaskChannel == ESkinVertexColorChannel::None
	// Index_2 Red:   bRecomputeTangent == true & RecomputeTangentsVertexMaskChannel == ESkinVertexColorChannel::Red
	// Index_3 Green: bRecomputeTangent == true & RecomputeTangentsVertexMaskChannel == ESkinVertexColorChannel::Green
	// Index_4 Blue:  bRecomputeTangent == true & RecomputeTangentsVertexMaskChannel == ESkinVertexColorChannel::Blue
	int32 ChannelIndex = 0;
	if (Section.bRecomputeTangent)
	{
		if (Section.RecomputeTangentsVertexMaskChannel == ESkinVertexColorChannel::None)
			ChannelIndex = 1;
		else
			ChannelIndex = FMath::Clamp<int32>(static_cast<int32>(Section.RecomputeTangentsVertexMaskChannel) + 2, 0, 4);
	}
	
	return RecomputeTangentsChannelLabels[ChannelIndex][0];
}

TSharedRef<SWidget> FPersonaMeshDetails::OnGenerateRecomputeTangentsSetting(int32 LODIndex, int32 SectionIndex)
{
	FMenuBuilder MenuBuilder(true, nullptr);
	for (int32 ChannelIndex = 0; ChannelIndex < 5; ChannelIndex++) { 
		FUIAction Action(FExecuteAction::CreateSP(this, &FPersonaMeshDetails::SetCurrentRecomputeTangentsSetting, LODIndex, SectionIndex, ChannelIndex));

		MenuBuilder.AddMenuEntry(RecomputeTangentsChannelLabels[ChannelIndex][0], RecomputeTangentsChannelLabels[ChannelIndex][1], FSlateIcon(), Action);
	}
	return MenuBuilder.MakeWidget();
}


void FPersonaMeshDetails::SetCurrentRecomputeTangentsSetting(int32 LODIndex, int32 SectionIndex, int32 Index)
{
	 USkeletalMesh* Mesh = GetPersonaToolkit()->GetMesh();
	if (Mesh == nullptr)
		return;

	check(Mesh->GetImportedModel());

	if (!Mesh->GetImportedModel()->LODModels.IsValidIndex(LODIndex))
		return;

	FSkeletalMeshLODModel& LODModel = Mesh->GetImportedModel()->LODModels[LODIndex];

	if (!LODModel.Sections.IsValidIndex(SectionIndex))
		return;

	FSkelMeshSection& Section = LODModel.Sections[SectionIndex];

	auto UpdatePolygonGroupRecomputeTangentVertexColor = [&Mesh, &LODModel, &Section, &SectionIndex, &Index]()
	{
		FScopedSuspendAlternateSkinWeightPreview ScopedSuspendAlternateSkinnWeightPreview(Mesh);
		{
			FScopedSkeletalMeshPostEditChange ScopedPostEditChange(Mesh);

			// Index_0 None:  bRecomputeTangent == false
			// Index_1 All:   bRecomputeTangent == true & RecomputeTangentsVertexMaskChannel == ESkinVertexColorChannel::None
			// Index_2 Red:   bRecomputeTangent == true & RecomputeTangentsVertexMaskChannel == ESkinVertexColorChannel::Red
			// Index_3 Green: bRecomputeTangent == true & RecomputeTangentsVertexMaskChannel == ESkinVertexColorChannel::Green
			// Index_4 Blue:  bRecomputeTangent == true & RecomputeTangentsVertexMaskChannel == ESkinVertexColorChannel::Blue
			Section.bRecomputeTangent = Index > 0;
			Section.RecomputeTangentsVertexMaskChannel = Index > 1 ? static_cast<ESkinVertexColorChannel>(Index - 2) : ESkinVertexColorChannel::None;
			
			for (int32 AfterSectionIndex = SectionIndex+1; AfterSectionIndex < LODModel.Sections.Num(); ++AfterSectionIndex)
			{
				if (LODModel.Sections[AfterSectionIndex].ChunkedParentSectionIndex == SectionIndex)
				{
					// Index: None-0, All-1, Red-2, Green-3, Blue-4
					LODModel.Sections[AfterSectionIndex].bRecomputeTangent = Index > 0;
					LODModel.Sections[AfterSectionIndex].RecomputeTangentsVertexMaskChannel = Index > 1 ? static_cast<ESkinVertexColorChannel>(Index-2) : ESkinVertexColorChannel::None;
				}
				else
				{
					break;
				}
			}
			// We display only the parent chunk
			check(Section.ChunkedParentSectionIndex == INDEX_NONE);
			SetSkelMeshSourceSectionUserData(LODModel, SectionIndex, Section.OriginalDataSectionIndex);
		}
	};


	{
		const FScopedTransaction Transaction(LOCTEXT("PersonaSetSectionRecomputeTangentFlag", "Persona editor: Set Recompute Tangent Vertex Color Mask For Section"));
		Mesh->Modify();
		UpdatePolygonGroupRecomputeTangentVertexColor();
	}
}


EVisibility FPersonaMeshDetails::GetOverrideUVDensityVisibililty() const
{
	//if (/*GetViewMode() == VMI_MeshUVDensityAccuracy*/ true)
	{
		return EVisibility::SelfHitTestInvisible;
	}
	/*
	else
	{
		return EVisibility::Collapsed;
	}*/
}

ECheckBoxState FPersonaMeshDetails::IsUVDensityOverridden(int32 MaterialIndex) const
{
	USkeletalMesh* Mesh = SkeletalMeshPtr.Get();
	if (!Mesh || Mesh->IsCompiling() || !Mesh->GetMaterials().IsValidIndex(MaterialIndex))
	{
		return ECheckBoxState::Undetermined;
	}
	else if (Mesh->GetMaterials()[MaterialIndex].UVChannelData.bOverrideDensities)
	{
		return ECheckBoxState::Checked;
	}
	else
	{
		return ECheckBoxState::Unchecked;
	}
}


void FPersonaMeshDetails::OnOverrideUVDensityChanged(ECheckBoxState NewState, int32 MaterialIndex)
{
	USkeletalMesh* Mesh = SkeletalMeshPtr.Get();
	if (NewState != ECheckBoxState::Undetermined && Mesh && Mesh->GetMaterials().IsValidIndex(MaterialIndex))
	{
		Mesh->GetMaterials()[MaterialIndex].UVChannelData.bOverrideDensities = (NewState == ECheckBoxState::Checked);
		Mesh->UpdateUVChannelData(true);
	}
}

EVisibility FPersonaMeshDetails::GetUVDensityVisibility(int32 MaterialIndex, int32 UVChannelIndex) const
{
	USkeletalMesh* Mesh = SkeletalMeshPtr.Get();
	if (/*MeshGetViewMode() == VMI_MeshUVDensityAccuracy && */ Mesh && IsUVDensityOverridden(MaterialIndex) == ECheckBoxState::Checked)
	{
		return EVisibility::SelfHitTestInvisible;
	}
	else
	{
		return EVisibility::Collapsed;
	}
}

TOptional<float> FPersonaMeshDetails::GetUVDensityValue(int32 MaterialIndex, int32 UVChannelIndex) const
{
	USkeletalMesh* Mesh = SkeletalMeshPtr.Get();
	if (Mesh && !Mesh->IsCompiling() && Mesh->GetMaterials().IsValidIndex(MaterialIndex))
	{
		float Value = Mesh->GetMaterials()[MaterialIndex].UVChannelData.LocalUVDensities[UVChannelIndex];
		return FMath::RoundToFloat(Value * 4.f) * .25f;
	}
	return TOptional<float>();
}

void FPersonaMeshDetails::SetUVDensityValue(float InDensity, ETextCommit::Type CommitType, int32 MaterialIndex, int32 UVChannelIndex)
{
	USkeletalMesh* Mesh = SkeletalMeshPtr.Get();
	if (Mesh && Mesh->GetMaterials().IsValidIndex(MaterialIndex))
	{
		Mesh->GetMaterials()[MaterialIndex].UVChannelData.LocalUVDensities[UVChannelIndex] = FMath::Max<float>(0, InDensity);
		Mesh->UpdateUVChannelData(true);
	}
}

int32 FPersonaMeshDetails::GetMaterialIndex(int32 LODIndex, int32 SectionIndex) const
{
	USkeletalMesh* SkelMesh = GetPersonaToolkit()->GetMesh();

	check(LODIndex < SkelMesh->GetLODNum());
	if (SkelMesh->IsCompiling())
	{
		return 0;
	}
	
	FSkeletalMeshModel* ImportedResource = SkelMesh->GetImportedModel();
	check(ImportedResource && ImportedResource->LODModels.IsValidIndex(LODIndex));
	int32 MaterialIndex = ImportedResource->LODModels[LODIndex].Sections[SectionIndex].MaterialIndex;
	FSkeletalMeshLODInfo& Info = *(SkelMesh->GetLODInfo(LODIndex));
	if (Info.LODMaterialMap.IsValidIndex(SectionIndex) && SkelMesh->GetMaterials().IsValidIndex(Info.LODMaterialMap[SectionIndex]))
	{
		return Info.LODMaterialMap[SectionIndex];
		
	}
		return MaterialIndex;
}

void FPersonaMeshDetails::OnSectionChanged(int32 LODIndex, int32 SectionIndex, int32 NewMaterialSlotIndex, FName NewMaterialSlotName)
{
	USkeletalMesh* Mesh = GetPersonaToolkit()->GetMesh();
	if(Mesh)
	{
		FSkeletalMeshModel* ImportedResource = Mesh->GetImportedModel();
		check(ImportedResource && ImportedResource->LODModels.IsValidIndex(LODIndex));
		const int32 TotalSectionCount = ImportedResource->LODModels[LODIndex].Sections.Num();

		check(TotalSectionCount > SectionIndex);

		FString NewMaterialSlotNameString = NewMaterialSlotName.ToString();
		NewMaterialSlotNameString.RemoveFromEnd(GSuffixDefaultMaterial, ESearchCase::CaseSensitive);
		FName CleanNewMaterialSlotName(*NewMaterialSlotNameString);

		int32 NewSkeletalMaterialIndex = INDEX_NONE;
		FName NewImportedMaterialSlotName = NAME_None;
		const TArray<FSkeletalMaterial>& MeshMaterials = Mesh->GetMaterials();
		for (int SkeletalMaterialIndex = 0; SkeletalMaterialIndex < MeshMaterials.Num(); ++SkeletalMaterialIndex)
		{
			if (NewMaterialSlotIndex == SkeletalMaterialIndex && MeshMaterials[SkeletalMaterialIndex].MaterialSlotName == CleanNewMaterialSlotName)
			{
				NewSkeletalMaterialIndex = SkeletalMaterialIndex;
				NewImportedMaterialSlotName = MeshMaterials[SkeletalMaterialIndex].ImportedMaterialSlotName;
				break;
			}
		}

		check(NewSkeletalMaterialIndex != INDEX_NONE);

		// Begin a transaction for undo/redo the first time we encounter a material to replace.  
		// There is only one transaction for all replacement
		FScopedTransaction Transaction(LOCTEXT("PersonaOnSectionChangedTransaction", "Persona editor: Section material slot changed"));
		Mesh->Modify();
		{
			FScopedSkeletalMeshPostEditChange ScopedPostEditChange(Mesh);
			int32 NumSections = ImportedResource->LODModels[LODIndex].Sections.Num();
			FSkeletalMeshLODInfo& Info = *(Mesh->GetLODInfo(LODIndex));

			auto SetLODMaterialMapValue = [&LODIndex, &Info, &ImportedResource](int32 InSectionIndex, int32 OverrideMaterialIndex)
			{
				if (ImportedResource->LODModels[LODIndex].Sections[InSectionIndex].MaterialIndex == OverrideMaterialIndex)
				{
					if (Info.LODMaterialMap.IsValidIndex(InSectionIndex))
					{
						Info.LODMaterialMap[InSectionIndex] = INDEX_NONE;
					}
				}
				else
				{
					while (Info.LODMaterialMap.Num() <= InSectionIndex)
					{
						Info.LODMaterialMap.Add(INDEX_NONE);
					}
					check(InSectionIndex < Info.LODMaterialMap.Num());
					Info.LODMaterialMap[InSectionIndex] = OverrideMaterialIndex;
				}
			};

			SetLODMaterialMapValue(SectionIndex, NewSkeletalMaterialIndex);
			//Set the chunked section 
			for (int32 SectionIdx = SectionIndex+1; SectionIdx < NumSections; SectionIdx++)
			{
				if (ImportedResource->LODModels[LODIndex].Sections[SectionIdx].ChunkedParentSectionIndex == SectionIndex)
				{
					SetLODMaterialMapValue(SectionIdx, NewSkeletalMaterialIndex);
				}
				else
				{
					//Chunked section are contiguous
					break;
				}
			}
		}
		// Redraw viewports to reflect the material changes 
		GUnrealEd->RedrawLevelEditingViewports();
	}
}

//
// Generate slate UI for Clothing category
//
void FPersonaMeshDetails::CustomizeClothingProperties(IDetailLayoutBuilder& DetailLayout, IDetailCategoryBuilder& ClothingFilesCategory)
{
	TSharedRef<IPropertyHandle> ClothingAssetsProperty = DetailLayout.GetProperty(USkeletalMesh::GetMeshClothingAssetsMemberName(), USkeletalMesh::StaticClass());

	if( ClothingAssetsProperty->IsValidHandle() )
	{
		TSharedRef<FDetailArrayBuilder> ClothingAssetsPropertyBuilder = MakeShareable( new FDetailArrayBuilder( ClothingAssetsProperty ) );
		ClothingAssetsPropertyBuilder->OnGenerateArrayElementWidget( FOnGenerateArrayElementWidget::CreateSP( this, &FPersonaMeshDetails::OnGenerateElementForClothingAsset, &DetailLayout) );

		ClothingFilesCategory.AddCustomBuilder(ClothingAssetsPropertyBuilder, false);
	}
}

//
// Generate each ClothingAsset array entry
//
void FPersonaMeshDetails::OnGenerateElementForClothingAsset( TSharedRef<IPropertyHandle> StructProperty, int32 ElementIndex, IDetailChildrenBuilder& ChildrenBuilder, IDetailLayoutBuilder* DetailLayout )
{
	if (const USkeletalMesh* const CurrentMesh = GetPersonaToolkit()->GetMesh())
	{
		if (const UClothingAssetBase* const CurrentAsset = CurrentMesh->GetMeshClothingAssets()[ElementIndex])
		{
			const FSlateFontInfo DetailFontInfo = IDetailLayoutBuilder::GetDetailFont();

			constexpr bool bStartExpanded = true;
			IDetailGroup& Group = ChildrenBuilder.AddGroup(CurrentAsset->GetFName(), LOCTEXT("ClothingAssets_GroupName", "ClothingAssets"), bStartExpanded);
			Group.HeaderRow()
				.NameContent()
				.HAlign(HAlign_Fill)
				[
					SNew(STextBlock)
						.Text(FText::Format(LOCTEXT("ClothingAsset_Label", "Index [ {0} ]"), FText::AsNumber(ElementIndex)))
						.Font(DetailFontInfo)
				]
				.ValueContent()
				[
					SNew(STextBlock)
						.Text(FText::FromString(CurrentAsset->GetName()))
						.Font(DetailFontInfo)
				];
			Group.AddWidgetRow()
				.WholeRowContent()
				[
					MakeClothingDetailsWidget(ElementIndex)
				];
		}
	}
}

TSharedRef<SUniformGridPanel> FPersonaMeshDetails::MakeClothingDetailsWidget(int32 AssetIndex) const
{
	const TSharedRef<SUniformGridPanel> Grid = SNew(SUniformGridPanel).SlotPadding(2.0f);

	if (const USkeletalMesh* const SkeletalMesh = GetPersonaToolkit()->GetMesh())
	{
		if (UClothingAssetBase* const ClothingAsset = SkeletalMesh->GetMeshClothingAssets()[AssetIndex])
		{
			const FSlateFontInfo DetailFont = IDetailLayoutBuilder::GetDetailFont();
			const FSlateFontInfo DetailFontBold = IDetailLayoutBuilder::GetDetailFontBold();
			const TArray<TPair<FText, FText>> Stats = ClothingAsset->GetStats();

			int32 RowNumber = 0;
			int32 ColumnNumber = 0;
			for (const TPair<FText, FText>& Stat : Stats)
			{
				if (Stat.Value.IsEmpty())  // Header
				{
					if (ColumnNumber)
					{
						RowNumber += 2;
						ColumnNumber = 0;
					}
				}
				Grid->AddSlot(ColumnNumber, RowNumber)
					.HAlign(HAlign_Center)
					[
						SNew(STextBlock)
							.Font(DetailFont)
							.Text(Stat.Key)
					];
				Grid->AddSlot(ColumnNumber, RowNumber + 1)
					.HAlign(HAlign_Center)
					[
						SNew(STextBlock)
							.Font(DetailFont)
							.Text(Stat.Value)
					];
				++ColumnNumber;
			}
		}
	}
	return Grid;
}

void FPersonaMeshDetails::UpdateClothingEntries()
{
	USkeletalMesh* Mesh = GetPersonaToolkit()->GetMesh();

	NewClothingAssetEntries.Empty();

	ClothingNoneEntry = MakeShared<FClothingEntry>();
	ClothingNoneEntry->AssetIndex = INDEX_NONE;
	ClothingNoneEntry->Asset = nullptr;

	NewClothingAssetEntries.Add(ClothingNoneEntry);

	const int32 NumClothingAssets = Mesh->GetMeshClothingAssets().Num();
	for(int32 Idx = 0; Idx < NumClothingAssets; ++Idx)
	{
		if (UClothingAssetBase* const ClothingAsset = Mesh->GetMeshClothingAssets()[Idx])
		{
			if (UClothingAssetCommon* const ClothingAssetCommon = Cast<UClothingAssetCommon>(ClothingAsset))
			{
				const int32 NumAssetLods = ClothingAssetCommon->GetNumLods();
				for (int32 AssetLodIndex = 0; AssetLodIndex < NumAssetLods; ++AssetLodIndex)
				{
					TSharedPtr<FClothingEntry> NewEntry = MakeShared<FClothingEntry>();
					NewEntry->Asset = ClothingAsset;
					NewEntry->AssetIndex = Idx;
					NewEntry->AssetLodIndex = AssetLodIndex;
					NewClothingAssetEntries.Add(NewEntry);
				}
			}
			else
			{
				TSharedPtr<FClothingEntry> NewEntry = MakeShared<FClothingEntry>();
				NewEntry->Asset = ClothingAsset;
				NewEntry->AssetIndex = Idx;
				NewEntry->AssetLodIndex = INDEX_NONE;
				NewClothingAssetEntries.Add(NewEntry);
			}
		}
	}
}

void FPersonaMeshDetails::RefreshClothingComboBoxes()
{
	for(const SClothComboBoxPtr& BoxPtr : ClothComboBoxes)
	{
		if(BoxPtr.IsValid())
		{
			BoxPtr->RefreshOptions();
		}
	}
}

void FPersonaMeshDetails::OnClothingComboBoxOpening()
{
	UpdateClothingEntries();
	RefreshClothingComboBoxes();
}

TSharedRef<SWidget> FPersonaMeshDetails::OnGenerateWidgetForClothingEntry(TSharedPtr<FClothingEntry> InEntry)
{
	const UClothingAssetBase* const Asset = InEntry->Asset.Get();

	FText EntryText;
	if(Asset)
	{
		EntryText = (InEntry->AssetLodIndex != INDEX_NONE) ?
			FText::Format(LOCTEXT("ClothingAssetEntry_Name", "{0} - LOD{1}"), FText::FromString(Asset->GetName()), FText::AsNumber(InEntry->AssetLodIndex)) : 
			FText::FromString(Asset->GetName());
	}
	else
	{
		EntryText = LOCTEXT("NoClothingEntry", "None");
	}

	return SNew(STextBlock)
		.Text(EntryText);
}

FText FPersonaMeshDetails::OnGetClothingComboText(int32 InLodIdx, int32 InSectionIdx) const
{
	const USkeletalMesh* const Mesh = GetPersonaToolkit()->GetMesh();

	if (Mesh && !Mesh->IsCompiling())
	{
		if (const UClothingAssetBase* const ClothingAsset = Cast<const UClothingAssetBase>(Mesh->GetSectionClothingAsset(InLodIdx, InSectionIdx)))
		{
			if (const UClothingAssetCommon* const ClothingAssetCommon = Cast<const UClothingAssetCommon>(Mesh->GetSectionClothingAsset(InLodIdx, InSectionIdx)))
			{
				if (ClothingAssetCommon->LodMap.IsValidIndex(InLodIdx))
				{
					const int32 ClothingLOD = ClothingAssetCommon->LodMap[InLodIdx];
					return FText::Format(LOCTEXT("ClothingAssetEntry_Name", "{0} - LOD{1}"), FText::FromString(ClothingAsset->GetName()), FText::AsNumber(ClothingLOD));
				}
			}
			else
			{
				// Non ClothingAssetCommon clothing data cannot have a different LOD index to the section's LOD index, so just show the name
				return FText::FromString(ClothingAsset->GetName());
			}
		}
	}

	return LOCTEXT("ClothingCombo_None", "None");
}

void FPersonaMeshDetails::OnClothingSelectionChanged(TSharedPtr<FClothingEntry> InNewEntry, ESelectInfo::Type InSelectType, int32 BoxIndex, int32 InLodIdx, int32 InSectionIdx)
{
	if (!InNewEntry.IsValid())
	{
		return;
	}
	USkeletalMesh* Mesh = SkeletalMeshPtr.Get();

	if (Mesh->GetImportedModel() == nullptr || !Mesh->GetImportedModel()->LODModels.IsValidIndex(InLodIdx))
	{
		return;
	}

	FSkeletalMeshLODModel& LODModel = Mesh->GetImportedModel()->LODModels[InLodIdx];
	const FSkelMeshSection& Section = LODModel.Sections[InSectionIdx];
	FScopedSuspendAlternateSkinWeightPreview ScopedSuspendAlternateSkinnWeightPreview(Mesh);
	{
		FScopedSkeletalMeshPostEditChange ScopedPostEditChange(Mesh);
		FScopedTransaction Transaction(LOCTEXT("PersonaOnSectionClothChangedTransaction", "Persona editor: Section cloth changed"));
		Mesh->Modify();

		FSkelMeshSourceSectionUserData& OriginalSectionData = LODModel.UserSectionsData.FindOrAdd(Section.OriginalDataSectionIndex);
		auto ClearOriginalSectionUserData = [&OriginalSectionData]()
		{
			OriginalSectionData.CorrespondClothAssetIndex = INDEX_NONE;
			OriginalSectionData.ClothingData.AssetGuid = FGuid();
			OriginalSectionData.ClothingData.AssetLodIndex = INDEX_NONE;
		};
		if (UClothingAssetBase* const ClothingAsset = InNewEntry->Asset.Get())
		{
			ClothingAsset->Modify();

			// Look for a currently bound asset an unbind it if necessary first
			if (UClothingAssetBase* CurrentAsset = Mesh->GetSectionClothingAsset(InLodIdx, InSectionIdx))
			{
				CurrentAsset->Modify();
				CurrentAsset->UnbindFromSkeletalMesh(Mesh, InLodIdx, InSectionIdx);
				ClearOriginalSectionUserData();
			}

			const int32 AssetLodIndex = InNewEntry->AssetLodIndex != INDEX_NONE ? InNewEntry->AssetLodIndex : InLodIdx;  //  Use the mesh LOD index if the asset type doesn't allow independent LOD setting
			if (!ClothingAsset->BindToSkeletalMesh(Mesh, InLodIdx, InSectionIdx, AssetLodIndex))
			{
				// We failed to bind the clothing asset, reset box selection to "None"
				SClothComboBoxPtr BoxPtr = ClothComboBoxes[BoxIndex];
				if (BoxPtr.IsValid())
				{
					BoxPtr->SetSelectedItem(ClothingNoneEntry);
				}
			}
			else
			{
				//Successful bind so set the SectionUserData
				int32 AssetIndex = INDEX_NONE;
				verify(Mesh->GetMeshClothingAssets().Find(ClothingAsset, AssetIndex));
				OriginalSectionData.CorrespondClothAssetIndex = static_cast<int16>(AssetIndex);
				OriginalSectionData.ClothingData.AssetGuid = ClothingAsset->GetAssetGuid();
				OriginalSectionData.ClothingData.AssetLodIndex = AssetLodIndex;
			}
		}
		else if (Mesh)
		{
			//User set none, so unbind anything that is bind
			if (UClothingAssetBase* CurrentAsset = Mesh->GetSectionClothingAsset(InLodIdx, InSectionIdx))
			{
				CurrentAsset->Modify();
				CurrentAsset->UnbindFromSkeletalMesh(Mesh, InLodIdx, InSectionIdx);
				ClearOriginalSectionUserData();
			}
		}
	}
}

bool FPersonaMeshDetails::IsClothingPanelEnabled() const
{
	return !GEditor->bIsSimulatingInEditor && !GEditor->PlayWorld;
}

bool FPersonaMeshDetails::CanDeleteMaterialElement(int32 LODIndex, int32 SectionIndex) const
{
	// Only allow deletion of extra elements
	return SectionIndex != 0;
}

void FPersonaMeshDetails::OnPreviewMeshChanged(USkeletalMesh* OldSkeletalMesh, USkeletalMesh* NewMesh)
{
	if (IsApplyNeeded())
	{
		ForceLayoutRebuild();
	}
}

bool FPersonaMeshDetails::FilterOutBakePose(const FAssetData& AssetData, USkeleton* Skeleton) const
{
	return !(Skeleton && Skeleton->IsCompatibleForEditor(AssetData));
}

#undef LOCTEXT_NAMESPACE
