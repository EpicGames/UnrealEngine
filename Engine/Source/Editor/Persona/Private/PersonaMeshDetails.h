// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Widgets/SBoxPanel.h"
#include "EngineDefines.h"
#include "Engine/SkeletalMesh.h"
#include "PropertyHandle.h"
#include "IDetailCustomNodeBuilder.h"
#include "IDetailCustomization.h"
#include "SkeletalMeshReductionSettings.h"
#include "Widgets/Input/SComboBox.h"

DECLARE_LOG_CATEGORY_EXTERN(LogSkeletalMeshPersonaMeshDetail, Log, All);

struct FAssetData;
class FNaniteSkeletalMeshLayout;
class FDetailWidgetRow;
class FPersonaMeshDetails;
class IDetailChildrenBuilder;
class IDetailLayoutBuilder;
class IPersonaToolkit;
class SUniformGridPanel;
struct FSectionLocalizer;
class IDetailCategoryBuilder;
class IDetailGroup;
class FScopedSkeletalMeshPostEditChange;

//The parameter is the LOD index so we know which LOD need to be rebuild
DECLARE_DELEGATE_RetVal_OneParam(bool, FIsLODSettingsEnabledDelegate, int32)
DECLARE_DELEGATE_OneParam(FModifyMeshLODSettingsDelegate, int32)

/**
 * Struct to uniquely identify clothing applied to a material section
 * Contains index into the ClothingAssets array and the submesh index.
 */
struct FClothAssetSubmeshIndex
{
	FClothAssetSubmeshIndex(int32 InAssetIndex, int32 InSubmeshIndex)
		:	AssetIndex(InAssetIndex)
		,	SubmeshIndex(InSubmeshIndex)
	{}

	int32 AssetIndex;
	int32 SubmeshIndex;

	bool operator==(const FClothAssetSubmeshIndex& Other) const
	{
		return (AssetIndex	== Other.AssetIndex 
			&& SubmeshIndex	== Other.SubmeshIndex
			);
	}
};

struct FClothingComboInfo
{
	/* Per-material clothing combo boxes, array size must be same to # of sections */
	TArray<TSharedPtr< class STextComboBox >>		ClothingComboBoxes;
	/* Clothing combo box strings */
	TArray<TSharedPtr<FString> >			ClothingComboStrings;
	/* Mapping from a combo box string to the asset and submesh it was generated from */
	TMap<FString, FClothAssetSubmeshIndex>	ClothingComboStringReverseLookup;
	/* The currently-selected index from each clothing combo box */
	TArray<int32>							ClothingComboSelectedIndices;
};

struct FSectionLocalizer
{
	FSectionLocalizer(int32 InLODIndex, int32 InSectionIndex)
		: LODIndex(InLODIndex)
		, SectionIndex(InSectionIndex)
	{}

	bool operator==(const FSectionLocalizer& Other) const
	{
		return (LODIndex == Other.LODIndex && SectionIndex == Other.SectionIndex);
	}

	bool operator!=(const FSectionLocalizer& Other) const
	{
		return !((*this) == Other);
	}

	int32 LODIndex;
	int32 SectionIndex;
};

class FSkeletalMeshReductionSettingsLayout : public IDetailCustomNodeBuilder, public TSharedFromThis<FSkeletalMeshReductionSettingsLayout>
{
public:
	FSkeletalMeshReductionSettingsLayout(FSkeletalMeshOptimizationSettings& InReductionettings, bool InbIsLODModelbuildDataAvailable, int32 InLODIndex, FIsLODSettingsEnabledDelegate InIsLODSettingsEnabledDelegate, FModifyMeshLODSettingsDelegate InModifyMeshLODSettingsDelegate);
	virtual ~FSkeletalMeshReductionSettingsLayout() = default;

	enum EImportanceType
	{
		ID_Silhouette,
		ID_Texture,
		ID_Shading,
		ID_Skinning
	};

	DECLARE_DELEGATE_RetVal(float, FGetFloatDelegate);
	DECLARE_DELEGATE_OneParam(FSetFloatDelegate, float);

	DECLARE_DELEGATE_RetVal(int32, FGetIntegerDelegate);
	DECLARE_DELEGATE_OneParam(FSetIntegerDelegate, int32);

	DECLARE_DELEGATE_RetVal(uint32, FGetUnsignedIntegerDelegate);
	DECLARE_DELEGATE_OneParam(FSetUnsignedIntegerDelegate, uint32);

	DECLARE_DELEGATE_RetVal(ECheckBoxState, FGetCheckBoxStateDelegate);
	DECLARE_DELEGATE_OneParam(FSetCheckBoxStateDelegate, ECheckBoxState);

	void UnbindReductionSettings()
	{
		IsLODSettingsEnabledDelegate.Unbind();
	}
private:
	/** IDetailCustomNodeBuilder Interface*/
	virtual void SetOnRebuildChildren(FSimpleDelegate InOnRegenerateChildren) override {}
	virtual void GenerateHeaderRowContent(FDetailWidgetRow& NodeRow) override;
	virtual void GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder) override;
	virtual void Tick(float DeltaTime) override {}
	virtual bool RequiresTick() const override { return false; }
	virtual FName GetName() const override { static FName MeshReductionSettings("SkeletalMeshOptimizationSettings"); return MeshReductionSettings; }
	virtual bool InitiallyCollapsed() const override { return true; }

	bool IsReductionEnabled() const;

	//Custom Row Add utilities
	FDetailWidgetRow& AddFloatRow(IDetailChildrenBuilder& ChildrenBuilder, const FText RowTitleText, const FText RowNameContentText, const FText RowNameContentTootlipText, FName RowTag, const float MinSliderValue, const float MaxSliderValue, FGetFloatDelegate GetterDelegate, FSetFloatDelegate SetterDelegate);
	FDetailWidgetRow& AddBoolRow(IDetailChildrenBuilder& ChildrenBuilder, const FText RowTitleText, const FText RowNameContentText, const FText RowNameContentToolitipText, FName RowTag, FGetCheckBoxStateDelegate GetterDelegate, FSetCheckBoxStateDelegate SetterDelegate);
	FDetailWidgetRow& AddIntegerRow(IDetailChildrenBuilder& ChildrenBuilder, const FText RowTitleText, const FText RowNameContentText, const FText RowNameContentTootlipText, FName RowTag, const int32 MinSliderValue, const int32 MaxSliderValue, FGetIntegerDelegate GetterDelegate, FSetIntegerDelegate SetterDelegate);
	FDetailWidgetRow& AddUnsignedIntegerRow(IDetailChildrenBuilder& ChildrenBuilder, const FText RowTitleText, const FText RowNameContentText, const FText RowNameContentTootlipText, FName RowTag, const uint32 MinSliderValue, const uint32 MaxSliderValue, FGetUnsignedIntegerDelegate GetterDelegate, FSetUnsignedIntegerDelegate SetterDelegate);
	void AddBaseLODRow(IDetailChildrenBuilder& ChildrenBuilder);

	void SetPercentAndAbsoluteVisibility(FDetailWidgetRow& Row, SkeletalMeshTerminationCriterion FirstCriterion, SkeletalMeshTerminationCriterion SecondCriterion);

	int32 GetBaseLODValue() const
	{
		return ReductionSettings.BaseLOD;
	}
	void SetBaseLODValue(int32 Value)
	{
		ReductionSettings.BaseLOD = Value;
	}
	TSharedRef<class SWidget> FillReductionMethodMenu();
	FText GetReductionMethodText() const;

	TSharedRef<class SWidget> FillReductionImportanceMenu(const EImportanceType Importance);
	FText GetReductionImportanceText(const EImportanceType Importance) const;

	TSharedRef<class SWidget> FillReductionTerminationCriterionMenu();
	FText GetReductionTerminationCriterionText() const;

	float GetNumTrianglesPercentage() const;
	void SetNumTrianglesPercentage(float Value);

	float GetNumVerticesPercentage() const;
	void SetNumVerticesPercentage(float Value);

	int32 GetNumMaxTrianglesCount() const;
	void SetNumMaxTrianglesCount(int32 Value);

	int32 GetNumMaxVerticesCount() const;
	void SetNumMaxVerticesCount(int32 Value);

	uint32 GetNumMaxTrianglesPercentageCount() const;
	void SetNumMaxTrianglesPercentageCount(uint32 Value);

	uint32 GetNumMaxVerticesPercentageCount() const;
	void SetNumMaxVerticesPercentageCount(uint32 Value);

	float GetAccuracyPercentage() const;
	void SetAccuracyPercentage(float Value);

	ECheckBoxState ShouldRecomputeNormals() const;
	void OnRecomputeNormalsChanged(ECheckBoxState NewState);

	float GetNormalsThreshold() const;
	void SetNormalsThreshold(float Value);

	float GetWeldingThreshold() const;
	void SetWeldingThreshold(float Value);

	ECheckBoxState GetLockEdges() const;
	void SetLockEdges(ECheckBoxState NewState);

	ECheckBoxState GetLockColorBounaries() const;
	void SetLockColorBounaries(ECheckBoxState NewState);

	ECheckBoxState GetImproveTrianglesForCloth() const;
	void SetImproveTrianglesForCloth(ECheckBoxState NewState);

	ECheckBoxState GetEnforceBoneBoundaries() const;
	void SetEnforceBoneBoundaries(ECheckBoxState NewState);

	ECheckBoxState GetMergeCoincidentVertBones() const;
	void SetMergeCoincidentVertBones(ECheckBoxState NewState);

	float GetVolumeImportance() const;
	void SetVolumeImportance(float Value);
	
	ECheckBoxState GetRemapMorphTargets() const;
	void SetRemapMorphTargets(ECheckBoxState NewState);

	int32 GetMaxBonesPerVertex() const;
	void SetMaxBonesPerVertex(int32 Value);

	// Used the the thrid-party UI.  
	EVisibility GetVisibiltyIfCurrentReductionMethodIsNot(SkeletalMeshOptimizationType ReductionType) const;

	// Used by the native tool UI.
	EVisibility ShowIfCurrentCriterionIs(TArray<SkeletalMeshTerminationCriterion> TerminationCriterionArray) const;

	/** Detect usage of thirdparty vs native tool */
	bool UseNativeLODTool() const;
	bool UseNativeReductionTool() const;

	/**
	Used to hide parameters that only make sense for the third party tool.
	@return EVisibility::Visible if we are using the simplygon tool, otherwise EVisibility::Hidden
	*/
	EVisibility GetVisibilityForThirdPartyTool() const;

private:
	FSkeletalMeshOptimizationSettings& ReductionSettings;
	bool bIsLODModelbuildDataAvailable;
	int32 LODIndex;
	FIsLODSettingsEnabledDelegate IsLODSettingsEnabledDelegate;
	FModifyMeshLODSettingsDelegate ModifyMeshLODSettingsDelegate;

	UEnum* EnumReductionMethod;
	UEnum* EnumImportance;
	UEnum* EnumTerminationCriterion;

	//Use this data to keep a valid reference so the helper lambda can have persistent data
	//Helper lambda are use with spinner to not do a transaction when spinning with mouse movement
	struct FSliderStateData
	{
		float MovementValueFloat = 0.0f;
		int32 MovementValueInt = 0;
		uint32 MovementValueUnsignedInt = 0;
		bool bSliderActiveMode = false;
	};
	TArray<FSliderStateData> SliderStateDataArray;
};

class FSkeletalMeshBuildSettingsLayout : public IDetailCustomNodeBuilder, public TSharedFromThis<FSkeletalMeshBuildSettingsLayout>
{
public:
	FSkeletalMeshBuildSettingsLayout(FSkeletalMeshBuildSettings& InBuildSettings, int32 InLODIndex, FIsLODSettingsEnabledDelegate InIsBuildSettingsEnabledDelegate, FModifyMeshLODSettingsDelegate InModifyMeshLODSettingsDelegate);
	virtual ~FSkeletalMeshBuildSettingsLayout() = default;


	DECLARE_DELEGATE_RetVal(float, FGetFloatDelegate);
	DECLARE_DELEGATE_OneParam(FSetFloatDelegate, float);

	DECLARE_DELEGATE_RetVal(int32, FGetIntegerDelegate);
	DECLARE_DELEGATE_OneParam(FSetIntegerDelegate, int32);

	void UnbindBuildSettings()
	{
		IsBuildSettingsEnabledDelegate.Unbind();
	}

private:
	/** IDetailCustomNodeBuilder Interface*/
	virtual void SetOnRebuildChildren(FSimpleDelegate InOnRegenerateChildren) override {}
	virtual void GenerateHeaderRowContent(FDetailWidgetRow& NodeRow) override;
	virtual void GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder) override;
	virtual void Tick(float DeltaTime) override {}
	virtual bool RequiresTick() const override { return false; }
	virtual FName GetName() const override { static FName MeshBuildSettings("MeshBuildSettings"); return MeshBuildSettings; }
	virtual bool InitiallyCollapsed() const override { return true; }

	bool IsBuildEnabled() const;

	//Custom Row Add utilities
	FDetailWidgetRow& AddFloatRow(IDetailChildrenBuilder& ChildrenBuilder, const FText RowTitleText, const FText RowNameContentText, const FText RowNameContentTootlipText, const float MinSliderValue, const float MaxSliderValue, FGetFloatDelegate GetterDelegate, FSetFloatDelegate SetterDelegate);
	FDetailWidgetRow& AddIntegerRow(
		IDetailChildrenBuilder& ChildrenBuilder,
		const FText& RowTitleText,
		const FText& RowNameContentText,
		const FText& RowNameContentTooltipText,
		FName RowTag,
		const int32 MinSliderValue,
		const int32 MaxSliderValue,
		const FGetIntegerDelegate& GetterDelegate,
		const FSetIntegerDelegate& SetterDelegate);

	float GetThresholdPosition() const;
	void SetThresholdPosition(float Value);
	
	float GetThresholdTangentNormal() const;
	void SetThresholdTangentNormal(float Value);
	
	float GetThresholdUV() const;
	void SetThresholdUV(float Value);

	float GetMorphThresholdPosition() const;
	void SetMorphThresholdPosition(float Value);

	int32 GetBoneInfluenceLimit() const;
	void SetBoneInfluenceLimit(int32 Value);

	ECheckBoxState ShouldRecomputeNormals() const;
	ECheckBoxState ShouldRecomputeTangents() const;
	ECheckBoxState ShouldUseMikkTSpace() const;
	ECheckBoxState ShouldComputeWeightedNormals() const;
	ECheckBoxState ShouldRemoveDegenerates() const;
	ECheckBoxState ShouldUseHighPrecisionTangentBasis() const;
	ECheckBoxState ShouldUseHighPrecisionSkinWeights() const;
	ECheckBoxState ShouldUseFullPrecisionUVs() const;
	ECheckBoxState ShouldUseBackwardsCompatibleF16TruncUVs() const;
	ECheckBoxState ShouldOptimizeForInstancing() const;

	void OnRecomputeNormalsChanged(ECheckBoxState NewState);
	void OnRecomputeTangentsChanged(ECheckBoxState NewState);
	void OnUseMikkTSpaceChanged(ECheckBoxState NewState);
	void OnComputeWeightedNormalsChanged(ECheckBoxState NewState);
	void OnRemoveDegeneratesChanged(ECheckBoxState NewState);
	void OnUseHighPrecisionTangentBasisChanged(ECheckBoxState NewState);
	void OnUseHighPrecisionSkinWeightsChanged(ECheckBoxState NewState);
	void OnUseFullPrecisionUVsChanged(ECheckBoxState NewState);
	void OnUseBackwardsCompatibleF16TruncUVsChanged(ECheckBoxState NewState);
	void OnOptimizeForInstancingChanged(ECheckBoxState NewState);

private:
	FSkeletalMeshBuildSettings& BuildSettings;
	int32 LODIndex;
	FIsLODSettingsEnabledDelegate IsBuildSettingsEnabledDelegate;
	FModifyMeshLODSettingsDelegate ModifyMeshLODSettingsDelegate;

	//Use this data to keep a valid reference so the helper lambda can have persistent data
	//Helper lambda are use with spinner to not do a transaction when spinning with mouse movement
	struct FSliderStateData
	{
		float MovementValueFloat = 0.0f;
		int32 MovementValueInt = 0;
		bool bSliderActiveMode = false;
	};
	TArray<FSliderStateData> SliderStateDataArray;
};


class FPersonaMeshDetails : public IDetailCustomization
{
public:
	FPersonaMeshDetails(TSharedRef<class IPersonaToolkit> InPersonaToolkit);
	~FPersonaMeshDetails();

	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance(TWeakPtr<class IPersonaToolkit> InPersonaToolkit);

	/** IDetailCustomization interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailLayout ) override;

public:
	void ForceLayoutRebuild();
	void RequestLayoutUpdate();

private:
	//Prevent attribute change calling post edit change
	void OnAttributePreChangePreventPostEditChange(int32 LODIndex, FName LODInfoPropertyName);
	void OnAttributeChangedPreventPostEditChange(const int32 LODIndex, const FName LODInfoPropertyName, const bool bForceComponentRefresh);
	void PreventAttributePostEditChange(TSharedPtr<IPropertyHandle> AttributeHandle, const int32 LODIndex, const FName PropertyName, const bool bForceComponentRefresh);

	//This function customize the LODInfo temporary object
	void CustomizeLODInfoSetingsDetails(IDetailLayoutBuilder& DetailLayout, class ULODInfoUILayout* LODInfoUILayout, TSharedRef<IPropertyHandle> LODInfoProperty, IDetailCategoryBuilder& LODCategory);

	FReply AddMaterialSlot();

	FText GetMaterialArrayText() const;

	/**
	 * Called by the material list widget when we need to get new materials for the list
	 *
	 * @param OutMaterials	Handle to a material list builder that materials should be added to
	 */
	void OnGetSectionsForView( class ISectionListBuilder& OutSections, int32 LODIndex );

	/**
	 * Called when a user drags a new material over a list item to replace it
	 *
	 * @param NewMaterial	The material that should replace the existing material
	 * @param PrevMaterial	The material that should be replaced
	 * @param SlotIndex		The index of the slot on the component where materials should be replaces
	 * @param bReplaceAll	If true all materials in the slot should be replaced not just ones using PrevMaterial
	 */
	void OnSectionChanged(int32 LODIndex, int32 SectionIndex, int32 NewMaterialSlotIndex, FName NewMaterialSlotName);

	/**
	* Called by the material list widget when we need to get new materials for the list
	*
	* @param OutMaterials	Handle to a material list builder that materials should be added to
	*/
	void OnGetMaterialsForArray(class IMaterialListBuilder& OutMaterials, int32 LODIndex);

	/**
	* Called when a user drags a new material over a list item to replace it
	*
	* @param NewMaterial	The material that should replace the existing material
	* @param PrevMaterial	The material that should be replaced
	* @param SlotIndex		The index of the slot on the component where materials should be replaces
	* @param bReplaceAll	If true all materials in the slot should be replaced not just ones using PrevMaterial
	*/
	void OnMaterialArrayChanged(UMaterialInterface* NewMaterial, UMaterialInterface* PrevMaterial, int32 SlotIndex, bool bReplaceAll, int32 LODIndex);

	
	/**
	 * Called by the material list widget on generating each name widget
	 *
	 * @param Material		The material that is being displayed
	 * @param SlotIndex		The index of the material slot
	 */
	TSharedRef<SWidget> OnGenerateCustomNameWidgetsForSection(int32 LodIndex, int32 SectionIndex);

	/**
	 * Called by the material list widget on generating each thumbnail widget
	 *
	 * @param Material		The material that is being displayed
	 * @param SlotIndex		The index of the material slot
	 */
	TSharedRef<SWidget> OnGenerateCustomSectionWidgetsForSection(int32 LODIndex, int32 SectionIndex);

	bool IsSectionEnabled(int32 LodIndex, int32 SectionIndex) const;
	EVisibility ShowEnabledSectionDetail(int32 LodIndex, int32 SectionIndex) const;
	EVisibility ShowDisabledSectionDetail(int32 LodIndex, int32 SectionIndex) const;
	void OnSectionEnabledChanged(int32 LodIndex, int32 SectionIndex, bool bEnable);

	TOptional<int8> GetSectionGenerateUpToValue(int32 LodIndex, int32 SectionIndex) const;
	void SetSectionGenerateUpToValue(int8 Value, int32 LodIndex, int32 SectionIndex);
	void SetSectionGenerateUpToValueCommitted(int8 Value, ETextCommit::Type CommitInfo, int32 LodIndex, int32 SectionIndex);
	EVisibility ShowSectionGenerateUpToSlider(int32 LodIndex, int32 SectionIndex) const;
	ECheckBoxState IsGenerateUpToSectionEnabled(int32 LodIndex, int32 SectionIndex) const;
	void OnSectionGenerateUpToChanged(ECheckBoxState NewState, int32 LodIndex, int32 SectionIndex);

	TSharedRef<SWidget> OnGenerateLodComboBoxForLodPicker();
	EVisibility LodComboBoxVisibilityForLodPicker() const;
	bool IsLodComboBoxEnabledForLodPicker() const;

	/*
	 * Generate the context menu to choose the LOD we will display the picker list
	*/
	TSharedRef<SWidget> OnGenerateLodMenuForLodPicker();
	FText GetCurrentLodName() const;
	FText GetCurrentLodTooltip() const;
	static bool GetAutoLod(USkeletalMeshComponent* InMeshComponent);
	static int32 GetCurrentDisplayLODIndex(USkeletalMeshComponent* InMeshComponent);

	void SetCurrentLOD(int32 NewLodIndex);

	void UpdateLODCategoryVisibility() const;

	FText GetMaterialNameText(int32 MaterialIndex)const ;
	void OnMaterialNameCommitted(const FText& InValue, ETextCommit::Type CommitType, int32 MaterialIndex);

	FText GetOriginalImportMaterialNameText(int32 MaterialIndex)const;

	/**
	* Called by the material list widget on generating name side content
	*
	* @param Material		The material that is being displayed
	* @param MaterialIndex	The index of the material slot
	*/
	TSharedRef<SWidget> OnGenerateCustomNameWidgetsForMaterialArray(UMaterialInterface* Material, int32 MaterialIndex);
	
	/**
	* Called by the material list widget on generating each thumbnail widget
	*
	* @param Material		The material that is being displayed
	* @param MaterialIndex	The index of the material slot
	*/
	TSharedRef<SWidget> OnGenerateCustomMaterialWidgetsForMaterialArray(UMaterialInterface* Material, int32 MaterialIndex, int32 LODIndex);

	/**
	* Called by the material list widget on generating each thumbnail widget
	* Those generated widget will be stack under the material list row
	*
	* @param Material		The material that is being displayed
	* @param MaterialIndex	The index of the material slot
	*/
	TSharedRef<SWidget> OnGenerateMaterialListExtraBottomWidget(UMaterialInterface* Material, int32 MaterialIndex, int32 LODIndex);

	/* If the material list is dirty this function will return true */
	bool OnMaterialListDirty();

	bool CanDeleteMaterialSlot(int32 MaterialIndex) const;

	void OnDeleteMaterialSlot(int32 MaterialIndex);

	/* Call when user change the material slot overlay material. */
	void OnMaterialSlotOverlayMaterialChanged(UMaterialInterface* NewOverlayMaterial, int32 MaterialIndex);

	/**
	* Handler for check box display based on whether the material is highlighted
	*
	* @param MaterialIndex	The material index that is being selected
	*/
	ECheckBoxState IsMaterialSelected(int32 MaterialIndex) const;

	/**
	* Handler for changing highlight status on a material
	*
	* @param MaterialIndex	The material index that is being selected
	*/
	void OnMaterialSelectedChanged(ECheckBoxState NewState, int32 MaterialIndex);

	/**
	* Handler for check box display based on whether the material is isolated
	*
	* @param MaterialIndex	The material index that is being isolate
	*/
	ECheckBoxState IsIsolateMaterialEnabled(int32 MaterialIndex) const;

	/**
	* Handler for changing isolated status on a material
	*
	* @param MaterialIndex	The material index that is being isolate
	*/
	void OnMaterialIsolatedChanged(ECheckBoxState NewState, int32 MaterialIndex);


	/**
	 * Handler for check box display based on whether the material is highlighted
	 *
	 * @param SectionIndex	The material section that is being tested
	 */
	ECheckBoxState IsSectionSelected(int32 SectionIndex) const;

	/**
	 * Handler for changing highlight status on a material
	 *
	 * @param SectionIndex	The material section that is being tested
	 */
	void OnSectionSelectedChanged(ECheckBoxState NewState, int32 SectionIndex);

	/**
	* Handler for check box display based on whether the material is isolated
	*
	* @param SectionIndex	The material section that is being tested
	*/
	ECheckBoxState IsIsolateSectionEnabled(int32 SectionIndex) const;

	/**
	* Handler for changing isolated status on a material
	*
	* @param SectionIndex	The material section that is being tested
	*/
	void OnSectionIsolatedChanged(ECheckBoxState NewState, int32 SectionIndex);

	/**
	* Handler for check box display based on whether the material has shadow casting enabled
	*
	* @param LODIndex	The LODIndex we want to change
	* @param SectionIndex	The SectionIndex we change the ShadowCasting flag
	*/
	ECheckBoxState IsSectionShadowCastingEnabled(int32 LODIndex, int32 SectionIndex) const;

	/**
	* Handler for changing shadow casting status on a section
	*
	* @param LODIndex	The LODIndex we want to change
	* @param SectionIndex	The SectionIndex we change the ShadowCasting flag
	*/
	void OnSectionShadowCastingChanged(ECheckBoxState NewState, int32 LODIndex, int32 SectionIndex);

	/**
	* Handler for check box display based on whether the material has VisibleInRayTracing enabled
	*
	* @param LODIndex	The LODIndex we want to change
	* @param SectionIndex	The SectionIndex we change the VisibleInRayTracing flag
	*/
	ECheckBoxState IsSectionVisibleInRayTracingEnabled(int32 LODIndex, int32 SectionIndex) const;

	/**
	* Handler for changing VisibleInRayTracing status on a section
	*
	* @param LODIndex	The LODIndex we want to change
	* @param SectionIndex	The SectionIndex we change the VisibleInRayTracing flag
	*/
	void OnSectionVisibleInRayTracingChanged(ECheckBoxState NewState, int32 LODIndex, int32 SectionIndex);

	/**
	* Handler for selecting which vertex color to mask the blending of recomputing tangents
	*
	* @param LODIndex	The LODIndex we want to change
	* @param SectionIndex	The SectionIndex we change the RecomputeTangent
	*/
	TSharedRef<class SWidget> OnGenerateRecomputeTangentsSetting(int32 LODIndex, int32 SectionIndex);
	FText GetCurrentRecomputeTangentsSetting(int32 LODIndex, int32 SectionIndex) const;
	void SetCurrentRecomputeTangentsSetting(int32 LODIndex, int32 SectionIndex, int32 Index);

	/**
	 * Handler for enabling delete button on materials
	 *
	 * @param SectionIndex - index of the section to check
	 */
	bool CanDeleteMaterialElement(int32 LODIndex, int32 SectionIndex) const;

	/** Creates the UI for Current LOD panel */
	void AddLODLevelCategories(IDetailLayoutBuilder& DetailLayout);

	/** Get a material index from LOD index and section index */
	int32 GetMaterialIndex(int32 LODIndex, int32 SectionIndex) const;

	/** for LOD settings category */
	void CustomizeLODSettingsCategories(IDetailLayoutBuilder& DetailLayout);

	/** Called when a LOD is imported. Refreshes the UI. */
	void OnAssetPostLODImported(UObject* InObject, int32 InLODIndex);
	void OnAssetReimport(UObject* InObject);
	/** Called from the PersonalMeshDetails UI to import a LOD. */
	void OnImportLOD(TSharedPtr<FString> NewValue, ESelectInfo::Type SelectInfo, IDetailLayoutBuilder* DetailLayout);
	void UpdateLODNames();
	int32 GetLODCount() const;
	void OnLODCountChanged(int32 NewValue);
	void OnLODCountCommitted(int32 InValue, ETextCommit::Type CommitInfo);
	FText GetLODCountTooltip() const;
	FText GetLODImportedText(int32 LODIndex) const;

	FText GetMaterialSlotNameText(int32 MaterialIndex) const;

	void OnNoRefStreamingLODBiasChanged(int32 NewValue, FName QualityLevel);
	void OnNoRefStreamingLODBiasCommitted(int32 InValue, ETextCommit::Type CommitInfo, FName QualityLevel);
	int32 GetNoRefStreamingLODBias(FName QualityLevel) const;
	TSharedRef<SWidget> GetNoRefStreamingLODBiasWidget(FName QualityLevelName) const;
	bool AddNoRefStreamingLODBiasOverride(FName QualityLevelName);
	bool RemoveNoRefStreamingLODBiasOverride(FName QualityLevelName);
	TArray<FName> GetNoRefStreamingLODBiasOverrideNames() const;
	FText GetNoRefStreamingLODBiasTooltip() const;

	void OnMinQualityLevelLodChanged(int32 NewValue, FName QualityLevel);
	void OnMinQualityLevelLodCommitted(int32 InValue, ETextCommit::Type CommitInfo, FName QualityLevel);
	int32 GetMinQualityLevelLod(FName QualityLevel) const;
	TSharedRef<SWidget> GetMinQualityLevelLodWidget(FName QualityLevelName) const;
	bool AddMinLodQualityLevelOverride(FName QualityLevelName);
	bool RemoveMinLodQualityLevelOverride(FName QualityLevelName);
	TArray<FName> GetMinQualityLevelLodOverrideNames() const;
	FReply ResetToDefault();
	FPerPlatformInt GetMinLod();

	/** apply LOD changes if the user modified LOD reduction settings */
	FReply OnApplyChanges();
	/** regenerate one specific LOD Index no dependencies*/
	void RegenerateOneLOD(int32 LODIndex);
	/** regenerate the specific all LODs dependent of InLODIndex. This is not regenerating the InLODIndex*/
	void RegenerateDependentLODs(int32 LODIndex);
	/** Apply specified LOD Index */
	FReply ApplyLODChanges(int32 LODIndex);
	/** Apply specified LOD Index */
	FReply RegenerateLOD(int32 LODIndex);
	/** Removes the specified lod from the skeletal mesh */
	FReply RemoveOneLOD(int32 LODIndex);
	/** Removes any editable geometry from the given LOD, making it auto-generated */
	FReply RemoveGeometryFromLOD(int32 LODIndex);
	/** Restore the LOD imported data if the LOD is no longer reduced */
	void RestoreNonReducedLOD(int32 LODIndex);
	/** hide properties which don't need to be showed to end users */
	void HideUnnecessaryProperties(IDetailLayoutBuilder& DetailLayout);

	// Handling functions for post process blueprint selection combo box
	void OnPostProcessBlueprintChanged(IDetailLayoutBuilder* DetailBuilder);
	FString GetCurrentPostProcessBlueprintPath() const;
	bool OnShouldFilterPostProcessBlueprint(const FAssetData& AssetData) const;
	void OnSetPostProcessBlueprint(const FAssetData& AssetData, TSharedRef<IPropertyHandle> BlueprintProperty);

	/** Access the persona toolkit ptr. It should always be valid in the lifetime of this customization */
	TSharedRef<IPersonaToolkit> GetPersonaToolkit() const { check(PersonaToolkitPtr.IsValid()); return PersonaToolkitPtr.Pin().ToSharedRef(); }
	bool HasValidPersonaToolkit() const { return PersonaToolkitPtr.IsValid(); }

	EVisibility GetOverrideUVDensityVisibililty() const;
	ECheckBoxState IsUVDensityOverridden(int32 MaterialIndex) const;
	void OnOverrideUVDensityChanged(ECheckBoxState NewState, int32 MaterialIndex);

	EVisibility GetUVDensityVisibility(int32 MaterialIndex, int32 UVChannelIndex) const;
	TOptional<float> GetUVDensityValue(int32 MaterialIndex, int32 UVChannelIndex) const;
	void SetUVDensityValue(float InDensity, ETextCommit::Type CommitType, int32 MaterialIndex, int32 UVChannelIndex);

	SVerticalBox::FSlot& GetUVDensitySlot(int32 MaterialIndex, int32 UVChannelIndex) const;

	// Used to control the type of reimport to do with a named parameter
	enum class EReimportButtonType : uint8
	{
		Reimport,
		ReimportWithNewFile
	};

	// Handler for reimport buttons in LOD details
	FReply OnReimportLodClicked(EReimportButtonType InReimportType, int32 InLODIndex);

	void OnCopySectionList(int32 LODIndex);
	bool OnCanCopySectionList(int32 LODIndex) const;
	void OnPasteSectionList(int32 LODIndex);

	void OnCopySectionItem(int32 LODIndex, int32 SectionIndex);
	bool OnCanCopySectionItem(int32 LODIndex, int32 SectionIndex) const;
	void OnPasteSectionItem(int32 LODIndex, int32 SectionIndex);

	void OnCopyMaterialList();
	bool OnCanCopyMaterialList() const;
	void OnPasteMaterialList();

	void OnCopyMaterialItem(int32 CurrentSlot);
	bool OnCanCopyMaterialItem(int32 CurrentSlot) const;
	void OnPasteMaterialItem(int32 CurrentSlot);

	void OnPreviewMeshChanged(USkeletalMesh* OldSkeletalMesh, USkeletalMesh* NewMesh);
	
	bool FilterOutBakePose(const struct FAssetData& AssetData, USkeleton* Skeleton) const;
	bool FilterOutBakePose(const struct FAssetData& AssetData, TObjectPtr<USkeleton> Skeleton) const { return FilterOutBakePose(AssetData, Skeleton.Get()); }

	FText GetLODCustomModeNameContent(int32 LODIndex) const;
	ECheckBoxState IsLODCustomModeCheck(int32 LODIndex) const;
	void SetLODCustomModeCheck(ECheckBoxState NewState, int32 LODIndex);
	bool IsLODCustomModeEnable(int32 LODIndex) const;

	/** Gets the max LOD that can be set from the lod count slider (current num plus an interval) */
	TOptional<int32> GetLodSliderMaxValue() const;

	void CustomizeSkinWeightProfiles(IDetailLayoutBuilder& DetailLayout);
	TSharedRef<SWidget> CreateSkinWeightProfileMenuContent();
public:

	bool IsApplyNeeded() const;
	bool IsGenerateAvailable() const;
	void ApplyChanges();
	FText GetApplyButtonText() const;

private:
	// Container for the objects to display
	TWeakObjectPtr<USkeletalMesh> SkeletalMeshPtr;

	// Reference the persona toolkit
	TWeakPtr<class IPersonaToolkit> PersonaToolkitPtr;

	/** Nanite settings for the details panel. */
	TSharedPtr<FNaniteSkeletalMeshLayout> NaniteSettings;

	IDetailLayoutBuilder* MeshDetailLayout;

	//This is the mockup UObjects to modify a copy of the LODInfo
	TArray<class ULODInfoUILayout*> LODInfoUILayouts;
	TArray<TSharedRef<IDetailsView>> LODInfoUILayoutDetailsViews;

	/** LOD import options */
	TArray<TSharedPtr<FString> > LODNames;
	/** Helper value that corresponds to the 'Number of LODs' spinbox.*/
	int32 LODCount;

	/* This is to know if material are used by any LODs sections. */
	TMap<int32, TArray<FSectionLocalizer>> MaterialUsedMap;

	TArray<class IDetailCategoryBuilder*> LodCategories;
	IDetailCategoryBuilder* LodCustomCategory;

	bool CustomLODEditMode;
	TArray<bool> DetailDisplayLODs;

	/*
	 * Helper to keep the old GenerateUpTo slider value to register transaction correctly.
	 * The key is the union of LOD index and section index.
	 */
	TMap<int64, int8> OldGenerateUpToSliderValues;

	/*
	 * This prevent showing the delete material slot warning dialog more then once per editor session
	 */
	bool bDeleteWarningConsumed;

private:
	TArray<TUniquePtr<FScopedSkeletalMeshPostEditChange>> ChangeScopes;

	// info about clothing combo boxes for multiple LOD
	TArray<FClothingComboInfo>				ClothingComboLODInfos;
	TArray<int32> ClothingSelectedSubmeshIndices;

	// Menu entry for clothing dropdown
	struct FClothingEntry
	{
		// Asset index inside the mesh
		int32 AssetIndex;

		// LOD index inside the clothing asset
		int32 AssetLodIndex;

		// Pointer back to the asset for this clothing entry
		TWeakObjectPtr<UClothingAssetBase> Asset;
	};

	// Cloth combo box tracking for refreshes post-import/creation
	typedef SComboBox<TSharedPtr<FClothingEntry>> SClothComboBox;
	typedef TSharedPtr<SClothComboBox> SClothComboBoxPtr;
	TArray<SClothComboBoxPtr> ClothComboBoxes;

	// Clothing entries available to bind to the mesh
	TArray<TSharedPtr<FClothingEntry>> NewClothingAssetEntries;

	// Cached item in above array that is used as the "None" entry in the list
	TSharedPtr<FClothingEntry> ClothingNoneEntry;

	// Update the list of valid entries
	void UpdateClothingEntries();

	// Refreshes clothing combo boxes that are currently active
	void RefreshClothingComboBoxes();

	// Called as clothing combo boxes open to validate option entries
	void OnClothingComboBoxOpening();

	// Generate a widget for the clothing details panel
	TSharedRef<SWidget> OnGenerateWidgetForClothingEntry(TSharedPtr<FClothingEntry> InEntry);

	// Get the current text for the clothing selection combo box for the specified LOD and section
	FText OnGetClothingComboText(int32 InLodIdx, int32 InSectionIdx) const;

	// Callback when the clothing asset is changed
	void OnClothingSelectionChanged(TSharedPtr<FClothingEntry> InNewEntry, ESelectInfo::Type InSelectType, int32 BoxIndex, int32 InLodIdx, int32 InSectionIdx);

	// If the clothing details widget is editable
	bool IsClothingPanelEnabled() const;

	/* Generate slate UI for Clothing category */
	void CustomizeClothingProperties(class IDetailLayoutBuilder& DetailLayout, class IDetailCategoryBuilder& ClothingFilesCategory);

	/* Generate each ClothingAsset array entry */
	void OnGenerateElementForClothingAsset( TSharedRef<IPropertyHandle> ElementProperty, int32 ElementIndex, IDetailChildrenBuilder& ChildrenBuilder, IDetailLayoutBuilder* DetailLayout );

	/* Make uniform grid widget for Apex details */
	TSharedRef<SUniformGridPanel> MakeClothingDetailsWidget(int32 AssetIndex) const;

	/* Removes a clothing asset */ 
	FReply OnRemoveClothingAssetClicked(int32 AssetIndex, IDetailLayoutBuilder* DetailLayout);

	/* Create LOD setting assets from current setting */
	FReply OnSaveLODSettings();

	/** LOD Settings Selected */
	void OnLODSettingsSelected(const FAssetData& AssetData);

	/** LOD Info editing is enabled? LODIndex == -1, then it just verifies if the asset exists */
	bool IsLODInfoEditingEnabled(int32 LODIndex) const;
	bool IsMinLodEnable() const;
	bool IsQualityLevelMinLodEnable() const;
	void ModifyMeshLODSettings(int32 LODIndex);

	TMap<int32, TSharedPtr<FSkeletalMeshBuildSettingsLayout>> BuildSettingsWidgetsPerLOD;
	TMap<int32, TSharedPtr<FSkeletalMeshReductionSettingsLayout>> ReductionSettingsWidgetsPerLOD;

	// Property handle used to determine if the VertexColorImportOverride property should be enabled.
	TSharedPtr<IPropertyHandle> VertexColorImportOptionHandle;

	// Property handle used during UI construction
	TSharedPtr<IPropertyHandle> VertexColorImportOverrideHandle;

	// Delegate implementation of FOnInstancedPropertyIteration used during DataImport UI construction
	void OnInstancedFbxSkeletalMeshImportDataPropertyIteration(IDetailCategoryBuilder& BaseCategory, IDetailGroup* PropertyGroup, TSharedRef<IPropertyHandle>& Property) const;

	// Delegate used at runtime to determine the state of the VertexOverrideColor property
	bool GetVertexOverrideColorEnabledState() const;

	// Called when the skeletal mesh has finished rebuilding. This may affect some settings, such as vertex attributes.
	void OnMeshRebuildCompleted(USkeletalMesh* InMesh);
};
