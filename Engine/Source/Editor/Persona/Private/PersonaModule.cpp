// Copyright Epic Games, Inc. All Rights Reserved.


#include "PersonaModule.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/MessageDialog.h"
#include "Misc/FeedbackContext.h"
#include "Misc/ScopedSlowTask.h"
#include "Modules/ModuleManager.h"
#include "EditorModeRegistry.h"
#include "PropertyEditorModule.h"
#include "IDetailsView.h"
#include "Materials/Material.h"
#include "IPersonaPreviewScene.h"
#include "Logging/TokenizedMessage.h"
#include "AssetRegistry/ARFilter.h"
#include "AnimGraphDefinitions.h"
#include "Toolkits/ToolkitManager.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "SkeletalMeshSocketDetails.h"
#include "AnimNotifyDetails.h"
#include "AnimGraphNodeDetails.h"
#include "BlendProfileCustomization.h"
#include "BlendProfileInterfaceWrapperDetailsCustomization.h"
#include "Engine/SkinnedAssetCommon.h"
#include "AnimInstanceDetails.h"
#include "IEditableSkeleton.h"
#include "IPersonaToolkit.h"
#include "PersonaToolkit.h"
#include "TabSpawners.h"
#include "SSkeletonAnimNotifies.h"
#include "SAssetFamilyShortcutBar.h"
#include "Animation/AnimMontage.h"
#include "SMontageEditor.h"
#include "SSequenceEditor.h"
#include "Animation/AnimComposite.h"
#include "SAnimCompositeEditor.h"
#include "Animation/AnimStreamable.h"
#include "SAnimStreamableEditor.h"
#include "Animation/PoseAsset.h"
#include "SPoseEditor.h"
#include "Animation/BlendSpace.h"
#include "SAnimationBlendSpace.h"
#include "Animation/BlendSpace1D.h"
#include "Animation/AimOffsetBlendSpace.h"
#include "Animation/AimOffsetBlendSpace1D.h"
#include "SAnimationDlgs.h"
#include "FbxMeshUtils.h"
#include "IAssetTools.h"
#include "AssetToolsModule.h"
#include "Logging/MessageLog.h"
#include "AnimationEditorUtils.h"
#include "DesktopPlatformModule.h"
#include "FbxAnimUtils.h"
#include "PersonaAssetFamilyManager.h"
#include "AnimGraphNode_Slot.h"
#include "Customization/AnimGraphNodeSlotDetails.h"
#include "Customization/BlendSpaceDetails.h"
#include "Customization/BlendParameterDetails.h"
#include "Customization/InterpolationParameterDetails.h"
#include "SkeletonSelectionEditMode.h"
#include "PersonaEditorModeManager.h"
#include "PreviewSceneCustomizations.h"
#include "SSkeletonSlotNames.h"
#include "PersonaMeshDetails.h"
#include "Animation/MorphTarget.h"
#include "EditorDirectories.h"
#include "PersonaCommonCommands.h"
#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Animation/AnimNotifies/AnimNotify.h"
#include "Animation/AnimNotifies/AnimNotifyState.h"
#include "Animation/AnimBoneCompressionSettings.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Framework/Notifications/NotificationManager.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "PhysicsAssetRenderUtils.h"
#include "PersonaPreviewSceneDescription.h"
#include "PersonaPreviewSceneAnimationController.h"
#include "PersonaPreviewSceneRefPoseController.h"
#include "AssetViewerSettings.h"
#include "Customization/SkeletalMeshRegionCustomization.h"
#include "AnimationEditorUtils.h"
#include "Factories/AnimSequenceFactory.h"
#include "Factories/PoseAssetFactory.h"
#include "ScopedTransaction.h"
#include "AnimPreviewInstance.h"
#include "ISequenceRecorder.h"
#include "SkinWeightProfileCustomization.h"
#include "SAnimationBlendSpaceGridWidget.h"
#include "SAnimSequenceCurveEditor.h"
#include "AnimSequenceTimelineCommands.h"
#include "SAnimCurvePicker.h"
#include "SAnimAttributePicker.h"
#include "SAnimMontageSectionsPanel.h"
#include "SPersonaToolBox.h"
#include "Animation/AnimSequenceHelpers.h"
#include "SkeletalMeshReferenceSectionDetails.h"
#include "PersonaToolMenuContext.h"
#include "ToolMenu.h"
#include "ContentBrowserMenuContexts.h"
#include "SAssetView.h"
#include "ToolMenus.h"
#include "SAnimAssetFindReplace.h"
#include "SMorphTargetViewer.h"

IMPLEMENT_MODULE( FPersonaModule, Persona );

const FName PersonaAppName = FName(TEXT("PersonaApp"));

const FEditorModeID FPersonaEditModes::SkeletonSelection(TEXT("PersonaSkeletonSelection"));

#define LOCTEXT_NAMESPACE "PersonaModule"

void FPersonaModule::StartupModule()
{
	MenuExtensibilityManager = MakeShareable(new FExtensibilityManager);
	ToolBarExtensibilityManager = MakeShareable(new FExtensibilityManager);

	//Call this to make sure AnimGraph module is setup
	FModuleManager::Get().LoadModuleChecked(TEXT("AnimGraph"));
	
	// Call this to make sure MDTEditor module is setup
	FModuleManager::Get().LoadModuleChecked(TEXT("MirrorDataTableEditor"));

	// Make sure the advanced preview scene module is loaded 
	FModuleManager::Get().LoadModuleChecked("AdvancedPreviewScene");

	UPhysicsAssetRenderUtilities::Initialise();

	{
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.RegisterCustomClassLayout( "SkeletalMeshSocket", FOnGetDetailCustomizationInstance::CreateStatic( &FSkeletalMeshSocketDetails::MakeInstance ) );
		PropertyModule.RegisterCustomClassLayout( "EditorNotifyObject", FOnGetDetailCustomizationInstance::CreateStatic(&FAnimNotifyDetails::MakeInstance));
		PropertyModule.RegisterCustomClassLayout( "AnimGraphNode_Base", FOnGetDetailCustomizationInstance::CreateStatic( &FAnimGraphNodeDetails::MakeInstance ) );
		PropertyModule.RegisterCustomClassLayout( "AnimInstance", FOnGetDetailCustomizationInstance::CreateStatic(&FAnimInstanceDetails::MakeInstance));
		PropertyModule.RegisterCustomClassLayout("BlendSpace", FOnGetDetailCustomizationInstance::CreateStatic(&FBlendSpaceDetails::MakeInstance));	

		PropertyModule.RegisterCustomPropertyTypeLayout("BlendProfile", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FBlendProfileCustomization::MakeInstance));
		PropertyModule.RegisterCustomPropertyTypeLayout("BlendProfileInterfaceWrapper", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FBlendProfileInterfaceWrapperCustomization::MakeInstance));
		PropertyModule.RegisterCustomPropertyTypeLayout( "InputScaleBias", FOnGetPropertyTypeCustomizationInstance::CreateStatic( &FInputScaleBiasCustomization::MakeInstance ) );
		PropertyModule.RegisterCustomPropertyTypeLayout( "BoneReference", FOnGetPropertyTypeCustomizationInstance::CreateStatic( &FBoneReferenceCustomization::MakeInstance ) );
		PropertyModule.RegisterCustomPropertyTypeLayout("BoneSocketTarget", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FBoneSocketTargetCustomization::MakeInstance));
		PropertyModule.RegisterCustomPropertyTypeLayout( "PreviewMeshCollectionEntry", FOnGetPropertyTypeCustomizationInstance::CreateStatic( &FPreviewMeshCollectionEntryCustomization::MakeInstance ) );

		PropertyModule.RegisterCustomPropertyTypeLayout("BlendParameter", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FBlendParameterDetails::MakeInstance));
		PropertyModule.RegisterCustomPropertyTypeLayout("InterpolationParameter", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FInterpolationParameterDetails::MakeInstance));

		PropertyModule.RegisterCustomPropertyTypeLayout("SkinWeightProfileInfo", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FSkinWeightProfileCustomization::MakeInstance));

		PropertyModule.RegisterCustomPropertyTypeLayout("SkeletalMeshSamplingRegionBoneFilter", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraSkeletalMeshRegionBoneFilterDetails::MakeInstance));
		PropertyModule.RegisterCustomPropertyTypeLayout("SkeletalMeshSamplingRegionMaterialFilter", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNiagaraSkeletalMeshRegionMaterialFilterDetails::MakeInstance));

		PropertyModule.RegisterCustomPropertyTypeLayout("SectionReference", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FSectionReferenceCustomization::MakeInstance));
	}

	// Register the editor modes
	FEditorModeRegistry::Get().RegisterMode<FSkeletonSelectionEditMode>(FPersonaEditModes::SkeletonSelection, LOCTEXT("SkeletonSelectionEditMode", "Skeleton Selection"), FSlateIcon(), false);

	FPersonaCommonCommands::Register();
	FAnimSequenceTimelineCommands::Register();

	FKismetEditorUtilities::RegisterOnBlueprintCreatedCallback(this, UAnimNotify::StaticClass(), FKismetEditorUtilities::FOnBlueprintCreated::CreateRaw(this, &FPersonaModule::HandleNewAnimNotifyBlueprintCreated));
	FKismetEditorUtilities::RegisterOnBlueprintCreatedCallback(this, UAnimNotifyState::StaticClass(), FKismetEditorUtilities::FOnBlueprintCreated::CreateRaw(this, &FPersonaModule::HandleNewAnimNotifyStateBlueprintCreated));

	GetMutableDefault<UPersonaOptions>()->SetFlags(RF_Transactional);

	RegisterToolMenuExtensions();
}

void FPersonaModule::ShutdownModule()
{
	FKismetEditorUtilities::UnregisterAutoBlueprintNodeCreation(this);

	// Unregister the editor modes
	FEditorModeRegistry::Get().UnregisterMode(FPersonaEditModes::SkeletonSelection);

	MenuExtensibilityManager.Reset();
	ToolBarExtensibilityManager.Reset();

	// Unregister when shut down
	if(FModuleManager::Get().IsModuleLoaded("PropertyEditor"))
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.UnregisterCustomClassLayout("SkeletalMeshSocket");
		PropertyModule.UnregisterCustomClassLayout("EditorNotifyObject");
		PropertyModule.UnregisterCustomClassLayout("AnimGraphNode_Base");
		PropertyModule.UnregisterCustomClassLayout("AnimInstance");
		PropertyModule.UnregisterCustomClassLayout("BlendSpace");

		PropertyModule.UnregisterCustomPropertyTypeLayout("InputScaleBias");
		PropertyModule.UnregisterCustomPropertyTypeLayout("BoneReference");

		PropertyModule.UnregisterCustomPropertyTypeLayout("BlendParameter");
		PropertyModule.UnregisterCustomPropertyTypeLayout("InterpolationParameter");

		PropertyModule.UnregisterCustomPropertyTypeLayout("SkeletalMeshSamplingRegionBoneFilter");
		PropertyModule.UnregisterCustomPropertyTypeLayout("SkeletalMeshSamplingRegionMaterialFilter");
	}
}

TSharedRef<IPersonaToolkit> FPersonaModule::CreatePersonaToolkit(UObject* InAsset, const FPersonaToolkitArgs& PersonaToolkitArgs, USkeleton* InSkeleton) const
{
	TSharedRef<FPersonaToolkit> NewPersonaToolkit(new FPersonaToolkit());
	NewPersonaToolkit->Initialize(InAsset, PersonaToolkitArgs, InSkeleton);
	return NewPersonaToolkit;
}

TSharedRef<IPersonaToolkit> FPersonaModule::CreatePersonaToolkit(USkeleton* InSkeleton, const FPersonaToolkitArgs& PersonaToolkitArgs) const
{
	TSharedRef<FPersonaToolkit> NewPersonaToolkit(new FPersonaToolkit());
	NewPersonaToolkit->Initialize(InSkeleton, PersonaToolkitArgs);
	return NewPersonaToolkit;
}

TSharedRef<IPersonaToolkit> FPersonaModule::CreatePersonaToolkit(UAnimationAsset* InAnimationAsset, const FPersonaToolkitArgs& PersonaToolkitArgs) const
{
	TSharedRef<FPersonaToolkit> NewPersonaToolkit(new FPersonaToolkit());
	NewPersonaToolkit->Initialize(InAnimationAsset, PersonaToolkitArgs);
	return NewPersonaToolkit;
}

TSharedRef<IPersonaToolkit> FPersonaModule::CreatePersonaToolkit(USkeletalMesh* InSkeletalMesh, const FPersonaToolkitArgs& PersonaToolkitArgs) const
{
	TSharedRef<FPersonaToolkit> NewPersonaToolkit(new FPersonaToolkit());
	NewPersonaToolkit->Initialize(InSkeletalMesh, PersonaToolkitArgs);
	return NewPersonaToolkit;
}

TSharedRef<IPersonaToolkit> FPersonaModule::CreatePersonaToolkit(UAnimBlueprint* InAnimBlueprint, const FPersonaToolkitArgs& PersonaToolkitArgs) const
{
	TSharedRef<FPersonaToolkit> NewPersonaToolkit(new FPersonaToolkit());
	NewPersonaToolkit->Initialize(InAnimBlueprint, PersonaToolkitArgs);
	return NewPersonaToolkit;
}

TSharedRef<IPersonaToolkit> FPersonaModule::CreatePersonaToolkit(UPhysicsAsset* InPhysicsAsset, const FPersonaToolkitArgs& PersonaToolkitArgs) const
{
	TSharedRef<FPersonaToolkit> NewPersonaToolkit(new FPersonaToolkit());
	NewPersonaToolkit->Initialize(InPhysicsAsset, PersonaToolkitArgs);
	return NewPersonaToolkit;
}

TSharedRef<class IAssetFamily> FPersonaModule::CreatePersonaAssetFamily(const UObject* InAsset) const
{
	return FPersonaAssetFamilyManager::Get().CreatePersonaAssetFamily(InAsset);
}

void FPersonaModule::BroadcastAssetFamilyChange() const
{
	return FPersonaAssetFamilyManager::Get().BroadcastAssetFamilyChange();
}

void FPersonaModule::RecordAssetOpened(const FAssetData& InAssetData) const
{
	return FPersonaAssetFamilyManager::Get().RecordAssetOpened(InAssetData);
}

TSharedRef<SWidget> FPersonaModule::CreateAssetFamilyShortcutWidget(const TSharedRef<class FWorkflowCentricApplication>& InHostingApp, const TSharedRef<class IAssetFamily>& InAssetFamily) const
{
	return SNew(SAssetFamilyShortcutBar, InHostingApp, InAssetFamily);
}

TSharedRef<class FWorkflowTabFactory> FPersonaModule::CreateDetailsTabFactory(const TSharedRef<class FWorkflowCentricApplication>& InHostingApp, FOnDetailsCreated InOnDetailsCreated) const
{
	return MakeShareable(new FPersonaDetailsTabSummoner(InHostingApp, InOnDetailsCreated));
}

TSharedRef<class FWorkflowTabFactory> FPersonaModule::CreatePersonaViewportTabFactory(const TSharedRef<class FWorkflowCentricApplication>& InHostingApp, const FPersonaViewportArgs& InArgs) const
{
	return MakeShareable(new FPreviewViewportSummoner(InHostingApp, InArgs, 0));
}

void FPersonaModule::RegisterPersonaViewportTabFactories(FWorkflowAllowedTabSet& TabSet, const TSharedRef<class FWorkflowCentricApplication>& InHostingApp, const FPersonaViewportArgs& InArgs) const
{
	TabSet.RegisterFactory(MakeShareable(new FPreviewViewportSummoner(InHostingApp, InArgs, 0)));
	TabSet.RegisterFactory(MakeShareable(new FPreviewViewportSummoner(InHostingApp, InArgs, 1)));
	TabSet.RegisterFactory(MakeShareable(new FPreviewViewportSummoner(InHostingApp, InArgs, 2)));
	TabSet.RegisterFactory(MakeShareable(new FPreviewViewportSummoner(InHostingApp, InArgs, 3)));
}

TSharedRef<class FWorkflowTabFactory> FPersonaModule::CreateAnimNotifiesTabFactory(const TSharedRef<class FWorkflowCentricApplication>& InHostingApp, const TSharedRef<class IEditableSkeleton>& InEditableSkeleton, FOnObjectsSelected InOnObjectsSelected) const
{
	return MakeShareable(new FSkeletonAnimNotifiesSummoner(InHostingApp, InEditableSkeleton, InOnObjectsSelected));
}

TSharedRef<class FWorkflowTabFactory> FPersonaModule::CreateCurveViewerTabFactory(const TSharedRef<class FWorkflowCentricApplication>& InHostingApp, const TSharedPtr<class IEditableSkeleton>& InEditableSkeleton, const TSharedRef<IPersonaPreviewScene>& InPreviewScene, FOnObjectsSelected InOnObjectsSelected, bool bInReadOnlyCurveViewer) const
{
	return MakeShareable(new FAnimCurveViewerTabSummoner(InHostingApp, InEditableSkeleton, InPreviewScene, InOnObjectsSelected, bInReadOnlyCurveViewer));
}

TSharedRef<class FWorkflowTabFactory> FPersonaModule::CreateCurveMetadataEditorTabFactory(const TSharedRef<class FWorkflowCentricApplication>& InHostingApp, UObject* InMetadataHost, const TSharedRef<IPersonaPreviewScene>& InPreviewScene, FOnObjectsSelected InOnObjectsSelected) const
{
	return MakeShared<FAnimCurveMetadataEditorTabSummoner>(InHostingApp, InMetadataHost, InPreviewScene, InOnObjectsSelected);
}

TSharedRef<class FWorkflowTabFactory> FPersonaModule::CreateRetargetSourcesTabFactory(const TSharedRef<class FWorkflowCentricApplication>& InHostingApp, const TSharedRef<class IEditableSkeleton>& InEditableSkeleton, FSimpleMulticastDelegate& InOnPostUndo) const
{
	return MakeShareable(new FRetargetSourcesTabSummoner(InHostingApp, InEditableSkeleton, InOnPostUndo));
}

TSharedRef<class FWorkflowTabFactory> FPersonaModule::CreateAdvancedPreviewSceneTabFactory(const TSharedRef<class FWorkflowCentricApplication>& InHostingApp, const TSharedRef<IPersonaPreviewScene>& InPreviewScene) const
{
	return MakeShareable(new FAdvancedPreviewSceneTabSummoner(InHostingApp, InPreviewScene));
}

TSharedRef<class FWorkflowTabFactory> FPersonaModule::CreateAdvancedPreviewSceneTabFactory(const TSharedRef<class FWorkflowCentricApplication>& InHostingApp, const TSharedRef<IPersonaPreviewScene>& InPreviewScene, const FPersonaAdvancedPreviewSceneTabSettings& DetailCustomizationSettings) const
{
	return MakeShareable(new FAdvancedPreviewSceneTabSummoner(InHostingApp, InPreviewScene, DetailCustomizationSettings));
}

TSharedRef<class FWorkflowTabFactory> FPersonaModule::CreateAnimationAssetBrowserTabFactory(const TSharedRef<class FWorkflowCentricApplication>& InHostingApp, const TSharedRef<IPersonaToolkit>& InPersonaToolkit, FOnOpenNewAsset InOnOpenNewAsset, FOnAnimationSequenceBrowserCreated InOnAnimationSequenceBrowserCreated, bool bInShowHistory) const
{
	return MakeShareable(new FAnimationAssetBrowserSummoner(InHostingApp, InPersonaToolkit, InOnOpenNewAsset, InOnAnimationSequenceBrowserCreated, bInShowHistory));
}

TSharedRef<class FWorkflowTabFactory> FPersonaModule::CreateAssetDetailsTabFactory(const TSharedRef<class FWorkflowCentricApplication>& InHostingApp, FOnGetAsset InOnGetAsset, FOnDetailsCreated InOnDetailsCreated) const
{
	return MakeShareable(new FAssetPropertiesSummoner(InHostingApp, InOnGetAsset, InOnDetailsCreated));
}

TSharedRef<class FWorkflowTabFactory> FPersonaModule::CreateMorphTargetTabFactory(const TSharedRef<class FWorkflowCentricApplication>& InHostingApp, const TSharedRef<IPersonaPreviewScene>& InPreviewScene, FSimpleMulticastDelegate& OnPostUndo) const
{
	TSharedRef<IMorphTargetViewer> ViewerWidget = CreateDefaultMorphTargetViewerWidget(InPreviewScene, OnPostUndo);
	return MakeShareable(new FMorphTargetTabSummoner(InHostingApp, ViewerWidget));
}

TSharedRef<FWorkflowTabFactory> FPersonaModule::CreateMorphTargetTabFactory(
	const TSharedRef<class FWorkflowCentricApplication>& InHostingApp, TSharedRef<IMorphTargetViewer> InViewerWidget) const
{
	return MakeShareable(new FMorphTargetTabSummoner(InHostingApp, InViewerWidget));
}

TSharedRef<IMorphTargetViewer> FPersonaModule::CreateDefaultMorphTargetViewerWidget(
	const TSharedRef<IPersonaPreviewScene>& InPreviewScene,
	FSimpleMulticastDelegate& OnPostUndo) const
{
	return SNew(SMorphTargetViewer, InPreviewScene, OnPostUndo);	
}

TSharedRef<class FWorkflowTabFactory> FPersonaModule::CreateAnimBlueprintPreviewTabFactory(const TSharedRef<class FBlueprintEditor>& InBlueprintEditor, const TSharedRef<IPersonaPreviewScene>& InPreviewScene) const
{
	return MakeShareable(new FAnimBlueprintPreviewEditorSummoner(InBlueprintEditor, InPreviewScene));
}

TSharedRef<class FWorkflowTabFactory> FPersonaModule::CreatePoseWatchTabFactory(const TSharedRef<class FBlueprintEditor>& InBlueprintEditor) const
{
	return MakeShareable(new FPoseWatchManagerSummoner(InBlueprintEditor));
}

TSharedRef<class FWorkflowTabFactory> FPersonaModule::CreateAnimBlueprintAssetOverridesTabFactory(const TSharedRef<class FBlueprintEditor>& InBlueprintEditor, UAnimBlueprint* InAnimBlueprint, FSimpleMulticastDelegate& InOnPostUndo) const
{
	return MakeShareable(new FAnimBlueprintParentPlayerEditorSummoner(InBlueprintEditor, InOnPostUndo));
}

TSharedRef<class FWorkflowTabFactory> FPersonaModule::CreateSkeletonSlotNamesTabFactory(const TSharedRef<class FWorkflowCentricApplication>& InHostingApp, const TSharedRef<class IEditableSkeleton>& InEditableSkeleton, FOnObjectSelected InOnObjectSelected) const
{
	return MakeShareable(new FSkeletonSlotNamesSummoner(InHostingApp, InEditableSkeleton, InOnObjectSelected));
}

TSharedRef<SWidget> FPersonaModule::CreateBlendSpacePreviewWidget(const FBlendSpacePreviewArgs& InArgs) const
{
	return
		SNew(SBlendSpaceGridWidget)
		.Cursor(EMouseCursor::Crosshairs)
		.BlendSpaceBase(InArgs.PreviewBlendSpace)
		.Position(InArgs.PreviewPosition)
		.FilteredPosition(InArgs.PreviewFilteredPosition)
		.ReadOnly(true)
		.ShowAxisLabels(false)
		.ShowSettingsButtons(false)
		.OnGetBlendSpaceSampleName(InArgs.OnGetBlendSpaceSampleName);
}

TSharedRef<FWorkflowTabFactory> FPersonaModule::CreatePersonaToolboxTabFactory(const TSharedRef<FPersonaAssetEditorToolkit>& InHostingApp) const
{
	return MakeShareable(new FToolBoxSummoner(InHostingApp));
}

TSharedRef<FWorkflowTabFactory> FPersonaModule::CreateAttributeViewerTabFactory(const TSharedRef<FWorkflowCentricApplication>& InHostingApp, const TSharedRef<IPersonaPreviewScene>& InPreviewScene) const
{
	return MakeShareable(new FAnimAttributeViewerTabSummoner(InHostingApp, InPreviewScene));
}

TSharedRef<class FWorkflowTabFactory> FPersonaModule::CreateAnimMontageSectionsTabFactory(const TSharedRef<class FWorkflowCentricApplication>& InHostingApp, const TSharedRef<IPersonaToolkit>& InPersonaToolkit, FSimpleMulticastDelegate& InOnSectionsChanged) const
{
	return MakeShareable(new FAnimMontageSectionsSummoner(InHostingApp, InPersonaToolkit, InOnSectionsChanged));
}

TSharedRef<FWorkflowTabFactory> FPersonaModule::CreateAnimAssetFindReplaceTabFactory(const TSharedRef<FWorkflowCentricApplication>& InHostingApp, const FAnimAssetFindReplaceConfig& InConfig) const
{
	return MakeShared<FAnimAssetFindReplaceSummoner>(InHostingApp, InConfig);
}

TSharedRef<SWidget> FPersonaModule::CreateFindReplaceWidget(const FAnimAssetFindReplaceConfig& InConfig) const
{
	return SNew(SAnimAssetFindReplace)
		.Config(InConfig);
}

TSharedRef<SWidget> FPersonaModule::CreateEditorWidgetForAnimDocument(const TSharedRef<IAnimationEditor>& InHostingApp, UObject* InAnimAsset, const FAnimDocumentArgs& InArgs, FString& OutDocumentLink)
{
	TSharedPtr<SWidget> Result = SNullWidget::NullWidget;
	if (InAnimAsset)
	{
		TWeakPtr<IAnimationEditor> WeakHostingApp = InHostingApp;
		auto OnEditCurves = [WeakHostingApp](UAnimSequenceBase* InAnimSequence, const TArray<IAnimationEditor::FCurveEditInfo>& InCurveInfo, const TSharedPtr<ITimeSliderController>& InExternalTimeSliderController)
		{ 
			WeakHostingApp.Pin()->EditCurves(InAnimSequence, InCurveInfo, InExternalTimeSliderController);
		};

		if (UAnimSequence* Sequence = Cast<UAnimSequence>(InAnimAsset))
		{
			Result = SNew(SSequenceEditor, InArgs.PreviewScene.Pin().ToSharedRef(), InArgs.EditableSkeleton.Pin().ToSharedRef(), InHostingApp->GetToolkitCommands())
				.Sequence(Sequence)
				.OnObjectsSelected(InArgs.OnDespatchObjectsSelected)
				.OnInvokeTab(InArgs.OnDespatchInvokeTab)
				.OnEditCurves_Lambda(OnEditCurves);

			OutDocumentLink = TEXT("Engine/Animation/Sequences");
		}
		else if (UAnimComposite* Composite = Cast<UAnimComposite>(InAnimAsset))
		{
			Result = SNew(SAnimCompositeEditor, InArgs.PreviewScene.Pin().ToSharedRef(), InArgs.EditableSkeleton.Pin().ToSharedRef(), InHostingApp->GetToolkitCommands())
				.Composite(Composite)
				.OnObjectsSelected(InArgs.OnDespatchObjectsSelected)
				.OnInvokeTab(InArgs.OnDespatchInvokeTab)
				.OnEditCurves_Lambda(OnEditCurves);

			OutDocumentLink = TEXT("Engine/Animation/AnimationComposite");
		}
		else if (UAnimMontage* Montage = Cast<UAnimMontage>(InAnimAsset))
		{
			FMontageEditorRequiredArgs RequiredArgs(InArgs.PreviewScene.Pin().ToSharedRef(), InArgs.EditableSkeleton.Pin().ToSharedRef(), InArgs.OnSectionsChanged, InHostingApp->GetToolkitCommands());

			Result = SNew(SMontageEditor, RequiredArgs)
				.Montage(Montage)
				.OnSectionsChanged(InArgs.OnDespatchSectionsChanged)
				.OnInvokeTab(InArgs.OnDespatchInvokeTab)
				.OnObjectsSelected(InArgs.OnDespatchObjectsSelected)
				.OnEditCurves_Lambda(OnEditCurves);

			OutDocumentLink = TEXT("Engine/Animation/AnimMontage");
		}
		else if (UAnimStreamable* StreamableAnim = Cast<UAnimStreamable>(InAnimAsset))
		{
			Result = SNew(SAnimStreamableEditor, InArgs.PreviewScene.Pin().ToSharedRef(), InArgs.EditableSkeleton.Pin().ToSharedRef(), InHostingApp->GetToolkitCommands())
				.StreamableAnim(StreamableAnim)
				.OnObjectsSelected(InArgs.OnDespatchObjectsSelected)
				.OnInvokeTab(InArgs.OnDespatchInvokeTab)
				.OnEditCurves_Lambda(OnEditCurves);

			OutDocumentLink = TEXT("Engine/Animation/Sequences");
		}
		else if (UPoseAsset* PoseAsset = Cast<UPoseAsset>(InAnimAsset))
		{
			Result = SNew(SPoseEditor, InArgs.PersonaToolkit.Pin().ToSharedRef(), InArgs.EditableSkeleton.Pin().ToSharedRef(), InArgs.PreviewScene.Pin().ToSharedRef())
				.PoseAsset(PoseAsset);

			OutDocumentLink = TEXT("Engine/Animation/Sequences");
		}
		else if (UBlendSpace* BlendSpace = Cast<UBlendSpace>(InAnimAsset))
		{
			Result = SNew(SBlendSpaceEditor, InArgs.PreviewScene.Pin().ToSharedRef())
				.BlendSpace(BlendSpace);

			if (Cast<UAimOffsetBlendSpace>(InAnimAsset))
			{
				OutDocumentLink = TEXT("Engine/Animation/AimOffset");
			}
			else
			{
				OutDocumentLink = TEXT("Engine/Animation/Blendspaces");
			}
		}
		else if (UBlendSpace1D* BlendSpace1D = Cast<UBlendSpace1D>(InAnimAsset))
		{
			Result = SNew(SBlendSpaceEditor, InArgs.PreviewScene.Pin().ToSharedRef())
				.BlendSpace(BlendSpace1D);

			if (Cast<UAimOffsetBlendSpace1D>(InAnimAsset))
			{
				OutDocumentLink = TEXT("Engine/Animation/AimOffset");
			}
			else
			{
				OutDocumentLink = TEXT("Engine/Animation/Blendspaces");
			}
		}
	}

	if (Result.IsValid())
	{
		InAnimAsset->SetFlags(RF_Transactional);
	}

	return Result.ToSharedRef();
}

TSharedRef<IAnimSequenceCurveEditor> FPersonaModule::CreateCurveWidgetForAnimDocument(const TSharedRef<class FWorkflowCentricApplication>& InHostingApp, const TSharedRef<IPersonaPreviewScene>& InPreviewScene, UAnimSequenceBase* InAnimSequence, const TSharedPtr<ITimeSliderController>& InExternalTimeSliderController, const TSharedPtr<FTabManager>& InTabManager)
{
	return SNew(SAnimSequenceCurveEditor, InPreviewScene, InAnimSequence)
		.ExternalTimeSliderController(InExternalTimeSliderController)
		.TabManager(InTabManager);
}

void FPersonaModule::CustomizeMeshDetails(const TSharedRef<IDetailsView>& InDetailsView, const TSharedRef<IPersonaToolkit>& InPersonaToolkit)
{
	InDetailsView->SetGenericLayoutDetailsDelegate(FOnGetDetailCustomizationInstance::CreateStatic(&FPersonaMeshDetails::MakeInstance, TWeakPtr<IPersonaToolkit>(InPersonaToolkit)));
}

void FPersonaModule::ImportNewAsset(USkeleton* InSkeleton, EFBXImportType DefaultImportType)
{
	TSharedRef<SImportPathDialog> NewAnimDlg =
		SNew(SImportPathDialog);

	if (NewAnimDlg->ShowModal() != EAppReturnType::Cancel)
	{
		FString AssetPath = NewAnimDlg->GetAssetPath();

		UFbxImportUI* ImportUI = NewObject<UFbxImportUI>();
		ImportUI->Skeleton = InSkeleton;
		ImportUI->MeshTypeToImport = DefaultImportType;

		FbxMeshUtils::SetImportOption(ImportUI);

		// now I have to set skeleton on it. 
		FAssetToolsModule& AssetToolsModule = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools");
		AssetToolsModule.Get().ImportAssetsWithDialog(AssetPath);
	}
}

void PopulateWithAssets(const FTopLevelAssetPath& ClassName, FName SkeletonMemberName, const FString& SkeletonString, TArray<FAssetData>& OutAssets)
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

	FARFilter Filter;
	Filter.ClassPaths.Add(ClassName);
	Filter.TagsAndValues.Add(SkeletonMemberName, SkeletonString);

	AssetRegistryModule.Get().GetAssets(Filter, OutAssets);
}

void FPersonaModule::TestSkeletonCurveMetaDataForUse(const TSharedRef<IEditableSkeleton>& InEditableSkeleton) const
{
	const USkeleton& Skeleton = InEditableSkeleton->GetSkeleton();
	const FString SkeletonString = FAssetData(&Skeleton).GetExportTextName();

	TArray<FAssetData> SkeletalMeshes;
	PopulateWithAssets(USkeletalMesh::StaticClass()->GetClassPathName(), USkeletalMesh::GetSkeletonMemberName(), SkeletonString, SkeletalMeshes);
	TArray<FAssetData> Animations;
	PopulateWithAssets(UAnimSequence::StaticClass()->GetClassPathName(), USkeletalMesh::GetSkeletonMemberName(), SkeletonString, Animations);

	FText TimeTakenMessage = FText::Format(LOCTEXT("TimeTakenWarning", "In order to verify curve usage all Skeletal Meshes and Animations that use this skeleton will be loaded, this may take some time.\n\nProceed?\n\nNumber of Meshes: {0}\nNumber of Animations: {1}"), FText::AsNumber(SkeletalMeshes.Num()), FText::AsNumber(Animations.Num()));

	if (FMessageDialog::Open(EAppMsgType::YesNo, TimeTakenMessage) == EAppReturnType::Yes)
	{
		const FText LoadingStatusUpdate = FText::Format(LOCTEXT("VerifyCurves_LoadingAllAnimations", "Loading all animations for skeleton '{0}'"), FText::FromString(Skeleton.GetName()));
		{
			FScopedSlowTask LoadingAnimSlowTask(static_cast<float>(Animations.Num()), LoadingStatusUpdate);
			LoadingAnimSlowTask.MakeDialog();

			// Loop through all animations to load then, this makes sure smart names are all up to date
			for (const FAssetData& Anim : Animations)
			{
				LoadingAnimSlowTask.EnterProgressFrame();
				UAnimSequence* Seq = Cast<UAnimSequence>(Anim.GetAsset());
			}
		}

		// Grab all curve names for this skeleton
		TArray<FName> UnusedNames;
		Skeleton.GetCurveMetaDataNames(UnusedNames);

		const FText ProcessingStatusUpdate = FText::Format(LOCTEXT("VerifyCurves_ProcessingCurveUsage", "Looking at curve useage for each skeletal mesh of skeleton '{0}'"), FText::FromString(Skeleton.GetName()));
		{
			FScopedSlowTask LoadingSkelMeshSlowTask(static_cast<float>(SkeletalMeshes.Num()), ProcessingStatusUpdate);
			LoadingSkelMeshSlowTask.MakeDialog();

			for (int32 MeshIdx = 0; MeshIdx < SkeletalMeshes.Num(); ++MeshIdx)
			{
				LoadingSkelMeshSlowTask.EnterProgressFrame();

				const USkeletalMesh* Mesh = Cast<USkeletalMesh>(SkeletalMeshes[MeshIdx].GetAsset());

				// Filter morph targets from curves
				const TArray<UMorphTarget*>& MorphTargets = Mesh->GetMorphTargets();
				for (int32 I = 0; I < MorphTargets.Num(); ++I)
				{
					const int32 CurveIndex = UnusedNames.RemoveSingleSwap(MorphTargets[I]->GetFName(), EAllowShrinking::No);
				}

				// Filter material params from curves
				for (const FSkeletalMaterial& Mat : Mesh->GetMaterials())
				{
					if (UnusedNames.Num() == 0)
					{
						break; // Done
					}

					UMaterial* Material = (Mat.MaterialInterface != nullptr) ? Mat.MaterialInterface->GetMaterial() : nullptr;
					if (Material)
					{
						TArray<FMaterialParameterInfo> OutParameterInfo;
						TArray<FGuid> OutParameterIds;

						// Retrieve all scalar parameter names from the material
						Mat.MaterialInterface->GetAllScalarParameterInfo(OutParameterInfo, OutParameterIds);

						for (FMaterialParameterInfo SPInfo : OutParameterInfo)
						{
							UnusedNames.RemoveSingleSwap(SPInfo.Name);
						}
					}
				}
			}
		}

		FMessageLog CurveOutput("Persona");
		CurveOutput.NewPage(LOCTEXT("PersonaMessageLogName", "Persona"));

		bool bFoundIssue = false;

		const FText ProcessingAnimStatusUpdate = FText::Format(LOCTEXT("FindUnusedCurves_ProcessingSkeletalMeshes", "Finding animations that reference unused curves on skeleton '{0}'"), FText::FromString(Skeleton.GetName()));
		{
			FScopedSlowTask ProcessingAnimationsSlowTask(static_cast<float>(Animations.Num()), ProcessingAnimStatusUpdate);
			ProcessingAnimationsSlowTask.MakeDialog();

			for (const FAssetData& Anim : Animations)
			{
				ProcessingAnimationsSlowTask.EnterProgressFrame();
				UAnimSequence* Seq = Cast<UAnimSequence>(Anim.GetAsset());

				TSharedPtr<FTokenizedMessage> Message;
				for (const FFloatCurve& Curve : Seq->GetDataModel()->GetCurveData().FloatCurves)
				{
					if (UnusedNames.Contains(Curve.GetName()))
					{
						bFoundIssue = true;
						if (!Message.IsValid())
						{
							Message = CurveOutput.Warning();
							Message->AddToken(FAssetNameToken::Create(Anim.GetObjectPathString(), FText::FromName(Anim.AssetName)));
							Message->AddToken(FTextToken::Create(LOCTEXT("VerifyCurves_FoundAnimationsWithUnusedReferences", "References the following curves that are not used for either morph targets or material parameters and so may be unneeded")));
						}
						CurveOutput.Info(FText::FromName(Curve.GetName()));
					}
				}
			}
		}

		if (bFoundIssue)
		{
			CurveOutput.Notify();
		}
	}
}

void FPersonaModule::ApplyCompression(TArray<TWeakObjectPtr<UAnimSequence>>& AnimSequences, bool bPickBoneSettingsOverride)
{
	UAnimBoneCompressionSettings* OverrideSettings = nullptr;
	if (bPickBoneSettingsOverride)
	{
		FAnimationCompressionSelectionDialogConfig DialogConfig;

		UAnimBoneCompressionSettings* CurrentSettings = nullptr;
		if (AnimSequences.Num() != 0)
		{
			CurrentSettings = AnimSequences[0]->BoneCompressionSettings;
			for (TWeakObjectPtr<UAnimSequence>& AnimSeq : AnimSequences)
			{
				if (CurrentSettings != AnimSeq->BoneCompressionSettings)
				{
					// One of the sequences in the list has a different settings asset, use the default behavior
					CurrentSettings = nullptr;
					break;
				}
			}
		}

		DialogConfig.DefaultSelectedAsset = CurrentSettings;

		FAssetData AssetData = AnimationEditorUtils::CreateModalAnimationCompressionSelectionDialog(DialogConfig);
		if (AssetData.IsValid())
		{
			OverrideSettings = Cast<UAnimBoneCompressionSettings>(AssetData.GetAsset());
		}
		else
		{
			// No asset selected but we need an override, do nothing
			return;
		}
	}

	TArray<UAnimSequence*> AnimSequencePtrs;
	AnimSequencePtrs.Reserve(AnimSequences.Num());

	for (TWeakObjectPtr<UAnimSequence>& AnimSeq : AnimSequences)
	{
		AnimSequencePtrs.Add(AnimSeq.Get());
	}

	AnimationEditorUtils::ApplyCompressionAlgorithm(AnimSequencePtrs, OverrideSettings);
}

bool FPersonaModule::ExportToFBX(TArray<TWeakObjectPtr<UAnimSequence>>& AnimSequences, USkeletalMesh* SkeletalMesh)
{
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	bool bResult = false;

	if (DesktopPlatform)
	{
		if (SkeletalMesh == NULL)
		{
			FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("ExportToFBXExportMissingSkeletalMesh", "ERROR: Missing skeletal mesh"));
			return bResult;
		}

		if (AnimSequences.Num() > 0)
		{
			//Get parent window for dialogs
			TSharedPtr<SWindow> RootWindow = FGlobalTabmanager::Get()->GetRootWindow();

			void* ParentWindowWindowHandle = NULL;

			if (RootWindow.IsValid() && RootWindow->GetNativeWindow().IsValid())
			{
				ParentWindowWindowHandle = RootWindow->GetNativeWindow()->GetOSWindowHandle();
			}

			//Cache anim file names
			TArray<FString> AnimFileNames;
			for (auto Iter = AnimSequences.CreateIterator(); Iter; ++Iter)
			{
				AnimFileNames.Add(Iter->Get()->GetName() + TEXT(".fbx"));
			}

			IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
			FString DestinationFolder;

			const FString Title = LOCTEXT("ExportFBXsToFolderTitle", "Choose a destination folder for the FBX file(s)").ToString();

			if (AnimSequences.Num() > 1)
			{
				bool bFolderValid = false;
				// More than one file, just ask for directory
				while (!bFolderValid)
				{
					const bool bFolderSelected = DesktopPlatform->OpenDirectoryDialog(
						ParentWindowWindowHandle,
						Title,
						FEditorDirectories::Get().GetLastDirectory(ELastDirectory::GENERIC_EXPORT),
						DestinationFolder
						);

					if (!bFolderSelected)
					{
						// User canceled, return
						return bResult;
					}

					FEditorDirectories::Get().SetLastDirectory(ELastDirectory::GENERIC_EXPORT, DestinationFolder);
					FPaths::NormalizeFilename(DestinationFolder);

					//Check whether there are any fbx filename conflicts in this folder
					for (auto Iter = AnimFileNames.CreateIterator(); Iter; ++Iter)
					{
						FString& AnimFileName = *Iter;
						FString FullPath = DestinationFolder + "/" + AnimFileName;

						bFolderValid = true;
						if (PlatformFile.FileExists(*FullPath))
						{
							FFormatNamedArguments Args;
							Args.Add(TEXT("DestinationFolder"), FText::FromString(DestinationFolder));
							const FText DialogMessage = FText::Format(LOCTEXT("ExportToFBXFileOverwriteMessage", "Exporting to '{DestinationFolder}' will cause one or more existing FBX files to be overwritten. Would you like to continue?"), Args);
							EAppReturnType::Type DialogReturn = FMessageDialog::Open(EAppMsgType::YesNo, DialogMessage);
							bFolderValid = (EAppReturnType::Yes == DialogReturn);
							break;
						}
					}
				}
			}
			else
			{
				// One file only, ask for full filename.
				// Can set bFolderValid from the SaveFileDialog call as the window will handle 
				// duplicate files for us.
				TArray<FString> TempDestinationNames;
				bool bSave = DesktopPlatform->SaveFileDialog(
					ParentWindowWindowHandle,
					Title,
					FEditorDirectories::Get().GetLastDirectory(ELastDirectory::GENERIC_EXPORT),
					AnimSequences[0]->GetName(),
					"FBX  |*.fbx",
					EFileDialogFlags::None,
					TempDestinationNames
					);

				if (!bSave)
				{
					// Canceled
					return bResult;
				}
				check(TempDestinationNames.Num() == 1);
				check(AnimFileNames.Num() == 1);

				DestinationFolder = FPaths::GetPath(TempDestinationNames[0]);
				AnimFileNames[0] = FPaths::GetCleanFilename(TempDestinationNames[0]);

				FEditorDirectories::Get().SetLastDirectory(ELastDirectory::GENERIC_EXPORT, DestinationFolder);
			}


			const bool bShowCancel = false;
			const bool bShowProgressDialog = true;
			GWarn->BeginSlowTask(LOCTEXT("ExportToFBXProgress", "Exporting Animation(s) to FBX"), bShowProgressDialog, bShowCancel);

			// make sure to use SkeletalMesh, when export inside of Persona
			const int32 NumberOfAnimations = AnimSequences.Num();
			bool ExportBatch = (NumberOfAnimations > 1);
			bool ExportAll = false;
			bool ExportCancel = false;
			for (int32 i = 0; i < NumberOfAnimations; ++i)
			{
				GWarn->UpdateProgress(i, NumberOfAnimations);

				UAnimSequence* AnimSequence = AnimSequences[i].Get();

				FString FileName = FString::Printf(TEXT("%s/%s"), *DestinationFolder, *AnimFileNames[i]);

				FbxAnimUtils::ExportAnimFbx(*FileName, AnimSequence, SkeletalMesh, ExportBatch, ExportAll, ExportCancel);
				if (ExportBatch && ExportCancel)
				{
					//The user cancel the batch export
					break;
				}
				bResult |= !ExportCancel;
			}

			GWarn->EndSlowTask();
		}
	}
	return bResult;
}

void FPersonaModule::AddLoopingInterpolation(TArray<TWeakObjectPtr<UAnimSequence>>& AnimSequences)
{
	FText WarningMessage = LOCTEXT("AddLoopingInterpolation", "This will add an extra first frame at the end of the animation to create a better looping interpolation. This action cannot be undone. Would you like to proceed?");

	if (FMessageDialog::Open(EAppMsgType::YesNo, WarningMessage) == EAppReturnType::Yes)
	{
		for (TWeakObjectPtr<UAnimSequence>& Animation : AnimSequences)
		{
			// get first frame and add to the last frame and go through track
			// now calculating old animated space bases
			if (UAnimSequence* AnimSequence = Animation.Get())
			{
				UE::Anim::AnimationData::AddLoopingInterpolation(AnimSequence);
			}
		}
	}
}

void FPersonaModule::CustomizeBlueprintEditorDetails(const TSharedRef<class IDetailsView>& InDetailsView, FOnInvokeTab InOnInvokeTab)
{
	InDetailsView->RegisterInstancedCustomPropertyLayout(UAnimGraphNode_Slot::StaticClass(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FAnimGraphNodeSlotDetails::MakeInstance, InOnInvokeTab));

	InDetailsView->SetExtensionHandler(MakeShared<FAnimGraphNodeBindingExtension>());
}

IPersonaEditorModeManager* FPersonaModule::CreatePersonaEditorModeManager()
{
	return new FPersonaEditorModeManager();
}

namespace UE::Persona::Private
{
	struct FNotificationHandler : public TSharedFromThis<FNotificationHandler>
	{
		static void HandleApplyPreviewMesh(TSharedPtr<FNotificationHandler> InNotificationHandler, TWeakPtr<IPersonaToolkit> InWeakPersonaToolkit)
		{
			TSharedPtr<IPersonaToolkit> PinnedPersonaToolkit = InWeakPersonaToolkit.Pin();
			if (PinnedPersonaToolkit.IsValid())	// Toolkit can become invalid while the toast is open
			{
				PinnedPersonaToolkit->SetPreviewMesh(PinnedPersonaToolkit->GetPreviewScene()->GetPreviewMesh(), true);
				if (InNotificationHandler->Notification.IsValid())
				{
					InNotificationHandler->Notification->Fadeout();
				}
			}
		}

		TSharedPtr<SNotificationItem> Notification;
	};

	static TSharedRef<SWidget> CreatePreviewMeshComboButtonContents(TWeakPtr<IPersonaToolkit> InWeakPersonaToolkit)
	{
		static TWeakPtr<SNotificationItem> WeakPendingApplyMeshNotification = nullptr;

		FMenuBuilder MenuBuilder(true, nullptr);

		MenuBuilder.BeginSection(TEXT("ChoosePreviewMesh"), LOCTEXT("ChoosePreviewMesh", "Choose Preview Mesh"));
		{
			FAssetPickerConfig AssetPickerConfig;

			auto HandleAssetSelected = [InWeakPersonaToolkit](const FAssetData& AssetData)
			{
				if (InWeakPersonaToolkit.IsValid())
				{
					InWeakPersonaToolkit.Pin()->SetPreviewMesh(Cast<USkeletalMesh>(AssetData.GetAsset()), false);
				}

				if (InWeakPersonaToolkit.IsValid())	// SetPreviewMesh can invalidate the persona toolkit, so check it here before displaying toast
				{
					TSharedPtr<FNotificationHandler> NotificationHandler = MakeShared<FNotificationHandler>();

					FNotificationInfo Info(LOCTEXT("PreviewMeshSetTemporarily", "Preview mesh set temporarily"));
					Info.ExpireDuration = 10.0f;
					Info.bUseLargeFont = true;
					Info.ButtonDetails.Add(
						FNotificationButtonInfo(
							LOCTEXT("ApplyToAsset", "Apply To Asset"),
							LOCTEXT("ApplyToAssetToolTip", "The preview mesh has changed, but it will not be able to be saved until it is applied to the asset. Click here to make the change to the preview mesh persistent."),
							FSimpleDelegate::CreateStatic(&FNotificationHandler::HandleApplyPreviewMesh, NotificationHandler, InWeakPersonaToolkit),
							SNotificationItem::CS_Success));

					// Fade-out previously added SetPreviewMesh notification (if any)
					const TSharedPtr<SNotificationItem> PendingNotification = WeakPendingApplyMeshNotification.Pin();
					if(PendingNotification.IsValid())
					{
						PendingNotification->Fadeout();
					}

					NotificationHandler->Notification = FSlateNotificationManager::Get().AddNotification(Info);
					WeakPendingApplyMeshNotification = NotificationHandler->Notification;
					if (NotificationHandler->Notification.IsValid())
					{
						NotificationHandler->Notification->SetCompletionState(SNotificationItem::CS_Success);
					}

					FSlateApplication::Get().DismissAllMenus();
				}
			};

			AssetPickerConfig.OnAssetEnterPressed = FOnAssetEnterPressed::CreateLambda([HandleAssetSelected](const TArray<FAssetData>& SelectedAssetData)
			{
				if (SelectedAssetData.Num() == 1)
				{
					HandleAssetSelected(SelectedAssetData[0]);
				}
			});
			AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateLambda(HandleAssetSelected);
			AssetPickerConfig.bAllowNullSelection = false;
			AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;
			AssetPickerConfig.Filter.bRecursiveClasses = false;
			AssetPickerConfig.Filter.ClassPaths.Add(USkeletalMesh::StaticClass()->GetClassPathName());
			AssetPickerConfig.OnShouldFilterAsset = FOnShouldFilterAsset::CreateLambda([InWeakPersonaToolkit](const FAssetData& AssetData)
			{
				if (InWeakPersonaToolkit.IsValid() && InWeakPersonaToolkit.Pin()->GetSkeleton())
				{
					if (InWeakPersonaToolkit.Pin()->GetContext() == UPhysicsAsset::StaticClass()->GetFName())
					{
						return false;
					}

					FString TagValue;
					if (AssetData.GetTagValue("Skeleton", TagValue))
					{
						return !InWeakPersonaToolkit.Pin()->GetSkeleton()->IsCompatibleForEditor(AssetData);
					}
				}
				return true;
			});
			if (InWeakPersonaToolkit.IsValid())
			{
				AssetPickerConfig.InitialAssetSelection = FAssetData(InWeakPersonaToolkit.Pin()->GetPreviewScene()->GetPreviewMesh());
			}

			FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

			TSharedPtr<SBox> MenuEntry = SNew(SBox)
				.WidthOverride(300.0f)
				.HeightOverride(300.0f)
				[
					ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
				];

			MenuBuilder.AddWidget(MenuEntry.ToSharedRef(), FText::GetEmpty(), true);
		}
		MenuBuilder.EndSection();

		return MenuBuilder.MakeWidget();
	};

	static TSharedRef<SWidget> CreatePreviewAnimationComboButtonContents(TWeakPtr<IPersonaToolkit> InWeakPersonaToolkit)
	{
		FMenuBuilder MenuBuilder(true, nullptr);

		MenuBuilder.BeginSection(TEXT("ChoosePreviewAnimation"), LOCTEXT("ChoosePreviewAnimation", "Choose Preview Animation"));
		{
			FAssetPickerConfig AssetPickerConfig;
			AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateLambda([InWeakPersonaToolkit](const FAssetData& AssetData)
			{
				if (InWeakPersonaToolkit.IsValid())
				{
					TSharedRef<FAnimationEditorPreviewScene> PreviewScene = StaticCastSharedRef<FAnimationEditorPreviewScene>(InWeakPersonaToolkit.Pin()->GetPreviewScene());
					PreviewScene->GetPreviewSceneDescription()->SetPreviewController(UPersonaPreviewSceneAnimationController::StaticClass(), &PreviewScene.Get());

					UPersonaPreviewSceneAnimationController* AnimController = CastChecked<UPersonaPreviewSceneAnimationController>(PreviewScene->GetPreviewSceneDescription()->PreviewControllerInstance);
					AnimController->Animation = CastChecked<UAnimationAsset>(AssetData.GetAsset());
					AnimController->InitializeView(PreviewScene->GetPreviewSceneDescription(), &PreviewScene.Get());

					// Make sure any settings views are updated with the new settings
					UAssetViewerSettings::Get()->OnAssetViewerProfileAddRemoved().Broadcast();
				}

				FSlateApplication::Get().DismissAllMenus();
			});
			AssetPickerConfig.bAllowNullSelection = false;
			AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;
			AssetPickerConfig.Filter.bRecursiveClasses = true;
			AssetPickerConfig.Filter.ClassPaths.Add(UAnimationAsset::StaticClass()->GetClassPathName());
			AssetPickerConfig.OnShouldFilterAsset = FOnShouldFilterAsset::CreateLambda([InWeakPersonaToolkit](const FAssetData& AssetData)
			{
				if (InWeakPersonaToolkit.IsValid() && InWeakPersonaToolkit.Pin()->GetSkeleton())
				{
					FString TagValue;
					if (AssetData.GetTagValue("Skeleton", TagValue))
					{
						return !InWeakPersonaToolkit.Pin()->GetSkeleton()->IsCompatibleForEditor(AssetData);
					}
				}
				return true;
			});
			if (InWeakPersonaToolkit.IsValid())
			{
				AssetPickerConfig.InitialAssetSelection = FAssetData(InWeakPersonaToolkit.Pin()->GetPreviewScene()->GetPreviewAnimationAsset());
			}

			FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

			TSharedPtr<SBox> MenuEntry = SNew(SBox)
				.WidthOverride(300.0f)
				.HeightOverride(300.0f)
				[
					ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
				];

			MenuBuilder.AddWidget(MenuEntry.ToSharedRef(), FText::GetEmpty(), true);
		}
		MenuBuilder.EndSection();

		return MenuBuilder.MakeWidget();
	};

	static void ShowReferencePose(TWeakPtr<IPersonaToolkit> InWeakPersonaToolkit)
	{
		if (InWeakPersonaToolkit.IsValid())
		{
			TSharedRef<FAnimationEditorPreviewScene> PreviewScene = StaticCastSharedRef<FAnimationEditorPreviewScene>(InWeakPersonaToolkit.Pin()->GetPreviewScene());
			PreviewScene->GetPreviewSceneDescription()->SetPreviewController(UPersonaPreviewSceneRefPoseController::StaticClass(), &PreviewScene.Get());

			UPersonaPreviewSceneRefPoseController* AnimController = CastChecked<UPersonaPreviewSceneRefPoseController>(PreviewScene->GetPreviewSceneDescription()->PreviewControllerInstance);
			AnimController->bResetBoneTransforms = true;
			AnimController->InitializeView(PreviewScene->GetPreviewSceneDescription(), &PreviewScene.Get());

			// Reset this to false here as we dont want it to always reset bone transforms, only if they user picks it from the toolbar
			AnimController->bResetBoneTransforms = false;

			// Make sure any settings views are updated with the new settings
			UAssetViewerSettings::Get()->OnAssetViewerProfileAddRemoved().Broadcast();
		}
	}
}

void FPersonaModule::AddCommonToolbarExtensions(FToolBarBuilder& InToolbarBuilder, TSharedRef<IPersonaToolkit> PersonaToolkit, const FCommonToolbarExtensionArgs& InArgs)
{
	TWeakPtr<IPersonaToolkit> WeakPersonaToolkit = PersonaToolkit;

	if (InArgs.bPreviewMesh)
	{
		InToolbarBuilder.AddComboButton(
			FUIAction(),
			FOnGetContent::CreateStatic(&UE::Persona::Private::CreatePreviewMeshComboButtonContents, WeakPersonaToolkit),
			LOCTEXT("SetPreviewMesh", "Preview Mesh"),
			LOCTEXT("SetPreviewMeshTooltip", "Set a new preview skeletal mesh for the current asset (stored per-animation or per-skeleton)"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Persona.TogglePreviewAsset", "Persona.TogglePreviewAsset.Small")
		);
	}

	if (InArgs.bPreviewAnimation)
	{
		InToolbarBuilder.AddComboButton(
			FUIAction(),
			FOnGetContent::CreateStatic(&UE::Persona::Private::CreatePreviewAnimationComboButtonContents, WeakPersonaToolkit),
			LOCTEXT("SetPreviewAnimation", "Preview Animation"),
			LOCTEXT("SetPreviewAnimationTooltip", "Setup the scene to use a preview animation. More advanced settings are available in Preview Scene Settings."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Persona.TogglePreviewAnimation", "Persona.TogglePreviewAnimation.Small")
		);
	}

	if (InArgs.bReferencePose)
	{
		InToolbarBuilder.AddToolBarButton(
			FUIAction(FExecuteAction::CreateStatic(&UE::Persona::Private::ShowReferencePose, WeakPersonaToolkit)),
			NAME_None,
			LOCTEXT("ShowReferencePose", "Reference Pose"),
			LOCTEXT("ShowReferencePoseTooltip", "Show the reference pose. Clears all bone modifications. More advanced settings are available in Preview Scene Settings."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Persona.ToggleReferencePose", "Persona.ToggleReferencePose.Small")
		);
	}

	if (InArgs.bCreateAsset)
	{
		InToolbarBuilder.AddComboButton(
			FUIAction(),
			FOnGetContent::CreateStatic(&FPersonaModule::GenerateCreateAssetMenu, WeakPersonaToolkit),
			LOCTEXT("CreateAsset_Label", "Create Asset"),
			LOCTEXT("CreateAsset_ToolTip", "Create Assets for this skeleton."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Persona.CreateAsset")
		);
	}
}

TSharedRef<SWidget> FPersonaModule::CreateMultiCurvePicker(const USkeleton* InSkeleton, FOnCurvesPicked InOnCurvesPicked, FIsCurveNameMarkedForExclusion InIsCurveNameMarkedForExclusion)
{
	return SNew(SAnimCurvePicker, InSkeleton)
		.bEnableMultiselect(true)
		.OnCurvesPicked(InOnCurvesPicked)
		.IsCurveNameMarkedForExclusion(InIsCurveNameMarkedForExclusion);
}

TSharedRef<SWidget> FPersonaModule::CreateCurvePicker(const USkeleton* InSkeleton, FOnCurvePicked InOnCurvePicked, FIsCurveNameMarkedForExclusion InIsCurveNameMarkedForExclusion)
{
	return SNew(SAnimCurvePicker, InSkeleton)
		.OnCurvesPicked_Lambda([InOnCurvePicked](const TConstArrayView<FName> InCurves)
		{
			check(InCurves.Num() == 1);
			InOnCurvePicked.ExecuteIfBound(InCurves[0]);
		})
		.IsCurveNameMarkedForExclusion(InIsCurveNameMarkedForExclusion);
}

TSharedRef<SWidget> FPersonaModule::CreateAttributePicker(
	const TObjectPtr<const USkeleton> InSkeleton,
	FOnAttributesPicked InOnAttributesPicked,
	const bool bEnableMultiSelect,
	const bool bCanShowOtherSkeletonAttributes)
{
	return SNew(SAnimAttributePicker, InSkeleton)
		.OnAttributesPicked(InOnAttributesPicked)
		.bEnableMultiSelect(bEnableMultiSelect)
		.bCanShowOtherSkeletonAttributes(bCanShowOtherSkeletonAttributes);
}

TSharedRef<SWidget> FPersonaModule::CreateSkeletonNotifyPicker(FOnNotifyPicked InOnNotifyPicked, TSharedPtr<IEditableSkeleton> InEditableSkeleton, TSharedPtr<FAssetEditorToolkit> InHostingApp, bool bInShowSyncMarkers, bool bInShowNotifies, bool bInShowCompatibleSkeletonAssets, bool bInShowOtherAssets)
{
	return SNew(SSkeletonAnimNotifies, InHostingApp)
		.OnItemSelected(InOnNotifyPicked)
		.EditableSkeleton(InEditableSkeleton)
		.ShowSyncMarkers(bInShowSyncMarkers)
		.ShowNotifies(bInShowNotifies)
		.ShowCompatibleSkeletonAssets(bInShowCompatibleSkeletonAssets)
		.ShowOtherAssets(bInShowOtherAssets);
}

void FPersonaModule::RegisterNotifyHostAsset(const FTopLevelAssetPath& InAssetClassPath, const FNotifyHostAssetParameters& InParameters)
{
	NotifyHostParameters.Add(InAssetClassPath, InParameters);
	NotifyHostAssetVersion++;
}

void FPersonaModule::UnregisterNotifyHostAsset(const FTopLevelAssetPath& InAssetClassPath)
{
	NotifyHostParameters.Remove(InAssetClassPath);
}

const FPersonaModule::FNotifyHostAssetParameters* FPersonaModule::FindNotifyHostAsset(UObject* InAsset)
{
	check(InAsset);
	UClass* AssetClass = InAsset->GetClass();
	while(AssetClass)
	{
		if(const FNotifyHostAssetParameters* FoundParameters = NotifyHostParameters.Find(AssetClass->GetClassPathName()))
		{
			return FoundParameters;
		}
		AssetClass = AssetClass->GetSuperClass();
	}

	return nullptr;
}

TArray<FTopLevelAssetPath> FPersonaModule::GetAllNotifyHostAssetClassPaths() const
{
	TArray<FTopLevelAssetPath> AssetClassPaths;
	NotifyHostParameters.GenerateKeyArray(AssetClassPaths);
	return AssetClassPaths;
}

uint32 FPersonaModule::GetNotifyHostAssetVersion() const
{
	return NotifyHostAssetVersion;
}

void FPersonaModule::AddCommonMenuExtensions(UToolMenu* InToolMenu, const FCommonToolMenuExtensionArgs& InArgs)
{
	UPersonaToolMenuContext* PersonaToolMenuContext = InToolMenu->FindContext<UPersonaToolMenuContext>();
	if (PersonaToolMenuContext == nullptr)
	{
		return;
	}

	TWeakPtr<IPersonaToolkit> WeakPersonaToolkit = PersonaToolMenuContext->GetToolkit();

	FToolMenuSection& PersonaSection = InToolMenu->AddSection("Persona", LOCTEXT("PersonaSectionName", "Persona"));

	if (InArgs.bPreviewMesh)
	{
		PersonaSection.AddSubMenu(
			"PreviewMesh",
			LOCTEXT("SetPreviewMesh", "Preview Mesh"),
			LOCTEXT("SetPreviewMeshTooltip", "Set a new preview skeletal mesh for the current asset (stored per-animation or per-skeleton)"),
			FOnGetContent::CreateStatic(&UE::Persona::Private::CreatePreviewMeshComboButtonContents, WeakPersonaToolkit),
			false,
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Persona.TogglePreviewAsset", "Persona.TogglePreviewAsset.Small")
		);
	}

	if (InArgs.bPreviewAnimation)
	{
		PersonaSection.AddSubMenu(
			"PreviewAnimation",
			LOCTEXT("SetPreviewAnimation", "Preview Animation"),
			LOCTEXT("SetPreviewAnimationTooltip", "Setup the scene to use a preview animation. More advanced settings are available in Preview Scene Settings."),
			FOnGetContent::CreateStatic(&UE::Persona::Private::CreatePreviewAnimationComboButtonContents, WeakPersonaToolkit),
			false,
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Persona.TogglePreviewAnimation", "Persona.TogglePreviewAnimation.Small")
		);
	}

	if (InArgs.bReferencePose)
	{
		PersonaSection.AddMenuEntry(
			"ReferencePose",
			LOCTEXT("ShowReferencePose", "Reference Pose"),
			LOCTEXT("ShowReferencePoseTooltip", "Show the reference pose. Clears all bone modifications. More advanced settings are available in Preview Scene Settings."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Persona.ToggleReferencePose", "Persona.ToggleReferencePose.Small"),
			FUIAction(FExecuteAction::CreateStatic(&UE::Persona::Private::ShowReferencePose, WeakPersonaToolkit))
		);
	}

	if (InArgs.bCreateAsset)
	{
		PersonaSection.AddSubMenu(
			"CreateAsset",
			LOCTEXT("CreateAsset_Label", "Create Asset"),
			LOCTEXT("CreateAsset_ToolTip", "Create Assets for this skeleton."),
			FOnGetContent::CreateStatic(&FPersonaModule::GenerateCreateAssetMenu, WeakPersonaToolkit),
			false,
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Persona.CreateAsset")
		);
	}
}


void FPersonaModule::AddCommonToolbarExtensions(UToolMenu* InToolMenu, const FCommonToolMenuExtensionArgs& InArgs)
{
	UPersonaToolMenuContext* PersonaToolMenuContext = InToolMenu->FindContext<UPersonaToolMenuContext>();
	if (PersonaToolMenuContext == nullptr)
	{
		return;
	}

	TWeakPtr<IPersonaToolkit> WeakPersonaToolkit = PersonaToolMenuContext->GetToolkit();

	FToolMenuSection& PersonaSection = InToolMenu->AddSection("Persona", LOCTEXT("PersonaSectionName", "Persona"));

	if (InArgs.bPreviewMesh)
	{
		PersonaSection.AddEntry(FToolMenuEntry::InitComboButton(
			"PreviewMesh",
			FUIAction(),
			FOnGetContent::CreateStatic(&UE::Persona::Private::CreatePreviewMeshComboButtonContents, WeakPersonaToolkit),
			LOCTEXT("SetPreviewMesh", "Preview Mesh"),
			LOCTEXT("SetPreviewMeshTooltip", "Set a new preview skeletal mesh for the current asset (stored per-animation or per-skeleton)"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Persona.TogglePreviewAsset", "Persona.TogglePreviewAsset.Small"))
		);
	}

	if (InArgs.bPreviewAnimation)
	{
		PersonaSection.AddEntry(FToolMenuEntry::InitComboButton(
			"PreviewAnimation",
			FUIAction(),
			FOnGetContent::CreateStatic(&UE::Persona::Private::CreatePreviewAnimationComboButtonContents, WeakPersonaToolkit),
			LOCTEXT("SetPreviewAnimation", "Preview Animation"),
			LOCTEXT("SetPreviewAnimationTooltip", "Setup the scene to use a preview animation. More advanced settings are available in Preview Scene Settings."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Persona.TogglePreviewAnimation", "Persona.TogglePreviewAnimation.Small"))
		);
	}

	if (InArgs.bReferencePose)
	{
		PersonaSection.AddEntry(FToolMenuEntry::InitToolBarButton(
			"ReferencePose",
			FUIAction(FExecuteAction::CreateStatic(&UE::Persona::Private::ShowReferencePose, WeakPersonaToolkit)),
			LOCTEXT("ShowReferencePose", "Reference Pose"),
			LOCTEXT("ShowReferencePoseTooltip", "Show the reference pose. Clears all bone modifications. More advanced settings are available in Preview Scene Settings."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Persona.ToggleReferencePose", "Persona.ToggleReferencePose.Small"))
		);
	}

	if (InArgs.bCreateAsset)
	{
		PersonaSection.AddEntry(FToolMenuEntry::InitComboButton(
			"CreateAsset",
			FUIAction(),
			FOnGetContent::CreateStatic(&FPersonaModule::GenerateCreateAssetMenu, WeakPersonaToolkit),
			LOCTEXT("CreateAsset_Label", "Create Asset"),
			LOCTEXT("CreateAsset_ToolTip", "Create Assets for this skeleton."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Persona.CreateAsset"))
		);
	}
}

TSharedRef< SWidget > FPersonaModule::GenerateCreateAssetMenu(TWeakPtr<IPersonaToolkit> InWeakPersonaToolkit)
{
	IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();

	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, NULL);

	// Create Animation menu
	MenuBuilder.BeginSection("CreateAnimation", LOCTEXT("CreateAnimationMenuHeading", "Animation"));
	{
		if(AssetTools.IsAssetClassSupported(UAnimSequence::StaticClass()))
		{
			// create menu
			MenuBuilder.AddSubMenu(
				LOCTEXT("CreateAnimationSubmenu", "Create Animation"),
				LOCTEXT("CreateAnimationSubmenu_ToolTip", "Create Animation for this skeleton"),
				FNewMenuDelegate::CreateStatic(&FPersonaModule::FillCreateAnimationMenu, InWeakPersonaToolkit),
				false,
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Persona.AssetActions.CreateAnimAsset")
				);
		}

		if (AssetTools.IsAssetClassSupported(UPoseAsset::StaticClass()))
		{
			MenuBuilder.AddSubMenu(
				LOCTEXT("CreatePoseAssetSubmenu", "Create PoseAsset"),
				LOCTEXT("CreatePoseAsssetSubmenu_ToolTip", "Create PoseAsset for this skeleton"),
				FNewMenuDelegate::CreateStatic(&FPersonaModule::FillCreatePoseAssetMenu, InWeakPersonaToolkit),
				false,
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.PoseAsset")
			);
		}
	}
	MenuBuilder.EndSection();

	TSharedRef<IPersonaToolkit> PersonaToolkit = InWeakPersonaToolkit.Pin().ToSharedRef();
	TArray<TSoftObjectPtr<UObject>> Objects;
	if (PersonaToolkit->GetPreviewMesh())
	{
		Objects.Add(PersonaToolkit->GetPreviewMesh());
	}
	else
	{
		Objects.Add(PersonaToolkit->GetSkeleton());
	}

	AnimationEditorUtils::FillCreateAssetMenu(MenuBuilder, Objects, FAnimAssetCreated::CreateStatic(&FPersonaModule::HandleAssetCreated), false);

	return MenuBuilder.MakeWidget();
}

void FPersonaModule::FillCreateAnimationMenu(FMenuBuilder& MenuBuilder, TWeakPtr<IPersonaToolkit> InWeakPersonaToolkit)
{
	TSharedRef<IPersonaToolkit> PersonaToolkit = InWeakPersonaToolkit.Pin().ToSharedRef();
	TArray<TSoftObjectPtr<UObject>> Objects;
	if (PersonaToolkit->GetPreviewMesh())
	{
		Objects.Add(PersonaToolkit->GetPreviewMesh());
	}
	else
	{
		Objects.Add(PersonaToolkit->GetSkeleton());
	}

	const bool bDoNotShowNameDialog = false;
	const bool bAllowReplaceExisting = true;  // This avoids a crash when saving on top of the currently previewed animation (if false, it tries to generate using a PersonaToolkit that no longer exists, as the editor gets closed for deleting the asset)

	// create rig
	MenuBuilder.BeginSection("CreateAnimationSubMenu", LOCTEXT("CreateAnimationSubMenuHeading", "Create Animation"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("CreateAnimation_RefPose", "Reference Pose"),
			LOCTEXT("CreateAnimation_RefPose_Tooltip", "Create Animation from reference pose."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateStatic(&AnimationEditorUtils::ExecuteNewAnimAsset<UAnimSequenceFactory, UAnimSequence>, Objects, FString("_Sequence"), FAnimAssetCreated::CreateStatic(&FPersonaModule::CreateAnimation, EPoseSourceOption::ReferencePose, InWeakPersonaToolkit), bDoNotShowNameDialog, bAllowReplaceExisting),
				FCanExecuteAction()
				)
			);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("CreateAnimation_CurrentPose", "Current Pose"),
			LOCTEXT("CreateAnimation_CurrentPose_Tooltip", "Create Animation from current pose."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateStatic(&AnimationEditorUtils::ExecuteNewAnimAsset<UAnimSequenceFactory, UAnimSequence>, Objects, FString("_Sequence"), FAnimAssetCreated::CreateStatic(&FPersonaModule::CreateAnimation, EPoseSourceOption::CurrentPose, InWeakPersonaToolkit), bDoNotShowNameDialog, bAllowReplaceExisting),
				FCanExecuteAction()
				)
			);

		if(UAnimSequence* AnimSequence = Cast<UAnimSequence>(PersonaToolkit->GetAnimationAsset()))
		{
			MenuBuilder.AddSubMenu(
				LOCTEXT("CreateAnimation_CurrenAnimationSubMenu", "Current Animation"),
				LOCTEXT("CreateAnimation_CurrenAnimationSubMenu_ToolTip", "Create Animation from current animation"),
				FNewMenuDelegate::CreateStatic(&FPersonaModule::FillCreateAnimationFromCurrentAnimationMenu, InWeakPersonaToolkit),
				false,
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Persona.AssetActions.CreateAnimAsset")
			);
		}
	}
	MenuBuilder.EndSection();
}

void FPersonaModule::FillCreateAnimationFromCurrentAnimationMenu(FMenuBuilder& MenuBuilder, TWeakPtr<IPersonaToolkit> InWeakPersonaToolkit) 
{
	TSharedRef<IPersonaToolkit> PersonaToolkit = InWeakPersonaToolkit.Pin().ToSharedRef();
	TArray<TSoftObjectPtr<UObject>> Objects;

	if (PersonaToolkit->GetPreviewMesh())
	{
		Objects.Add(PersonaToolkit->GetPreviewMesh());
	}
	else
	{
		Objects.Add(PersonaToolkit->GetSkeleton());
	}

	const bool bDoNotShowNameDialog = false;
	const bool bAllowReplaceExisting = true;  // This avoids a crash when saving on top of the currently previewed animation (if false, it tries to generate using a PersonaToolkit that no longer exists, as the editor gets closed for deleting the asset)

	// create rig
	MenuBuilder.BeginSection("CreateAnimationSubMenu", LOCTEXT("CreateAnimationFromCurrentAnimationSubmenuHeading", "Create Animation"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("CreateAnimation_CurrentAnimation_AnimData", "Animation Data"),
			LOCTEXT("CreateAnimation_CurrentAnimation_AnimData_Tooltip", "Create Animation from Animation Source Data."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateStatic(&AnimationEditorUtils::ExecuteNewAnimAsset<UAnimSequenceFactory, UAnimSequence>, Objects, FString("_Sequence"), FAnimAssetCreated::CreateStatic(&FPersonaModule::CreateAnimation, EPoseSourceOption::CurrentAnimation_AnimData, InWeakPersonaToolkit), bDoNotShowNameDialog, bAllowReplaceExisting)
			)
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("CreateAnimation_CurrentAnimation_PreviewMesh", "Preview Mesh"),
			LOCTEXT("CreateAnimation_CurrentAnimation_PreviewMesh_Tooltip", "Create Animation by playing on the Current Preview Mesh, including Retargeting, Post Process Graph, or anything you see on the preview mesh."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateStatic(&AnimationEditorUtils::ExecuteNewAnimAsset<UAnimSequenceFactory, UAnimSequence>, Objects, FString("_Sequence"), FAnimAssetCreated::CreateStatic(&FPersonaModule::CreateAnimation, EPoseSourceOption::CurrentAnimation_PreviewMesh, InWeakPersonaToolkit), bDoNotShowNameDialog, bAllowReplaceExisting)
			)
		);
	}
	MenuBuilder.EndSection();
}

void FPersonaModule::FillCreatePoseAssetMenu(FMenuBuilder& MenuBuilder, TWeakPtr<IPersonaToolkit> InWeakPersonaToolkit)
{
	TSharedRef<IPersonaToolkit> PersonaToolkit = InWeakPersonaToolkit.Pin().ToSharedRef();
	TArray<TSoftObjectPtr<UObject>> Objects;

	if (PersonaToolkit->GetPreviewMesh())
	{
		Objects.Add(PersonaToolkit->GetPreviewMesh());
	}
	else
	{
		Objects.Add(PersonaToolkit->GetSkeleton());
	}

	const bool bDoNotShowNameDialog = false;
	const bool bAllowReplaceExisting = true;  // This avoids a crash when saving on top of the currently previewed animation (if false, it tries to generate using a PersonaToolkit that no longer exists, as the editor gets closed for deleting the asset)

	// create rig
	MenuBuilder.BeginSection("CreatePoseAssetSubMenu", LOCTEXT("CreatePoseAssetSubMenuHeading", "Create PoseAsset"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("CreatePoseAsset_CurrentPose", "Current Pose"),
			LOCTEXT("CreatePoseAsset_CurrentPose_Tooltip", "Create PoseAsset from current pose."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateStatic(&AnimationEditorUtils::ExecuteNewAnimAsset<UPoseAssetFactory, UPoseAsset>, Objects, FString("_PoseAsset"), FAnimAssetCreated::CreateStatic(&FPersonaModule::CreatePoseAsset, EPoseSourceOption::CurrentPose, InWeakPersonaToolkit), bDoNotShowNameDialog, bAllowReplaceExisting),
				FCanExecuteAction()
			)
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("CreatePoseAsset_CurrentAnimation", "Current Animation"),
			LOCTEXT("CreatePoseAsset_CurrentAnimation_Tooltip", "Create Animation from current animation."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateStatic(&AnimationEditorUtils::ExecuteNewAnimAsset<UPoseAssetFactory, UPoseAsset>, Objects, FString("_PoseAsset"), FAnimAssetCreated::CreateStatic(&FPersonaModule::CreatePoseAsset, EPoseSourceOption::CurrentAnimation_AnimData, InWeakPersonaToolkit), bDoNotShowNameDialog, bAllowReplaceExisting),
				FCanExecuteAction()
			)
		);
	}
	MenuBuilder.EndSection();

	// create pose asset
	MenuBuilder.BeginSection("InsertPoseSubMenuSection", LOCTEXT("InsertPoseSubMenuSubMenuHeading", "Insert Pose"));
	{
		MenuBuilder.AddSubMenu(
			LOCTEXT("InsertPoseSubmenu", "Insert Pose"),
			LOCTEXT("InsertPoseSubmenu_ToolTip", "Insert current pose to selected PoseAsset"),
			FNewMenuDelegate::CreateStatic(&FPersonaModule::FillInsertPoseMenu, InWeakPersonaToolkit),
			false,
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.PoseAsset")
		);
	}
	MenuBuilder.EndSection();
}

void FPersonaModule::FillInsertPoseMenu(FMenuBuilder& MenuBuilder, TWeakPtr<IPersonaToolkit> InWeakPersonaToolkit)
{
	FAssetPickerConfig AssetPickerConfig;

	TSharedRef<IPersonaToolkit> PersonaToolkit = InWeakPersonaToolkit.Pin().ToSharedRef();
	USkeleton* Skeleton = PersonaToolkit->GetSkeleton();

	/** The asset picker will only show skeletons */
	AssetPickerConfig.Filter.ClassPaths.Add(UPoseAsset::StaticClass()->GetClassPathName());
	AssetPickerConfig.Filter.bRecursiveClasses = false;
	AssetPickerConfig.bAllowNullSelection = false;
	AssetPickerConfig.Filter.TagsAndValues.Add(TEXT("Skeleton"), FAssetData(Skeleton).GetExportTextName());

	/** The delegate that fires when an asset was selected */
	AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateStatic(&FPersonaModule::InsertCurrentPoseToAsset, InWeakPersonaToolkit);

	/** The default view mode should be a list view */
	AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;

	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

	MenuBuilder.AddWidget(
		ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig),
		FText::GetEmpty()
	);
}

void FPersonaModule::InsertCurrentPoseToAsset(const FAssetData& NewPoseAssetData, TWeakPtr<IPersonaToolkit> InWeakPersonaToolkit)
{
	TSharedRef<IPersonaToolkit> PersonaToolkit = InWeakPersonaToolkit.Pin().ToSharedRef();
	UPoseAsset* PoseAsset = Cast<UPoseAsset>(NewPoseAssetData.GetAsset());
	FScopedTransaction ScopedTransaction(LOCTEXT("InsertPose", "Insert Pose"));

	if (PoseAsset)
	{
		PoseAsset->Modify();

		UDebugSkelMeshComponent* PreviewMeshComponent = PersonaToolkit->GetPreviewMeshComponent();
		if (PreviewMeshComponent)
		{
			FName NewPoseName = PoseAsset->AddPoseWithUniqueName(PreviewMeshComponent);
			FFormatNamedArguments Args;
			Args.Add(TEXT("PoseAsset"), FText::FromString(PoseAsset->GetName()));
			Args.Add(TEXT("PoseName"), FText::FromName(NewPoseName));
			FNotificationInfo Info(FText::Format(LOCTEXT("InsertPoseSucceeded", "The current pose has inserted to {PoseAsset} with {PoseName}"), Args));
			Info.ExpireDuration = 7.0f;
			Info.bUseLargeFont = false;
			TSharedPtr<SNotificationItem> Notification = FSlateNotificationManager::Get().AddNotification(Info);
			if (Notification.IsValid())
			{
				Notification->SetCompletionState(SNotificationItem::CS_Success);
			}
		}
	}

	// it doesn't work well if I leave the window open. The delegate goes weired or it stop showing the popups. 
	FSlateApplication::Get().DismissAllMenus();
}

bool FPersonaModule::CreateAnimation(const TArray<UObject*> NewAssets, const EPoseSourceOption Option, TWeakPtr<IPersonaToolkit> InWeakPersonaToolkit)
{
	bool bResult = true;
	if (NewAssets.Num() > 0)
	{
		TSharedRef<IPersonaToolkit> PersonaToolkit = InWeakPersonaToolkit.Pin().ToSharedRef();
		USkeletalMeshComponent* MeshComponent = PersonaToolkit->GetPreviewMeshComponent();
		UAnimSequence* Sequence = Cast<UAnimSequence>(PersonaToolkit->GetAnimationAsset());

		for (auto NewAsset : NewAssets)
		{
			UAnimSequence* NewAnimSequence = Cast<UAnimSequence>(NewAsset);
			if (NewAnimSequence)
			{
				switch (Option)
				{
				case EPoseSourceOption::ReferencePose:
					bResult &= NewAnimSequence->CreateAnimation(MeshComponent->GetSkeletalMeshAsset());
					break;
				case EPoseSourceOption::CurrentPose:
					bResult &= NewAnimSequence->CreateAnimation(MeshComponent);
					break;
				case EPoseSourceOption::CurrentAnimation_AnimData:
					bResult &= NewAnimSequence->CreateAnimation(Sequence);
					break;
				case EPoseSourceOption::CurrentAnimation_PreviewMesh:
				{
					ISequenceRecorder & RecorderModule = FModuleManager::Get().LoadModuleChecked<ISequenceRecorder>("SequenceRecorder");
					bResult &= RecorderModule.RecordSingleNodeInstanceToAnimation(MeshComponent, NewAnimSequence);
				}
					break;
				default: 
					ensure(false);
					break;
				}
			}
		}

		if (bResult)
		{
			HandleAssetCreated(NewAssets);

			// if it created based on current mesh component, 
			if (Option == EPoseSourceOption::CurrentPose)
			{
				UDebugSkelMeshComponent* PreviewMeshComponent = PersonaToolkit->GetPreviewMeshComponent();
				if (PreviewMeshComponent && PreviewMeshComponent->PreviewInstance)
				{
					PreviewMeshComponent->PreviewInstance->ResetModifiedBone();
				}
			}
		}
	}
	return true;
}

bool FPersonaModule::CreatePoseAsset(const TArray<UObject*> NewAssets, const EPoseSourceOption Option, TWeakPtr<IPersonaToolkit> InWeakPersonaToolkit)
{
	bool bResult = false;
	if (NewAssets.Num() > 0)
	{
		TSharedRef<IPersonaToolkit> PersonaToolkit = InWeakPersonaToolkit.Pin().ToSharedRef();
		UDebugSkelMeshComponent* PreviewComponent = PersonaToolkit->GetPreviewMeshComponent();
		UAnimSequence* Sequence = Cast<UAnimSequence>(PersonaToolkit->GetAnimationAsset());

		for (auto NewAsset : NewAssets)
		{
			UPoseAsset* NewPoseAsset = Cast<UPoseAsset>(NewAsset);
			if (NewPoseAsset)
			{
				switch (Option)
				{
				case EPoseSourceOption::CurrentPose:
					NewPoseAsset->AddPoseWithUniqueName(PreviewComponent);
					bResult = true;
					break;
				case EPoseSourceOption::CurrentAnimation_AnimData:
					NewPoseAsset->CreatePoseFromAnimation(Sequence);
					bResult = true;
					break;
				default:
					ensure(false);
					bResult = false; 
					break;
				}

			}
		}

		// if it contains error, warn them
		if (bResult)
		{
			HandleAssetCreated(NewAssets);

			// if it created based on current mesh component, 
			if (Option == EPoseSourceOption::CurrentPose)
			{
				PreviewComponent->PreviewInstance->ResetModifiedBone();
			}
		}
		else
		{
			FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("FailedToCreateAsset", "Failed to create asset"));
		}
	}
	return true;
}

bool FPersonaModule::HandleAssetCreated(const TArray<UObject*> NewAssets)
{
	if (NewAssets.Num() > 0)
	{
		FAssetRegistryModule::AssetCreated(NewAssets[0]);

		// forward to asset manager to open the asset for us
		FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>(TEXT("AssetTools"));
		TWeakPtr<IAssetTypeActions> AssetTypeActions = AssetToolsModule.Get().GetAssetTypeActionsForClass(NewAssets[0]->GetClass());
		if (AssetTypeActions.IsValid())
		{
			AssetTypeActions.Pin()->OpenAssetEditor(NewAssets);
		}
	}
	return true;
}

void FPersonaModule::HandleNewAnimNotifyBlueprintCreated(UBlueprint* InBlueprint)
{
	if (InBlueprint->BlueprintType == BPTYPE_Normal)
	{
		UEdGraph* const NewGraph = FBlueprintEditorUtils::CreateNewGraph(InBlueprint, "Received_Notify", UEdGraph::StaticClass(), UEdGraphSchema_K2::StaticClass());
		FBlueprintEditorUtils::AddFunctionGraph(InBlueprint, NewGraph, /*bIsUserCreated=*/ false, UAnimNotify::StaticClass());
		InBlueprint->LastEditedDocuments.Add(NewGraph);
	}
}

void FPersonaModule::HandleNewAnimNotifyStateBlueprintCreated(UBlueprint* InBlueprint)
{
	if (InBlueprint->BlueprintType == BPTYPE_Normal)
	{
		UEdGraph* const NewGraph = FBlueprintEditorUtils::CreateNewGraph(InBlueprint, "Received_NotifyTick", UEdGraph::StaticClass(), UEdGraphSchema_K2::StaticClass());
		FBlueprintEditorUtils::AddFunctionGraph(InBlueprint, NewGraph, /*bIsUserCreated=*/ false, UAnimNotifyState::StaticClass());
		InBlueprint->LastEditedDocuments.Add(NewGraph);
	}
}

TSharedRef<SWidget> FPersonaModule::CreateBlendSpaceEditWidget(UBlendSpace* InBlendSpace, const FBlendSpaceEditorArgs& InArgs) const
{
	return SNew(SBlendSpaceEditor)
		.BlendSpace(InBlendSpace)
		.DisplayScrubBar(false)
		.OnBlendSpaceNavigateUp(InArgs.OnBlendSpaceNavigateUp)
		.OnBlendSpaceNavigateDown(InArgs.OnBlendSpaceNavigateDown)
		.OnBlendSpaceCanvasDoubleClicked(InArgs.OnBlendSpaceCanvasDoubleClicked)
		.OnBlendSpaceSampleDoubleClicked(InArgs.OnBlendSpaceSampleDoubleClicked)
		.OnBlendSpaceSampleAdded(InArgs.OnBlendSpaceSampleAdded)
		.OnBlendSpaceSampleRemoved(InArgs.OnBlendSpaceSampleRemoved)
		.OnBlendSpaceSampleReplaced(InArgs.OnBlendSpaceSampleReplaced)
		.OnGetBlendSpaceSampleName(InArgs.OnGetBlendSpaceSampleName)
		.OnExtendSampleTooltip(InArgs.OnExtendSampleTooltip)
		.OnSetPreviewPosition(InArgs.OnSetPreviewPosition)
		.PreviewPosition(InArgs.PreviewPosition)
		.PreviewFilteredPosition(InArgs.PreviewFilteredPosition)
		.StatusBarName(InArgs.StatusBarName);
}

void FPersonaModule::RegisterToolMenuExtensions()
{
	// Register content browser menu extensions
	UToolMenu* ToolMenu = UToolMenus::Get()->ExtendMenu("ContentBrowser.AssetViewOptions");
	ToolMenu->AddDynamicSection("AnimationAssets", FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
	{
		if(UContentBrowserAssetViewContextMenuContext* MenuContext = InMenu->Context.FindContext<UContentBrowserAssetViewContextMenuContext>())
		{
			if(TSharedPtr<SAssetView> AssetView = MenuContext->AssetView.Pin())
			{
				// Check whether this asset view is showing animation-related assets
				TArray<FTopLevelAssetPath> BaseAnimAssetClassPaths =
				{
					UAnimationAsset::StaticClass()->GetClassPathName(),
					UAnimBlueprint::StaticClass()->GetClassPathName(),
					USkeletalMesh::StaticClass()->GetClassPathName(),
					USkeleton::StaticClass()->GetClassPathName()
				};
				TSet<FTopLevelAssetPath> AnimAssetClassPaths;
				IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();
				AssetRegistry.GetDerivedClassNames(BaseAnimAssetClassPaths, TSet<FTopLevelAssetPath>(), AnimAssetClassPaths);

				auto AssetClassMatches = [&AnimAssetClassPaths](const FTopLevelAssetPath& InPath)
				{
					for(const FTopLevelAssetPath& Path : AnimAssetClassPaths)
					{
						if(InPath == Path)
						{
							return true;
						}
					}
					return false;
				};

				// Add a menu item to allow incompatible skeletons to be selected if this is selecting animation assets
				const FARFilter& BackendFilter = AssetView->GetBackendFilter();
				if(BackendFilter.ClassPaths.ContainsByPredicate(AssetClassMatches))
				{
					FToolMenuSection& Section = InMenu->AddSection("AnimationAssets", LOCTEXT("AnimationAssetsSection", "Animation Assets"));
					Section.AddMenuEntry(
						"AllowIncompatibleSkeletons",
						LOCTEXT("AllowIncompatibleSkeletons", "Allow Incompatible Skeletons"),
						LOCTEXT("AllowIncompatibleSkeletonsTooltip", "Whether to allow animation assets that are incompatible with the current skeleton/skeletal mesh to be selected."),
						FSlateIcon(),
						FToolUIActionChoice(
							FUIAction(
								FExecuteAction::CreateLambda([WeakAssetView = TWeakPtr<SAssetView>(AssetView)]()
								{
									UPersonaOptions* PersonaOptions = GetMutableDefault<UPersonaOptions>();
									PersonaOptions->SetAllowIncompatibleSkeletonSelection(!PersonaOptions->GetAllowIncompatibleSkeletonSelection());
									if(TSharedPtr<SAssetView> AssetView = WeakAssetView.Pin())
									{
										AssetView->RequestSlowFullListRefresh();
									}
								}),
								FCanExecuteAction(),
								FIsActionChecked::CreateLambda([]()
								{
									return GetDefault<UPersonaOptions>()->bAllowIncompatibleSkeletonSelection;
								})
							)
						),
						EUserInterfaceActionType::ToggleButton
					);
				}
			}
		}
	}));
}

void FPersonaModule::RegisterBlendProfilePickerExtender(TSharedPtr<IBlendProfilePickerExtender> Extender)
{
	CustomBlendProfiles.Add(Extender);
}

void FPersonaModule::UnregisterBlendProfilePickerExtender(const FName ExtenderId)
{
	CustomBlendProfiles.RemoveAll([ExtenderId](const TSharedPtr<IBlendProfilePickerExtender>& Extender)
		{
			return Extender->GetId() == ExtenderId;
		});
}

const TArray<TSharedPtr<IBlendProfilePickerExtender>>& FPersonaModule::GetCustomBlendProfiles() const
{
	return CustomBlendProfiles;
}

#undef LOCTEXT_NAMESPACE
