// Copyright Epic Games, Inc. All Rights Reserved.

#include "PersonaPreviewSceneAnimationController.h"
#include "AnimationEditorPreviewScene.h"
#include "PersonaPreviewSceneDescription.h"
#include "IPersonaToolkit.h"
#include "IEditableSkeleton.h"
#include "DetailWidgetRow.h"
#include "PropertyCustomizationHelpers.h"
#include "Animation/Skeleton.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PersonaPreviewSceneAnimationController)

void UPersonaPreviewSceneAnimationController::InitializeView(UPersonaPreviewSceneDescription* SceneDescription, IPersonaPreviewScene* PreviewScene) const
{
	PreviewScene->SetPreviewAnimationAsset(Animation.LoadSynchronous());
}

void UPersonaPreviewSceneAnimationController::UninitializeView(UPersonaPreviewSceneDescription* SceneDescription, IPersonaPreviewScene* PreviewScene) const
{

}

IDetailPropertyRow* UPersonaPreviewSceneAnimationController::AddPreviewControllerPropertyToDetails(const TSharedRef<IPersonaToolkit>& PersonaToolkit, IDetailLayoutBuilder& DetailBuilder, IDetailCategoryBuilder& Category, const FProperty* Property, const EPropertyLocation::Type PropertyLocation)
{
	TArray<UObject*> ListOfPreviewController{ this };

	const USkeleton* Skeleton = PersonaToolkit->GetPreviewMeshComponent()->GetSkeletalMeshAsset() ? PersonaToolkit->GetPreviewMeshComponent()->GetSkeletalMeshAsset()->GetSkeleton() : nullptr;
	if (Skeleton)
	{
		IDetailPropertyRow* NewRow = Category.AddExternalObjectProperty(ListOfPreviewController, Property->GetFName(), PropertyLocation);
	
		NewRow->CustomWidget()
		.NameContent()
		[
			NewRow->GetPropertyHandle()->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MaxDesiredWidth(250.0f)
		.MinDesiredWidth(250.0f)
		[
			SNew(SObjectPropertyEntryBox)
			.AllowedClass(UAnimationAsset::StaticClass())
			.PropertyHandle(NewRow->GetPropertyHandle())
			.OnShouldFilterAsset(FOnShouldFilterAsset::CreateUObject(this, &UPersonaPreviewSceneAnimationController::HandleShouldFilterAsset, Skeleton))
			.ThumbnailPool(DetailBuilder.GetThumbnailPool())
		];
		return NewRow;
	}

	return nullptr;
}

bool UPersonaPreviewSceneAnimationController::HandleShouldFilterAsset(const FAssetData& InAssetData, const USkeleton* InSkeleton) const
{
	if (InSkeleton && InSkeleton->IsCompatibleForEditor(InAssetData))
	{
		return false;
	}

	return true;
}
