// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PersonaPreviewSceneController.h"
#include "UObject/SoftObjectPtr.h"
#include "PersonaPreviewSceneAnimationController.generated.h"

class UAnimationAsset;
class USkeleton;

UCLASS(DisplayName="Use Specific Animation")
class UPersonaPreviewSceneAnimationController : public UPersonaPreviewSceneController
{
public:
	GENERATED_BODY()

	/** The preview animation to use */
	UPROPERTY(EditAnywhere, Category = "Animation", meta = (DisplayThumbnail = true))
	TSoftObjectPtr<UAnimationAsset> Animation;

	virtual void InitializeView(UPersonaPreviewSceneDescription* SceneDescription, IPersonaPreviewScene* PreviewScene) const;
	virtual void UninitializeView(UPersonaPreviewSceneDescription* SceneDescription, IPersonaPreviewScene* PreviewScene) const;

	virtual IDetailPropertyRow* AddPreviewControllerPropertyToDetails(const TSharedRef<IPersonaToolkit>& PersonaToolkit, IDetailLayoutBuilder& DetailBuilder, IDetailCategoryBuilder& Category, const FProperty* Property, const EPropertyLocation::Type PropertyLocation);

	bool HandleShouldFilterAsset(const FAssetData& InAssetData, const USkeleton* InSkeleton) const;
};
