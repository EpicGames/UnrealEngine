// Copyright Epic Games, Inc. All Rights Reserved.

#include "PersonaPreviewSceneController.h"
#include "AnimationEditorPreviewScene.h"
#include "DetailCategoryBuilder.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PersonaPreviewSceneController)

IDetailPropertyRow* UPersonaPreviewSceneController::AddPreviewControllerPropertyToDetails(const TSharedRef<IPersonaToolkit>& PersonaToolkit, IDetailLayoutBuilder& DetailBuilder, IDetailCategoryBuilder& Category, const FProperty* Property, const EPropertyLocation::Type PropertyLocation)
{
	TArray<UObject*> ListOfPreviewController{ this };
	return Category.AddExternalObjectProperty(ListOfPreviewController, Property->GetFName(), PropertyLocation);
}
