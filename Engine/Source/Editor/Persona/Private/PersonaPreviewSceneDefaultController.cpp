// Copyright Epic Games, Inc. All Rights Reserved.

#include "PersonaPreviewSceneDefaultController.h"
#include "AnimationEditorPreviewScene.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PersonaPreviewSceneDefaultController)

void UPersonaPreviewSceneDefaultController::InitializeView(UPersonaPreviewSceneDescription* SceneDescription, IPersonaPreviewScene* PreviewScene) const
{
	PreviewScene->ShowDefaultMode();
}

void UPersonaPreviewSceneDefaultController::UninitializeView(UPersonaPreviewSceneDescription* SceneDescription, IPersonaPreviewScene* PreviewScene) const
{

}
