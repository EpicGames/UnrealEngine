// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PersonaPreviewSceneController.h"
#include "PersonaPreviewSceneDefaultController.generated.h"

UCLASS(DisplayName = "Default")
class UPersonaPreviewSceneDefaultController : public UPersonaPreviewSceneController
{
public:
	GENERATED_BODY()

	virtual void InitializeView(UPersonaPreviewSceneDescription* SceneDescription, IPersonaPreviewScene* PreviewScene) const;
	virtual void UninitializeView(UPersonaPreviewSceneDescription* SceneDescription, IPersonaPreviewScene* PreviewScene) const;
};
