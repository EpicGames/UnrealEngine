// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Package.h"
#include "UObject/SoftObjectPtr.h"
#include "Templates/SubclassOf.h"
#include "PersonaPreviewSceneController.h"
#include "PersonaPreviewSceneDescription.generated.h"

class UPreviewMeshCollection;
class USkeletalMesh;
class UDataAsset;
class UPreviewMeshCollection;
class UAnimBlueprint;
enum class EPreviewAnimationBlueprintApplicationMethod : uint8;

UCLASS()
class UPersonaPreviewSceneDescription : public UObject
{
public:
	GENERATED_BODY()

	// The method by which the preview is animated
	UPROPERTY(EditAnywhere, BlueprintReadOnly, NoClear, Category = "Animation")
	class TSubclassOf<UPersonaPreviewSceneController> PreviewController;

	UPROPERTY()
	TObjectPtr<UPersonaPreviewSceneController> PreviewControllerInstance;

	UPROPERTY()
	TArray<TObjectPtr<UPersonaPreviewSceneController>> PreviewControllerInstances;

	/** The preview mesh to use */
	UPROPERTY(EditAnywhere, Category = "Mesh", meta=(DisplayThumbnail=true))
	TSoftObjectPtr<USkeletalMesh> PreviewMesh;

	/** The preview anim blueprint to use */
	UPROPERTY(EditAnywhere, Category = "Animation Blueprint", meta=(DisplayThumbnail=true))
	TSoftObjectPtr<UAnimBlueprint> PreviewAnimationBlueprint;

	/** The method by which a preview animation blueprint is applied, either as an overlay layer, or as a linked instance */
	UPROPERTY(EditAnywhere, Category = "Animation Blueprint")
	EPreviewAnimationBlueprintApplicationMethod ApplicationMethod;

	/** The tag to use when applying a preview animation blueprint via LinkAnimGraphByTag */
	UPROPERTY(EditAnywhere, Category = "Animation Blueprint")
	FName LinkedAnimGraphTag;

	UPROPERTY(EditAnywhere, Category = "Additional Meshes")
	TSoftObjectPtr<UDataAsset> AdditionalMeshes;

	UPROPERTY()
	TObjectPtr<UPreviewMeshCollection> DefaultAdditionalMeshes;

	// Sets the current preview controller for the scene (handles uninitializing and initializing controllers)
	bool SetPreviewController(UClass* PreviewControllerClass, class IPersonaPreviewScene* PreviewScene)
	{
		if (!PreviewControllerClass->HasAnyClassFlags(CLASS_Abstract) &&
			PreviewControllerClass->IsChildOf(UPersonaPreviewSceneController::StaticClass()) &&
			(!PreviewControllerInstance || PreviewControllerClass != PreviewControllerInstance->GetClass()))
		{
			PreviewController = PreviewControllerClass;
			if (PreviewControllerInstance)
			{
				PreviewControllerInstance->UninitializeView(this, PreviewScene);
			}
			PreviewControllerInstance = GetControllerForClass(PreviewControllerClass);
			PreviewControllerInstance->InitializeView(this, PreviewScene);
			return true;
		}
		return false;
	}

private:
	// Gets created controller for the requested class (or creates one if none exists)
	UPersonaPreviewSceneController* GetControllerForClass(UClass* PreviewControllerClass)
	{
		for (UPersonaPreviewSceneController* Controller : PreviewControllerInstances)
		{
			if (Controller->GetClass() == PreviewControllerClass)
			{
				return Controller;
			}
		}

		UPersonaPreviewSceneController* NewController = NewObject<UPersonaPreviewSceneController>(GetTransientPackage(), PreviewControllerClass);
		PreviewControllerInstances.Add(NewController);
		return NewController;
	}
};
