// Copyright Epic Games, Inc. All Rights Reserved.

#include "PersonaPreviewSceneRefPoseController.h"
#include "AnimationEditorPreviewScene.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PersonaPreviewSceneRefPoseController)

void UPersonaPreviewSceneRefPoseController::InitializeView(UPersonaPreviewSceneDescription* SceneDescription, IPersonaPreviewScene* PreviewScene) const
{
	PreviewScene->ShowReferencePose(true, bResetBoneTransforms);
}

void UPersonaPreviewSceneRefPoseController::UninitializeView(UPersonaPreviewSceneDescription* SceneDescription, IPersonaPreviewScene* PreviewScene) const
{

}
