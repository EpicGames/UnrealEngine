// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PersonaPreviewSceneController.h"
#include "PersonaPreviewSceneRefPoseController.generated.h"

UCLASS(DisplayName = "Reference Pose")
class UPersonaPreviewSceneRefPoseController : public UPersonaPreviewSceneController
{
public:
	GENERATED_BODY()

	UPersonaPreviewSceneRefPoseController()
		: bResetBoneTransforms(false)
	{}

	/** Whether to reset bone transforms when the refpose is set */
	UPROPERTY()
	bool bResetBoneTransforms;

	virtual void InitializeView(UPersonaPreviewSceneDescription* SceneDescription, IPersonaPreviewScene* PreviewScene) const;
	virtual void UninitializeView(UPersonaPreviewSceneDescription* SceneDescription, IPersonaPreviewScene* PreviewScene) const;
};
