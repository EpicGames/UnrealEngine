// Copyright Epic Games, Inc. All Rights Reserved.

#include "PersonaPreviewSceneSkelMeshInstanceController.h"
#include "AnimationEditorPreviewScene.h"
#include "AnimPreviewInstance.h"
#include "DetailWidgetRow.h"
#include "Editor.h"
#include "EngineUtils.h"
#include "PersonaPreviewSceneDescription.h"
#include "IPersonaToolkit.h"
#include "Selection.h"
#include "Widgets/Layout/SGridPanel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PersonaPreviewSceneSkelMeshInstanceController)

#define LOCTEXT_NAMESPACE "UPersonaPreviewSceneSkelMeshInstanceController"

AActor* FSkeletalMeshDebugInstance::GetActor() const
{
	return SkeletalMeshComponent.IsValid() ? SkeletalMeshComponent->GetOwner() : nullptr;
}

void UPersonaPreviewSceneSkelMeshInstanceController::InitializeView(
	UPersonaPreviewSceneDescription* SceneDescription,
	IPersonaPreviewScene* PreviewScene) const
{
	constexpr bool bShowReferencePose = true;
	constexpr bool bResetTransforms = true;
	PreviewScene->ShowReferencePose(bShowReferencePose, bResetTransforms);
}

void UPersonaPreviewSceneSkelMeshInstanceController::UninitializeView(
	UPersonaPreviewSceneDescription* SceneDescription,
	IPersonaPreviewScene* PreviewScene) const
{
	if (UDebugSkelMeshComponent* PreviewMeshComponent = PreviewScene->GetPreviewMeshComponent())
	{
        PreviewMeshComponent->bTrackAttachedInstanceLOD = false;
		
		if (UAnimPreviewInstance* PreviewAnimInstance = PreviewMeshComponent->PreviewInstance)
		{
			PreviewAnimInstance->SetDebugSkeletalMeshComponent(nullptr);
		}
	}

	PreviewScene->ShowDefaultMode();
}

IDetailPropertyRow* UPersonaPreviewSceneSkelMeshInstanceController::AddPreviewControllerPropertyToDetails(
	const TSharedRef<IPersonaToolkit>& PersonaToolkit,
	IDetailLayoutBuilder& DetailBuilder,
	IDetailCategoryBuilder& Category,
	const FProperty* Property,
	const EPropertyLocation::Type PropertyLocation)
{
	TSharedPtr<IPersonaPreviewScene> PreviewScenePtr = PersonaToolkit->GetPreviewScene();
	const USkeletalMesh* SkeletalMesh = PersonaToolkit->GetPreviewMeshComponent()->GetSkeletalMeshAsset();
	if (!SkeletalMesh)
	{
		return nullptr;
	}

	if (Property->GetName() != GET_MEMBER_NAME_CHECKED(UPersonaPreviewSceneSkelMeshInstanceController, ActivePreviewInstance))
	{
		return nullptr;
	}

	// create custom widget to select preview instance
	const TArray<UObject*> ListOfPreviewController{ this };
	const FName PropertyName = GET_MEMBER_NAME_CHECKED(UPersonaPreviewSceneSkelMeshInstanceController, ActivePreviewInstance);
	IDetailPropertyRow* NewRow = Category.AddExternalObjectProperty(ListOfPreviewController, PropertyName, EPropertyLocation::Common);
	NewRow->CustomWidget()
	.NameContent()
	[
		NewRow->GetPropertyHandle()->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MaxDesiredWidth(250.0f)
	.MinDesiredWidth(250.0f)
	[
		SNew(SSkeletalMeshDebugSelectionWidget).PreviewScene(PreviewScenePtr)
	];
	
	return NewRow;
}

void SSkeletalMeshDebugSelectionWidget::Construct(const FArguments& InArgs)
{
	PreviewScenePtr = InArgs._PreviewScene;
	
	ChildSlot
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(FMargin(2, 0, 0, 0))
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			SAssignNew(InstanceComboBox, SComboBox<TSharedPtr<FSkeletalMeshDebugInstance>>)
			.OptionsSource(&AllMeshInstances)
			.OnGenerateWidget(this, &SSkeletalMeshDebugSelectionWidget::OnGenerateComboBoxItemWidget)
			.OnSelectionChanged(this, &SSkeletalMeshDebugSelectionWidget::OnSelectionChanged)
			[
				SNew(STextBlock)
				.Text_Lambda([this]()
				{
					return ActiveInstance.IsValid() ? ActiveInstance->DisplayName : LOCTEXT("PreviewDisabledText", "Preview Disabled");
				})
				.ColorAndOpacity_Lambda([this]()
				{
					const bool bIsPreviewingInstance = ActiveInstance.IsValid() && ActiveInstance->SkeletalMeshComponent.IsValid();
					return bIsPreviewingInstance ? FLinearColor::Green : FSlateColor::UseForeground();
				})
			]
		]
	];

	Refresh();

	FWorldDelegates::OnPostWorldInitialization.AddSPLambda(this, [this](UWorld* World, const UWorld::InitializationValues IVS) { Refresh(); });
	FWorldDelegates::OnWorldCleanup.AddSPLambda(this, [this](UWorld* World, bool bSessionEnded, bool bCleanupResources) { Refresh(); } );
	FWorldDelegates::OnPostDuplicate.AddSPLambda(this, [this](UWorld* World, bool bDuplicateForPIE, FWorldDelegates::FReplacementMap& ReplacementMap, TArray<UObject*>& ObjectsToFixReferences) { Refresh(); });
	FWorldDelegates::OnPostWorldRename.AddSPLambda(this, [this](UWorld* World) { Refresh(); } );
	
	FWorldDelegates::OnPIEReady.AddSPLambda(this, [this](UGameInstance*) { Refresh(); });
	FEditorDelegates::PostPIEStarted.AddSPLambda(this, [this](bool bSimulating) { Refresh(); });
	FEditorDelegates::PausePIE.AddSPLambda(this, [this](bool bSimulating) { Refresh(); });
	FEditorDelegates::ResumePIE.AddSPLambda(this, [this](bool bSimulating) { Refresh(); });
	FEditorDelegates::SingleStepPIE.AddSPLambda(this, [this](bool bSimulating) { Refresh(); });
	FEditorDelegates::EndPIE.AddSPLambda(this, [this](bool bSimulating) { Refresh(); });
	FEditorDelegates::CancelPIE.AddSPLambda(this, [this]() { Refresh(); });
	FEditorDelegates::OnNewActorsPlaced.AddSPLambda(this, [this](UObject*, const TArray<AActor*>&) { Refresh(); });
	FEditorDelegates::OnDeleteActorsBegin.AddSPLambda(this, [this]() { Refresh(); });
	FEditorDelegates::OnSwitchBeginPIEAndSIE.AddSPLambda(this, [this](bool bSimulating) { Refresh(); });
	
	USelection::SelectObjectEvent.AddSPLambda(this, [this](UObject* NewSelection) { Refresh(); });
	USelection::SelectionChangedEvent.AddSPLambda(this, [this](UObject* NewSelection) { Refresh(); });
	USelection::SelectNoneEvent.AddSPLambda(this, [this]() { Refresh(); });
}

void SSkeletalMeshDebugSelectionWidget::Refresh()
{
	// empty list of instances to refresh
	AllMeshInstances.Reset();
	
	// create a default/empty item used to disable preview
	TSharedPtr<FSkeletalMeshDebugInstance> EmptyItem = MakeShared<FSkeletalMeshDebugInstance>();
	EmptyItem->DisplayName = LOCTEXT("DefaultItemText", "None");
	AllMeshInstances.Emplace(EmptyItem);
	
	const IPersonaPreviewScene* PreviewScene = PreviewScenePtr.Get();
	if (!PreviewScene)
	{
		return;
	}
	
	const USkeletalMesh* PreviewMesh = PreviewScene->GetPreviewMesh();
	if (!PreviewMesh)
	{
		return;
	}

	const UDebugSkelMeshComponent* PreviewComponent = PreviewScene->GetPreviewMeshComponent();
	if (!PreviewComponent)
	{
		return;
	}

	// get all debug worlds
	TArray<TWeakObjectPtr<UWorld>> AllDebugWorlds = []()
	{
		TArray<TWeakObjectPtr<UWorld>> AllDebugWorlds;
		for (TObjectIterator<UWorld> It; It; ++It)
		{
			UWorld* World = *It;
			// include only PIE and worlds that own the persistent level (i.e. non-streaming levels).
			const bool bIsValidDebugWorld = (World != nullptr)
				&& (World->WorldType == EWorldType::PIE || World->WorldType == EWorldType::Editor || World->WorldType == EWorldType::EditorPreview)
				&& World->PersistentLevel != nullptr
				&& World->PersistentLevel->OwningWorld == World;
			if (!bIsValidDebugWorld)
			{
				continue;
			}
			AllDebugWorlds.Add(World);
		}
		return MoveTemp(AllDebugWorlds);
	}();
	
	// spin through all available worlds and find all skeletal mesh components using the target skeletal mesh
	for (int32 WorldIndex=0; WorldIndex<AllDebugWorlds.Num(); ++WorldIndex)
	{
		UWorld* World = AllDebugWorlds[WorldIndex].Get();
		if (!World)
		{
			// double-check because we have had crashes in TActorIterator below on null worlds
			continue;
		}
		
		for (TActorIterator<AActor> ActorItr(World); ActorItr; ++ActorItr)
		{
			const AActor* Actor = *ActorItr;
			if (!Actor)
			{
				continue;
			}

			constexpr bool bIncludeChildActors = true;
			Actor->ForEachComponent<USkeletalMeshComponent>(bIncludeChildActors,
				[this, PreviewMesh, PreviewComponent](USkeletalMeshComponent* Component)
			{
				if (!Component)
				{
					return;
				}

				const bool bUsingPreviewMesh = Component->GetSkeletalMeshAsset() == PreviewMesh;
				const bool bIsNotPreviewComponent = Component != PreviewComponent;
				if (bUsingPreviewMesh && bIsNotPreviewComponent)
				{
					TSharedPtr<FSkeletalMeshDebugInstance> NewInstance = MakeShared<FSkeletalMeshDebugInstance>();
					NewInstance->SkeletalMeshComponent = Component;
					AActor* Actor = NewInstance->GetActor();
					NewInstance->DisplayName = IsValid(Actor) ? FText::FromString(Actor->GetActorNameOrLabel()) : FText();
					AllMeshInstances.Emplace(NewInstance);
				}
			});
		}
	}
	
	// update bIsSelected for each debug instance based on if the actor is selected in the level editor
	{
		// default to NOT selected
		for (TSharedPtr<FSkeletalMeshDebugInstance>& MeshInstance : AllMeshInstances)
		{
			if (MeshInstance.IsValid())
			{
				MeshInstance->bIsSelected = false;
			}
		}

		// get the selected actors in the editor.
		const USelection* ActiveDebugActors = GEditor->GetSelectedActors();
		if (ActiveDebugActors == nullptr)
		{
			return;
		}
	
		// processed in reverse order, as we want the last selected item to be the one we pick.
		// there can only be one actor selected to preview, while many can be selected in the editor itself.
		for (int32 Index = ActiveDebugActors->Num() - 1; Index >= 0; --Index)
		{
			const AActor* Actor = Cast<AActor>(ActiveDebugActors->GetSelectedObject(Index));
			if (Actor == nullptr)
			{
				continue;
			}

			// is this an actor with a preview mesh?
			const TSharedPtr<FSkeletalMeshDebugInstance>* SelectedMeshInstance = AllMeshInstances.FindByPredicate
			([Actor](const TSharedPtr<FSkeletalMeshDebugInstance>& MeshInstance)
			{
				if (!MeshInstance.IsValid())
				{
					return false;
				}
			
				return MeshInstance->GetActor() == Actor;
			});

			// found a selected preview mesh?
			if (SelectedMeshInstance && SelectedMeshInstance->IsValid())
			{
				// mark it as selected and break out (only one/last selection considered for debug preview)
				SelectedMeshInstance->Get()->bIsSelected = true;
				break;
			}
		}
	} // END update selection state
	
	// restore active running instance if there is one AND it's still in the list of available instances
	TSharedPtr<FSkeletalMeshDebugInstance> ItemToActivate = EmptyItem; // default to empty item
	for (const TSharedPtr<FSkeletalMeshDebugInstance>& Instance : AllMeshInstances)
	{
		// search for actor with same name as was previously selected
		if (Instance->DisplayName.EqualTo(NameOfLastSelectedInstance, ETextComparisonLevel::Default))
		{
			ItemToActivate = Instance;
			break;
		}

		if (PreviewComponent->PreviewInstance && PreviewComponent->PreviewInstance->GetDebugSkeletalMeshComponent() == Instance->SkeletalMeshComponent)
		{
			ItemToActivate = Instance;
			break;
		}
	}

	// assign selection to combobox
	InstanceComboBox.Get()->SetSelectedItem(ItemToActivate);
}

TSharedRef<SWidget> SSkeletalMeshDebugSelectionWidget::OnGenerateComboBoxItemWidget(TSharedPtr<FSkeletalMeshDebugInstance> Item)
{
	// If we have the first item in the actor list, generate a special widget.
	const TWeakObjectPtr<USkeletalMeshComponent> Component = Item.IsValid() ? Item->SkeletalMeshComponent : nullptr;
	const AActor* Actor = Component.IsValid() ? Component->GetOwner() : nullptr;
	FText ActorName = Actor ? FText::FromString(Actor->GetActorNameOrLabel()) : LOCTEXT("DestroyedActorText", "<Destroyed>");

	const UWorld* World = Actor ? Actor->GetWorld() : nullptr;
	FText WorldName = World ? FText::FromString(GetDebugStringForWorld(World)) : LOCTEXT("DestroyedWorldText", "<Destroyed>");

	constexpr FLinearColor ActiveColor = FLinearColor(0.0f, 1.0f, 0.0f);
	constexpr FLinearColor EditorSelectedMarkColor = FLinearColor(1.0f, 0.5f, 0.0f);

	// special item to disable debugging.
	if (Actor == nullptr)
	{
		ActorName = LOCTEXT("DebugDisabledActorText", "None");
		WorldName = LOCTEXT("DebugDisabledWorldText", "Editor Preview World");
	}
	
	const TSharedPtr<SWidget> ItemWidget = 
	SNew(SGridPanel)
	+SGridPanel::Slot(0, 0)
	.Padding(2.0f)
	.HAlign(EHorizontalAlignment::HAlign_Right)
	[
		SNew(STextBlock)
		.Text(LOCTEXT("ActorName", "Actor:"))
		.ColorAndOpacity(FSlateColor::UseSubduedForeground())
	]
	+SGridPanel::Slot(1, 0)
	.Padding(2.0f)
	.HAlign(EHorizontalAlignment::HAlign_Left)
	[
		SNew(STextBlock)
		.Text(ActorName)
		.Font_Lambda([Item]()
		{
			if (Item.IsValid() && Item->bIsSelected)
			{
				return FAppStyle::GetFontStyle(TEXT("NormalFontBold"));
			}
			return FAppStyle::GetFontStyle(TEXT("NormalFont"));
		})
		.ColorAndOpacity_Lambda([this, Item, ActiveColor]() 
		{ 
			const AActor* ItemActor = Item.IsValid() ? Item->GetActor() : nullptr;
			return (ActiveInstance.IsValid() && ActiveInstance->GetActor() == ItemActor) ? ActiveColor : FSlateColor::UseForeground();
		})
	]
	+SGridPanel::Slot(2, 0)
	.Padding(2.0f)
	.HAlign(EHorizontalAlignment::HAlign_Left)
	[
		SNew(STextBlock)
		.Text_Lambda([Item]()
		{
			if (Item.IsValid() && Item->bIsSelected)
			{
				return LOCTEXT("SelectedText", "(Selected)");
			}
			return FText();
		})
		.TextStyle(FAppStyle::Get(), "RichTextBlock.Bold")
		.ColorAndOpacity(EditorSelectedMarkColor)
	]
	+SGridPanel::Slot(0, 1)
	.Padding(2.0f)
	.HAlign(EHorizontalAlignment::HAlign_Right)
	[
		SNew(STextBlock)
		.Text(LOCTEXT("WorldName", "World:"))
		.ColorAndOpacity(FSlateColor::UseSubduedForeground())
	]
	+SGridPanel::Slot(1, 1)
	.Padding(2.0f)
	.HAlign(EHorizontalAlignment::HAlign_Left)
	[
		SNew(STextBlock)
		.Text(WorldName)
		.ColorAndOpacity_Lambda([this, Item, ActiveColor]() 
		{ 
			const TObjectPtr<AActor> ItemActor = Item.IsValid() ? Item->GetActor() : nullptr;
			return (ActiveInstance.IsValid() && ActiveInstance->GetActor() == ItemActor) ? ActiveColor : FSlateColor::UseForeground();
		})
	];

	return ItemWidget.ToSharedRef();
}

void SSkeletalMeshDebugSelectionWidget::OnSelectionChanged(TSharedPtr<FSkeletalMeshDebugInstance> Item, ESelectInfo::Type SelectInfo)
{
	ActiveInstance = Item;
	if (SelectInfo != ESelectInfo::Type::Direct)
	{
		NameOfLastSelectedInstance = ActiveInstance->DisplayName;	
	}
			
	const IPersonaPreviewScene* PreviewScene = PreviewScenePtr.Get();
	if (!PreviewScene)
	{
		return;
	}

	UDebugSkelMeshComponent* PreviewMeshComponent = PreviewScene->GetPreviewMeshComponent();
	if (!PreviewMeshComponent)
	{
		return;
	}

	UAnimPreviewInstance* PreviewAnimInstance = PreviewMeshComponent->PreviewInstance;
	if (!PreviewAnimInstance)
	{
		return;
	}

	// reset to show no preview
	if (!Item->SkeletalMeshComponent.IsValid())
	{
		PreviewAnimInstance->SetDebugSkeletalMeshComponent(nullptr);
		PreviewMeshComponent->bTrackAttachedInstanceLOD = false;
		return;
	}

	// assign the preview mesh to the debug skel mesh component
	PreviewAnimInstance->SetDebugSkeletalMeshComponent(Item->SkeletalMeshComponent.Get());
	PreviewMeshComponent->bTrackAttachedInstanceLOD = true;
}

#undef LOCTEXT_NAMESPACE
