// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PersonaPreviewSceneController.h"
#include "Widgets/Input/SComboBox.h"
#include "PersonaPreviewSceneSkelMeshInstanceController.generated.h"

struct FSkeletalMeshDebugInstance
{
	FSkeletalMeshDebugInstance() = default;
	AActor* GetActor() const;
	
	// the component inside the PIE viewport, or nullptr when debugging is disabled. */
	TWeakObjectPtr<USkeletalMeshComponent> SkeletalMeshComponent = nullptr;
	// true if the actor is selected in the level editor (used to color UI to help user find intended instance)
	bool bIsSelected = false;
	// the display name
	FText DisplayName;
};

class SSkeletalMeshDebugSelectionWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SSkeletalMeshDebugSelectionWidget) {}
	SLATE_ARGUMENT(TSharedPtr<IPersonaPreviewScene>, PreviewScene)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:

	// gathers all skel mesh instances in all UWorlds and refresh the UI options
	void Refresh();

	// UI
	TSharedRef<SWidget> OnGenerateComboBoxItemWidget(TSharedPtr<FSkeletalMeshDebugInstance> Item);
	IDetailPropertyRow* GenerateInstancesCombobox(IDetailCategoryBuilder& Category);
	void OnSelectionChanged(TSharedPtr<FSkeletalMeshDebugInstance> Item, ESelectInfo::Type SelectInfo);

	// store the preview scene for callbacks to use
	TSharedPtr<IPersonaPreviewScene> PreviewScenePtr;
	// the combobox that contains the instance names
	TSharedPtr<SComboBox<TSharedPtr<FSkeletalMeshDebugInstance>>> InstanceComboBox;
	// all the instances of skeletal mesh components using the target skeletal mesh
	TArray<TSharedPtr<FSkeletalMeshDebugInstance>> AllMeshInstances;
	// the currently active component to copy the pose from 
	TSharedPtr<FSkeletalMeshDebugInstance> ActiveInstance;
	// the display name of the last instance selected by the user, used to restore their selection between PIE sessions
	FText NameOfLastSelectedInstance;
};

UCLASS(DisplayName = "Running Instance")
class UPersonaPreviewSceneSkelMeshInstanceController : public UPersonaPreviewSceneController
{
public:
	GENERATED_BODY()

	UPersonaPreviewSceneSkelMeshInstanceController() : ActivePreviewInstance(nullptr) {}

	/** The instance to preview. */
	UPROPERTY(EditAnywhere, Category = "Animation")
	TWeakObjectPtr<USkeletalMeshComponent> ActivePreviewInstance;

	// BEGIN UPersonaPreviewSceneController interface
	virtual void InitializeView(UPersonaPreviewSceneDescription* SceneDescription, IPersonaPreviewScene* PreviewScene) const override;
	virtual void UninitializeView(UPersonaPreviewSceneDescription* SceneDescription, IPersonaPreviewScene* PreviewScene) const override;
	virtual IDetailPropertyRow* AddPreviewControllerPropertyToDetails(
		const TSharedRef<IPersonaToolkit>& PersonaToolkit,
		IDetailLayoutBuilder& DetailBuilder,
		IDetailCategoryBuilder& Category,
		const FProperty* Property,
		const EPropertyLocation::Type PropertyLocation) override;
	// END UPersonaPreviewSceneController interface
};
