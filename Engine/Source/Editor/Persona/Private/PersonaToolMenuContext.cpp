// Copyright Epic Games, Inc. All Rights Reserved.

#include "PersonaToolMenuContext.h"
#include "IPersonaToolkit.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PersonaToolMenuContext)

USkeleton* UPersonaToolMenuContext::GetSkeleton() const
{
	if (HasValidToolkit())
	{
		return WeakToolkit.Pin()->GetSkeleton();
	}

	return nullptr;
}

UDebugSkelMeshComponent* UPersonaToolMenuContext::GetPreviewMeshComponent() const
{
	if (HasValidToolkit())
	{
		return WeakToolkit.Pin()->GetPreviewMeshComponent();
	}

	return nullptr;
}

USkeletalMesh* UPersonaToolMenuContext::GetMesh() const
{
	if (HasValidToolkit())
	{
		return WeakToolkit.Pin()->GetMesh();
	}

	return nullptr;
}

UAnimBlueprint* UPersonaToolMenuContext::GetAnimBlueprint() const
{
	if (HasValidToolkit())
	{
		return WeakToolkit.Pin()->GetAnimBlueprint();
	}

	return nullptr;
}

UAnimationAsset* UPersonaToolMenuContext::GetAnimationAsset() const
{
	if (HasValidToolkit())
	{
		return WeakToolkit.Pin()->GetAnimationAsset();
	}

	return nullptr;
}

void UPersonaToolMenuContext::SetToolkit(TSharedRef<IPersonaToolkit> InToolkit)
{
	WeakToolkit = InToolkit;
}

bool UPersonaToolMenuContext::HasValidToolkit() const
{
	return WeakToolkit.IsValid();
}

