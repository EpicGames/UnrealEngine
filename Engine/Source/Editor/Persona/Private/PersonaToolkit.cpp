// Copyright Epic Games, Inc. All Rights Reserved.

#include "PersonaToolkit.h"
#include "Modules/ModuleManager.h"
#include "Engine/SkeletalMesh.h"
#include "Animation/AnimationAsset.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "AnimationEditorPreviewScene.h"
#include "ISkeletonEditorModule.h"
#include "Animation/AnimBlueprint.h"
#include "GameFramework/WorldSettings.h"
#include "ScopedTransaction.h"
#include "PersonaModule.h"
#include "PersonaAssetFamily.h"
#include "Interfaces/Interface_PreviewMeshProvider.h"
#include "AnimationEditorPreviewActor.h"
#include "Subsystems/AssetEditorSubsystem.h"

FPersonaToolkit::FPersonaToolkit()
	: Skeleton(nullptr)
	, Mesh(nullptr)
	, AnimBlueprint(nullptr)
	, AnimationAsset(nullptr)
	, PhysicsAsset(nullptr)
	, Asset(nullptr)
	, InitialAssetClass(nullptr)
{
}

FPersonaToolkit::~FPersonaToolkit()
{
	PreviewScene.Reset();
}

void FPersonaToolkit::Initialize(UObject* InAsset, const FPersonaToolkitArgs& PersonaToolkitArgs, USkeleton* InSkeleton)
{
	Asset = InAsset;
	InitialAssetClass = Asset->GetClass();

	if(IInterface_PreviewMeshProvider* PreviewMeshInterface = Cast<IInterface_PreviewMeshProvider>(Asset))
	{
		Mesh = PreviewMeshInterface->GetPreviewMesh();
	}

	Skeleton = InSkeleton;

	CommonInitialSetup(PersonaToolkitArgs);
}

void FPersonaToolkit::Initialize(USkeleton* InSkeleton, const FPersonaToolkitArgs& PersonaToolkitArgs)
{
	check(InSkeleton);
	Skeleton = InSkeleton;
	InitialAssetClass = USkeleton::StaticClass();

	FPersonaAssetFamily AssetFamily(InSkeleton);
	Mesh = const_cast<USkeletalMesh*>(AssetFamily.GetAssetOfType<USkeletalMesh>().Get());

	CommonInitialSetup(PersonaToolkitArgs);
}

void FPersonaToolkit::Initialize(UAnimationAsset* InAnimationAsset, const FPersonaToolkitArgs& PersonaToolkitArgs)
{
	check(InAnimationAsset);
	AnimationAsset = InAnimationAsset;
	InitialAssetClass = UAnimationAsset::StaticClass();

	FPersonaAssetFamily AssetFamily(InAnimationAsset);
	Skeleton = const_cast<USkeleton*>(AssetFamily.GetAssetOfType<USkeleton>().Get());
	Mesh = const_cast<USkeletalMesh*>(AssetFamily.GetAssetOfType<USkeletalMesh>().Get());

	CommonInitialSetup(PersonaToolkitArgs);

	if (AnimationAsset != nullptr)
	{
		PreviewScene->SetPreviewAnimationAsset(AnimationAsset.Get());
	}
}

void FPersonaToolkit::Initialize(USkeletalMesh* InSkeletalMesh, const FPersonaToolkitArgs& PersonaToolkitArgs)
{
	check(InSkeletalMesh);
	Mesh = InSkeletalMesh;
	InitialAssetClass = USkeletalMesh::StaticClass();

	FPersonaAssetFamily AssetFamily(InSkeletalMesh);
	Skeleton = const_cast<USkeleton*>(AssetFamily.GetAssetOfType<USkeleton>().Get());

	CommonInitialSetup(PersonaToolkitArgs);
}

void FPersonaToolkit::Initialize(UAnimBlueprint* InAnimBlueprint, const FPersonaToolkitArgs& PersonaToolkitArgs)
{
	check(InAnimBlueprint);
	AnimBlueprint = InAnimBlueprint;
	InitialAssetClass = UAnimBlueprint::StaticClass();

	FPersonaAssetFamily AssetFamily(InAnimBlueprint);
	Skeleton = const_cast<USkeleton*>(AssetFamily.GetAssetOfType<USkeleton>().Get());
	Mesh = const_cast<USkeletalMesh*>(AssetFamily.GetAssetOfType<USkeletalMesh>().Get());

	CommonInitialSetup(PersonaToolkitArgs);

	if (InAnimBlueprint->bIsTemplate)
	{
		bPreviewMeshCanUseDifferentSkeleton = true;
	}
}

void FPersonaToolkit::Initialize(UPhysicsAsset* InPhysicsAsset, const FPersonaToolkitArgs& PersonaToolkitArgs)
{
	check(InPhysicsAsset);
	PhysicsAsset = InPhysicsAsset;
	InitialAssetClass = UPhysicsAsset::StaticClass();

	FPersonaAssetFamily AssetFamily(InPhysicsAsset);
	Skeleton = const_cast<USkeleton*>(AssetFamily.GetAssetOfType<USkeleton>().Get());
	Mesh = const_cast<USkeletalMesh*>(AssetFamily.GetAssetOfType<USkeletalMesh>().Get());

	CommonInitialSetup(PersonaToolkitArgs);
}

void FPersonaToolkit::CommonInitialSetup(const FPersonaToolkitArgs& PersonaToolkitArgs)
{
	if (PersonaToolkitArgs.bCreatePreviewScene)
	{
		CreatePreviewScene(PersonaToolkitArgs);
	}

	OnPreviewSceneSettingsCustomized = PersonaToolkitArgs.OnPreviewSceneSettingsCustomized;
	bPreviewMeshCanUseDifferentSkeleton = PersonaToolkitArgs.bPreviewMeshCanUseDifferentSkeleton;
}

void FPersonaToolkit::CreatePreviewScene(const FPersonaToolkitArgs& PersonaToolkitArgs)
{
	if (!PreviewScene.IsValid())
	{
		if (!EditableSkeleton.IsValid() && Skeleton.IsValid())
		{
			ISkeletonEditorModule& SkeletonEditorModule = FModuleManager::LoadModuleChecked<ISkeletonEditorModule>("SkeletonEditor");
			EditableSkeleton = SkeletonEditorModule.CreateEditableSkeleton(Skeleton.Get());
		}

		PreviewScene = MakeShareable(new FAnimationEditorPreviewScene(FPreviewScene::ConstructionValues().AllowAudioPlayback(true).ShouldSimulatePhysics(true), EditableSkeleton, AsShared()));

		PreviewScene->SetIsBeingConstructed(true);
		ON_SCOPE_EXIT
		{
			PreviewScene->SetIsBeingConstructed(false);
		};

		//Temporary fix for missing attached assets - MDW
		PreviewScene->GetWorld()->GetWorldSettings()->SetIsTemporarilyHiddenInEditor(false);

		if (PersonaToolkitArgs.OnPreviewSceneCreated.IsBound())
		{
			// Custom per-instance scene setup
			PersonaToolkitArgs.OnPreviewSceneCreated.Execute(PreviewScene.ToSharedRef());
		}
		else
		{
			// setup default scene
			AAnimationEditorPreviewActor* Actor = PreviewScene->GetWorld()->SpawnActor<AAnimationEditorPreviewActor>(AAnimationEditorPreviewActor::StaticClass(), FTransform::Identity);
			PreviewScene->SetActor(Actor);

			// Create the preview component
			UDebugSkelMeshComponent* SkeletalMeshComponent = NewObject<UDebugSkelMeshComponent>(Actor);
			if (GEditor->PreviewPlatform.GetEffectivePreviewFeatureLevel() <= ERHIFeatureLevel::ES3_1)
			{
				SkeletalMeshComponent->SetMobility(EComponentMobility::Static);
			}
			PreviewScene->AddComponent(SkeletalMeshComponent, FTransform::Identity);
			PreviewScene->SetPreviewMeshComponent(SkeletalMeshComponent);

			// set root component, so we can attach to it. 
			Actor->SetRootComponent(SkeletalMeshComponent);
		}

		// allow external systems to add components or otherwise manipulate the scene
		FPersonaModule& PersonaModule = FModuleManager::GetModuleChecked<FPersonaModule>(TEXT("Persona"));
		PersonaModule.OnPreviewSceneCreated().Broadcast(PreviewScene.ToSharedRef());
		
		// if not mesh editor, we allow it to override mesh
		const bool bAllowOverrideMesh = GetContext() != USkeletalMesh::StaticClass()->GetFName();
		// Force validation of preview attached assets (catch case of never doing it if we dont have a valid preview mesh)
		PreviewScene->ValidatePreviewAttachedAssets(nullptr);
		PreviewScene->RefreshAdditionalMeshes(false);
		PreviewScene->SetAllowAdditionalMeshes(bAllowOverrideMesh);

		bool bSetMesh = false;
		// Set the mesh
		if (Mesh.IsValid())
		{
			PreviewScene->SetPreviewMesh(Mesh.Get(), bAllowOverrideMesh);
			bSetMesh = true;
			
		}

		if (!bSetMesh && Skeleton.IsValid())
		{
			//If no preview mesh set, just find the first mesh that uses this skeleton
			USkeletalMesh* PreviewMesh = Skeleton->FindCompatibleMesh();
			if (PreviewMesh)
			{
				PreviewScene->SetPreviewMesh(PreviewMesh);
				if(EditableSkeleton.IsValid())
				{
					EditableSkeleton->SetPreviewMesh(PreviewMesh);
				}
			}
		}
	}
}

USkeleton* FPersonaToolkit::GetSkeleton() const
{
	return Skeleton.Get();
}

TSharedPtr<class IEditableSkeleton> FPersonaToolkit::GetEditableSkeleton() const
{
	return EditableSkeleton;
}

UDebugSkelMeshComponent* FPersonaToolkit::GetPreviewMeshComponent() const
{
	return PreviewScene.IsValid() ? PreviewScene->GetPreviewMeshComponent() : nullptr;
}

USkeletalMesh* FPersonaToolkit::GetMesh() const
{
	return Mesh.Get();
}

void FPersonaToolkit::SetMesh(class USkeletalMesh* InSkeletalMesh)
{
	if (InSkeletalMesh != nullptr && Skeleton.IsValid())
	{
		check(InSkeletalMesh->GetSkeleton() == Skeleton);
	}

	Mesh = InSkeletalMesh;
}

UAnimBlueprint* FPersonaToolkit::GetAnimBlueprint() const
{
	return AnimBlueprint.Get();
}

UAnimationAsset* FPersonaToolkit::GetAnimationAsset() const
{
	return AnimationAsset.Get();
}

void FPersonaToolkit::SetAnimationAsset(class UAnimationAsset* InAnimationAsset)
{
	USkeleton* PreviousAnimSkeleton = (AnimationAsset != nullptr) ? AnimationAsset->GetSkeleton() : nullptr;

	if (InAnimationAsset != nullptr)
	{
		ensure(Skeleton->IsCompatibleForEditor(InAnimationAsset->GetSkeleton()));
	}

	AnimationAsset = InAnimationAsset;
	
	if(AnimationAsset.IsValid())
	{
		check(InitialAssetClass == UAnimationAsset::StaticClass());

		USkeletalMesh* NewPreviewMesh = GetPreviewMesh();
		const USkeleton* CurrentAnimSkeleton = AnimationAsset->GetSkeleton();

		if (NewPreviewMesh == nullptr && PreviousAnimSkeleton != CurrentAnimSkeleton)
		{
			NewPreviewMesh = CurrentAnimSkeleton->GetPreviewMesh();
		}

		if (NewPreviewMesh)
		{
			GetPreviewScene()->SetPreviewMesh(NewPreviewMesh, false);
		}
	}
}

TSharedRef<IPersonaPreviewScene> FPersonaToolkit::GetPreviewScene() const
{
	return PreviewScene.ToSharedRef();
}

USkeletalMesh* FPersonaToolkit::GetPreviewMesh() const
{
	if (InitialAssetClass == UAnimationAsset::StaticClass())
	{
		check(AnimationAsset.IsValid());
		return AnimationAsset->GetPreviewMesh();
	}
	else if (InitialAssetClass == UAnimBlueprint::StaticClass())
	{
		check(AnimBlueprint.IsValid());
		return AnimBlueprint->GetPreviewMesh();
	}
	else if (InitialAssetClass == UPhysicsAsset::StaticClass())
	{
		check(PhysicsAsset.IsValid());
		return PhysicsAsset->GetPreviewMesh();
	}
	else if(InitialAssetClass == USkeletalMesh::StaticClass())
	{
		check(Mesh.IsValid());
		return Mesh.Get();
	}
	else if(InitialAssetClass == USkeleton::StaticClass())
	{
		check(Skeleton.IsValid());
		return Skeleton->GetPreviewMesh();
	}
	else if(IInterface_PreviewMeshProvider* PreviewMeshInterface = Cast<IInterface_PreviewMeshProvider>(Asset))
	{
		return PreviewMeshInterface->GetPreviewMesh();
	}

	return nullptr;
}

void FPersonaToolkit::SetPreviewMesh(class USkeletalMesh* InSkeletalMesh, bool bSetPreviewMeshInAsset)
{
	// Cant set preview mesh on a skeletal mesh (makes for a confusing experience!)
	if (InitialAssetClass != USkeletalMesh::StaticClass())
	{
		// If the skeleton itself is changing, then we need to re-open the asset editor
		bool bReOpenEditor = false;
		if(InSkeletalMesh != nullptr && EditableSkeleton.IsValid() && InSkeletalMesh->GetSkeleton() != &EditableSkeleton->GetSkeleton())
		{
			bReOpenEditor = true;
			bSetPreviewMeshInAsset = true;
		}

		if(bSetPreviewMeshInAsset)
		{
			if (InitialAssetClass == UAnimationAsset::StaticClass())
			{
				FScopedTransaction Transaction(NSLOCTEXT("PersonaToolkit", "SetAnimationPreviewMesh", "Set Animation Preview Mesh"));

				check(AnimationAsset.IsValid());
				AnimationAsset->SetPreviewMesh(InSkeletalMesh);
			}
			else if (InitialAssetClass == UAnimBlueprint::StaticClass())
			{
				FScopedTransaction Transaction(NSLOCTEXT("PersonaToolkit", "SetAnimBlueprintPreviewMesh", "Set Animation Blueprint Preview Mesh"));

				check(AnimBlueprint.IsValid());
				AnimBlueprint->SetPreviewMesh(InSkeletalMesh);
			}
			else if (InitialAssetClass == UPhysicsAsset::StaticClass())
			{
				FScopedTransaction Transaction(NSLOCTEXT("PersonaToolkit", "SetPhysicsAssetPreviewMesh", "Set Physics Asset Preview Mesh"));

				check(PhysicsAsset.IsValid());
				PhysicsAsset->SetPreviewMesh(InSkeletalMesh);
			}
			else if(IInterface_PreviewMeshProvider* PreviewMeshInterface = Cast<IInterface_PreviewMeshProvider>(Asset))
			{
				PreviewMeshInterface->SetPreviewMesh(InSkeletalMesh);
			}
			else if(EditableSkeleton.IsValid())
			{
				EditableSkeleton->SetPreviewMesh(InSkeletalMesh);
			}
		}

		if(bReOpenEditor)
		{
			UObject* AssetToReopen = nullptr;
			if (InitialAssetClass == UAnimationAsset::StaticClass())
			{
				AssetToReopen = AnimationAsset.Get();
			}
			else if (InitialAssetClass == UAnimBlueprint::StaticClass())
			{
				AssetToReopen = AnimBlueprint.Get();
			}
			else if (InitialAssetClass == UPhysicsAsset::StaticClass())
			{
				AssetToReopen = PhysicsAsset.Get();
			}
			else if (InitialAssetClass == USkeleton::StaticClass())
			{
				AssetToReopen = Skeleton.Get();
			}
			else if(IInterface_PreviewMeshProvider* PreviewMeshInterface = Cast<IInterface_PreviewMeshProvider>(Asset))
			{
				AssetToReopen = Asset.Get();
			}

			check(AssetToReopen);

			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->CloseAllEditorsForAsset(AssetToReopen);
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(AssetToReopen);
			return;
		}
	}

	// if it's here, it allows to replace 
	GetPreviewScene()->SetPreviewMesh(InSkeletalMesh, false);
}

void FPersonaToolkit::SetPreviewAnimationBlueprint(UAnimBlueprint* InAnimBlueprint)
{
	// Only allowed for anim blueprints
	if (InitialAssetClass == UAnimBlueprint::StaticClass())
	{
		FScopedTransaction Transaction(NSLOCTEXT("PersonaToolkit", "SetAnimBlueprintPreviewBlueprint", "Set Animation Blueprint Preview Blueprint"));

		check(AnimBlueprint.IsValid());
		AnimBlueprint->SetPreviewAnimationBlueprint(InAnimBlueprint);

		// Note setting the 'edited' blueprint as an overlay here
		GetPreviewScene()->SetPreviewAnimationBlueprint(InAnimBlueprint, AnimBlueprint.Get());
	}
}

UAnimBlueprint* FPersonaToolkit::GetPreviewAnimationBlueprint() const
{
	if (InitialAssetClass == UAnimBlueprint::StaticClass())
	{
		check(AnimBlueprint.IsValid());
		return AnimBlueprint->GetPreviewAnimationBlueprint();
	}

	return nullptr;
}

int32 FPersonaToolkit::GetCustomData(const int32 Key) const
{
	if (!CustomEditorData.Contains(Key))
	{
		return INDEX_NONE;
	}
	return CustomEditorData[Key];
}

void FPersonaToolkit::SetCustomData(const int32 Key, const int32 CustomData)
{
	CustomEditorData.FindOrAdd(Key) = CustomData;
}

void FPersonaToolkit::CustomizeSceneSettings(IDetailLayoutBuilder& DetailBuilder)
{
	OnPreviewSceneSettingsCustomized.ExecuteIfBound(DetailBuilder);
}

FName FPersonaToolkit::GetContext() const
{
	if (InitialAssetClass != nullptr)
	{
		return InitialAssetClass->GetFName();
	}

	return NAME_None;
}


bool FPersonaToolkit::CanPreviewMeshUseDifferentSkeleton() const
{
	return bPreviewMeshCanUseDifferentSkeleton;
}
