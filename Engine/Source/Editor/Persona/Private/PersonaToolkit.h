// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IPersonaPreviewScene.h"
#include "IEditableSkeleton.h"
#include "IPersonaToolkit.h"
#include "PersonaModule.h"

class FAnimationEditorPreviewScene;
class UAnimationAsset;
class UAnimBlueprint;
class USkeletalMesh;
class UPhysicsAsset;
struct FPersonaToolkitArgs;

class FPersonaToolkit : public IPersonaToolkit, public TSharedFromThis<FPersonaToolkit>
{
public:
	FPersonaToolkit();
	virtual ~FPersonaToolkit();

	/** Initialize from various sources */
	void Initialize(UObject* InAsset, const FPersonaToolkitArgs& PersonaToolkitArgs, USkeleton* InSkeleton = nullptr);
	void Initialize(USkeleton* InSkeleton, const FPersonaToolkitArgs& PersonaToolkitArgs);
	void Initialize(UAnimationAsset* InAnimationAsset, const FPersonaToolkitArgs& PersonaToolkitArgs);
	void Initialize(USkeletalMesh* InSkeletalMesh, const FPersonaToolkitArgs& PersonaToolkitArgs);
	void Initialize(UAnimBlueprint* InAnimBlueprint, const FPersonaToolkitArgs& PersonaToolkitArgs);
	void Initialize(UPhysicsAsset* InPhysicsAsset, const FPersonaToolkitArgs& PersonaToolkitArgs);

	/** Optionally create a preview scene - note: creates an editable skeleton */
	void CreatePreviewScene(const FPersonaToolkitArgs& PersonaToolkitArgs);

	/** Store the options for the Preview Scene Settings details panel layout. */
	void SetPreviewSceneDetailsOptions(const FPersonaPreviewSceneDetailsOptions& Options);

	/** IPersonaToolkit interface */
	virtual class USkeleton* GetSkeleton() const override;
	virtual TSharedPtr<class IEditableSkeleton> GetEditableSkeleton() const override;
	virtual class UDebugSkelMeshComponent* GetPreviewMeshComponent() const override;
	virtual class USkeletalMesh* GetMesh() const override;
	virtual void SetMesh(class USkeletalMesh* InSkeletalMesh) override;
	virtual class UAnimBlueprint* GetAnimBlueprint() const override;
	virtual class UAnimationAsset* GetAnimationAsset() const override;
	virtual void SetAnimationAsset(class UAnimationAsset* InAnimationAsset) override;
	virtual class TSharedRef<IPersonaPreviewScene> GetPreviewScene() const override;
	virtual class USkeletalMesh* GetPreviewMesh() const override;
	virtual void SetPreviewMesh(class USkeletalMesh* InSkeletalMesh, bool bSetPreviewMeshInAsset = true) override;
	virtual void SetPreviewAnimationBlueprint(UAnimBlueprint* InAnimBlueprint) override;
	virtual UAnimBlueprint* GetPreviewAnimationBlueprint() const override;
	virtual int32 GetCustomData(const int32 Key) const override;
	virtual void SetCustomData(const int32 Key, const int32 CustomData) override;
	virtual void CustomizeSceneSettings(IDetailLayoutBuilder& DetailBuilder) override;
	virtual FName GetContext() const override;
	virtual bool CanPreviewMeshUseDifferentSkeleton() const override;

private:
	/** Common initialization */
	void CommonInitialSetup(const FPersonaToolkitArgs& PersonaToolkitArgs);
	
	/** The skeleton we are editing */
	TWeakObjectPtr<USkeleton> Skeleton;

	/** Editable skeleton */
	TSharedPtr<IEditableSkeleton> EditableSkeleton;

	/** The mesh we are editing */
	TWeakObjectPtr <USkeletalMesh> Mesh;

	/** The anim blueprint we are editing */
	TWeakObjectPtr<UAnimBlueprint> AnimBlueprint;

	/** the animation asset we are editing */
	TWeakObjectPtr<UAnimationAsset> AnimationAsset;

	/** the physics asset we are editing */
	TWeakObjectPtr<UPhysicsAsset> PhysicsAsset;

	/** The generic asset we are editing */
	TWeakObjectPtr<UObject> Asset;

	/** Allow custom data for this editor */
	TMap<int32, int32> CustomEditorData;

	/** Preview scene for the editor */
	TSharedPtr<FAnimationEditorPreviewScene> PreviewScene;

	/* Callback to customize details tab of Preview Scene Settings */
	FOnPreviewSceneSettingsCustomized::FDelegate OnPreviewSceneSettingsCustomized;

	/** The class of the initial asset we were created with */
	UClass* InitialAssetClass;

	/** A flag to indicate whether the preview mesh's skeleton can be incompatible with the editing skeleton */
	bool bPreviewMeshCanUseDifferentSkeleton = false;
};
