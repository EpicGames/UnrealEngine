// Copyright Epic Games, Inc. All Rights Reserved.

#include "PersonaUtils.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UnrealType.h"
#include "Engine/Blueprint.h"
#include "Editor.h"
#include "ComponentAssetBroker.h"
#include "Animation/AnimInstance.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Animation/AnimNodeBase.h"
#include "Animation/AnimBlueprint.h"
#include "Widgets/Text/STextBlock.h"
#include "Styling/CoreStyle.h"
#include "Widgets/Input/SComboButton.h"
#include "Styling/AppStyle.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Images/SImage.h"
#include "Animation/AnimNotifies/AnimNotify.h"
#include "Animation/AnimNotifies/AnimNotifyState.h"
#include "ClassViewerFilter.h"
#include "Modules/ModuleManager.h"

namespace PersonaUtils
{

USceneComponent* GetComponentForAttachedObject(USceneComponent* PreviewComponent, UObject* Object, const FName& AttachedTo)
{
	if (PreviewComponent)
	{
		for (USceneComponent* ChildComponent : PreviewComponent->GetAttachChildren())
		{
			UObject* Asset = FComponentAssetBrokerage::GetAssetFromComponent(ChildComponent);

			if (Asset == Object && ChildComponent->GetAttachSocketName() == AttachedTo)
			{
				return ChildComponent;
			}
		}
	}
	return nullptr;
}

int32 CopyPropertiesToCDO(UAnimInstance* InAnimInstance, const FCopyOptions& Options)
{
	check(InAnimInstance != nullptr);

	UAnimInstance* SourceInstance = InAnimInstance;
	UClass* AnimInstanceClass = SourceInstance->GetClass();
	UAnimInstance* TargetInstance = CastChecked<UAnimInstance>(AnimInstanceClass->GetDefaultObject());
	
	const bool bIsPreviewing = ( Options.Flags & ECopyOptions::PreviewOnly ) != 0;

	int32 CopiedPropertyCount = 0;

	// Copy properties from the instance to the CDO
	TSet<UObject*> ModifiedObjects;
	for( FProperty* Property = AnimInstanceClass->PropertyLink; Property != nullptr; Property = Property->PropertyLinkNext )
	{
		const bool bIsTransient = !!( Property->PropertyFlags & CPF_Transient );
		const bool bIsBlueprintReadonly = !!(Options.Flags & ECopyOptions::FilterBlueprintReadOnly) && !!( Property->PropertyFlags & CPF_BlueprintReadOnly );
		const bool bIsIdentical = Property->Identical_InContainer(SourceInstance, TargetInstance);
		const bool bIsAnimGraphNodeProperty = Property->IsA<FStructProperty>() && CastField<FStructProperty>(Property)->Struct->IsChildOf(FAnimNode_Base::StaticStruct());

		if( !bIsAnimGraphNodeProperty && !bIsTransient && !bIsIdentical && !bIsBlueprintReadonly)
		{
			const bool bIsSafeToCopy = !( Options.Flags & ECopyOptions::OnlyCopyEditOrInterpProperties ) || ( Property->HasAnyPropertyFlags( CPF_Edit | CPF_Interp ) );
			if( bIsSafeToCopy )
			{
				if (!Options.CanCopyProperty(*Property, *SourceInstance))
				{
					continue;
				}

				if( !bIsPreviewing )
				{
					if( !ModifiedObjects.Contains(TargetInstance) )
					{
						// Start modifying the target object
						TargetInstance->Modify();
						ModifiedObjects.Add(TargetInstance);
					}

					if( Options.Flags & ECopyOptions::CallPostEditChangeProperty )
					{
						TargetInstance->PreEditChange(Property);
					}

					EditorUtilities::CopySingleProperty(SourceInstance, TargetInstance, Property);

					if( Options.Flags & ECopyOptions::CallPostEditChangeProperty )
					{
						FPropertyChangedEvent PropertyChangedEvent(Property);
						TargetInstance->PostEditChangeProperty(PropertyChangedEvent);
					}
				}

				++CopiedPropertyCount;
			}
		}
	}

	if (!bIsPreviewing && CopiedPropertyCount > 0 && AnimInstanceClass->HasAllClassFlags(CLASS_CompiledFromBlueprint))
	{
		FBlueprintEditorUtils::MarkBlueprintAsModified(CastChecked<UBlueprint>(AnimInstanceClass->ClassGeneratedBy));
	}

	return CopiedPropertyCount;
}

void SetObjectBeingDebugged(UAnimBlueprint* InAnimBlueprint, UAnimInstance* InAnimInstance)
{
	UAnimBlueprint* PreviewAnimBlueprint = InAnimBlueprint->GetPreviewAnimationBlueprint();
		
	if (PreviewAnimBlueprint)
	{
		EPreviewAnimationBlueprintApplicationMethod ApplicationMethod = InAnimBlueprint->GetPreviewAnimationBlueprintApplicationMethod();
		if(ApplicationMethod == EPreviewAnimationBlueprintApplicationMethod::LinkedLayers)
		{
			// Make sure the object being debugged is the linked layer instance
			InAnimBlueprint->SetObjectBeingDebugged(InAnimInstance->GetLinkedAnimLayerInstanceByClass(InAnimBlueprint->GeneratedClass.Get()));
		}
		else if(ApplicationMethod == EPreviewAnimationBlueprintApplicationMethod::LinkedAnimGraph)
		{
			// Make sure the object being debugged is the linked instance
			InAnimBlueprint->SetObjectBeingDebugged(InAnimInstance->GetLinkedAnimGraphInstanceByTag(InAnimBlueprint->GetPreviewAnimationBlueprintTag()));
		}
	}
	else
	{
		// Make sure the object being debugged is the preview instance
		InAnimBlueprint->SetObjectBeingDebugged(InAnimInstance);
	}
}

TSharedRef<SWidget> MakeTrackButton(FText HoverText, FOnGetContent MenuContent, const TAttribute<bool>& HoverState)
{
	FSlateFontInfo SmallLayoutFont = FCoreStyle::GetDefaultFontStyle("Regular", 8);

	TSharedRef<STextBlock> ComboButtonText = SNew(STextBlock)
		.Text(HoverText)
		.Font(SmallLayoutFont)
		.ColorAndOpacity( FSlateColor::UseForeground() );

	TSharedRef<SComboButton> ComboButton =

		SNew(SComboButton)
		.HasDownArrow(false)
		.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
		.ForegroundColor( FSlateColor::UseForeground() )
		.OnGetMenuContent(MenuContent)
		.ContentPadding(FMargin(5, 2))
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.ButtonContent()
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FMargin(0,0,2,0))
			[
				SNew(SImage)
				.ColorAndOpacity( FSlateColor::UseForeground() )
				.Image(FAppStyle::GetBrush("ComboButton.Arrow"))
			]

			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				ComboButtonText
			]
		];

	auto GetRolloverVisibility = [WeakComboButton = TWeakPtr<SComboButton>(ComboButton), HoverState]()
	{
		TSharedPtr<SComboButton> ComboButton = WeakComboButton.Pin();
		if (HoverState.Get() || ComboButton->IsOpen())
		{
			return EVisibility::SelfHitTestInvisible;
		}
		else
		{
			return EVisibility::Collapsed;
		}
	};

	TAttribute<EVisibility> Visibility = TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateLambda(GetRolloverVisibility));
	ComboButtonText->SetVisibility(Visibility);

	return ComboButton;
}

template<typename NotifyTypeClass>
static TSharedRef<SWidget> MakeNewNotifyPicker(UAnimSequenceBase* Sequence, const FOnClassPicked& OnClassPicked)
{
	class FNotifyStateClassFilter : public IClassViewerFilter
	{
	public:
		FNotifyStateClassFilter(UAnimSequenceBase* InSequence)
			: Sequence(InSequence)
		{}

		bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
		{
			const bool bChildOfObjectClass = InClass->IsChildOf(NotifyTypeClass::StaticClass());
			const bool bMatchesFlags = !InClass->HasAnyClassFlags(CLASS_Hidden | CLASS_HideDropDown | CLASS_Deprecated | CLASS_Abstract);
			return bChildOfObjectClass && bMatchesFlags && CastChecked<NotifyTypeClass>(InClass->GetDefaultObject(false))->CanBePlaced(Sequence);
		}

		virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef< const IUnloadedBlueprintData > InUnloadedClassData, TSharedRef< FClassViewerFilterFuncs > InFilterFuncs) override
		{
			const bool bChildOfObjectClass = InUnloadedClassData->IsChildOf(NotifyTypeClass::StaticClass());
			const bool bMatchesFlags = !InUnloadedClassData->HasAnyClassFlags(CLASS_Hidden | CLASS_HideDropDown | CLASS_Deprecated | CLASS_Abstract);
			bool bValidToPlace = false;
			if (bChildOfObjectClass)
			{
				if (const UClass* NativeBaseClass = InUnloadedClassData->GetNativeParent())
				{
					bValidToPlace = CastChecked<NotifyTypeClass>(NativeBaseClass->GetDefaultObject(false))->CanBePlaced(Sequence);
				}
			}

			return bChildOfObjectClass && bMatchesFlags && bValidToPlace;
		}

		/** Sequence referenced by outer panel */
		UAnimSequenceBase* Sequence;
	};

	FClassViewerInitializationOptions InitOptions;
	InitOptions.Mode = EClassViewerMode::ClassPicker;
	InitOptions.bShowObjectRootClass = false;
	InitOptions.bShowUnloadedBlueprints = true;
	InitOptions.bShowNoneOption = false;
	InitOptions.bEnableClassDynamicLoading = true;
	InitOptions.bExpandRootNodes = true;
	InitOptions.NameTypeToDisplay = EClassViewerNameTypeToDisplay::DisplayName;
	InitOptions.ClassFilters.Add(MakeShared<FNotifyStateClassFilter>(Sequence));
	InitOptions.bShowBackgroundBorder = false;

	FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");

	return SNew(SBox)
		.MinDesiredWidth(300.0f)
		.MaxDesiredHeight(400.0f)
		[
			ClassViewerModule.CreateClassViewer(InitOptions, OnClassPicked)
		];
}

TSharedRef<SWidget> MakeAnimNotifyPicker(UAnimSequenceBase* Sequence, const FOnClassPicked& OnClassPicked)
{
	return MakeNewNotifyPicker<UAnimNotify>(Sequence, OnClassPicked);
}

TSharedRef<SWidget> MakeAnimNotifyStatePicker(UAnimSequenceBase* Sequence, const FOnClassPicked& OnClassPicked)
{
	return MakeNewNotifyPicker<UAnimNotifyState>(Sequence, OnClassPicked);
}

}
