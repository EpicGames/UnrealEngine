// Copyright Epic Games, Inc. All Rights Reserved.

#include "PhysicsAssetRenderUtils.h"
#include "AssetRegistry/AssetData.h"
#include "Features/IModularFeatures.h"
#include "Materials/MaterialInterface.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Components/SkeletalMeshComponent.h"
#include "Materials/Material.h"
#include "Preferences/PhysicsAssetEditorOptions.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "PhysicsEngine/PhysicsConstraintTemplate.h"
#include "PhysicsEngine/SkeletalBodySetup.h"
#include "Chaos/Core.h"
#include "Chaos/SkinnedTriangleMesh.h"
#include "SceneView.h"
#include "SkeletalMeshTypes.h"
#include "Styling/AppStyle.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Internationalization/TextKey.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PhysicsAssetRenderUtils)

#define LOCTEXT_NAMESPACE "PhysicsAssetRenderUtils"

// How large to make the constraint arrows.
// The factor of 60 was found experimentally, to look reasonable in comparison with the rest of the constraint visuals.
constexpr float ConstraintArrowScale = 60.0f;

////////////////////////////////////////
// Utility Functions 

FTransform GetConstraintMatrix(const USkeletalMeshComponent* const SkeletalMeshComponent, const UPhysicsConstraintTemplate* const ConstraintSetup, const EConstraintFrame::Type Frame, const int32 BoneIndex)
{
	if (!ConstraintSetup || !SkeletalMeshComponent)
	{
		return FTransform::Identity;
	}

	FTransform LFrame = ConstraintSetup->DefaultInstance.GetRefFrame(Frame);
	FTransform BoneTM = SkeletalMeshComponent->GetBoneTransform(BoneIndex);

	return LFrame * BoneTM;
}

void DrawWireStar(FPrimitiveDrawInterface* PDI, const FTransform& Transform, float Size, const FLinearColor& Color, uint8 DepthPriority, const float Thickness)
{
	const FVector Position = Transform.GetLocation();
	const FVector XAxis = Transform.GetUnitAxis(EAxis::X);
	const FVector YAxis = Transform.GetUnitAxis(EAxis::Y);
	const FVector ZAxis = Transform.GetUnitAxis(EAxis::Z);

	PDI->DrawLine(Position + Size * XAxis, Position - Size * XAxis, Color, DepthPriority, Thickness);
	PDI->DrawLine(Position + Size * YAxis, Position - Size * YAxis, Color, DepthPriority, Thickness);
	PDI->DrawLine(Position + Size * ZAxis, Position - Size * ZAxis, Color, DepthPriority, Thickness);
}

////////////////////////////////////////
// struct FPhysicsAssetRenderSettings

FPhysicsAssetRenderSettings::FPhysicsAssetRenderSettings()
	: CenterOfMassViewMode(EPhysicsAssetEditorCenterOfMassViewMode::None)
	, CollisionViewMode(EPhysicsAssetEditorCollisionViewMode::Solid)
	, ConstraintViewMode(EPhysicsAssetEditorConstraintViewMode::AllLimits)
	, ConstraintViewportManipulationFlags(EConstraintTransformComponentFlags::All)
	, ConstraintTransformComponentDisplayRelativeToDefaultFlags(EConstraintTransformComponentFlags::None)
	, ConstraintDrawSize(1.0f)
	, PhysicsBlend(1.0f)
	, bHideKinematicBodies(false)
	, bHideSimulatedBodies(false)
	, bHideBodyMass(false)
	, bRenderOnlySelectedConstraints(false)
	, bShowCOM_DEPRECATED(false)
	, bShowConstraintsAsPoints(false)
	, bHighlightOverlapingBodies(false)
	, bDrawViolatedLimits(false)
	, bHideCenterOfMassForKinematicBodies(true)
	, BoneUnselectedColor(170, 155, 225)
	, NoCollisionColor(200, 200, 200)
	, COMRenderColor(255, 255, 100)
	, COMRenderSize(2.0f)
	, COMRenderLineThickness(0.2f)
	, COMRenderMassTextOffsetScreenspace(8.0f)
	, InfluenceLineLength(2.0f)
	, BoneUnselectedMaterial(nullptr)
	, BoneNoCollisionMaterial(nullptr)
{}

void FPhysicsAssetRenderSettings::InitPhysicsAssetRenderSettings(UMaterialInterface* InBoneUnselectedMaterial, UMaterialInterface* InBoneNoCollisionMaterial)
{
	BoneUnselectedMaterial = InBoneUnselectedMaterial;
	check(BoneUnselectedMaterial);

	BoneNoCollisionMaterial = InBoneNoCollisionMaterial;
	check(BoneNoCollisionMaterial);
}

bool FPhysicsAssetRenderSettings::IsBodyHidden(const int32 BodyIndex) const
{
	return HiddenBodies.Contains(BodyIndex);
}

bool FPhysicsAssetRenderSettings::IsConstraintHidden(const int32 ConstraintIndex) const
{
	return HiddenConstraints.Contains(ConstraintIndex);
}

bool FPhysicsAssetRenderSettings::AreAnyBodiesHidden() const
{
	return HiddenBodies.Num() > 0;
}

bool FPhysicsAssetRenderSettings::AreAnyConstraintsHidden() const
{
	return HiddenConstraints.Num() > 0;
}

void FPhysicsAssetRenderSettings::HideBody(const int32 BodyIndex)
{
	if (!HiddenBodies.Contains(BodyIndex))
	{
		HiddenBodies.Add(BodyIndex);
	}
}

void FPhysicsAssetRenderSettings::ShowBody(const int32 BodyIndex)
{
	if (HiddenBodies.Contains(BodyIndex))
	{
		HiddenBodies.RemoveSwap(BodyIndex);
	}
}

void FPhysicsAssetRenderSettings::ShowAllBodies()
{
	HiddenBodies.Reset();
}

void FPhysicsAssetRenderSettings::ShowAllConstraints()
{
	HiddenConstraints.Reset();
}

void FPhysicsAssetRenderSettings::ShowAll()
{
	ShowAllBodies();
	ShowAllConstraints();
}

void FPhysicsAssetRenderSettings::HideAllBodies(const UPhysicsAsset* const PhysicsAsset)
{
	HiddenBodies.Reset();

	if (PhysicsAsset != nullptr)
	{
		for (int32 i = 0; i < PhysicsAsset->SkeletalBodySetups.Num(); ++i)
		{
			HiddenBodies.Add(i);
		}
	}
}

void FPhysicsAssetRenderSettings::HideAllConstraints(const UPhysicsAsset* const PhysicsAsset)
{
	HiddenConstraints.Reset();

	if (PhysicsAsset != nullptr)
	{
		for (int32 i = 0; i < PhysicsAsset->ConstraintSetup.Num(); ++i)
		{
			HiddenConstraints.Add(i);
		}
	}
}

void FPhysicsAssetRenderSettings::HideAll(const UPhysicsAsset* const PhysicsAsset)
{
	HideAllBodies(PhysicsAsset);
	HideAllConstraints(PhysicsAsset);
}

void FPhysicsAssetRenderSettings::ToggleShowBody(const int32 BodyIndex)
{
	if (IsBodyHidden(BodyIndex))
	{
		ShowBody(BodyIndex);
	}
	else
	{
		HideBody(BodyIndex);
	}
}

void FPhysicsAssetRenderSettings::ToggleShowConstraint(const int32 ConstraintIndex)
{
	if (IsConstraintHidden(ConstraintIndex))
	{
		ShowConstraint(ConstraintIndex);
	}
	else
	{
		HideConstraint(ConstraintIndex);
	}
}

void FPhysicsAssetRenderSettings::ToggleShowAllBodies(const UPhysicsAsset* const PhysicsAsset)
{
	if (AreAnyBodiesHidden())
	{
		ShowAllBodies();
	}
	else
	{
		HideAllBodies(PhysicsAsset);
	}
}

void FPhysicsAssetRenderSettings::ToggleShowAllConstraints(const UPhysicsAsset* const PhysicsAsset)
{
	if (AreAnyConstraintsHidden())
	{
		ShowAllConstraints();
	}
	else
	{
		HideAllConstraints(PhysicsAsset);
	}
}

void FPhysicsAssetRenderSettings::HideConstraint(const int32 ConstraintIndex)
{
	if (!HiddenConstraints.Contains(ConstraintIndex))
	{
		HiddenConstraints.Add(ConstraintIndex);
	}
}

void FPhysicsAssetRenderSettings::ShowConstraint(const int32 ConstraintIndex)
{
	if (HiddenConstraints.Contains(ConstraintIndex))
	{
		HiddenConstraints.RemoveSwap(ConstraintIndex);
	}
}

void FPhysicsAssetRenderSettings::SetHiddenBodies(const TArray<int32>& InHiddenBodies)
{
	HiddenBodies.Reset();
	HiddenBodies.Append(InHiddenBodies);
}

void FPhysicsAssetRenderSettings::SetHiddenConstraints(const TArray<int32>& InHiddenConstraints)
{
	HiddenConstraints.Reset();
	HiddenConstraints.Append(InHiddenConstraints);
}

EConstraintTransformComponentFlags FPhysicsAssetRenderSettings::GetConstraintViewportManipulationFlags() const
{
	return ConstraintViewportManipulationFlags;
}

bool FPhysicsAssetRenderSettings::IsDisplayingConstraintTransformComponentRelativeToDefault(const EConstraintTransformComponentFlags ComponentFlags) const
{
	return EnumHasAllFlags(ConstraintTransformComponentDisplayRelativeToDefaultFlags, ComponentFlags);
}

void FPhysicsAssetRenderSettings::SetDisplayConstraintTransformComponentRelativeToDefault(const EConstraintTransformComponentFlags ComponentFlags, const bool bShouldDisplayRelativeToDefault)
{
	ConstraintTransformComponentDisplayRelativeToDefaultFlags = (bShouldDisplayRelativeToDefault) ? (ConstraintTransformComponentDisplayRelativeToDefaultFlags | ComponentFlags) : (ConstraintTransformComponentDisplayRelativeToDefaultFlags & ~ComponentFlags);
}

void FPhysicsAssetRenderSettings::ResetEditorViewportOptions()
{
	const FPhysicsAssetRenderSettings DefaultObject = FPhysicsAssetRenderSettings();

	CenterOfMassViewMode = DefaultObject.CenterOfMassViewMode;
	CollisionViewMode = DefaultObject.CollisionViewMode;
	COMRenderSize = DefaultObject.COMRenderSize;
	ConstraintViewMode = DefaultObject.ConstraintViewMode;
	ConstraintDrawSize = DefaultObject.ConstraintDrawSize;
	PhysicsBlend = DefaultObject.PhysicsBlend;
	bHideKinematicBodies = DefaultObject.bHideKinematicBodies;
	bHideSimulatedBodies = DefaultObject.bHideSimulatedBodies;
	bHideBodyMass = DefaultObject.bHideBodyMass;
	bRenderOnlySelectedConstraints = DefaultObject.bRenderOnlySelectedConstraints;
	bShowConstraintsAsPoints = DefaultObject.bShowConstraintsAsPoints;
	bDrawViolatedLimits = DefaultObject.bDrawViolatedLimits;
	bHideCenterOfMassForKinematicBodies = DefaultObject.bHideCenterOfMassForKinematicBodies;
}

namespace PhysicsAssetRender
{
	void DebugDraw(USkeletalMeshComponent* const SkeletalMeshComponent, UPhysicsAsset* const PhysicsAsset, FPrimitiveDrawInterface* PDI)
	{
		// Draw Bodies.
		{
			auto TransformFn = [](const UPhysicsAsset* PhysicsAsset, const FTransform& BoneTM, const int32 BodyIndex, const EAggCollisionShape::Type PrimType, const int32 PrimIndex, const float Scale) { return GetPrimitiveTransform(PhysicsAsset, BoneTM, BodyIndex, PrimType, PrimIndex, Scale);  };
			auto ColorFn    = [](const int32 BodyIndex, const EAggCollisionShape::Type PrimitiveType, const int32 PrimitiveIndex, const FPhysicsAssetRenderSettings& Settings) { return GetPrimitiveColor(BodyIndex, PrimitiveType, PrimitiveIndex, Settings); };
			auto MaterialFn = [](const int32 BodyIndex, const EAggCollisionShape::Type PrimitiveType, const int32 PrimitiveIndex, const FPhysicsAssetRenderSettings& Settings) { return GetPrimitiveMaterial(BodyIndex, PrimitiveType, PrimitiveIndex, Settings); };
			auto HitProxyFn = [](const int32 BodyIndex, const EAggCollisionShape::Type PrimitiveType, const int32 PrimitiveIndex) { return nullptr; };

			DebugDrawBodies(SkeletalMeshComponent, PhysicsAsset, PDI, ColorFn, MaterialFn, TransformFn, HitProxyFn);
		}

		// Draw Constraints.
		{
			auto HitProxyFunctor = [](const int32) { return nullptr; };
			DebugDrawConstraints(SkeletalMeshComponent, PhysicsAsset, PDI, IsSelectedFn(), false, HitProxyFunctor);
		}
	}

	// Contains info needed for drawing, in a form that can be sorted by distance.
	struct DrawElement
	{
		DrawElement(
			FTransform        InTM,
			HHitProxy* InHitProxy,
			float             InScale,
			const FSceneView* InView,
			const FKShapeElem* InShapeElem,
			const FName& InBoneName
		)
			: TM(InTM), HitProxy(InHitProxy), Scale(InScale), ShapeElem(InShapeElem)
#ifdef UE_BUILD_DEBUG
			, BoneName(InBoneName)
#endif
		{
			// The TM position is at the center of the objects, so can be used directly for sorting.
			// Sorting by distance (rather than along the view direction) reduces flickering when
			// the camera is rotated without moving it.
			SortingMetric = static_cast<float>((TM.GetTranslation() - InView->ViewLocation).SquaredLength()); // Sorts by Distance
		}

		UMaterialInterface* Material = 0;
		FColor                Color = FColor(ForceInitToZero);
		FTransform            TM;
		HHitProxy* HitProxy = nullptr;
		float                 Scale;
		const FKShapeElem* ShapeElem = nullptr;
		float                 SortingMetric;
#ifdef UE_BUILD_DEBUG
		FName                 BoneName;
#endif
	};
	
	template< typename TElementContainer > void CollectDrawElements(TArray<DrawElement>& DrawElements, const EAggCollisionShape::Type PrimitiveType, const TElementContainer& ElementContainer, const FTransform& BoneTM, const float Scale, const uint32 BodyIndex, const FPhysicsAssetRenderSettings* const RenderSettings, UPhysicsAsset* const PhysicsAsset, FPrimitiveDrawInterface* PDI, GetPrimitiveRef< FColor > GetPrimitiveColor, GetPrimitiveRef< UMaterialInterface* > GetPrimitiveMaterial, GetPrimitiveTransformRef GetPrimitiveTransform, CreateBodyHitProxyFn CreateHitProxy)
	{
		if (PhysicsAsset && RenderSettings && PDI && PhysicsAsset->SkeletalBodySetups.IsValidIndex(BodyIndex))
		{
			for (int32 ElementIndex = 0; ElementIndex < ElementContainer.Num(); ++ElementIndex)
			{
				const FName BoneName = PhysicsAsset->SkeletalBodySetups[BodyIndex]->BoneName;

				DrawElement DE(
					GetPrimitiveTransform(PhysicsAsset, BoneTM, BodyIndex, PrimitiveType, ElementIndex, Scale),
					CreateHitProxy(BodyIndex, PrimitiveType, ElementIndex),
					Scale, PDI->View, &ElementContainer[ElementIndex], BoneName
				);

				if (RenderSettings->CollisionViewMode == EPhysicsAssetEditorCollisionViewMode::Solid || RenderSettings->CollisionViewMode == EPhysicsAssetEditorCollisionViewMode::SolidWireframe)
				{
					DE.Material = GetPrimitiveMaterial(BodyIndex, PrimitiveType, ElementIndex, *RenderSettings);
				}
				if (RenderSettings->CollisionViewMode == EPhysicsAssetEditorCollisionViewMode::SolidWireframe || RenderSettings->CollisionViewMode == EPhysicsAssetEditorCollisionViewMode::Wireframe)
				{
					DE.Color = GetPrimitiveColor(BodyIndex, PrimitiveType, ElementIndex, *RenderSettings);
				}
				if (DE.Material || DE.Color.A)
				{
					DrawElements.Add(DE);
				}
			}
		}
	}

	void DebugDrawBodies(USkeletalMeshComponent* const SkeletalMeshComponent, UPhysicsAsset* const PhysicsAsset, FPrimitiveDrawInterface* PDI, GetPrimitiveRef< FColor > GetPrimitiveColor, GetPrimitiveRef< UMaterialInterface* > GetPrimitiveMaterial, GetPrimitiveTransformRef GetPrimitiveTransform, CreateBodyHitProxyFn CreateHitProxy)
	{
		if (!SkeletalMeshComponent || !PhysicsAsset || !PhysicsAsset->PreviewSkeletalMesh)
		{
			// Nothing to draw without an asset, this can happen if the preview scene has no skeletal mesh
			return;
		}

		const FPhysicsAssetRenderSettings* const RenderSettings = UPhysicsAssetRenderUtilities::GetSettings(PhysicsAsset);

		if (!RenderSettings)
		{
			return;
		}

		// Draw bodies
		for (int32 i = 0; i < PhysicsAsset->SkeletalBodySetups.Num(); ++i)
		{
			if (!ensure(PhysicsAsset->SkeletalBodySetups[i]))
			{
				continue;
			}
			if ((PhysicsAsset->SkeletalBodySetups[i]->PhysicsType == EPhysicsType::PhysType_Kinematic &&
				RenderSettings->bHideKinematicBodies) ||
				(PhysicsAsset->SkeletalBodySetups[i]->PhysicsType == EPhysicsType::PhysType_Simulated &&
					RenderSettings->bHideSimulatedBodies)
				)
			{
				continue;
			}
			if (RenderSettings->IsBodyHidden(i))
			{
				continue;
			}

			const int32 BoneIndex = SkeletalMeshComponent->GetBoneIndex(PhysicsAsset->SkeletalBodySetups[i]->BoneName);

			// If we found a bone for it, draw the collision.
			// The logic is as follows; always render in the ViewMode requested when not in hit mode - but if we are in hit mode and the right editing mode, render as solid
			if (BoneIndex != INDEX_NONE)
			{
				FTransform BoneTM = SkeletalMeshComponent->GetBoneTransform(BoneIndex);
				const float Scale = static_cast<float>(BoneTM.GetScale3D().GetAbsMax());
				BoneTM.RemoveScaling();

				FKAggregateGeom* const AggGeom = &PhysicsAsset->SkeletalBodySetups[i]->AggGeom;

				TArray<DrawElement> DrawElements;

				CollectDrawElements(DrawElements, EAggCollisionShape::Sphere, AggGeom->SphereElems, BoneTM, Scale, i, RenderSettings, PhysicsAsset, PDI, GetPrimitiveColor, GetPrimitiveMaterial, GetPrimitiveTransform, CreateHitProxy);
				CollectDrawElements(DrawElements, EAggCollisionShape::Box, AggGeom->BoxElems, BoneTM, Scale, i, RenderSettings, PhysicsAsset, PDI, GetPrimitiveColor, GetPrimitiveMaterial, GetPrimitiveTransform, CreateHitProxy);
				CollectDrawElements(DrawElements, EAggCollisionShape::Sphyl, AggGeom->SphylElems, BoneTM, Scale, i, RenderSettings, PhysicsAsset, PDI, GetPrimitiveColor, GetPrimitiveMaterial, GetPrimitiveTransform, CreateHitProxy);
				CollectDrawElements(DrawElements, EAggCollisionShape::Convex, AggGeom->ConvexElems, BoneTM, Scale, i, RenderSettings, PhysicsAsset, PDI, GetPrimitiveColor, GetPrimitiveMaterial, GetPrimitiveTransform, CreateHitProxy);
				CollectDrawElements(DrawElements, EAggCollisionShape::TaperedCapsule, AggGeom->TaperedCapsuleElems, BoneTM, Scale, i, RenderSettings, PhysicsAsset, PDI, GetPrimitiveColor, GetPrimitiveMaterial, GetPrimitiveTransform, CreateHitProxy);
				CollectDrawElements(DrawElements, EAggCollisionShape::LevelSet, AggGeom->LevelSetElems, BoneTM, Scale, i, RenderSettings, PhysicsAsset, PDI, GetPrimitiveColor, GetPrimitiveMaterial, GetPrimitiveTransform, CreateHitProxy);
				CollectDrawElements(DrawElements, EAggCollisionShape::SkinnedLevelSet, AggGeom->SkinnedLevelSetElems, BoneTM, Scale, i, RenderSettings, PhysicsAsset, PDI, GetPrimitiveColor, GetPrimitiveMaterial, GetPrimitiveTransform, CreateHitProxy);
				CollectDrawElements(DrawElements, EAggCollisionShape::MLLevelSet, AggGeom->MLLevelSetElems, BoneTM, Scale, i, RenderSettings, PhysicsAsset, PDI, GetPrimitiveColor, GetPrimitiveMaterial, GetPrimitiveTransform, CreateHitProxy);
				CollectDrawElements(DrawElements, EAggCollisionShape::SkinnedTriangleMesh, AggGeom->SkinnedTriangleMeshElems, BoneTM, Scale, i, RenderSettings, PhysicsAsset, PDI, GetPrimitiveColor, GetPrimitiveMaterial, GetPrimitiveTransform, CreateHitProxy);

				// Sort elements
				DrawElements.Sort([](const DrawElement& DE1, const DrawElement& DE2) {
					return DE1.SortingMetric > DE2.SortingMetric;
					});

				// Render sorted elements.
				for (const DrawElement& DE : DrawElements)
				{
					PDI->SetHitProxy(DE.HitProxy);
					if (DE.Material)
					{
						DE.ShapeElem->DrawElemSolid(PDI, DE.TM, DE.Scale, DE.Material->GetRenderProxy());
					}
					if (DE.Color.A)
					{
						DE.ShapeElem->DrawElemWire(PDI, DE.TM, DE.Scale, DE.Color);
					}
				}
	
				PDI->SetHitProxy(NULL);
			}
		}
	}

	void DebugDrawCenterOfMass(USkeletalMeshComponent* const SkeletalMeshComponent, UPhysicsAsset* const PhysicsAsset, FPrimitiveDrawInterface* PDI, TFunctionRef< FVector(const uint32) > GetCoMPosition, TFunctionRef< bool(const uint32) > IsSelected, TFunctionRef< bool(const uint32) > IsHidden, CreateCoMHitProxyFn CreateHitProxy)
	{
		static const float SelectedItemRenderSizeMultiplier = 1.5f;

		const FPhysicsAssetRenderSettings* const RenderSettings = UPhysicsAssetRenderUtilities::GetSettings(PhysicsAsset);

		if (RenderSettings && (RenderSettings->CenterOfMassViewMode != EPhysicsAssetEditorCenterOfMassViewMode::None))
		{
			const bool bDrawSelectedOnly = RenderSettings->CenterOfMassViewMode == EPhysicsAssetEditorCenterOfMassViewMode::Selected;

			for (int32 BodyIndex = 0, BodyCount = SkeletalMeshComponent->Bodies.Num(); BodyIndex < BodyCount; ++BodyIndex)
			{
				const bool bIsSelected = IsSelected(BodyIndex);

				if (!IsHidden(BodyIndex) && (!bDrawSelectedOnly || bIsSelected))
				{
					const FBodyInstance* const BodyInstance = SkeletalMeshComponent->Bodies[BodyIndex];

					if (BodyInstance && BodyInstance->IsValidBodyInstance() && !(RenderSettings->bHideCenterOfMassForKinematicBodies && IsBodyKinematic(PhysicsAsset, BodyIndex)))
					{
						float COMRenderSize = RenderSettings->COMRenderSize;
						float COMRenderLineThickness = RenderSettings->COMRenderLineThickness;

						if (bIsSelected)
						{
							COMRenderSize *= SelectedItemRenderSizeMultiplier;
							COMRenderLineThickness *= SelectedItemRenderSizeMultiplier;
						}

						FTransform CoMMarkerTM(GetCoMPosition(BodyIndex));
						
						if (BodyInstance->GetBodySetup())
						{
							const FName BoneName = BodyInstance->GetBodySetup()->BoneName;
							CoMMarkerTM.SetRotation(SkeletalMeshComponent->GetBoneTransform(BoneName).GetRotation());
						}

						PDI->SetHitProxy(CreateHitProxy(BodyIndex));
						DrawWireStar(PDI, CoMMarkerTM, COMRenderSize, RenderSettings->COMRenderColor, SDPG_Foreground, COMRenderLineThickness);
						PDI->SetHitProxy(NULL);
					}
				}
			}
		}
	}

	void DebugDrawConstraints(USkeletalMeshComponent* const SkeletalMeshComponent, UPhysicsAsset* const PhysicsAsset, FPrimitiveDrawInterface* PDI, TFunction< bool(const uint32) > IsSelected, const bool bRunningSimulation, CreateConstraintHitProxyFn CreateHitProxy)
	{
		check(SkeletalMeshComponent);
		check(PhysicsAsset);

		const FPhysicsAssetRenderSettings* const RenderSettings = UPhysicsAssetRenderUtilities::GetSettings(PhysicsAsset);

		if (!RenderSettings)
		{
			return;
		}

		if (RenderSettings->ConstraintViewMode != EPhysicsAssetEditorConstraintViewMode::None)
		{
			const bool bIsSelectedStateAvailable = (IsSelected != nullptr);
			const bool bRenderOnlySelected = bIsSelectedStateAvailable && RenderSettings->bRenderOnlySelectedConstraints;

			for (int32 ConstraintIndex = 0; ConstraintIndex < PhysicsAsset->ConstraintSetup.Num(); ++ConstraintIndex)
			{
				const bool bConstraintSelected = bIsSelectedStateAvailable && IsSelected(ConstraintIndex);

				if ((!bRenderOnlySelected || bConstraintSelected) &&
					!RenderSettings->IsConstraintHidden(ConstraintIndex))
				{
					const bool bDrawLimits = (RenderSettings->ConstraintViewMode == EPhysicsAssetEditorConstraintViewMode::AllLimits) || bConstraintSelected;
					const bool bDrawSelected = !bRunningSimulation && bConstraintSelected;

					PDI->SetHitProxy(CreateHitProxy(ConstraintIndex));
					
					UPhysicsConstraintTemplate* const ConstraintSetup = PhysicsAsset->ConstraintSetup[ConstraintIndex];
					check(ConstraintSetup);
					if (ConstraintSetup)
					{
						const int32 BoneIndex1 = SkeletalMeshComponent->GetBoneIndex(ConstraintSetup->DefaultInstance.ConstraintBone1);
						const int32 BoneIndex2 = SkeletalMeshComponent->GetBoneIndex(ConstraintSetup->DefaultInstance.ConstraintBone2);

						// if bone doesn't exist, do not draw it. It crashes in random points when we try to manipulate. 
						if ((BoneIndex1 != INDEX_NONE) && (BoneIndex2 != INDEX_NONE))
						{
							const float DrawScale = ConstraintArrowScale * RenderSettings->ConstraintDrawSize;
							FTransform Con1Frame = GetConstraintMatrix(SkeletalMeshComponent, ConstraintSetup, EConstraintFrame::Frame1, BoneIndex1);
							FTransform Con2Frame = GetConstraintMatrix(SkeletalMeshComponent, ConstraintSetup, EConstraintFrame::Frame2, BoneIndex2);

							// Remove scaling from constraint frame transforms so that constraint limit cones etc are all drawn at the same scale.
							Con1Frame.RemoveScaling();
							Con2Frame.RemoveScaling();

							ConstraintSetup->DefaultInstance.DrawConstraint(PDI, RenderSettings->ConstraintDrawSize, DrawScale, bDrawLimits, bDrawSelected, Con1Frame, Con2Frame, RenderSettings->bShowConstraintsAsPoints, RenderSettings->bDrawViolatedLimits);

						}
					}

					PDI->SetHitProxy(NULL);
				}
			}
		}
	}

	FTransform GetPrimitiveTransform(const UPhysicsAsset* PhysicsAsset, const FTransform& BoneTM, const int32 BodyIndex, const EAggCollisionShape::Type PrimType, const int32 PrimIndex, const float Scale)
	{
		UBodySetup* SharedBodySetup = PhysicsAsset->SkeletalBodySetups[BodyIndex];
		FVector Scale3D(Scale);

		FTransform PrimTM = FTransform::Identity;

		if (PrimType == EAggCollisionShape::Sphere)
		{
			PrimTM = SharedBodySetup->AggGeom.SphereElems[PrimIndex].GetTransform();
		}
		else if (PrimType == EAggCollisionShape::Box)
		{
			PrimTM = SharedBodySetup->AggGeom.BoxElems[PrimIndex].GetTransform();
		}
		else if (PrimType == EAggCollisionShape::Sphyl)
		{
			PrimTM = SharedBodySetup->AggGeom.SphylElems[PrimIndex].GetTransform();
		}
		else if (PrimType == EAggCollisionShape::Convex)
		{
			PrimTM = SharedBodySetup->AggGeom.ConvexElems[PrimIndex].GetTransform();
		}
		else if (PrimType == EAggCollisionShape::TaperedCapsule)
		{
			PrimTM = SharedBodySetup->AggGeom.TaperedCapsuleElems[PrimIndex].GetTransform();
		}
		else
		{
			// Should never reach here
			check(0);
		}

		PrimTM.ScaleTranslation(Scale3D);
		return PrimTM * BoneTM;
	}

	FColor GetPrimitiveColor(const int32 BodyIndex, const EAggCollisionShape::Type PrimitiveType, const int32 PrimitiveIndex, const FPhysicsAssetRenderSettings& Settings)
	{
		return (PrimitiveType == EAggCollisionShape::TaperedCapsule) ? Settings.NoCollisionColor : Settings.BoneUnselectedColor;
	}

	UMaterialInterface* GetPrimitiveMaterial(const int32 BodyIndex, const EAggCollisionShape::Type PrimitiveType, const int32 PrimitiveIndex, const FPhysicsAssetRenderSettings& Settings)
	{
		return (PrimitiveType == EAggCollisionShape::TaperedCapsule) ? Settings.BoneNoCollisionMaterial : Settings.BoneUnselectedMaterial;
	}
}

UPhysicsAssetRenderUtilities::UPhysicsAssetRenderUtilities()
: PhysicsAssetRenderInterface(nullptr)
{}

UPhysicsAssetRenderUtilities::~UPhysicsAssetRenderUtilities()
{
	delete PhysicsAssetRenderInterface;
}


void UPhysicsAssetRenderUtilities::Initialise()
{
	if (UPhysicsAssetRenderUtilities* PhysicsAssetRenderUtilities = GetMutableDefault<UPhysicsAssetRenderUtilities>())
	{
		PhysicsAssetRenderUtilities->InitialiseImpl();

		if (!PhysicsAssetRenderUtilities->PhysicsAssetRenderInterface)
		{
			PhysicsAssetRenderUtilities->PhysicsAssetRenderInterface = new FPhysicsAssetRenderInterface;
		}

		IModularFeatures::Get().RegisterModularFeature(IPhysicsAssetRenderInterface::GetModularFeatureName(), 
			PhysicsAssetRenderUtilities->PhysicsAssetRenderInterface);
	}
}

void UPhysicsAssetRenderUtilities::OnAssetRenamed(FAssetData const& AssetInfo, const FString& InOldPhysicsAssetPathName)
{
	FPhysicsAssetRenderSettings Temp;

	const uint32 OldPhysicsAssetPathNameHash = GetPathNameHash(InOldPhysicsAssetPathName);

	if (IdToSettingsMap.RemoveAndCopyValue(OldPhysicsAssetPathNameHash, Temp))
	{
		const uint32 PhysicsAssetPathNameHash = GetPathNameHash(AssetInfo.GetObjectPathString());
		IdToSettingsMap.Add(PhysicsAssetPathNameHash, Temp);
		SaveConfig();
	}
}

void UPhysicsAssetRenderUtilities::OnAssetRemoved(UObject* Object)
{
	if (Object)
	{
		const uint32 PhysicsAssetPathNameHash = GetPathNameHash(Object->GetPathName());
		IdToSettingsMap.Remove(PhysicsAssetPathNameHash);
	}
}

FPhysicsAssetRenderSettings* UPhysicsAssetRenderUtilities::GetSettings(const UPhysicsAsset* InPhysicsAsset)
{
	const uint32 PhysicsAssetPathNameHash = GetPathNameHash(InPhysicsAsset);

	if (PhysicsAssetPathNameHash)
	{
		return GetSettings(PhysicsAssetPathNameHash);
	}

	return nullptr;
}

FPhysicsAssetRenderSettings* UPhysicsAssetRenderUtilities::GetSettings(const FString& InPhysicsAssetPathName)
{
	const uint32 PhysicsAssetPathNameHash = GetPathNameHash(InPhysicsAssetPathName);

	if (PhysicsAssetPathNameHash)
	{
		return GetSettings(PhysicsAssetPathNameHash);
	}

	return nullptr;
}

FPhysicsAssetRenderSettings* UPhysicsAssetRenderUtilities::GetSettings(const uint32 InPhysicsAssetPathNameHash)
{
	if (UPhysicsAssetRenderUtilities* PhysicsAssetRenderUtilities = GetMutableDefault<UPhysicsAssetRenderUtilities>())
	{
		return PhysicsAssetRenderUtilities->GetSettingsImpl(InPhysicsAssetPathNameHash);
	}

	return nullptr;
}

void UPhysicsAssetRenderUtilities::InitialiseImpl()
{
	LoadConfig();

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	AssetRegistryModule.Get().OnAssetRenamed().AddUObject(this, &UPhysicsAssetRenderUtilities::OnAssetRenamed);
	AssetRegistryModule.Get().OnInMemoryAssetDeleted().AddUObject(this, &UPhysicsAssetRenderUtilities::OnAssetRemoved);

	BoneUnselectedMaterial = LoadObject<UMaterialInterface>(NULL, TEXT("/Engine/EditorMaterials/PhAT_UnselectedMaterial.PhAT_UnselectedMaterial"), NULL, LOAD_None, NULL);
	BoneNoCollisionMaterial = LoadObject<UMaterialInterface>(NULL, TEXT("/Engine/EditorMaterials/PhAT_NoCollisionMaterial.PhAT_NoCollisionMaterial"), NULL, LOAD_None, NULL);
}

FPhysicsAssetRenderSettings* UPhysicsAssetRenderUtilities::GetSettingsImpl(const uint32 InPhysicsAssetPathNameHash)
{
	FPhysicsAssetRenderSettings* Settings = IdToSettingsMap.Find(InPhysicsAssetPathNameHash);

	if (!Settings)
	{
		FPhysicsAssetRenderSettings& NewSettings = IdToSettingsMap.Add(InPhysicsAssetPathNameHash, FPhysicsAssetRenderSettings());
		NewSettings.InitPhysicsAssetRenderSettings(BoneUnselectedMaterial, BoneNoCollisionMaterial);
		Settings = &NewSettings;
	}

	return Settings;
}

uint32 UPhysicsAssetRenderUtilities::GetPathNameHash(const UPhysicsAsset* InPhysicsAsset)
{
	if (InPhysicsAsset)
	{
		return GetPathNameHash(InPhysicsAsset->GetPathName());
	}

	return 0;
}
	
uint32 UPhysicsAssetRenderUtilities::GetPathNameHash(const FString& InPhysicsAssetPathName)
{
	return TextKeyUtil::HashString(InPhysicsAssetPathName);
}

// class FPhysicsAssetRenderInterface
void FPhysicsAssetRenderInterface::DebugDraw(USkeletalMeshComponent* const SkeletalMeshComponent, UPhysicsAsset* const PhysicsAsset, FPrimitiveDrawInterface* PDI)
{
	PhysicsAssetRender::DebugDraw(SkeletalMeshComponent, PhysicsAsset, PDI);
}

void FPhysicsAssetRenderInterface::DebugDrawBodies(USkeletalMeshComponent* const SkeletalMeshComponent, UPhysicsAsset* const PhysicsAsset, FPrimitiveDrawInterface* PDI, const FColor& PrimitiveColorOverride)
{
	auto TransformFn = [](const UPhysicsAsset* PhysicsAsset, const FTransform& BoneTM, const int32 BodyIndex, const EAggCollisionShape::Type PrimType, const int32 PrimIndex, const float Scale) { return PhysicsAssetRender::GetPrimitiveTransform(PhysicsAsset, BoneTM, BodyIndex, PrimType, PrimIndex, Scale);  };
	auto HitProxyFn  = [](const int32 BodyIndex, const EAggCollisionShape::Type PrimitiveType, const int32 PrimitiveIndex) { return nullptr; };

	auto MaterialFn = [](const int32 BodyIndex, const EAggCollisionShape::Type PrimitiveType, const int32 PrimitiveIndex, const FPhysicsAssetRenderSettings& Settings)
	{ 
		return PhysicsAssetRender::GetPrimitiveMaterial(BodyIndex, PrimitiveType, PrimitiveIndex, Settings); 
	};

	auto ColorFn = [&PrimitiveColorOverride](const int32 BodyIndex, const EAggCollisionShape::Type PrimitiveType, const int32 PrimitiveIndex, const FPhysicsAssetRenderSettings& Settings)
	{ 
		if (PrimitiveColorOverride.A > (1.0f - KINDA_SMALL_NUMBER))
		{
			return PrimitiveColorOverride;
		}

		const FLinearColor LinearOverride = PrimitiveColorOverride.ReinterpretAsLinear();
		return FMath::Lerp(PhysicsAssetRender::GetPrimitiveColor(BodyIndex, PrimitiveType, PrimitiveIndex, Settings).ReinterpretAsLinear(), LinearOverride, LinearOverride.A).QuantizeRound();
	};

	PhysicsAssetRender::DebugDrawBodies(SkeletalMeshComponent, PhysicsAsset, PDI, ColorFn, MaterialFn, TransformFn, HitProxyFn);
}

void FPhysicsAssetRenderInterface::DebugDrawConstraints(USkeletalMeshComponent* const SkeletalMeshComponent, UPhysicsAsset* const PhysicsAsset, FPrimitiveDrawInterface* PDI)
{
	auto HitProxyFn   = [](const int32) { return nullptr; };
	PhysicsAssetRender::DebugDrawConstraints(SkeletalMeshComponent, PhysicsAsset, PDI, PhysicsAssetRender::IsSelectedFn(), false, HitProxyFn);
}

void FPhysicsAssetRenderInterface::SaveConfig()
{
	if (UPhysicsAssetRenderUtilities* PhysicsAssetRenderUtilities = GetMutableDefault<UPhysicsAssetRenderUtilities>())
	{
		PhysicsAssetRenderUtilities->SaveConfig();
	}
}

void FPhysicsAssetRenderInterface::ToggleShowAllBodies(class UPhysicsAsset* const PhysicsAsset)
{
	if (FPhysicsAssetRenderSettings* PhysicsAssetRenderSettings = UPhysicsAssetRenderUtilities::GetSettings(PhysicsAsset))
	{
		PhysicsAssetRenderSettings->ToggleShowAllBodies(PhysicsAsset);
	}
}

void FPhysicsAssetRenderInterface::ToggleShowAllConstraints(class UPhysicsAsset* const PhysicsAsset)
{
	if (FPhysicsAssetRenderSettings* PhysicsAssetRenderSettings = UPhysicsAssetRenderUtilities::GetSettings(PhysicsAsset))
	{
		PhysicsAssetRenderSettings->ToggleShowAllConstraints(PhysicsAsset);
	}
}

bool FPhysicsAssetRenderInterface::AreAnyBodiesHidden(class UPhysicsAsset* const PhysicsAsset)
{
	if (FPhysicsAssetRenderSettings* PhysicsAssetRenderSettings = UPhysicsAssetRenderUtilities::GetSettings(PhysicsAsset))
	{
		return PhysicsAssetRenderSettings->AreAnyBodiesHidden();
	}

	return false;
}

bool FPhysicsAssetRenderInterface::AreAnyConstraintsHidden(class UPhysicsAsset* const PhysicsAsset)
{
	if (FPhysicsAssetRenderSettings* PhysicsAssetRenderSettings = UPhysicsAssetRenderUtilities::GetSettings(PhysicsAsset))
	{
		return PhysicsAssetRenderSettings->AreAnyConstraintsHidden();
	}

	return false;
}

EConstraintTransformComponentFlags FPhysicsAssetRenderInterface::GetConstraintViewportManipulationFlags(class UPhysicsAsset* const PhysicsAsset)
{
	if (FPhysicsAssetRenderSettings* PhysicsAssetRenderSettings = UPhysicsAssetRenderUtilities::GetSettings(PhysicsAsset))
	{
		return PhysicsAssetRenderSettings->GetConstraintViewportManipulationFlags();
	}

	return EConstraintTransformComponentFlags::None;
}

bool FPhysicsAssetRenderInterface::IsDisplayingConstraintTransformComponentRelativeToDefault(class UPhysicsAsset* const PhysicsAsset, const EConstraintTransformComponentFlags ComponentFlags)
{
	if (FPhysicsAssetRenderSettings* PhysicsAssetRenderSettings = UPhysicsAssetRenderUtilities::GetSettings(PhysicsAsset))
	{
		return PhysicsAssetRenderSettings->IsDisplayingConstraintTransformComponentRelativeToDefault(ComponentFlags);
	}

	return false;
}

void FPhysicsAssetRenderInterface::SetDisplayConstraintTransformComponentRelativeToDefault(class UPhysicsAsset* const PhysicsAsset, const EConstraintTransformComponentFlags ComponentFlags, const bool bShouldDisplayRelativeToDefault)
{
	if (FPhysicsAssetRenderSettings* PhysicsAssetRenderSettings = UPhysicsAssetRenderUtilities::GetSettings(PhysicsAsset))
	{
		PhysicsAssetRenderSettings->SetDisplayConstraintTransformComponentRelativeToDefault(ComponentFlags, bShouldDisplayRelativeToDefault);
	}
}

bool IsBodyKinematic(const UPhysicsAsset* const PhysicsAsset, const int32 BodyIndex)
{
	if (PhysicsAsset && PhysicsAsset->SkeletalBodySetups.IsValidIndex(BodyIndex))
	{
		return PhysicsAsset->SkeletalBodySetups[BodyIndex]->PhysicsType == EPhysicsType::PhysType_Kinematic;
	}

	return false;
}

#undef LOCTEXT_NAMESPACE
