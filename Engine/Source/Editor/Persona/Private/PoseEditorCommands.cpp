// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseEditorCommands.h"

#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"

#define LOCTEXT_NAMESPACE "PoseEditorCommands"

void FPoseEditorCommands::RegisterCommands()
{
	UI_COMMAND(PasteAllNames, "Paste All Pose Names", "Paste all pose names from clipboard", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(UpdatePoseToCurrent, "Update Pose to Current", "Updates the selected pose to match the pose currently shown in the viewport", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(AddPoseFromCurrent, "Add Pose from Current", "Adds a new pose matching the pose currently shown in the viewport", EUserInterfaceActionType::Button, FInputChord());	
	UI_COMMAND(AddPoseFromReference, "Add Pose from Reference", "Adds a new pose matching the Skeleton its reference pose", EUserInterfaceActionType::Button, FInputChord());
}

#undef LOCTEXT_NAMESPACE
