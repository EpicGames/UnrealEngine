// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "HAL/Platform.h"
#include "Internationalization/Internationalization.h"
#include "Styling/AppStyle.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

class FUICommandInfo;

/**
* Class containing commands for pose editor
*/
class FPoseEditorCommands : public TCommands<FPoseEditorCommands>
{
public:
	FPoseEditorCommands()
	: TCommands<FPoseEditorCommands>
		(
			TEXT("PoseEditor"), // Context name for fast lookup
			NSLOCTEXT("Contexts", "PoseEditor", "Pose Editor"), // Localized context name for displaying
			NAME_None, // Parent context name.  
			FAppStyle::GetAppStyleSetName() // Icon Style Set
		)
	{}

	/** Initialize commands */
	virtual void RegisterCommands() override;

	/** Paste all names */
	TSharedPtr< FUICommandInfo > PasteAllNames;

	/** Update selected pose to match viewport */
	TSharedPtr< FUICommandInfo > UpdatePoseToCurrent;

	TSharedPtr< FUICommandInfo > AddPoseFromCurrent;	
	TSharedPtr< FUICommandInfo > AddPoseFromReference;
};


