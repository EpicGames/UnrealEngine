// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseWatchManagerColumnColor.h"
#include "Framework/Application/SlateApplication.h"
#include "Layout/WidgetPath.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Styling/AppStyle.h"
#include "Editor.h"
#include "Widgets/Views/STreeView.h"
#include "IPoseWatchManager.h"
#include "Widgets/Colors/SColorPicker.h"
#include "PoseWatchManagerElementTreeItem.h"
#include "PoseWatchManagerPoseWatchTreeItem.h"
#include "Engine/PoseWatch.h"

#define LOCTEXT_NAMESPACE "PoseWatchManagerColumnColor"

class SColorBoxWidget : public SBox
{
public:
	SLATE_BEGIN_ARGS(SColorBoxWidget) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, IPoseWatchManagerTreeItem* InTreeItem)
	{
		TreeItem = InTreeItem;

		ChildSlot
		[
			SNew(SBox)
			.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
			.VAlign(VAlign_Center)
			[
				SAssignNew(ColorWidgetBackgroundBorder, SBorder)
				.Padding(1.f)
				.BorderImage(FAppStyle::Get().GetBrush("ColorPicker.RoundedSolidBackground"))
				.BorderBackgroundColor(this, &SColorBoxWidget::GetColorWidgetBorderColor)
				.VAlign(VAlign_Center)
				[
					SNew(SOverlay)
					+ SOverlay::Slot()
					.VAlign(VAlign_Center)
					[
						SAssignNew(ColorPickerParentWidget, SColorBlock)
						.AlphaBackgroundBrush(FAppStyle::Get().GetBrush("ColorPicker.RoundedAlphaBackground"))
						.Color(this, &SColorBoxWidget::OnGetColorForColorBlock)
						.ShowBackgroundForAlpha(true)
						.AlphaDisplayMode(EColorBlockAlphaDisplayMode::Ignore)
						.OnMouseButtonDown(this, &SColorBoxWidget::OnMouseButtonDownColorBlock)
						.Size(FVector2D(16.0f, 16.0f))
						.CornerRadius(FVector4(4.0f, 4.0f, 4.0f, 4.0f))
					]
				]
			]
		];
	}

	FLinearColor OnGetColorForColorBlock() const
	{
		return TreeItem->GetColor();
	}

	void OnSetColorFromColorPicker(FLinearColor NewColor)
	{
		TreeItem->SetColor(NewColor.ToFColorSRGB());
	}

	void OnColorPickerCancelled(FLinearColor OriginalColor)
	{
		TreeItem->SetColor(OriginalColor.ToFColorSRGB());
	}

	FReply OnMouseButtonDownColorBlock(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		FColorPickerArgs PickerArgs;
		{
			PickerArgs.bUseAlpha = false;
			PickerArgs.DisplayGamma = TAttribute<float>::Create(TAttribute<float>::FGetter::CreateUObject(GEngine, &UEngine::GetDisplayGamma));
			PickerArgs.OnColorCommitted = FOnLinearColorValueChanged::CreateSP(this, &SColorBoxWidget::OnSetColorFromColorPicker);
			PickerArgs.OnColorPickerCancelled = FOnColorPickerCancelled::CreateSP(this, &SColorBoxWidget::OnColorPickerCancelled);
			PickerArgs.InitialColor = TreeItem->GetColor();
			PickerArgs.ParentWidget = ColorPickerParentWidget;
			PickerArgs.OptionalOwningDetailsView = ColorPickerParentWidget;
			FWidgetPath ParentWidgetPath;
			if (FSlateApplication::Get().FindPathToWidget(ColorPickerParentWidget.ToSharedRef(), ParentWidgetPath))
			{
				PickerArgs.bOpenAsMenu = FSlateApplication::Get().FindMenuInWidgetPath(ParentWidgetPath).IsValid();
			}
		}

		OpenColorPicker(PickerArgs);

		return FReply::Handled();
	}

	FSlateColor GetColorWidgetBorderColor() const
	{
		static const FSlateColor HoveredColor = FAppStyle::Get().GetSlateColor("Colors.Hover");
		static const FSlateColor DefaultColor = FAppStyle::Get().GetSlateColor("Colors.InputOutline");
		return ColorWidgetBackgroundBorder->IsHovered() ? HoveredColor : DefaultColor;
	}

private:
	IPoseWatchManagerTreeItem* TreeItem;
	TSharedPtr<SWidget> ColorPickerParentWidget;
	TSharedPtr<SWidget > ColorWidgetBackgroundBorder;
};

SHeaderRow::FColumn::FArguments FPoseWatchManagerColumnColor::ConstructHeaderRowColumn()
{
	return SHeaderRow::Column(GetColumnID())
		.FixedWidth(24.f)
		.HAlignHeader(HAlign_Left)
		.VAlignHeader(VAlign_Center)
		.HAlignCell(HAlign_Center)
		.VAlignCell(VAlign_Center)
		.DefaultTooltip(FText::FromName(GetColumnID()))
		.HeaderContentPadding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
		[
			SNew(SImage)
			.ColorAndOpacity(FSlateColor::UseForeground())
			.Image(FAppStyle::Get().GetBrush("ColorPicker.Mode"))
		];
}


const TSharedRef<SWidget> FPoseWatchManagerColumnColor::ConstructRowWidget(FPoseWatchManagerTreeItemRef TreeItem, const STableRow<FPoseWatchManagerTreeItemPtr>& Row)
{
	if (TreeItem->ShouldDisplayColorPicker())
	{
		return SNew(SColorBoxWidget, TreeItem->CastTo<IPoseWatchManagerTreeItem>());
	}
	return SNullWidget::NullWidget;
}

#undef LOCTEXT_NAMESPACE

