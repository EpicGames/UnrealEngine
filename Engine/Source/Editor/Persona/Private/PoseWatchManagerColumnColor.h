// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "Widgets/Views/SHeaderRow.h"
#include "PoseWatchManagerPublicTypes.h"
#include "IPoseWatchManagerColumn.h"
#include "IPoseWatchManagerTreeItem.h"

template<typename ItemType> class STableRow;

/**
 * A PoseWatchManager column which handles setting and visualizing a pose watch's color
 */
class FPoseWatchManagerColumnColor : public IPoseWatchManagerColumn
{
public:
	FPoseWatchManagerColumnColor() {};
	virtual ~FPoseWatchManagerColumnColor() = default;

	static FName GetID() { return FPoseWatchManagerBuiltInColumnTypes::Color(); }

	// Begin IPoseWatchManagerColumn Implementation
	virtual FName GetColumnID() override { return GetID();  }
	virtual SHeaderRow::FColumn::FArguments ConstructHeaderRowColumn() override;
	virtual const TSharedRef<SWidget> ConstructRowWidget(FPoseWatchManagerTreeItemRef TreeItem, const STableRow<FPoseWatchManagerTreeItemPtr>& Row) override;
	virtual bool SupportsSorting() const override { return false; }
	// End IPoseWatchManagerColumn Implementation
};
