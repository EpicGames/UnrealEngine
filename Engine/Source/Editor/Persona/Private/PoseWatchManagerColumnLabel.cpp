// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseWatchManagerColumnLabel.h"
#include "PoseWatchManagerFwd.h"
#include "PoseWatchManagerStandaloneTypes.h"
#include "IPoseWatchManagerTreeItem.h"
#include "IPoseWatchManager.h"

#define LOCTEXT_NAMESPACE "PoseWatchManagerColumnLabel"

SHeaderRow::FColumn::FArguments FPoseWatchManagerItemLabelColumn::ConstructHeaderRowColumn()
{
	return SHeaderRow::Column(GetColumnID())
		.FillWidth(5.0f);
}
const TSharedRef<SWidget> FPoseWatchManagerItemLabelColumn::ConstructRowWidget(FPoseWatchManagerTreeItemRef TreeItem, const STableRow<FPoseWatchManagerTreeItemPtr>& Row)
{
	IPoseWatchManager* PoseWatchManager = WeakPoseWatchManager.Pin().Get();
	check(PoseWatchManager);
	return TreeItem->GenerateLabelWidget(*PoseWatchManager, Row);
}

void FPoseWatchManagerItemLabelColumn::PopulateSearchStrings(const IPoseWatchManagerTreeItem& Item, TArray< FString >& OutSearchStrings) const
{
	OutSearchStrings.Add(Item.GetDisplayString());
}

#undef LOCTEXT_NAMESPACE
