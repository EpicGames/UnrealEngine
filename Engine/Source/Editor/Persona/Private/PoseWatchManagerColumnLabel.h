// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "Widgets/Views/SHeaderRow.h"
#include "IPoseWatchManager.h"
#include "PoseWatchManagerPublicTypes.h"
#include "PoseWatchManagerStandaloneTypes.h"
#include "IPoseWatchManagerColumn.h"

template<typename ItemType> class STableRow;

/** A column for the PoseWatchManager that displays the item label */
class FPoseWatchManagerItemLabelColumn : public IPoseWatchManagerColumn
{

public:
	FPoseWatchManagerItemLabelColumn(IPoseWatchManager& PoseWatchManager)
		: WeakPoseWatchManager(StaticCastSharedRef<IPoseWatchManager>(PoseWatchManager.AsShared()))
	{}

	static FName GetID() { return FPoseWatchManagerBuiltInColumnTypes::Label(); }

	// Begin IPoseWatchManagerColumn Implementation
	virtual FName GetColumnID() override { return GetID(); }
	virtual SHeaderRow::FColumn::FArguments ConstructHeaderRowColumn() override;
	virtual const TSharedRef< SWidget > ConstructRowWidget(FPoseWatchManagerTreeItemRef TreeItem, const STableRow<FPoseWatchManagerTreeItemPtr>& Row) override;
	virtual void PopulateSearchStrings(const IPoseWatchManagerTreeItem& Item, TArray< FString >& OutSearchStrings) const override;
	// End IPoseWatchManagerColumn Implementation

private:
	TWeakPtr<IPoseWatchManager> WeakPoseWatchManager;
};
