// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseWatchManagerColumnVisibility.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Images/SImage.h"
#include "Styling/AppStyle.h"
#include "Editor.h"
#include "Widgets/Views/STreeView.h"
#include "IPoseWatchManager.h"
#include "PoseWatchManagerPoseWatchTreeItem.h"

#define LOCTEXT_NAMESPACE "PoseWatchManagerColumnVisibility"

namespace PoseWatchManager
{
	class SVisibilityWidget : public SImage
	{
	public:
		SLATE_BEGIN_ARGS(SVisibilityWidget) {}
		SLATE_END_ARGS()

			void Construct(const FArguments& InArgs, TWeakPtr<FPoseWatchManagerColumnVisibility> InWeakColumn, TWeakPtr<IPoseWatchManager> InWeakPoseWatchManager, TWeakPtr<IPoseWatchManagerTreeItem> InWeakTreeItem, const STableRow<FPoseWatchManagerTreeItemPtr>* InRow)
		{
			WeakTreeItem = InWeakTreeItem;
			WeakPoseWatchManager = InWeakPoseWatchManager;
			WeakColumn = InWeakColumn;

			Row = InRow;

			SImage::Construct(
				SImage::FArguments()
				.ColorAndOpacity(this, &SVisibilityWidget::GetForegroundColor)
				.Image(this, &SVisibilityWidget::GetBrush)
			);

			static const FName NAME_VisibleHoveredBrush = TEXT("Level.VisibleHighlightIcon16x");
			static const FName NAME_VisibleNotHoveredBrush = TEXT("Level.VisibleIcon16x");
			static const FName NAME_NotVisibleHoveredBrush = TEXT("Level.NotVisibleHighlightIcon16x");
			static const FName NAME_NotVisibleNotHoveredBrush = TEXT("Level.NotVisibleIcon16x");

			VisibleHoveredBrush = FAppStyle::Get().GetBrush(NAME_VisibleHoveredBrush);
			VisibleNotHoveredBrush = FAppStyle::Get().GetBrush(NAME_VisibleNotHoveredBrush);

			NotVisibleHoveredBrush = FAppStyle::Get().GetBrush(NAME_NotVisibleHoveredBrush);
			NotVisibleNotHoveredBrush = FAppStyle::Get().GetBrush(NAME_NotVisibleNotHoveredBrush);
		}

	private:
		FReply HandleClick()
		{
			TSharedPtr<IPoseWatchManagerTreeItem> TreeItem = WeakTreeItem.Pin();
			TreeItem->SetIsVisible(!TreeItem->GetVisibility());
			return FReply::Handled();
		}

		/** Called when the mouse button is pressed down on this widget */
		virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
		{
			if (MouseEvent.GetEffectingButton() != EKeys::LeftMouseButton)
			{
				return FReply::Unhandled();
			}

			return HandleClick();
		}

		/** Get the brush for this widget */
		const FSlateBrush* GetBrush() const
		{
			if (IsVisible())
			{
				return IsHovered() ? VisibleHoveredBrush : VisibleNotHoveredBrush;
			}
			else
			{
				return IsHovered() ? NotVisibleHoveredBrush : NotVisibleNotHoveredBrush;
			}
		}

		virtual FSlateColor GetForegroundColor() const
		{
			auto TreeItem = WeakTreeItem.Pin();
			const bool bIsSelected = WeakPoseWatchManager.Pin()->GetTree().IsItemSelected(TreeItem.ToSharedRef());

			if (!TreeItem->IsEnabled())
			{
				return FPoseWatchManagerCommonLabelData::DisabledColor;
			}

			if (IsHovered() && !bIsSelected)
			{
				return FAppStyle::Get().GetSlateColor("Colors.ForegroundHover");
			}

			return FSlateColor::UseForeground();
		}

		static bool IsVisible(const FPoseWatchManagerTreeItemPtr& Item, const TSharedPtr<FPoseWatchManagerColumnVisibility>& Column)
		{
			return (Column.IsValid() && Item && Item.IsValid()) ? Item->GetVisibility() : false;
		}

		bool IsVisible() const
		{
			TSharedPtr<FPoseWatchManagerColumnVisibility> Column = WeakColumn.Pin();
			TSharedPtr<IPoseWatchManagerTreeItem> TreeItem = WeakTreeItem.Pin();
			return (TreeItem.IsValid() && Column.IsValid()) ? IsVisible(TreeItem, Column) : false;
		}

		void SetIsVisible(const bool bVisible)
		{
			TSharedPtr<IPoseWatchManagerTreeItem> TreeItem = WeakTreeItem.Pin();
			TreeItem.Get()->SetIsVisible(bVisible);
			WeakPoseWatchManager.Pin()->Refresh();
		}

		/** The tree item we relate to */
		TWeakPtr<IPoseWatchManagerTreeItem> WeakTreeItem;

		TWeakPtr<IPoseWatchManager> WeakPoseWatchManager;

		/** Weak pointer back to the column */
		TWeakPtr<FPoseWatchManagerColumnVisibility> WeakColumn;

		/** Weak pointer back to the row */
		const STableRow<FPoseWatchManagerTreeItemPtr>* Row;

		/** Visibility brushes for the various states */
		const FSlateBrush* VisibleHoveredBrush;
		const FSlateBrush* VisibleNotHoveredBrush;
		const FSlateBrush* NotVisibleHoveredBrush;
		const FSlateBrush* NotVisibleNotHoveredBrush;
	};
}

SHeaderRow::FColumn::FArguments FPoseWatchManagerColumnVisibility::ConstructHeaderRowColumn()
{
	return SHeaderRow::Column(GetColumnID())
		.FixedWidth(24.f)
		.HAlignHeader(HAlign_Left)
		.VAlignHeader(VAlign_Center)
		.HAlignCell(HAlign_Center)
		.VAlignCell(VAlign_Center)
		.DefaultTooltip(FText::FromName(GetColumnID()))
		.HeaderContentPadding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
		[
			SNew(SImage)
			.ColorAndOpacity(FSlateColor::UseForeground())
			.Image(FAppStyle::Get().GetBrush("Level.VisibleIcon16x"))
		];
}

const TSharedRef<SWidget> FPoseWatchManagerColumnVisibility::ConstructRowWidget(FPoseWatchManagerTreeItemRef TreeItem, const STableRow<FPoseWatchManagerTreeItemPtr>& Row)
{
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(PoseWatchManager::SVisibilityWidget, SharedThis(this), WeakPoseWatchManager, TreeItem, &Row)
		];
}

#undef LOCTEXT_NAMESPACE

