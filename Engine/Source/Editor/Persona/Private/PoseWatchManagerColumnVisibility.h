// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "Widgets/Views/SHeaderRow.h"
#include "PoseWatchManagerPublicTypes.h"
#include "IPoseWatchManagerColumn.h"
#include "IPoseWatchManager.h"

template<typename ItemType> class STableRow;

/**
 * A PoseWatchManager column which handles setting and visualizing a pose watch's visibility
 */
class FPoseWatchManagerColumnVisibility : public IPoseWatchManagerColumn
{
public:
	FPoseWatchManagerColumnVisibility(IPoseWatchManager& PoseWatchManager)
		: WeakPoseWatchManager(StaticCastSharedRef<IPoseWatchManager>(PoseWatchManager.AsShared()))
	{}

	// Begin IPoseWatchManagerColumn Implementation
	static FName GetID() { return FPoseWatchManagerBuiltInColumnTypes::Visibility(); }
	virtual FName GetColumnID() override { return GetID(); }
	virtual SHeaderRow::FColumn::FArguments ConstructHeaderRowColumn() override;
	virtual const TSharedRef< SWidget > ConstructRowWidget(FPoseWatchManagerTreeItemRef TreeItem, const STableRow<FPoseWatchManagerTreeItemPtr>& Row) override;
	// End IPoseWatchManagerColumn Implementation

private:
	TWeakPtr<IPoseWatchManager> WeakPoseWatchManager;
};
