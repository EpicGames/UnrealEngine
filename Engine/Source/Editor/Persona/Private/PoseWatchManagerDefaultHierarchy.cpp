// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseWatchManagerDefaultHierarchy.h"
#include "PoseWatchManagerElementTreeItem.h"
#include "PoseWatchManagerFolderTreeItem.h"
#include "PoseWatchManagerPoseWatchTreeItem.h"
#include "PoseWatchManagerDefaultMode.h"
#include "EditorFolderUtils.h"
#include "Engine/PoseWatch.h"
#include "Animation/AnimBlueprint.h"
#include "SPoseWatchManager.h"

FPoseWatchManagerDefaultHierarchy::FPoseWatchManagerDefaultHierarchy(FPoseWatchManagerDefaultMode* InMode) : Mode(InMode) {}

FPoseWatchManagerTreeItemPtr FPoseWatchManagerDefaultHierarchy::FindParent(const IPoseWatchManagerTreeItem& Item, const TMap<FObjectKey, FPoseWatchManagerTreeItemPtr>& Items) const
{
	FObjectKey ParentIdentifier = nullptr;

	if (const FPoseWatchManagerFolderTreeItem* FolderTreeItem = Item.CastTo<FPoseWatchManagerFolderTreeItem>())
	{
		ParentIdentifier = FolderTreeItem->PoseWatchFolder->GetParent();
	}
	else if (const FPoseWatchManagerPoseWatchTreeItem* PoseWatchTreeItem = Item.CastTo<FPoseWatchManagerPoseWatchTreeItem>())
	{
		ParentIdentifier = PoseWatchTreeItem->PoseWatch->GetParent();
	}
	else if (const FPoseWatchManagerElementTreeItem* ElementTreeItem = Item.CastTo<FPoseWatchManagerElementTreeItem>())
	{
		ParentIdentifier = ElementTreeItem->PoseWatchElement->GetParent();
	}

	const FPoseWatchManagerTreeItemPtr* ParentTreeItem = Items.Find(ParentIdentifier);
	if (ParentTreeItem)
	{
		return *ParentTreeItem;
	}

	return nullptr;
}

void FPoseWatchManagerDefaultHierarchy::CreateItems(TArray<FPoseWatchManagerTreeItemPtr>& OutItems) const
{
	UAnimBlueprint* AnimBlueprint = Mode->PoseWatchManager->AnimBlueprint;
	for (int32 Index = 0; Index < AnimBlueprint->PoseWatchFolders.Num(); ++Index)
	{
		TObjectPtr<UPoseWatchFolder>& PoseWatchFolder = AnimBlueprint->PoseWatchFolders[Index];
		if (PoseWatchFolder)
		{
			FPoseWatchManagerTreeItemPtr PoseWatchFolderItem = Mode->PoseWatchManager->CreateItemFor<FPoseWatchManagerFolderTreeItem>(PoseWatchFolder);
			OutItems.Add(PoseWatchFolderItem);
		}
		else
		{
			AnimBlueprint->PoseWatchFolders.RemoveAtSwap(Index);
		}
	}
	for (int32 Index = 0; Index < AnimBlueprint->PoseWatches.Num(); ++Index)
	{
		TObjectPtr<UPoseWatch>& PoseWatch = AnimBlueprint->PoseWatches[Index];
		if (PoseWatch)
		{
			if (!PoseWatch->GetShouldDeleteOnDeselect())
			{
				FPoseWatchManagerTreeItemPtr PoseWatchItem = Mode->PoseWatchManager->CreateItemFor<FPoseWatchManagerPoseWatchTreeItem>(PoseWatch);

				OutItems.Add(PoseWatchItem);

				for (const TObjectPtr<UPoseWatchElement>& CurrentElement : PoseWatch->GetElements())
				{
					FPoseWatchManagerTreeItemPtr ElementItem = Mode->PoseWatchManager->CreateItemFor<FPoseWatchManagerElementTreeItem>(CurrentElement);
					OutItems.Add(ElementItem);
				}
			}
		}
		else
		{
			AnimBlueprint->PoseWatches.RemoveAtSwap(Index);
		}
	}
}