// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PoseWatchManagerFwd.h"
#include "IPoseWatchManagerTreeItem.h"

class FPoseWatchManagerDefaultMode;

class FPoseWatchManagerDefaultHierarchy
{
public:
	FPoseWatchManagerDefaultHierarchy(FPoseWatchManagerDefaultMode* InMode);

	FPoseWatchManagerTreeItemPtr FindParent(const IPoseWatchManagerTreeItem& Item, const TMap<FObjectKey, FPoseWatchManagerTreeItemPtr>& Items) const;

	void CreateItems(TArray<FPoseWatchManagerTreeItemPtr>& OutItems) const;

	FPoseWatchManagerDefaultMode* Mode;
};