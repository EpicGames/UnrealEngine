// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseWatchManagerDefaultMode.h"
#include "AnimationEditorUtils.h"
#include "Engine/PoseWatch.h"
#include "SGraphNode.h"
#include "SGraphPanel.h"
#include "IAnimationBlueprintEditor.h"
#include "PoseWatchManagerElementTreeItem.h"
#include "PoseWatchManagerFolderTreeItem.h"
#include "PoseWatchManagerPoseWatchTreeItem.h"
#include "SPoseWatchManager.h"

#define LOCTEXT_NAMESPACE "PoseWatchDefaultMode"

/** Functor which can be used to get weak actor pointers from a selection */
struct FWeakPoseWatchSelector
{
	bool operator()(const TWeakPtr<IPoseWatchManagerTreeItem>& Item, TWeakObjectPtr<UPoseWatch>& DataOut) const;
};

FPoseWatchManagerDefaultMode::FPoseWatchManagerDefaultMode(SPoseWatchManager* InPoseWatchManager)
	: PoseWatchManager(InPoseWatchManager)
{
	Rebuild();
}

void FPoseWatchManagerDefaultMode::Rebuild()
{
	Hierarchy = MakeUnique<FPoseWatchManagerDefaultHierarchy>(this);
}

bool FPoseWatchManagerDefaultMode::ParseDragDrop(FPoseWatchManagerDragDropPayload& OutPayload, const FDragDropOperation& Operation) const
{
	if (Operation.IsOfType<FPoseWatchManagerDragDropOp>())
	{
		const auto& OutlinerOp = static_cast<const FPoseWatchManagerDragDropOp&>(Operation);
		if (const auto& PoseWatchOp = OutlinerOp.GetSubOp<FPoseWatchDragDropOp>())
		{
			OutPayload.DraggedItem = PoseWatchManager->GetTreeItem(PoseWatchOp->PoseWatch.Get());
		}
		if (const auto& FolderOp = OutlinerOp.GetSubOp<FPoseWatchFolderDragDropOp>())
		{
			OutPayload.DraggedItem = PoseWatchManager->GetTreeItem(FolderOp->PoseWatchFolder.Get());
		}
		return true;
	}

	return false;
}

FPoseWatchManagerDragValidationInfo FPoseWatchManagerDefaultMode::ValidateDrop(const IPoseWatchManagerTreeItem& DropTarget, const FPoseWatchManagerDragDropPayload& Payload) const
{
	if (Payload.DraggedItem.IsValid())
	{
		TSharedPtr<IPoseWatchManagerTreeItem> PayloadItem = Payload.DraggedItem.Pin();

		// Pose watches cannot be a parent
		if (const FPoseWatchManagerPoseWatchTreeItem* PoseWatchDropTarget = DropTarget.CastTo<FPoseWatchManagerPoseWatchTreeItem>())
		{
			return FPoseWatchManagerDragValidationInfo(EPoseWatchManagerDropCompatibility::Incompatible, LOCTEXT("PoseWatchBadParent", "A pose watch cannot be a parent"));
		}

		if (const FPoseWatchManagerElementTreeItem* ElementDropTarget = DropTarget.CastTo<FPoseWatchManagerElementTreeItem>())
		{
			return FPoseWatchManagerDragValidationInfo(EPoseWatchManagerDropCompatibility::Incompatible, LOCTEXT("ElementBadParent", "A pose watch element cannot be a parent"));
		}

		// Drop target must either be a valid UPoseWatchFolder or nullptr denoting the root of the tree
		UPoseWatchFolder* TargetFolder = nullptr;
		FText TargetFolderLabel = LOCTEXT("Root", "root");
		if (DropTarget.IsValid())
		{
			if (const FPoseWatchManagerFolderTreeItem* DropTargetFolder = DropTarget.CastTo<FPoseWatchManagerFolderTreeItem>())
			{
				TargetFolder = DropTargetFolder->PoseWatchFolder.Get();
				if (TargetFolder)
				{
					TargetFolderLabel = TargetFolder->GetLabel();
				}
			}
		}

		if (FPoseWatchManagerPoseWatchTreeItem* PayloadPoseWatchItem = PayloadItem->CastTo<FPoseWatchManagerPoseWatchTreeItem>())
		{
			// Dropping a pose watch into a folder
			UPoseWatch* PayloadPoseWatch = PayloadPoseWatchItem->PoseWatch.Get();
			check(PayloadPoseWatch);

			if (PayloadPoseWatch->IsIn(TargetFolder))
			{
				FText ValidationText = FText::Format(LOCTEXT("PoseWatchAlreadyInDirectory", "This pose watch is already inside {0}"), TargetFolderLabel);
				return FPoseWatchManagerDragValidationInfo(EPoseWatchManagerDropCompatibility::Incompatible, ValidationText);
			}

			if (!PayloadPoseWatch->IsLabelUniqueInParent(PayloadPoseWatch->GetLabel(), TargetFolder))
			{
				FText ValidationText = FText::Format(LOCTEXT("PoseWatchAlreadyExistsInFolder", "A pose watch with that name already exists inside {0}"), TargetFolderLabel);
				return FPoseWatchManagerDragValidationInfo(EPoseWatchManagerDropCompatibility::Incompatible, ValidationText);
			}

			FText ValidationText = FText::Format(LOCTEXT("MovePoseWatchIntoFolder", "Move {0} into {1}"), PayloadPoseWatch->GetLabel(), TargetFolderLabel);
			return FPoseWatchManagerDragValidationInfo(EPoseWatchManagerDropCompatibility::Compatible, ValidationText);
		}
		else if (FPoseWatchManagerFolderTreeItem* PayloadFolderItem = PayloadItem->CastTo<FPoseWatchManagerFolderTreeItem>())
		{
			// Dropping a folder into a folder
			UPoseWatchFolder* PayloadFolder = PayloadFolderItem->PoseWatchFolder.Get();
			check(PayloadFolder);

			if (PayloadFolder == TargetFolder)
			{
				return FPoseWatchManagerDragValidationInfo(EPoseWatchManagerDropCompatibility::Incompatible, LOCTEXT("FolderInItself", "A folder cannot contain itself"));
			}

			if (PayloadFolder->IsIn(TargetFolder))
			{
				FText ValidationText = FText::Format(LOCTEXT("FolderAlreadyInDirectory", "This folder is already inside {0}"), TargetFolderLabel);
				return FPoseWatchManagerDragValidationInfo(EPoseWatchManagerDropCompatibility::Incompatible, ValidationText);
			}

			if (TargetFolder && TargetFolder->IsDescendantOf(PayloadFolder))
			{
				return FPoseWatchManagerDragValidationInfo(EPoseWatchManagerDropCompatibility::Incompatible, LOCTEXT("ParentFolderNotChildren", "Parent folders cannot be children"));
			}

			if (!PayloadFolder->IsLabelUniqueInParent(PayloadFolder->GetLabel(), TargetFolder))
			{
				FText ValidationText = FText::Format(LOCTEXT("FolderNameConflictInFolder", "A folder with that name already exists inside {0}"), TargetFolderLabel);
				return FPoseWatchManagerDragValidationInfo(EPoseWatchManagerDropCompatibility::Incompatible, ValidationText);
			}

			FText ValidationText = FText::Format(LOCTEXT("MoveFolderIntoFolder", "Move {0} into {1}"), PayloadFolder->GetLabel(), TargetFolderLabel);
			return FPoseWatchManagerDragValidationInfo(EPoseWatchManagerDropCompatibility::Compatible, ValidationText);
		}
	}

	return FPoseWatchManagerDragValidationInfo(EPoseWatchManagerDropCompatibility::Incompatible, FText());
}

void FPoseWatchManagerDefaultMode::OnDrop(IPoseWatchManagerTreeItem& DropTarget, const FPoseWatchManagerDragDropPayload& Payload, const FPoseWatchManagerDragValidationInfo& ValidationInfo) const
{
	check(Payload.DraggedItem.IsValid());
	check(DropTarget.IsA<FPoseWatchManagerFolderTreeItem>());
	const FPoseWatchManagerFolderTreeItem* FolderDropTarget = DropTarget.CastTo<FPoseWatchManagerFolderTreeItem>();
	const TWeakPtr<IPoseWatchManagerTreeItem> PayloadItem = Payload.DraggedItem;

	if (const FPoseWatchManagerFolderTreeItem* ChildFolderItem = PayloadItem.Pin()->CastTo<FPoseWatchManagerFolderTreeItem>())
	{
		ChildFolderItem->PoseWatchFolder->MoveTo(FolderDropTarget->PoseWatchFolder.Get());
		return;
	}
	if (const FPoseWatchManagerPoseWatchTreeItem* ChildPoseWatchItem = PayloadItem.Pin()->CastTo<FPoseWatchManagerPoseWatchTreeItem>())
	{
		ChildPoseWatchItem->PoseWatch->MoveTo(FolderDropTarget->PoseWatchFolder.Get());
		return;
	}
	check(false);
}

TSharedPtr<FDragDropOperation> FPoseWatchManagerDefaultMode::CreateDragDropOperation(const TArray<FPoseWatchManagerTreeItemPtr>& InTreeItems) const
{
	check(InTreeItems.Num() == 1)
	FPoseWatchManagerTreeItemPtr TreeItem = InTreeItems[0];

	FPoseWatchManagerDragDropPayload DraggedObjects(TreeItem);

	TSharedPtr<FPoseWatchManagerDragDropOp> OutlinerOp = MakeShareable(new FPoseWatchManagerDragDropOp());

	if (FPoseWatchManagerPoseWatchTreeItem* PoseWatchTreeItem = TreeItem->CastTo<FPoseWatchManagerPoseWatchTreeItem>())
	{
		TSharedPtr<FPoseWatchDragDropOp> Operation = MakeShareable(new FPoseWatchDragDropOp);
		Operation->Init(TWeakObjectPtr<UPoseWatch>(PoseWatchTreeItem->PoseWatch));
		OutlinerOp->AddSubOp(Operation);
	}

	else if (FPoseWatchManagerFolderTreeItem* FolderTreeItem = TreeItem->CastTo<FPoseWatchManagerFolderTreeItem>())
	{
		TSharedPtr<FPoseWatchFolderDragDropOp> Operation = MakeShareable(new FPoseWatchFolderDragDropOp);
		Operation->Init(TWeakObjectPtr<UPoseWatchFolder>(FolderTreeItem->PoseWatchFolder));
		OutlinerOp->AddSubOp(Operation);
	}

	OutlinerOp->Construct();
	return OutlinerOp;
}

FReply FPoseWatchManagerDefaultMode::OnDragOverItem(const FDragDropEvent& Event, const IPoseWatchManagerTreeItem& Item) const
{
	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE