// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PoseWatchManagerFwd.h"
#include "PoseWatchManagerDefaultHierarchy.h"

struct FPoseWatchManagerItemSelection;
class SPoseWatchManager;

class FPoseWatchManagerDefaultMode
{
public:
	FPoseWatchManagerDefaultMode(SPoseWatchManager* InPoseWatchManager);

	void Rebuild();

	bool ParseDragDrop(FPoseWatchManagerDragDropPayload& OutPayload, const FDragDropOperation& Operation) const;
	
	FPoseWatchManagerDragValidationInfo ValidateDrop(const IPoseWatchManagerTreeItem& DropTarget, const FPoseWatchManagerDragDropPayload& Payload) const;
	
	void OnDrop(IPoseWatchManagerTreeItem& DropTarget, const FPoseWatchManagerDragDropPayload& Payload, const FPoseWatchManagerDragValidationInfo& ValidationInfo) const;
	
	TSharedPtr<FDragDropOperation> CreateDragDropOperation(const TArray<FPoseWatchManagerTreeItemPtr>& InTreeItems) const;
	
	FReply OnDragOverItem(const FDragDropEvent& Event, const IPoseWatchManagerTreeItem& Item) const;

	int32 GetTypeSortPriority(const IPoseWatchManagerTreeItem& Item) const { return 0; }


	SPoseWatchManager* PoseWatchManager;

	TUniquePtr<FPoseWatchManagerDefaultHierarchy> Hierarchy;
};