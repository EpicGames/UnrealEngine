// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseWatchManagerDragDrop.h"

#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "Misc/Attribute.h"
#include "SlotBase.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWidget.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SPoseWatchManager"


FPoseWatchManagerDragDropOp::FPoseWatchManagerDragDropOp()
	: OverrideText()
	, OverrideIcon(nullptr)
{}

EVisibility FPoseWatchManagerDragDropOp::GetOverrideVisibility() const
{
	return OverrideText.IsEmpty() && OverrideIcon == nullptr ? EVisibility::Collapsed : EVisibility::Visible;
}

EVisibility FPoseWatchManagerDragDropOp::GetDefaultVisibility() const
{
	return OverrideText.IsEmpty() && OverrideIcon == nullptr ? EVisibility::Visible : EVisibility::Collapsed;
}

TSharedPtr<SWidget> FPoseWatchManagerDragDropOp::GetDefaultDecorator() const
{
	TSharedRef<SVerticalBox> VerticalBox = SNew(SVerticalBox);

	VerticalBox->AddSlot()
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("Graph.ConnectorFeedback.Border"))
		.Visibility(this, &FPoseWatchManagerDragDropOp::GetOverrideVisibility)
		.Content()
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.0f, 0.0f, 3.0f, 0.0f)
			[
				SNew(SImage)
				.Image(this, &FPoseWatchManagerDragDropOp::GetOverrideIcon)
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(this, &FPoseWatchManagerDragDropOp::GetOverrideText)
			]
		]
	];

	for (auto& SubOp : SubOps)
	{
		auto Content = SubOp->GetDefaultDecorator();
		if (Content.IsValid())
		{
			Content->SetVisibility(TAttribute<EVisibility>(this, &FPoseWatchManagerDragDropOp::GetDefaultVisibility));
			VerticalBox->AddSlot()[Content.ToSharedRef()];
		}
	}

	return VerticalBox;
}

#undef LOCTEXT_NAMESPACE
