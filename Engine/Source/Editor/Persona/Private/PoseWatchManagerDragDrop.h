// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "DragAndDrop/CompositeDragDropOp.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "Engine/PoseWatch.h"
#include "HAL/PlatformMath.h"
#include "Input/DragAndDrop.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#define UE_API PERSONA_API

class SWidget;
class UClass;
struct FSlateBrush;
struct IPoseWatchManagerTreeItem;

/** Enum to describe the compatibility of a drag drop operation */
enum class EPoseWatchManagerDropCompatibility : uint8
{
	Compatible,
	Incompatible
};

struct FPoseWatchManagerDragDropPayload
{
	FPoseWatchManagerDragDropPayload(const FDragDropOperation& InOperation = FDragDropOperation())
		: SourceOperation(InOperation)
	{}

	template<typename TreeType>
	FPoseWatchManagerDragDropPayload(const TreeType& InDraggedItem, const FDragDropOperation& InOperation = FDragDropOperation())
		: SourceOperation(InOperation)
	{
		DraggedItem = InDraggedItem;
	}

	TWeakPtr<IPoseWatchManagerTreeItem> DraggedItem;

	/** The source FDragDropOperation */
	const FDragDropOperation& SourceOperation;
};

/** Struct used for validation of a drag/drop operation in the pose watch manager */
struct FPoseWatchManagerDragValidationInfo
{
	/** The tooltip type to display on the operation */
	EPoseWatchManagerDropCompatibility CompatibilityType;

	/** The tooltip text to display on the operation */
	FText ValidationText;

	/** Construct this validation information out of a tooltip type and some text */
	FPoseWatchManagerDragValidationInfo(const EPoseWatchManagerDropCompatibility InCompatibilityType, const FText InValidationText)
		: CompatibilityType(InCompatibilityType)
		, ValidationText(InValidationText)
	{}

	/** Return a generic invalid result */
	static FPoseWatchManagerDragValidationInfo Invalid()
	{
		return FPoseWatchManagerDragValidationInfo(EPoseWatchManagerDropCompatibility::Incompatible, FText());
	}

	/** @return true if this operation is valid, false otherwise */
	bool IsValid() const
	{
		return CompatibilityType == EPoseWatchManagerDropCompatibility::Compatible;
	}
};

/** A drag/drop operation that was started from the pose watch manager */
struct FPoseWatchManagerDragDropOp : public FCompositeDragDropOp
{
	DRAG_DROP_OPERATOR_TYPE(FPoseWatchManagerDragDropOp, FCompositeDragDropOp);

	UE_API FPoseWatchManagerDragDropOp();

	using FDragDropOperation::Construct;

	void ResetTooltip()
	{
		OverrideText = FText();
		OverrideIcon = nullptr;
	}

	void SetTooltip(FText InOverrideText, const FSlateBrush* InOverrideIcon)
	{
		OverrideText = InOverrideText;
		OverrideIcon = InOverrideIcon;
	}

	UE_API virtual TSharedPtr<SWidget> GetDefaultDecorator() const override;

private:

	UE_API EVisibility GetOverrideVisibility() const;
	UE_API EVisibility GetDefaultVisibility() const;

	FText OverrideText;
	FText GetOverrideText() const { return OverrideText; }

	const FSlateBrush* OverrideIcon;
	const FSlateBrush* GetOverrideIcon() const { return OverrideIcon; }
};


class FPoseWatchDragDropOp : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FPoseWatchDragDropOp, FDecoratedDragDropOp)

	/** Pose Watch that we are dragging */
	TWeakObjectPtr<UPoseWatch> PoseWatch;

	void Init(const TWeakObjectPtr<UPoseWatch>& InPoseWatch)
	{
		PoseWatch = InPoseWatch;

		// Set text and icon
		UClass* CommonSelClass = NULL;
		CurrentIconBrush = FAppStyle::Get().GetBrush(TEXT("ClassIcon.PoseAsset"));
		CurrentHoverText = PoseWatch->GetLabel();

		SetupDefaults();
	}
};

class FPoseWatchFolderDragDropOp : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FPoseWatchFolderDragDropOp, FDecoratedDragDropOp)

	/** Pose Watch Folder that we are dragging */
	TWeakObjectPtr<UPoseWatchFolder> PoseWatchFolder;

	void Init(const TWeakObjectPtr<UPoseWatchFolder>& InPoseWatchFolder)
	{
		PoseWatchFolder = InPoseWatchFolder;

		CurrentIconBrush = FAppStyle::Get().GetBrush(TEXT("SceneOutliner.FolderClosed"));
		CurrentHoverText = PoseWatchFolder->GetLabel();


		SetupDefaults();
	}
};

#undef UE_API
