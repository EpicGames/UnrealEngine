// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseWatchManagerElementTreeItem.h"
#include "Textures/SlateIcon.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Framework/Commands/UIAction.h"
#include "ToolMenus.h"
#include "Styling/AppStyle.h"
#include "PoseWatchManagerDragDrop.h"
#include "SPoseWatchManager.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Engine/PoseWatch.h"

#define LOCTEXT_NAMESPACE "PoseWatchManagerElementTreeItem"

struct PERSONA_API SPoseWatchManagerElementTreeLabel : FPoseWatchManagerCommonLabelData, public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SPoseWatchManagerElementTreeLabel) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, FPoseWatchManagerElementTreeItem& NodeWatchTreeItem, IPoseWatchManager& PoseWatchManager, const STableRow<FPoseWatchManagerTreeItemPtr>& InRow)
	{
		WeakPoseWatchManager = StaticCastSharedRef<IPoseWatchManager>(PoseWatchManager.AsShared());

		TreeItemPtr = StaticCastSharedRef<FPoseWatchManagerElementTreeItem>(NodeWatchTreeItem.AsShared());
		WeakPoseWatchElementPtr = NodeWatchTreeItem.PoseWatchElement;

		HighlightText = PoseWatchManager.GetFilterHighlightText();

		TSharedPtr<SInlineEditableTextBlock> InlineTextBlock;

		TSharedRef<SHorizontalBox> MainContent = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		[
			SAssignNew(InlineTextBlock, SInlineEditableTextBlock)
			.Text(this, &SPoseWatchManagerElementTreeLabel::GetDisplayText)
			.ToolTipText(this, &SPoseWatchManagerElementTreeLabel::GetTooltipText)
			.HighlightText(HighlightText)
			.ColorAndOpacity(this, &SPoseWatchManagerElementTreeLabel::GetForegroundColor)
			.OnTextCommitted(this, &SPoseWatchManagerElementTreeLabel::OnLabelCommitted)
			.OnVerifyTextChanged(this, &SPoseWatchManagerElementTreeLabel::OnVerifyItemLabelChanged)
			.OnEnterEditingMode(this, &SPoseWatchManagerElementTreeLabel::OnEnterEditingMode)
			.OnExitEditingMode(this, &SPoseWatchManagerElementTreeLabel::OnExitEditingMode)
			.IsSelected(FIsSelected::CreateSP(&InRow, &STableRow<FPoseWatchManagerTreeItemPtr>::IsSelectedExclusively))
		];

		if (WeakPoseWatchManager.Pin())
		{
			NodeWatchTreeItem.RenameRequestEvent.BindSP(InlineTextBlock.Get(), &SInlineEditableTextBlock::EnterEditingMode);
		}

		ChildSlot
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FPoseWatchManagerDefaultTreeItemMetrics::IconPadding())
			[
				SNew(SBox)
				.WidthOverride(FPoseWatchManagerDefaultTreeItemMetrics::IconSize())
				.HeightOverride(FPoseWatchManagerDefaultTreeItemMetrics::IconSize())
				[
					SNew(SImage)
					.Image(this, &SPoseWatchManagerElementTreeLabel::GetIcon)
					.ToolTipText(this, &SPoseWatchManagerElementTreeLabel::GetIconTooltip)
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]

			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.VAlign(VAlign_Center)
			.Padding(0.0f, 0.0f)
			[
				MainContent
			]
		];
	}

private:
	TWeakPtr<FPoseWatchManagerElementTreeItem> TreeItemPtr;
	TWeakObjectPtr<UPoseWatchElement> WeakPoseWatchElementPtr;
	TAttribute<FText> HighlightText;
	bool bInEditingMode = false;

	FText GetDisplayText() const
	{
		return WeakPoseWatchElementPtr.Get()->GetLabel();
	}

	FText GetTooltipText() const
	{
		if (!TreeItemPtr.Pin()->IsEnabled())
		{
			return LOCTEXT("NodeWatchDisabled", "This node watch element is disabled because it is not connected to the 'Output Pose' node");
		}
		return GetDisplayText();
	}

	const FSlateBrush* GetIcon() const
	{
		FName IconBrushName = TEXT("ClassIcon.Default");
		if (const UPoseWatchElement* PoseWatchElement = WeakPoseWatchElementPtr.Get())
		{
			IconBrushName = PoseWatchElement->GetIconName();
		}

		return FAppStyle::Get().GetBrush(IconBrushName);
	}

	const FSlateBrush* GetIconOverlay() const
	{
		return nullptr;
	}

	FText GetIconTooltip() const
	{
		return LOCTEXT("PoseWatchElement", "Component");
	}

	FSlateColor GetForegroundColor() const
	{
		if (const TOptional<FLinearColor> BaseColor = FPoseWatchManagerCommonLabelData::GetForegroundColor(*TreeItemPtr.Pin()))
		{
			return BaseColor.GetValue();
		}
		return FSlateColor::UseForeground();
	}

	bool OnVerifyItemLabelChanged(const FText& InLabel, FText& OutErrorMessage)
	{
		return WeakPoseWatchElementPtr->ValidateLabelRename(InLabel, OutErrorMessage);
	}

	void OnLabelCommitted(const FText& InLabel, ETextCommit::Type InCommitInfo)
	{
		check(WeakPoseWatchElementPtr->SetLabel(InLabel));
		WeakPoseWatchManager.Pin()->FullRefresh();
		WeakPoseWatchManager.Pin()->SetKeyboardFocus();
	}

	void OnEnterEditingMode()
	{
		bInEditingMode = true;
	}

	void OnExitEditingMode()
	{
		bInEditingMode = false;
	}
};

const EPoseWatchTreeItemType FPoseWatchManagerElementTreeItem::Type(EPoseWatchTreeItemType::Element);

FPoseWatchManagerElementTreeItem::FPoseWatchManagerElementTreeItem(TWeakObjectPtr<UPoseWatchElement> InPoseWatchElement)
	: IPoseWatchManagerTreeItem(Type)
	, ID(InPoseWatchElement.Get())
	, PoseWatchElement(InPoseWatchElement)
{
	check(InPoseWatchElement.Get());
}

FObjectKey FPoseWatchManagerElementTreeItem::GetID() const
{
	return ID;
}

FString FPoseWatchManagerElementTreeItem::GetDisplayString() const
{
	return PoseWatchElement->GetLabel().ToString();
}

bool FPoseWatchManagerElementTreeItem::IsAssignedFolder() const
{
	return false;
}

TSharedRef<SWidget> FPoseWatchManagerElementTreeItem::GenerateLabelWidget(IPoseWatchManager& PoseWatchManager, const STableRow<FPoseWatchManagerTreeItemPtr>& InRow)
{
	return SNew(SPoseWatchManagerElementTreeLabel, *this, PoseWatchManager, InRow);
}

bool FPoseWatchManagerElementTreeItem::GetVisibility() const
{
	return PoseWatchElement->GetIsVisible();
}

void FPoseWatchManagerElementTreeItem::SetIsVisible(const bool bVisible)
{
	PoseWatchElement->SetIsVisible(bVisible);
}

TSharedPtr<SWidget> FPoseWatchManagerElementTreeItem::CreateContextMenu()
{
	return SNullWidget::NullWidget;
}

void FPoseWatchManagerElementTreeItem::OnRemoved() {}

bool FPoseWatchManagerElementTreeItem::IsEnabled() const
{
	return PoseWatchElement->GetIsEnabled();
}

FColor FPoseWatchManagerElementTreeItem::GetColor() const
{
	return PoseWatchElement->GetColor();
}

void FPoseWatchManagerElementTreeItem::SetColor(const FColor& InColor)
{
	PoseWatchElement->SetColor(InColor);
}

bool FPoseWatchManagerElementTreeItem::ShouldDisplayColorPicker() const
{
	return PoseWatchElement->HasColor();
}

#undef LOCTEXT_NAMESPACE
