// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "IPoseWatchManagerTreeItem.h"

class UPoseWatchElement;

struct FPoseWatchManagerElementTreeItem : IPoseWatchManagerTreeItem
{
public:
	FPoseWatchManagerElementTreeItem(TWeakObjectPtr<UPoseWatchElement> InPoseWatchElement);

	/** Constant identifier for this tree item */
	const FObjectKey ID;

	/** The node watch component this item is describing */
	TWeakObjectPtr<UPoseWatchElement> PoseWatchElement;

	/** Static type identifier for this tree item class */
	static const EPoseWatchTreeItemType Type;

	bool IsValid() const override { return PoseWatchElement.IsValid(); }

	/* Begin IPoseWatchManagerTreeItem Implementation */
	virtual FObjectKey GetID() const override;
	virtual FString GetDisplayString() const override;
	virtual TSharedRef<SWidget> GenerateLabelWidget(IPoseWatchManager& Outliner, const STableRow<FPoseWatchManagerTreeItemPtr>& InRow) override;
	virtual bool GetVisibility() const override;
	virtual bool IsAssignedFolder() const;
	virtual void SetIsVisible(const bool bVisible) override;
	virtual TSharedPtr<SWidget> CreateContextMenu() override;
	virtual void OnRemoved() override;
	virtual bool IsEnabled() const override;
	virtual FColor GetColor() const override;
	virtual void SetColor(const FColor& InColor) override;
	virtual bool ShouldDisplayColorPicker() const override;
	/* End IPoseWatchManagerTreeItem Implementation */
};
