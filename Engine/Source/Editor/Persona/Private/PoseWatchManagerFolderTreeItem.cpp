// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseWatchManagerFolderTreeItem.h"
#include "Textures/SlateIcon.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Framework/Commands/UIAction.h"
#include "ToolMenus.h"
#include "Styling/AppStyle.h"
#include "PoseWatchManagerDragDrop.h"
#include "SPoseWatchManager.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Engine/PoseWatch.h"

#define LOCTEXT_NAMESPACE "PoseWatchManagerFolderTreeItem"

const EPoseWatchTreeItemType FPoseWatchManagerFolderTreeItem::Type(EPoseWatchTreeItemType::Folder);

FPoseWatchManagerFolderTreeItem::FPoseWatchManagerFolderTreeItem(UPoseWatchFolder* InPoseWatchFolder)
	: IPoseWatchManagerTreeItem(Type)
	, ID(InPoseWatchFolder)
	, PoseWatchFolder(InPoseWatchFolder)
{
}

bool FPoseWatchManagerFolderTreeItem::IsAssignedFolder() const
{
	return PoseWatchFolder->IsAssignedFolder();
}

FObjectKey FPoseWatchManagerFolderTreeItem::GetID() const
{
	return ID;
}

FString FPoseWatchManagerFolderTreeItem::GetDisplayString() const
{
	return PoseWatchFolder->GetLabel().ToString();
}

void FPoseWatchManagerFolderTreeItem::SetIsVisible(const bool bVisible)
{
	PoseWatchFolder->SetIsVisible(bVisible);
}

TSharedPtr<SWidget> FPoseWatchManagerFolderTreeItem::CreateContextMenu()
{
	FMenuBuilder MenuBuilder(true, nullptr);

	UPoseWatchFolder* FolderItemPtr = PoseWatchFolder.Get();
	MenuBuilder.BeginSection(FName(), LOCTEXT("PoseWatchFolder", "Pose Watch Folder"));
	MenuBuilder.AddMenuEntry(
		LOCTEXT("DeleteFolder", "Delete Folder"),
		LOCTEXT("DeleteFolderDescription", "Delete this folder and move its contents to this folder's parent"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([FolderItemPtr, this]() {
				FolderItemPtr->OnRemoved();
			})
		)
	);
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}


struct SPoseWatchManagerFolderTreeItemLabel : FPoseWatchManagerCommonLabelData, public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SPoseWatchManagerFolderTreeItemLabel) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, FPoseWatchManagerFolderTreeItem& FolderItem, IPoseWatchManager& PoseWatchManager, const STableRow<FPoseWatchManagerTreeItemPtr>& InRow)
	{
		TreeItemPtr = StaticCastSharedRef<FPoseWatchManagerFolderTreeItem>(FolderItem.AsShared());
		WeakPoseWatchManager = StaticCastSharedRef<IPoseWatchManager>(PoseWatchManager.AsShared());

		TSharedPtr<SInlineEditableTextBlock> InlineTextBlock = SNew(SInlineEditableTextBlock)
			.Text(this, &SPoseWatchManagerFolderTreeItemLabel::GetDisplayText)
			.HighlightText(PoseWatchManager.GetFilterHighlightText())
			.ColorAndOpacity(this, &SPoseWatchManagerFolderTreeItemLabel::GetForegroundColor)
			.OnTextCommitted(this, &SPoseWatchManagerFolderTreeItemLabel::OnLabelCommitted)
			.OnVerifyTextChanged(this, &SPoseWatchManagerFolderTreeItemLabel::OnVerifyItemLabelChanged)
			.IsSelected(FIsSelected::CreateSP(&InRow, &STableRow<FPoseWatchManagerTreeItemPtr>::IsSelectedExclusively))
			.IsReadOnly_Lambda([this]() { return !CanExecuteRenameRequest(*TreeItemPtr.Pin()); });

		FolderItem.RenameRequestEvent.BindSP(InlineTextBlock.Get(), &SInlineEditableTextBlock::EnterEditingMode);

		ChildSlot
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FPoseWatchManagerDefaultTreeItemMetrics::IconPadding())
			[
				SNew(SBox)
				.WidthOverride(FPoseWatchManagerDefaultTreeItemMetrics::IconSize())
				.HeightOverride(FPoseWatchManagerDefaultTreeItemMetrics::IconSize())
			[
				SNew(SImage)
				.Image(this, &SPoseWatchManagerFolderTreeItemLabel::GetIcon)
			]
		]

		+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.VAlign(VAlign_Center)
			.Padding(0.0f, 2.0f)
			[
				InlineTextBlock.ToSharedRef()
			]
		];
	}

private:
	TWeakPtr<FPoseWatchManagerFolderTreeItem> TreeItemPtr;

	FText GetDisplayText() const
	{
		return TreeItemPtr.Pin()->PoseWatchFolder->GetLabel();
	}

	const FSlateBrush* GetIcon() const
	{
		auto TreeItem = TreeItemPtr.Pin();
		if (!TreeItem.IsValid())
		{
			return FAppStyle::Get().GetBrush(TEXT("SceneOutliner.FolderClosed"));
		}

		if (TreeItemPtr.Pin()->IsExpanded() && TreeItem->PoseWatchFolder->HasChildren())
		{
			return FAppStyle::Get().GetBrush(TEXT("SceneOutliner.FolderOpen"));
		}
		else
		{
			return FAppStyle::Get().GetBrush(TEXT("SceneOutliner.FolderClosed"));
		}
	}

	FSlateColor GetForegroundColor() const
	{
		if (auto BaseColor = FPoseWatchManagerCommonLabelData::GetForegroundColor(*TreeItemPtr.Pin()))
		{
			return BaseColor.GetValue();
		}

		return FSlateColor::UseForeground();
	}

	bool OnVerifyItemLabelChanged(const FText& InLabel, FText& OutErrorMessage)
	{
		return TreeItemPtr.Pin()->PoseWatchFolder.Get()->ValidateLabelRename(InLabel, OutErrorMessage);
	}

	void OnLabelCommitted(const FText& InLabel, ETextCommit::Type InCommitInfo)
	{
		check(TreeItemPtr.Pin()->PoseWatchFolder.Get()->SetLabel(InLabel));
		WeakPoseWatchManager.Pin()->FullRefresh();
		WeakPoseWatchManager.Pin()->SetKeyboardFocus();
	}
};

TSharedRef<SWidget> FPoseWatchManagerFolderTreeItem::GenerateLabelWidget(IPoseWatchManager& Outliner, const STableRow<FPoseWatchManagerTreeItemPtr>& InRow)
{
	return SNew(SPoseWatchManagerFolderTreeItemLabel, *this, Outliner, InRow);
}

void FPoseWatchManagerFolderTreeItem::OnRemoved()
{
	PoseWatchFolder->OnRemoved();
}

bool FPoseWatchManagerFolderTreeItem::GetVisibility() const
{
	return PoseWatchFolder.IsValid() ? PoseWatchFolder.Get()->GetIsVisible() : false;
}

bool FPoseWatchManagerFolderTreeItem::HasChildren() const
{
	return PoseWatchFolder.IsValid() ? PoseWatchFolder.Get()->HasChildren() : false;
}

void FPoseWatchManagerFolderTreeItem::SetIsExpanded(const bool bIsExpanded)
{
	if (PoseWatchFolder.IsValid())
	{
		PoseWatchFolder.Get()->SetIsExpanded(bIsExpanded);
	}
}

bool FPoseWatchManagerFolderTreeItem::IsExpanded() const
{
	return PoseWatchFolder.IsValid() && PoseWatchFolder.Get()->GetIsExpanded();
}

#undef LOCTEXT_NAMESPACE
