// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "IPoseWatchManagerTreeItem.h"

class UPoseWatch;
class UPoseWatchFolder;

struct FPoseWatchManagerFolderTreeItem : IPoseWatchManagerTreeItem
{
public:
	FPoseWatchManagerFolderTreeItem(UPoseWatchFolder* InPoseWatchFolder);

	/** Constant identifier for this tree item */
	const FObjectKey ID;
	
	/** The pose watch folder this item is describing */
	TWeakObjectPtr<UPoseWatchFolder> PoseWatchFolder;
	
	/** Static type identifier for this tree item class */
	static const EPoseWatchTreeItemType Type;

	bool IsValid() const override { return PoseWatchFolder.IsValid(); }

	/* Begin IPoseWatchManagerTreeItem Implementation */
	virtual FObjectKey GetID() const override;
	virtual FString GetDisplayString() const override;
	virtual TSharedRef<SWidget> GenerateLabelWidget(IPoseWatchManager& Outliner, const STableRow<FPoseWatchManagerTreeItemPtr>& InRow) override;
	virtual TSharedPtr<SWidget> CreateContextMenu() override;
	virtual bool GetVisibility() const override;
	virtual bool IsAssignedFolder() const override;
	virtual void SetIsVisible(const bool bVisible) override;
	virtual bool HasChildren() const override;
	virtual void OnRemoved() override;
	virtual void SetIsExpanded(const bool bIsExpanded) override;
	virtual bool IsExpanded() const override;
	/* End IPoseWatchManagerTreeItem Implementation */
};
