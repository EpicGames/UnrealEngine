// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/CoreDelegates.h"

class IPoseWatchManager;
class IPoseWatchManagerColumn;

class UPoseWatch;
class UPoseWatchFolder;

struct FPoseWatchManagerInitializationOptions;

struct IPoseWatchManagerTreeItem;
struct FPoseWatchManagerPoseWatchTreeItem;
struct FPoseWatchManagerFolderTreeItem;

struct FObjectKey;


struct FPoseWatchManagerDragDropPayload;
struct FPoseWatchManagerDragValidationInfo;