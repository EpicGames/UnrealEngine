// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseWatchManagerPoseWatchTreeItem.h"
#include "Textures/SlateIcon.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Framework/Commands/UIAction.h"
#include "ToolMenus.h"
#include "Styling/AppStyle.h"
#include "PoseWatchManagerDragDrop.h"
#include "SPoseWatchManager.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Engine/PoseWatch.h"

#define LOCTEXT_NAMESPACE "PoseWatchManagerPoseWatchTreeItem"

struct PERSONA_API SPoseWatchManagerPoseWatchTreeLabel : FPoseWatchManagerCommonLabelData, public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SPoseWatchManagerPoseWatchTreeLabel) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, FPoseWatchManagerPoseWatchTreeItem& PoseWatchTreeItem, IPoseWatchManager& PoseWatchManager, const STableRow<FPoseWatchManagerTreeItemPtr>& InRow)
	{
		WeakPoseWatchManager = StaticCastSharedRef<IPoseWatchManager>(PoseWatchManager.AsShared());

		TreeItemPtr = StaticCastSharedRef<FPoseWatchManagerPoseWatchTreeItem>(PoseWatchTreeItem.AsShared());
		WeakPoseWatchPtr = PoseWatchTreeItem.PoseWatch;

		HighlightText = PoseWatchManager.GetFilterHighlightText();

		TSharedPtr<SInlineEditableTextBlock> InlineTextBlock = SNew(SInlineEditableTextBlock)
			.Text(this, &SPoseWatchManagerPoseWatchTreeLabel::GetDisplayText)
			.HighlightText(PoseWatchManager.GetFilterHighlightText())
			.ColorAndOpacity(this, &SPoseWatchManagerPoseWatchTreeLabel::GetForegroundColor)
			.OnTextCommitted(this, &SPoseWatchManagerPoseWatchTreeLabel::OnLabelCommitted)
			.OnVerifyTextChanged(this, &SPoseWatchManagerPoseWatchTreeLabel::OnVerifyItemLabelChanged)
			.IsSelected(FIsSelected::CreateSP(&InRow, &STableRow<FPoseWatchManagerTreeItemPtr>::IsSelectedExclusively))
			.IsReadOnly_Lambda([this]() { return !CanExecuteRenameRequest(*TreeItemPtr.Pin()); });

		if (WeakPoseWatchManager.Pin())
		{
			PoseWatchTreeItem.RenameRequestEvent.BindSP(InlineTextBlock.Get(), &SInlineEditableTextBlock::EnterEditingMode);
		}

		ChildSlot
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.VAlign(VAlign_Center)
			.Padding(0.0f, 2.0f)
			[
				InlineTextBlock.ToSharedRef()
			]
		];
	}

private:
	TWeakPtr<FPoseWatchManagerPoseWatchTreeItem> TreeItemPtr;
	TWeakObjectPtr<UPoseWatch> WeakPoseWatchPtr;
	TAttribute<FText> HighlightText;
	bool bInEditingMode = false;

	FText GetDisplayText() const
	{
		if (const UPoseWatch* PoseWatch = WeakPoseWatchPtr.Get())
		{
			return PoseWatch->GetLabel();
		}
		return FText();
	}

	FText GetTooltipText() const
	{
		if (!TreeItemPtr.Pin()->IsEnabled())
		{
			return LOCTEXT("PoseWatchDisabled", "This pose watch is disabled because it is not being evaluated");
		}
		return GetDisplayText();
	}

	FText GetTypeText() const
	{
		if (const UPoseWatch* PoseWatch = WeakPoseWatchPtr.Get())
		{
			return FText::FromName(PoseWatch->Node->GetFName());
		}
		return FText();
	}

	EVisibility GetTypeTextVisibility() const
	{
		return HighlightText.Get().IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible;
	}

	const FSlateBrush* GetIcon() const
	{
		return FAppStyle::Get().GetBrush("AnimGraph.PoseWatch.Icon");
	}

	const FSlateBrush* GetIconOverlay() const
	{
		return nullptr;
	}

	FText GetIconTooltip() const
	{
		return LOCTEXT("PoseWatch", "Pose Watch");
	}

	FSlateColor GetForegroundColor() const
	{
		if (auto BaseColor = FPoseWatchManagerCommonLabelData::GetForegroundColor(*TreeItemPtr.Pin()))
		{
			return BaseColor.GetValue();
		}
		return FSlateColor::UseForeground();
	}

	bool OnVerifyItemLabelChanged(const FText& InLabel, FText& OutErrorMessage)
	{
		return WeakPoseWatchPtr->ValidateLabelRename(InLabel, OutErrorMessage);
	}

	void OnLabelCommitted(const FText& InLabel, ETextCommit::Type InCommitInfo)
	{
		bool bRenameSuccessful = WeakPoseWatchPtr->SetLabel(InLabel);
		check(bRenameSuccessful);
		WeakPoseWatchManager.Pin()->FullRefresh();
		WeakPoseWatchManager.Pin()->SetKeyboardFocus();
	}

	void OnEnterEditingMode()
	{
		bInEditingMode = true;
	}

	void OnExitEditingMode()
	{
		bInEditingMode = false;
	}
};

const EPoseWatchTreeItemType FPoseWatchManagerPoseWatchTreeItem::Type(EPoseWatchTreeItemType::PoseWatch);

FPoseWatchManagerPoseWatchTreeItem::FPoseWatchManagerPoseWatchTreeItem(UPoseWatch* InPoseWatch)
	: IPoseWatchManagerTreeItem(Type)
	, ID(InPoseWatch)
	, PoseWatch(InPoseWatch)
{
	check(InPoseWatch);
}

FObjectKey FPoseWatchManagerPoseWatchTreeItem::GetID() const
{
	return ID;
}

FString FPoseWatchManagerPoseWatchTreeItem::GetDisplayString() const
{
	return PoseWatch->GetLabel().ToString();
}

bool FPoseWatchManagerPoseWatchTreeItem::IsAssignedFolder() const
{
	return PoseWatch->IsAssignedFolder();
}

TSharedRef<SWidget> FPoseWatchManagerPoseWatchTreeItem::GenerateLabelWidget(IPoseWatchManager& PoseWatchManager, const STableRow<FPoseWatchManagerTreeItemPtr>& InRow)
{
	return SNew(SPoseWatchManagerPoseWatchTreeLabel, *this, PoseWatchManager, InRow);
}

bool FPoseWatchManagerPoseWatchTreeItem::GetVisibility() const
{
	return PoseWatch->GetIsVisible();
}

void FPoseWatchManagerPoseWatchTreeItem::SetIsVisible(const bool bVisible)
{
	PoseWatch->SetIsVisible(bVisible);
}

TSharedPtr<SWidget> FPoseWatchManagerPoseWatchTreeItem::CreateContextMenu()
{
	FMenuBuilder MenuBuilder(true, nullptr);

	UPoseWatch* PoseWatchPtr = PoseWatch.Get();
	MenuBuilder.BeginSection(FName(), LOCTEXT("PoseWatch", "Pose Watch"));
	MenuBuilder.AddMenuEntry(
		LOCTEXT("DeletePoseWatch", "Delete Pose Watch"),
		LOCTEXT("DeleteFolderDescription", "Delete the selected pose watch"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([PoseWatchPtr]() {
				PoseWatchPtr->OnRemoved();
			})
		)
	);
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void FPoseWatchManagerPoseWatchTreeItem::OnRemoved()
{
	PoseWatch->OnRemoved();
}

bool FPoseWatchManagerPoseWatchTreeItem::IsEnabled() const
{
	return PoseWatch->GetIsEnabled();
}

void FPoseWatchManagerPoseWatchTreeItem::SetIsExpanded(const bool bIsExpanded)
{
	PoseWatch->SetIsExpanded(bIsExpanded);
}

bool FPoseWatchManagerPoseWatchTreeItem::IsExpanded() const
{
	return PoseWatch->GetIsExpanded();
}

#undef LOCTEXT_NAMESPACE
