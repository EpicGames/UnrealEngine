// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "IPoseWatchManagerTreeItem.h"

class UPoseWatch;

struct FPoseWatchManagerPoseWatchTreeItem : IPoseWatchManagerTreeItem
{
public:
	FPoseWatchManagerPoseWatchTreeItem(UPoseWatch* InPoseWatch);
	
	/** Constant identifier for this tree item */
	const FObjectKey ID;
	
	/** The pose watch this item is describing */
	TWeakObjectPtr<UPoseWatch> PoseWatch;

	/** Static type identifier for this tree item class */
	static const EPoseWatchTreeItemType Type;

	bool IsValid() const override { return PoseWatch.IsValid();	}

	/* Begin IPoseWatchManagerTreeItem Implementation */
	virtual FObjectKey GetID() const override;
	virtual FString GetDisplayString() const override;
	virtual TSharedRef<SWidget> GenerateLabelWidget(IPoseWatchManager& Outliner, const STableRow<FPoseWatchManagerTreeItemPtr>& InRow) override;
	virtual bool GetVisibility() const override;
	virtual bool IsAssignedFolder() const;
	virtual void SetIsVisible(const bool bVisible) override;
	virtual TSharedPtr<SWidget> CreateContextMenu() override;
	virtual void OnRemoved() override;
	virtual bool IsEnabled() const override;
	virtual void SetIsExpanded(const bool bIsExpanded) override;
	virtual bool IsExpanded() const override;
	/* End IPoseWatchManagerTreeItem Implementation */
};
