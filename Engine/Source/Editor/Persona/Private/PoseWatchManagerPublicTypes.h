// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Margin.h"

/** Container for built in column types. Function-static so they are available without linking */
struct FPoseWatchManagerBuiltInColumnTypes
{
	static const FName& Visibility()
	{
		static FName Visibility("Visibility");
		return Visibility;
	}

	static const FName& Label()
	{
		static FName Label("Item Label");
		return Label;
	}

	static FName& Color()
	{
		static FName Color("Color");
		return Color;
	}
};

enum class EPoseWatchManagerColumnVisibility : uint8
{
	Visible,
	Invisible,
};

struct FPoseWatchManagerInitializationOptions
{
	/** The blueprint editor this pose watch manager is reflecting */
	TWeakPtr<class FBlueprintEditor> BlueprintEditor;

	FPoseWatchManagerInitializationOptions() {}
};

struct FPoseWatchManagerDefaultTreeItemMetrics
{
	static float	RowHeight() { return 20.f; };
	static float	IconSize() { return 16; };
	static FMargin	IconPadding() { return FMargin(0.f, 1.f, 6.f, 1.f); };
};