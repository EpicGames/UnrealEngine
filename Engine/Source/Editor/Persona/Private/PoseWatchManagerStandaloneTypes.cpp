// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseWatchManagerStandaloneTypes.h"

#include "EditorActorFolders.h"
#include "Framework/Application/SlateApplication.h"
#include "IPoseWatchManagerTreeItem.h"
#include "IPoseWatchManager.h"
#include "PoseWatchManagerDragDrop.h"
#include "PoseWatchManagerDefaultMode.h"

#define LOCTEXT_NAMESPACE "PoseWatchManagerStandaloneTypes"

const FLinearColor FPoseWatchManagerCommonLabelData::DisabledColor(0.15f, 0.15f, 0.15f);

TOptional<FLinearColor> FPoseWatchManagerCommonLabelData::GetForegroundColor(const IPoseWatchManagerTreeItem& TreeItem) const
{
	if (!TreeItem.IsValid() || !TreeItem.IsEnabled())
	{
		return DisabledColor;
	}

	// Darken items that aren't suitable targets for an active drag and drop action
	if (FSlateApplication::Get().IsDragDropping())
	{
		TSharedPtr<FDragDropOperation> DragDropOp = FSlateApplication::Get().GetDragDroppingContent();

		FPoseWatchManagerDragDropPayload DraggedObjects(*DragDropOp);
		const auto Outliner = WeakPoseWatchManager.Pin();
		if (Outliner->GetMode()->ParseDragDrop(DraggedObjects, *DragDropOp) && !Outliner->GetMode()->ValidateDrop(TreeItem, DraggedObjects).IsValid())
		{
			return DisabledColor;
		}
	}

	return TOptional<FLinearColor>();
}

bool FPoseWatchManagerCommonLabelData::CanExecuteRenameRequest(const IPoseWatchManagerTreeItem& Item) const
{
	return true;
}

#undef LOCTEXT_NAMESPACE
