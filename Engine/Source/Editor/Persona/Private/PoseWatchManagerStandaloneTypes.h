// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Paths.h"
#include "UObject/ObjectKey.h"
#include "Templates/MaxSizeof.h"

#define UE_API PERSONA_API

struct IPoseWatchManagerTreeItem;
class IPoseWatchManager;

typedef TSharedPtr<IPoseWatchManagerTreeItem> FPoseWatchManagerTreeItemPtr;
typedef TSharedRef<IPoseWatchManagerTreeItem> FPoseWatchManagerTreeItemRef;

typedef TMap<FObjectKey, FPoseWatchManagerTreeItemPtr> FPoseWatchManagerTreeItemMap;

enum EPoseWatchTreeItemType
{
	PoseWatch,
	Folder,
	Element
};

struct FPoseWatchManagerCommonLabelData
{
	TWeakPtr<IPoseWatchManager> WeakPoseWatchManager;
	static UE_API const FLinearColor DisabledColor;

	UE_API TOptional<FLinearColor> GetForegroundColor(const IPoseWatchManagerTreeItem& TreeItem) const;

	UE_API bool CanExecuteRenameRequest(const IPoseWatchManagerTreeItem& Item) const;
};


struct FPoseWatchManagerHierarchyChangedData
{
	enum
	{
		Added,
		FullRefresh
	} Type;

	// This event may pass one of two kinds of data, depending on the type of event
	TArray<FPoseWatchManagerTreeItemPtr> Items;

	/** Actions to apply to items */
	uint8 ItemActions = 0;
};

#undef UE_API
