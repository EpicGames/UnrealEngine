// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IPropertyTypeCustomization.h"
#include "IPersonaPreviewScene.h"
#include "IDetailCustomization.h"
#include "PropertyHandle.h"
#include "IEditableSkeleton.h"
#include "IPersonaToolkit.h"
#include "AnimationEditorPreviewScene.h"
#include "PersonaModule.h"

struct FAssetData;
class FDetailWidgetRow;
class IDetailChildrenBuilder;
class IDetailLayoutBuilder;
class IPropertyUtilities;
class UPreviewMeshCollectionFactory;
class USkeleton;

// An entry in the preview mode choice box
struct FPersonaModeComboEntry
{
	// The preview controller class for this entry
	UClass* Class;

	//The localized label for this entry to show in the combo box
	FText Text;

	FPersonaModeComboEntry(UClass* InClass)
		: Class(InClass)
		, Text(InClass->GetDisplayNameText())
	{}
};

class FPreviewSceneDescriptionCustomization : public IDetailCustomization
{
public:
	FPreviewSceneDescriptionCustomization(const FString& InSkeletonName, const TSharedRef<class IPersonaToolkit>& InPersonaToolkit);
	FPreviewSceneDescriptionCustomization(const FString& InSkeletonName, const TSharedRef<class IPersonaToolkit>& InPersonaToolkit, const FPersonaAdvancedPreviewSceneTabSettings& InDetailCustomizationSettings);

	~FPreviewSceneDescriptionCustomization();

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

private:
	EVisibility GetSaveButtonVisibility(TSharedRef<IPropertyHandle> InAdditionalMeshesProperty) const;

	FReply OnSaveCollectionClicked(TSharedRef<IPropertyHandle> InAdditionalMeshesProperty, IDetailLayoutBuilder* DetailLayoutBuilder);

	bool HandleShouldFilterAsset(const FAssetData& InAssetData, FName InTag, bool bCanUseDifferentSkeleton);

	bool HandleShouldFilterAdditionalMesh(const FAssetData& InAssetData, bool bCanUseDifferentSkeleton);

	// Helper function for making the widgets of each item in the preview controller combo box
	TSharedRef<SWidget> MakeControllerComboEntryWidget(TSharedPtr<FPersonaModeComboEntry> InItem) const;

	// Delegate for getting the current preview controller text
	FText GetCurrentPreviewControllerText() const;

	// Called when the combo box selection changes, when a new parameter type is selected
	void OnComboSelectionChanged(TSharedPtr<FPersonaModeComboEntry> InSelectedItem, ESelectInfo::Type SelectInfo);

	// Called when user changes the preview controller type
	void HandlePreviewControllerPropertyChanged();

	void HandleMeshChanged(const FAssetData& InAssetData);

	void HandlePreviewAnimBlueprintChanged(const FAssetData& InAssetData);
	
	// Called when the anim blueprint being edited is compiled.
	void HandleAnimBlueprintCompiled(UBlueprint* Blueprint);

	void HandleAdditionalMeshesChanged(const FAssetData& InAssetData, IDetailLayoutBuilder* DetailLayoutBuilder);

	void HandleAllowDifferentSkeletonsCheckedStateChanged(ECheckBoxState CheckState);

	ECheckBoxState HandleAllowDifferentSkeletonsIsChecked() const;

	void HandleUseCustomAnimBPCheckedStateChanged(ECheckBoxState CheckState);

	ECheckBoxState HandleUseCustomAnimBPIsChecked() const;

	// Reinitialize the preview controller in the preview scene.
	void ReinitializePreviewController();

private:
	/** Cached skeleton name to check for asset registry tags */
	FString SkeletonName;

	/** The persona toolkit we are associated with */
	TWeakPtr<class IPersonaToolkit> PersonaToolkit;

	/** Preview scene we will be editing */
	TWeakPtr<class FAnimationEditorPreviewScene> PreviewScene;

	/** Editable Skeleton scene we will be editing */
	TWeakPtr<class IEditableSkeleton> EditableSkeleton;

	/** Factory to use when creating mesh collections */
	UPreviewMeshCollectionFactory* FactoryToUse;

	// Names of all preview controllers for choice UI
	TArray<TSharedPtr<FPersonaModeComboEntry>> ControllerItems;

	/** This is list of class available to filter asset by. This list doesn't change once loaded, so only collect once */
	static TArray<FTopLevelAssetPath> AvailableClassNameList;

	// Our layout builder (cached so we can refresh)
	IDetailLayoutBuilder* MyDetailLayout;

	TWeakObjectPtr<UDataAsset>	DataAssetToDisplay;

	FPersonaAdvancedPreviewSceneTabSettings DetailCustomizationSettings;

	/**
	* Called to get the visibility of the replace button
	*/
	bool GetReplaceVisibility(TSharedPtr<IPropertyHandle> PropertyHandle) const;
	/**
	* Called when reset to base is clicked
	*/
	void OnResetToBaseClicked(TSharedPtr<IPropertyHandle> PropertyHandle);
	void OnResetAdditionalMeshes();

	TSharedPtr<IPropertyHandle> AdditionalMeshesProperty;
};

class FPreviewMeshCollectionEntryCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShareable(new FPreviewMeshCollectionEntryCustomization(nullptr));
	}

	FPreviewMeshCollectionEntryCustomization(const TSharedPtr<IPersonaPreviewScene>& InPreviewScene)
		: PreviewScene(InPreviewScene)
	{}

	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override {}

private:
	bool HandleShouldFilterAsset(const FAssetData& InAssetData, FString SkeletonName, USkeleton* Skeleton);

	void HandleMeshChanged(const FAssetData& InAssetData);

	void HandleMeshesArrayChanged(TSharedPtr<IPropertyUtilities> PropertyUtilities);

private:
	/** Preview scene we will be editing */
	TWeakPtr<class IPersonaPreviewScene> PreviewScene;
};
