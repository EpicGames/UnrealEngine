// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAnimAssetFindReplace.h"

#include "AnimAssetFindReplaceCurves.h"
#include "ContentBrowserDataSource.h"
#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"
#include "PropertyCustomizationHelpers.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "ToolMenus.h"
#include "ToolMenuSection.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Misc/ScopedSlowTask.h"
#include "ScopedTransaction.h"
#include "SkeletalMeshCompiler.h"
#include "Widgets/Views/SListView.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/AnimPhysObjectVersion.h"
#include "UObject/ObjectVersion.h"
#include "Widgets/Input/SHyperlink.h"
#include "UObject/UObjectIterator.h"
#include "AssetToolsModule.h"

#define LOCTEXT_NAMESPACE "SAnimAssetFindReplace"

FAnimAssetFindReplaceSummoner::FAnimAssetFindReplaceSummoner(TSharedPtr<FAssetEditorToolkit> InHostingApp, const FAnimAssetFindReplaceConfig& InConfig)
	: FWorkflowTabFactory(FPersonaTabs::FindReplaceID, InHostingApp)
	, Config(InConfig)
{
	TabLabel = LOCTEXT("AnimAssetFindReplaceTabLabel", "Find/Replace");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Persona.Tabs.FindReplace");
}

TSharedRef<SWidget> FAnimAssetFindReplaceSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	return SNew(SAnimAssetFindReplace)
		.Config(Config);
}

TSharedPtr<SToolTip> FAnimAssetFindReplaceSummoner::CreateTabToolTipWidget(const FWorkflowTabSpawnInfo& Info) const
{
	return IDocumentation::Get()->CreateToolTip(LOCTEXT("WindowTooltip", "This tab lets you search and replace curve and notify names across multiple assets"), nullptr, TEXT("Shared/Editors/Persona"), TEXT("AnimationFindReplace_Window"));
}

namespace AnimAssetFindReplacePrivate
{

TSharedPtr<SAnimAssetFindReplace> GetWidgetFromContext(const FToolMenuContext& InContext)
{
	if(UAnimAssetFindReplaceContext* Context = InContext.FindContext<UAnimAssetFindReplaceContext>())
	{
		return Context->Widget.Pin();
	}

	return nullptr;
}

}

void SAnimAssetFindReplace::Construct(const FArguments& InArgs)
{
	Config = InArgs._Config;

	// Create instances of all processor types we can use
	for (TObjectIterator<UClass> It; It; ++It)
	{
		if(!It->HasAnyClassFlags(CLASS_Abstract) && It->IsChildOf(UAnimAssetFindReplaceProcessor::StaticClass()))
		{
			UAnimAssetFindReplaceProcessor* Processor = NewObject<UAnimAssetFindReplaceProcessor>(GetTransientPackage(), *It, NAME_None, RF_Transient);
			Processor->Initialize(SharedThis(this));
			Processors.Add(*It, Processor);

			if(*It == Config.InitialProcessorClass)
			{
				CurrentProcessor = Processor;
			}
		}
	}

	// Assign a default processor if we didn't have one set
	if(CurrentProcessor == nullptr && Processors.Num() > 0)
	{
		CurrentProcessor = Processors.FindRef(UAnimAssetFindReplaceCurves::StaticClass());
	}

	const FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");

	AssetPickerConfig.SelectionMode = ESelectionMode::Multi;
	AssetPickerConfig.InitialAssetViewType = EAssetViewType::Column;
	AssetPickerConfig.Filter = MakeARFilter();
	AssetPickerConfig.OnShouldFilterAsset = FOnShouldFilterAsset::CreateSP(this, &SAnimAssetFindReplace::HandleFilterAsset);
	AssetPickerConfig.RefreshAssetViewDelegates.Add(&RefreshAssetViewDelegate);
	AssetPickerConfig.GetCurrentSelectionDelegates.Add(&GetCurrentSelectionDelegate);
	AssetPickerConfig.SetFilterDelegates.Add(&SetARFilterDelegate);
	AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateLambda([this](const FAssetData& InAssetData)
	{
		bAssetsSelected = InAssetData.IsValid();
	});
	AssetPickerConfig.OnAssetDoubleClicked = FOnAssetDoubleClicked::CreateLambda([this](const FAssetData& InAssetData)
	{
		if (UAssetEditorSubsystem* EditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>())
		{
			EditorSubsystem->OpenEditorForAsset(InAssetData.ToSoftObjectPath());
		}
	});
	AssetPickerConfig.CustomColumns.Add(
		FAssetViewCustomColumn(
				"AssetResults",
				LOCTEXT("ResultsColumnLabel", "Results"),
				LOCTEXT("ResultsColumnTooltip", "The matching results that are in each asset"),
				UObject::FAssetRegistryTag::TT_Alphabetical,
				FOnGetCustomAssetColumnData::CreateLambda([this](FAssetData& InAssetData, FName InColumnName)
				{
					return CurrentProcessor->GetFindResultStringFromAssetData(InAssetData);
				})
			));

	TSet<UClass*> ClassesWithAssetRegistryTagsSet;
	for(TPair<TSubclassOf<UAnimAssetFindReplaceProcessor>, UAnimAssetFindReplaceProcessor*> ProcessorPair : Processors)
	{
		for(const UClass* AssetType : ProcessorPair.Value->GetSupportedAssetTypes())
		{
			ClassesWithAssetRegistryTagsSet.Add(const_cast<UClass*>(AssetType));
		}
	}

	TArray<UClass*> ClassesWithAssetRegistryTags = ClassesWithAssetRegistryTagsSet.Array();
	TArray<UClass*> DerivedClassesWithAssetRegistryTags;
	for(UClass* AssetType : ClassesWithAssetRegistryTags)
	{
		GetDerivedClasses(AssetType, DerivedClassesWithAssetRegistryTags);
	}
	ClassesWithAssetRegistryTags.Append(DerivedClassesWithAssetRegistryTags);

	for(UClass* Class : ClassesWithAssetRegistryTags)
	{
		UObject* DefaultObject = Class->GetDefaultObject();
		FAssetRegistryTagsContextData TagsContext(DefaultObject, EAssetRegistryTagsCaller::Uncategorized);
		DefaultObject->GetAssetRegistryTags(TagsContext);

		for(const TPair<FName,UObject::FAssetRegistryTag>& TagPair : TagsContext.Tags)
		{
			if(TagPair.Value.Type != UObject::FAssetRegistryTag::TT_Hidden)
			{
				AssetPickerConfig.HiddenColumnNames.AddUnique(TagPair.Key.ToString());
			}
		}
	}

	AssetPickerConfig.HiddenColumnNames.Add(ContentBrowserItemAttributes::ItemDiskSize.ToString());
	AssetPickerConfig.HiddenColumnNames.Add(ContentBrowserItemAttributes::VirtualizedData.ToString());
	AssetPickerConfig.HiddenColumnNames.Add(TEXT("Path"));
	AssetPickerConfig.HiddenColumnNames.Add(TEXT("Class"));
	AssetPickerConfig.HiddenColumnNames.Add(TEXT("RevisionControl"));
	AssetPickerConfig.bShowPathInColumnView = true;
	AssetPickerConfig.bShowTypeInColumnView = true;
	AssetPickerConfig.bSortByPathInColumnView = false;
	AssetPickerConfig.bFocusSearchBoxWhenOpened = false;

	UToolMenu* Toolbar = UToolMenus::Get()->FindMenu("AnimAssetFindReplaceToolbar");
	if(Toolbar == nullptr)
	{
		Toolbar = UToolMenus::Get()->RegisterMenu("AnimAssetFindReplaceToolbar", NAME_None, EMultiBoxType::SlimHorizontalToolBar);
		Toolbar->StyleName = "CalloutToolbar";	// This style displays button text

		{
			FToolMenuSection& Section = Toolbar->AddSection("FindReplaceOptions");

			Section.AddDynamicEntry("ProcessorSelector", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
			{
				if(TSharedPtr<SAnimAssetFindReplace> Widget = AnimAssetFindReplacePrivate::GetWidgetFromContext(InSection.Context))
				{
					InSection.AddEntry(
						FToolMenuEntry::InitWidget(
							"ProcessorSelector",
							SNew(SHorizontalBox)
							+SHorizontalBox::Slot()
							.AutoWidth()
							.Padding(5.0f, 0.0f)
							.VAlign(VAlign_Center)
							[
								SNew(STextBlock)
								.Text(LOCTEXT("FindLabel", "Find:"))
							]
							+SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Center)
							[
								SNew(SClassPropertyEntryBox)
									.MetaClass(UAnimAssetFindReplaceProcessor::StaticClass())
									.AllowNone(false)
									.ShowDisplayNames(true)
									.HideViewOptions(true)
									.SelectedClass_Lambda([WeakWidget = TWeakPtr<SAnimAssetFindReplace>(Widget)]() -> UClass*
									{
										if(TSharedPtr<SAnimAssetFindReplace> PinnedWidget = WeakWidget.Pin())
										{
											return PinnedWidget->GetCurrentProcessor()->GetClass();
										}

										return nullptr;
									})
									.OnSetClass_Lambda([WeakWidget = TWeakPtr<SAnimAssetFindReplace>(Widget)](const UClass* InClass)
									{
										if(TSharedPtr<SAnimAssetFindReplace> PinnedWidget = WeakWidget.Pin())
										{
											PinnedWidget->SetCurrentProcessor(const_cast<UClass*>(InClass));
										}
									})
							],
							FText::GetEmpty(), true));
				}
			}));
		}

		{
			FToolMenuSection& Section = Toolbar->AddSection("FindReplaceActions");
		
			FToolUIAction RefreshButton;
			RefreshButton.ExecuteAction = FToolMenuExecuteAction::CreateLambda([](const FToolMenuContext& InContext)
			{
				if(TSharedPtr<SAnimAssetFindReplace> Widget = AnimAssetFindReplacePrivate::GetWidgetFromContext(InContext))
				{
					Widget->RequestRefreshUI();
				}
			});

			Section.AddEntry(
				FToolMenuEntry::InitToolBarButton(
					"Refresh",
					RefreshButton,
					LOCTEXT("RefreshRadioLabel", "Refresh"),
					LOCTEXT("RefreshRadioTooltip", "Refresh search results."),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Find"),
					EUserInterfaceActionType::Button));
		}

		{
			FToolMenuSection& Section = Toolbar->AddSection("ProcessorOptions");
			Section.AddDynamicEntry("ProcessorOptions", FNewToolMenuSectionDelegate::CreateLambda([this](FToolMenuSection& InSection)
			{
				if(TSharedPtr<SAnimAssetFindReplace> Widget = AnimAssetFindReplacePrivate::GetWidgetFromContext(InSection.Context))
				{
					Widget->GetCurrentProcessor()->ExtendToolbar(InSection);
				}
			}));
		}
	}

	ToolbarContext = NewObject<UAnimAssetFindReplaceContext>();
	ToolbarContext->Widget = SharedThis(this);

	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SAssignNew(ToolbarContainer, SBox)
			[
				UToolMenus::Get()->GenerateWidget("AnimAssetFindReplaceToolbar", FToolMenuContext(ToolbarContext))
			]
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(5.0f, 10.0f)
		[
			SAssignNew(FindReplaceWidgetContainer, SBox)
			[
				CurrentProcessor->MakeFindReplaceWidget()
			]
		]
		+SVerticalBox::Slot()
		.FillHeight(1.0f)
		.Padding(5.0f, 10.0f)
		[
			ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(10.0f)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			[
				SNew(SHyperlink)
				.Visibility_Lambda([this]()
				{
					return OldAssets.Num() > 0 ? EVisibility::Visible : EVisibility::Collapsed;
				})
				.Text_Lambda([this]()
				{
					return FText::Format(LOCTEXT("UnindexedAssetWarningFormat", "{0} assets could not be indexed, load them now?"), FText::AsNumber(OldAssets.Num()));
				})
				.OnNavigate_Lambda([this]()
				{
					// Load all old unindexed assets
					FScopedSlowTask SlowTask(OldAssets.Num(), FText::Format(LOCTEXT("LoadingUnindexedAssetsFormat", "Loading {0} Unindexed Assets..."), FText::AsNumber(OldAssets.Num())));
					SlowTask.MakeDialog(true);
					
					for(const FAssetData& AssetData : OldAssets)
					{
						SlowTask.EnterProgressFrame();

						AssetData.GetAsset();

						if(SlowTask.ShouldCancel())
						{
							break;
						}
					}

					// Ensure all meshes are compiled after the load, as asset registry data isnt available correctly until they are
					FSkinnedAssetCompilingManager::Get().FinishAllCompilation();

					RequestRefreshCachedData();
				})
			]
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.HAlign(HAlign_Right)
			[
				SNew(SUniformGridPanel)
				.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
				.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
				.MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))
				+SUniformGridPanel::Slot(0, 0)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.Text(LOCTEXT("RemoveButton", "Remove"))
					.ToolTipText(LOCTEXT("RemoveButtonTooltip", "Remove selected items"))
					.IsEnabled_Lambda([this]()
					{
						return bAssetsSelected && CurrentProcessor->CanCurrentlyRemove();
					})
					.OnClicked(this, &SAnimAssetFindReplace::HandleRemove)
				]
				+SUniformGridPanel::Slot(1, 0)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.Text(LOCTEXT("RemoveAllButton", "Remove All"))
					.ToolTipText(LOCTEXT("RemoveAllButtonTooltip", "Remove all matching items"))
					.IsEnabled_Lambda([this]()
					{
						return bFoundAssets && CurrentProcessor->CanCurrentlyRemove();
					})
					.OnClicked(this, &SAnimAssetFindReplace::HandleRemoveAll)
				]
				+SUniformGridPanel::Slot(2, 0)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.Text(LOCTEXT("ReplaceButton", "Replace"))
					.ToolTipText(LOCTEXT("ReplaceButtonTooltip", "Replace selected items"))
					.IsEnabled_Lambda([this]()
					{
						return bAssetsSelected && CurrentProcessor->CanCurrentlyReplace();
					})
					.OnClicked(this, &SAnimAssetFindReplace::HandleReplace)
				]
				+SUniformGridPanel::Slot(3, 0)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.Text(LOCTEXT("ReplaceAllButton", "Replace All"))
					.ToolTipText(LOCTEXT("ReplaceAllButtonTooltip", "Replace all matching items"))
					.IsEnabled_Lambda([this]()
					{
						return bFoundAssets && CurrentProcessor->CanCurrentlyReplace();
					})
					.OnClicked(this, &SAnimAssetFindReplace::HandleReplaceAll)
				]
			]
		]
	];

	RequestRefreshUI();

	RegisterActiveTimer(0.0f, FWidgetActiveTimerDelegate::CreateLambda([this](double InCurrentTime, float InDeltaTime)
	{
		CurrentProcessor->FocusInitialWidget();
		return EActiveTimerReturnType::Stop;
	}));
	
	RegisterActiveTimer(1.0f / 60.0f, FWidgetActiveTimerDelegate::CreateLambda([this](double InCurrentTime, float InDeltaTime)
	{
		if(bRefreshUIRequested)
		{
			bRefreshUIRequested = false;
			ToolbarContainer->SetContent(UToolMenus::Get()->GenerateWidget("AnimAssetFindReplaceToolbar", FToolMenuContext(ToolbarContext)));
			FindReplaceWidgetContainer->SetContent(CurrentProcessor->MakeFindReplaceWidget());
			bRefreshCachedDataRequested = true;
		}

		if(bRefreshCachedDataRequested)
		{
			bRefreshCachedDataRequested = false;
			CurrentProcessor->RefreshCachedData();
			bRefreshSearchResultsRequested = true;
		}

		if(bRefreshSearchResultsRequested)
		{
			bRefreshSearchResultsRequested = false;
			RefreshSearchResults();
		}

		return EActiveTimerReturnType::Continue;
	}));
}

void SAnimAssetFindReplace::SetCurrentProcessor(TSubclassOf<UAnimAssetFindReplaceProcessor> InProcessorClass)
{
	TObjectPtr<UAnimAssetFindReplaceProcessor>* FoundProcessor = Processors.Find(InProcessorClass);
	if(FoundProcessor)
	{
		CurrentProcessor = *FoundProcessor;
	}

	RequestRefreshUI();
}

UAnimAssetFindReplaceProcessor* SAnimAssetFindReplace::GetProcessor(TSubclassOf<UAnimAssetFindReplaceProcessor> InProcessorClass) const
{
	const TObjectPtr<UAnimAssetFindReplaceProcessor>* FoundProcessor = Processors.Find(InProcessorClass);
	return FoundProcessor ? *FoundProcessor : nullptr;
}

void SAnimAssetFindReplace::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObjects(Processors);
	Collector.AddReferencedObject(ToolbarContext);
}

FString SAnimAssetFindReplace::GetReferencerName() const
{
	return TEXT("SAnimAssetFindReplace");
}

FARFilter SAnimAssetFindReplace::MakeARFilter() const
{
	FARFilter Filter;
	for(const UClass* AssetClass : CurrentProcessor->GetSupportedAssetTypes())
	{
		Filter.ClassPaths.Add(AssetClass->GetClassPathName());
	}
	Filter.bRecursiveClasses = true;

	return Filter;
}

void SAnimAssetFindReplace::RefreshSearchResults()
{
	bFoundAssets = false;
	OldAssets.Empty();
	AssetPickerConfig.Filter = MakeARFilter();
	SetARFilterDelegate.ExecuteIfBound(AssetPickerConfig.Filter);
	RefreshAssetViewDelegate.ExecuteIfBound(true);
}

bool SAnimAssetFindReplace::ShouldFilterOutAsset(const FAssetData& InAssetData, bool& bOutIsOldAsset) const
{
	FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>(TEXT("AssetTools"));
	
	const bool bIsEditableAsset = AssetToolsModule.Get().GetWritableFolderPermissionList()->PassesStartsWithFilter(InAssetData.PackageName);
	if (!bIsEditableAsset)
	{
		return true;
	}

	return CurrentProcessor->ShouldFilterOutAsset(InAssetData, bOutIsOldAsset);
}

bool SAnimAssetFindReplace::HandleFilterAsset(const FAssetData& InAssetData)
{
	bool bIsOldAsset = false;
	const bool bShouldFilterOut = ShouldFilterOutAsset(InAssetData, bIsOldAsset);
	bFoundAssets |= !bShouldFilterOut;
	if(bIsOldAsset)
	{
		OldAssets.Add(InAssetData);
	}
	return bShouldFilterOut;
}

FReply SAnimAssetFindReplace::HandleReplace()
{
	if(GetCurrentSelectionDelegate.IsBound())
	{
		TArray<FAssetData> SelectedAssets = GetCurrentSelectionDelegate.Execute();
		if(SelectedAssets.Num() > 0)
		{
			ReplaceInAssets(SelectedAssets);
		}
	}

	return FReply::Handled();
}

FReply SAnimAssetFindReplace::HandleReplaceAll()
{
	// Apply current filter
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");

	TArray<FAssetData> FilteredAssets;
	AssetRegistryModule.Get().GetAssets(AssetPickerConfig.Filter,FilteredAssets);

	FilteredAssets.RemoveAll([this](const FAssetData& InAssetData)
	{
		bool bIsOldAsset = false;
		return ShouldFilterOutAsset(InAssetData, bIsOldAsset);
	});

	ReplaceInAssets(FilteredAssets);

	return FReply::Handled();
}

void SAnimAssetFindReplace::ReplaceInAssets(const TArray<FAssetData>& InAssetDatas)
{
	const FText TypeName = CurrentProcessor->GetClass()->GetDisplayNameText();
	const FText MessageText = FText::Format(LOCTEXT("ReplacingTaskStatus", "Replacing {0} in {1} Assets..."), TypeName, FText::AsNumber(InAssetDatas.Num()));
	FScopedSlowTask ScopedSlowTask(static_cast<float>(InAssetDatas.Num()), MessageText);
	ScopedSlowTask.MakeDialog(true);

	FScopedTransaction ScopedTransaction(FText::Format(LOCTEXT("ReplaceTransaction", "Replace {0}."), TypeName));
	
	for(const FAssetData& AssetData : InAssetDatas)
	{
		ScopedSlowTask.EnterProgressFrame();

		ReplaceInAsset(AssetData);

		if(ScopedSlowTask.ShouldCancel())
		{
			break;
		}
	}

	RequestRefreshCachedData();
}

void SAnimAssetFindReplace::ReplaceInAsset(const FAssetData& InAssetData) const
{
	CurrentProcessor->ReplaceInAsset(InAssetData);
}

FReply SAnimAssetFindReplace::HandleRemove()
{
	if(GetCurrentSelectionDelegate.IsBound())
	{
		TArray<FAssetData> SelectedAssets = GetCurrentSelectionDelegate.Execute();
		if(SelectedAssets.Num() > 0)
		{
			RemoveInAssets(SelectedAssets);
		}
	}

	return FReply::Handled();
}

FReply SAnimAssetFindReplace::HandleRemoveAll()
{
	// Apply current filter
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");

	TArray<FAssetData> FilteredAssets;
	AssetRegistryModule.Get().GetAssets(AssetPickerConfig.Filter,FilteredAssets);

	FilteredAssets.RemoveAll([this](const FAssetData& InAssetData)
	{
		bool bIsOldAsset = false;
		return ShouldFilterOutAsset(InAssetData, bIsOldAsset);
	});

	RemoveInAssets(FilteredAssets);

	return FReply::Handled();
}

void SAnimAssetFindReplace::RemoveInAssets(const TArray<FAssetData>& InAssetDatas)
{
	const FText TypeName = CurrentProcessor->GetClass()->GetDisplayNameText();
	const FText MessageText = FText::Format(LOCTEXT("RemovingTaskStatus", "Removing {0} in {1} Assets..."), TypeName, FText::AsNumber(InAssetDatas.Num()));
	FScopedSlowTask ScopedSlowTask(static_cast<float>(InAssetDatas.Num()), MessageText);
	ScopedSlowTask.MakeDialog(true);

	FScopedTransaction ScopedTransaction(FText::Format(LOCTEXT("RemoveTransaction", "Remove {0}."), TypeName));

	for(const FAssetData& AssetData : InAssetDatas)
	{
		ScopedSlowTask.EnterProgressFrame();

		RemoveInAsset(AssetData);

		if(ScopedSlowTask.ShouldCancel())
		{
			break;
		}
	}

	RequestRefreshCachedData();
}

void SAnimAssetFindReplace::RemoveInAsset(const FAssetData& InAssetData) const
{
	CurrentProcessor->RemoveInAsset(InAssetData);
}

#undef LOCTEXT_NAMESPACE
