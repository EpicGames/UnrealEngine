// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "IContentBrowserSingleton.h"
#include "IDocumentation.h"
#include "PersonaTabs.h"
#include "Widgets/SWindow.h"
#include "Widgets/SCompoundWidget.h"
#include "WorkflowOrientedApp/WorkflowTabFactory.h"
#include "AnimAssetFindReplace.h"

class SBox;
class SAutoCompleteSearchBox;

namespace AnimAssetFindReplacePrivate
{
extern TSharedPtr<SAnimAssetFindReplace> GetWidgetFromContext(const FToolMenuContext& InContext);
}

struct FAnimAssetFindReplaceSummoner : public FWorkflowTabFactory
{
public:
	FAnimAssetFindReplaceSummoner(TSharedPtr<FAssetEditorToolkit> InHostingApp, const FAnimAssetFindReplaceConfig& InConfig);

	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;
	virtual TSharedPtr<SToolTip> CreateTabToolTipWidget(const FWorkflowTabSpawnInfo& Info) const override;

private:
	FAnimAssetFindReplaceConfig Config;
};

class SAnimAssetFindReplace : public IAnimAssetFindReplace, public FGCObject
{
public:
	friend class UAnimAssetFindReplaceProcessor;

	SLATE_BEGIN_ARGS(SAnimAssetFindReplace) {}

	SLATE_ARGUMENT(FAnimAssetFindReplaceConfig, Config)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	// IAnimAssetFindReplace interface
	virtual void SetCurrentProcessor(TSubclassOf<UAnimAssetFindReplaceProcessor> InProcessor) override;
	virtual UAnimAssetFindReplaceProcessor* GetCurrentProcessor() const override { return CurrentProcessor; }
	virtual UAnimAssetFindReplaceProcessor* GetProcessor(TSubclassOf<UAnimAssetFindReplaceProcessor> InProcessor) const override;

private:
	// FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override;

	void RefreshSearchResults();

	bool HandleFilterAsset(const FAssetData& InAssetData);

	FReply HandleReplace();

	FReply HandleReplaceAll();

	void ReplaceInAssets(const TArray<FAssetData>& InAssetDatas);

	void ReplaceInAsset(const FAssetData& InAssetData) const;

	FReply HandleRemove();

	FReply HandleRemoveAll();
	
	void RemoveInAssets(const TArray<FAssetData>& InAssetDatas);

	void RemoveInAsset(const FAssetData& InAssetData) const;

	bool ShouldFilterOutAsset(const FAssetData& InAssetData, bool& bOutIsOldAsset) const;

	// Request a 'full' refresh. Rebuilds find/replace widgets and also falls through to bRefreshRequested
	void RequestRefreshUI() { bRefreshUIRequested = true; }

	// Request a refresh of cached data and search results. Re-caches data also falls through to bRefreshSearchResultsRequested
	void RequestRefreshCachedData() { bRefreshCachedDataRequested = true; }

	// Refresh search results
	void RequestRefreshSearchResults() { bRefreshSearchResultsRequested = true; }

	FARFilter MakeARFilter() const;

private:
	FAnimAssetFindReplaceConfig Config;

	FAssetPickerConfig AssetPickerConfig;

	EAnimAssetFindReplaceMode Mode = EAnimAssetFindReplaceMode::Find;

	TMap<TSubclassOf<UAnimAssetFindReplaceProcessor>, TObjectPtr<UAnimAssetFindReplaceProcessor>> Processors;

	UAnimAssetFindReplaceProcessor* CurrentProcessor = nullptr;

	FRefreshAssetViewDelegate RefreshAssetViewDelegate;

	FGetCurrentSelectionDelegate GetCurrentSelectionDelegate;

	FSetARFilterDelegate SetARFilterDelegate;

	bool bAssetsSelected = false;

	bool bFoundAssets = false;

	bool bRefreshCachedDataRequested = false;

	bool bRefreshUIRequested = false;

	bool bRefreshSearchResultsRequested = false;

	TArray<FAssetData> OldAssets;

	TObjectPtr<UAnimAssetFindReplaceContext> ToolbarContext;

	TSharedPtr<SBox> ToolbarContainer;

	TSharedPtr<SBox> FindReplaceWidgetContainer;
};
