// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAnimAttributePicker.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "IEditableSkeleton.h"
#include "Animation/AnimationAsset.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "SListViewSelectorDropdownMenu.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Layout/SMenuOwner.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Input/SCheckBox.h"
#include "Engine/SkeletalMesh.h"
#include "Widgets/Input/SHyperlink.h"
#include "SkinnedAssetCompiler.h"

#define LOCTEXT_NAMESPACE "SAnimAttributePicker"

const FName SAnimAttributePicker::FColumnIds::AttributeName = "AttributeName";
const FName SAnimAttributePicker::FColumnIds::BoneName = "BoneName";
const FName SAnimAttributePicker::FColumnIds::AttributeType = "AttributeType";

void SAnimAttributePicker::Construct(const FArguments& InArgs, const TObjectPtr<const USkeleton> InSkeleton)
{
	OnAttributesPicked = InArgs._OnAttributesPicked;
	Skeleton = InSkeleton;
	bShowOtherSkeletonAttributes = false;

	SAssignNew(SearchBox, SSearchBox)
		.HintText(LOCTEXT("SearchBoxHint", "Search"))
		.OnTextChanged(this, &SAnimAttributePicker::HandleFilterTextChanged);

	SAssignNew(NameListView, SListView<TSharedPtr<FAttributeData>>)
		.SelectionMode(InArgs._bEnableMultiSelect ? ESelectionMode::Multi : ESelectionMode::Single)
		.OnSelectionChanged_Lambda([this, InArgs](TSharedPtr<FAttributeData> InItem, ESelectInfo::Type SelectInfo)
			{
				if (InItem && !InArgs._bEnableMultiSelect)
				{
					FAnimationAttributeIdentifier AttributeIdentifier(
						InItem->AttributeName,
						InItem->BoneIndex,
						InItem->BoneName,
						FSoftObjectPath::ConstructFromStringPath(InItem->AttributeType.ToString()));

					OnAttributesPicked.ExecuteIfBound(TArray<FAnimationAttributeIdentifier>{ AttributeIdentifier });
				}
			})
		.ListItemsSource(&ShownAttributes)
		.HeaderRow(
			SNew(SHeaderRow)
			+ SHeaderRow::Column(FColumnIds::AttributeName)
				.DefaultLabel(LOCTEXT("AttributeName", "Attribute Name"))
				.SortMode(this, &SAnimAttributePicker::GetColumnSortMode, FColumnIds::AttributeName)
				.OnSort(this, &SAnimAttributePicker::OnSortModeChanged)
			+ SHeaderRow::Column(FColumnIds::BoneName)
				.DefaultLabel(LOCTEXT("BoneName", "Bone Name"))
				.SortMode(this, &SAnimAttributePicker::GetColumnSortMode, FColumnIds::BoneName)
				.OnSort(this, &SAnimAttributePicker::OnSortModeChanged)
			+ SHeaderRow::Column(FColumnIds::AttributeType)
				.DefaultLabel(LOCTEXT("AttributeType", "Attribute Type"))
				.SortMode(this, &SAnimAttributePicker::GetColumnSortMode, FColumnIds::AttributeType)
				.OnSort(this, &SAnimAttributePicker::OnSortModeChanged)
		)
		.OnGenerateRow(this, &SAnimAttributePicker::HandleGenerateRow);

	const float HorizontalPadding = 8.0f;
	const float VerticalPadding = 2.0f;
	const float WeightOverride = 300.0f;

	TSharedPtr<SVerticalBox> VerticalBox;

	ChildSlot
	[
		SNew(SMenuOwner)
		[
			SNew(SListViewSelectorDropdownMenu<TSharedPtr<FAttributeData>>, SearchBox, NameListView)
			[
				SAssignNew(VerticalBox, SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(HorizontalPadding, VerticalPadding)
				[
					SearchBox.ToSharedRef()
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(HorizontalPadding, VerticalPadding)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SHyperlink)
						.Visibility_Lambda([this]()
						{
							return OldAssets.Num() > 0 ? EVisibility::Visible : EVisibility::Collapsed;
						})
						.Text_Lambda([this]()
						{
							return FText::Format(LOCTEXT("UnindexedAssetWarningFormat", "{0} assets could not be indexed, load them now?"), FText::AsNumber(OldAssets.Num()));
						})
						.OnNavigate_Lambda([this]()
						{
							// Load all old unindexed assets
							FScopedSlowTask SlowTask(OldAssets.Num(), FText::Format(LOCTEXT("LoadingUnindexedAssetsFormat", "Loading {0} Unindexed Assets..."), FText::AsNumber(OldAssets.Num())));
							SlowTask.MakeDialog(true);

							for (const FAssetData& AssetData : OldAssets)
							{
								SlowTask.EnterProgressFrame();

								AssetData.GetAsset();

								if (SlowTask.ShouldCancel())
								{
									break;
								}
							}

							// Ensure all meshes are compiled after the load, as asset registry data isnt available correctly until they are
							FSkinnedAssetCompilingManager::Get().FinishAllCompilation();

							RefreshListItems();
						})
					]
				]
				+ SVerticalBox::Slot()
				.FillHeight(1.0f)
				.Padding(HorizontalPadding, VerticalPadding)
				[
					SNew(SBox)
					.WidthOverride(WeightOverride)
					.HeightOverride(WeightOverride)
					[
						SNew(SOverlay)
						+ SOverlay::Slot()
						[
							SNew(SBorder)
							.BorderImage(FAppStyle::GetBrush("Graph.StateNode.Body"))
							.BorderBackgroundColor(FAppStyle::Get().GetSlateColor("Colors.Input"))
						]
						+ SOverlay::Slot()
						[
							SNew(SScrollBox)
							.Orientation(EOrientation::Orient_Vertical)
							+ SScrollBox::Slot()
							.HAlign(HAlign_Fill)
							.VAlign(VAlign_Fill)
							[
								NameListView.ToSharedRef()
							]
						]
					]
				]
			]
		]
	];

	if (Skeleton.IsValid() && InArgs._bCanShowOtherSkeletonAttributes)
	{
		VerticalBox->AddSlot()
			.AutoHeight()
			.Padding(HorizontalPadding, VerticalPadding)
			[
				SNew(SCheckBox)
					.ToolTipText(LOCTEXT("ShowOtherSkeletonsTooltip", "Whether to show all attributes or just the attributes from the current skeleton"))
					.IsChecked_Lambda([this]()
					{
						return bShowOtherSkeletonAttributes ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
					})
					.OnCheckStateChanged_Lambda([this](ECheckBoxState InState)
					{
						bShowOtherSkeletonAttributes = (InState == ECheckBoxState::Checked);
						RefreshListItems();
					})
					.Content()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("ShowOtherSkeletons", "Show attributes from other skeletons"))
					]
			];
	}

	if (InArgs._bEnableMultiSelect)
	{
		VerticalBox->AddSlot()
			.AutoHeight()
			.Padding(HorizontalPadding, VerticalPadding)
			[
				SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), "PrimaryButton")
					.Text(LOCTEXT("ConfirmButton", "Confirm"))
					.IsEnabled_Lambda([this]()
						{
							return NameListView->GetNumItemsSelected() > 0;
						})
					.OnClicked_Lambda([this]()
						{
							TArray<TSharedPtr<FAttributeData>> SelectionPtrs;
							NameListView->GetSelectedItems(SelectionPtrs);
							check(SelectionPtrs.Num() > 0);

							TArray<FAnimationAttributeIdentifier> SelectedAttributes;

							for (TSharedPtr<FAttributeData> Selected : SelectionPtrs)
							{
								FAnimationAttributeIdentifier Identifier(
									Selected->AttributeName,
									Selected->BoneIndex,
									Selected->BoneName,
									FSoftObjectPath::ConstructFromStringPath(Selected->AttributeType.ToString()));

								SelectedAttributes.Add(MoveTemp(Identifier));
							}

							OnAttributesPicked.ExecuteIfBound(SelectedAttributes);

							return FReply::Handled();
						})
			];
	}

	RefreshListItems();
}

class SAttributeDataRow : public SMultiColumnTableRow<TSharedPtr<SAnimAttributePicker::FAttributeData>>
{
public:
	SLATE_BEGIN_ARGS(SAttributeDataRow) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, TSharedPtr<SAnimAttributePicker::FAttributeData> InItem, const FString* InFilterText)
	{
		Item = InItem;
		FilterText = InFilterText;
		
		SMultiColumnTableRow<TSharedPtr<SAnimAttributePicker::FAttributeData>>::Construct(
			SMultiColumnTableRow<TSharedPtr<SAnimAttributePicker::FAttributeData>>::FArguments(), InOwnerTableView);
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		if (ColumnName == SAnimAttributePicker::FColumnIds::AttributeName)
		{
			return SNew(STextBlock)
				.Text(FText::FromName(Item->AttributeName))
				.HighlightText_Lambda([this]() { return FText::FromString(*FilterText); });
		}
		else if (ColumnName == SAnimAttributePicker::FColumnIds::BoneName)
		{
			return SNew(STextBlock)
				.Text(FText::FromName(Item->BoneName));
		}
		else
		{
			return SNew(STextBlock)
				.Text(FText::FromName(Item->AttributeType));
		}
	}

private:
	TSharedPtr<SAnimAttributePicker::FAttributeData> Item;
	const FString* FilterText;
};

TSharedRef<ITableRow> SAnimAttributePicker::HandleGenerateRow(TSharedPtr<FAttributeData> InItem, const TSharedRef<STableViewBase>& InOwnerTable)
{
	return SNew(SAttributeDataRow, InOwnerTable, InItem, &FilterText);
}

void SAnimAttributePicker::RefreshListItems()
{
	const USkeleton* CurrentSkeleton = Skeleton.Get();
	FString CurrentSkeletonName;
	if (CurrentSkeleton)
	{
		CurrentSkeletonName = FAssetData(CurrentSkeleton).GetExportTextName();
	}

	ShownAttributes.Reset();
	FoundUniqueAttributes.Reset();

	{
		const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");

		FARFilter Filter;
		Filter.bRecursiveClasses = true;
		Filter.ClassPaths.Append({ UAnimationAsset::StaticClass()->GetClassPathName() });

		TArray<FAssetData> FoundAssetData;
		AssetRegistryModule.Get().GetAssets(Filter, FoundAssetData);

		OldAssets.Empty();

		for (FAssetData& AssetData : FoundAssetData)
		{
			if (!bShowOtherSkeletonAttributes)
			{
				if (!CurrentSkeletonName.IsEmpty())
				{
					const FString SkeletonName = AssetData.GetTagValueRef<FString>(USkeleton::StaticClass()->GetFName());
					if (SkeletonName != CurrentSkeletonName)
					{
						continue;
					}
				}
			}

			const FString TagValue = AssetData.GetTagValueRef<FString>(USkeleton::AttributeTag);
			if (!TagValue.IsEmpty())
			{
				FAnimationAttributeIdentifierArray AttributesIdentifierArray;
				FAnimationAttributeIdentifierArray::StaticStruct()->ImportText(*TagValue, &AttributesIdentifierArray, nullptr, PPF_None, GLog, FAnimationAttributeIdentifierArray::StaticStruct()->GetName());
				
				for (const FAnimationAttributeIdentifier& AttrId : AttributesIdentifierArray.AttributeIdentifiers)
				{
					FoundUniqueAttributes.Add(FAttributeData(AttrId));
				}
			}
			else
			{
				OldAssets.Add(AssetData);
			}
		}
	}

	FilterAvailableAttributes();
}

void SAnimAttributePicker::FilterAvailableAttributes()
{
	ShownAttributes.Reset();

	// Exact filtering
	for (const FAttributeData& AttributeData : FoundUniqueAttributes)
	{
		if (FilterText.IsEmpty() || AttributeData.AttributeName.ToString().Contains(FilterText))
		{
			ShownAttributes.Add(MakeShared<FAttributeData>(AttributeData));
		}
	}

	// Alphabetical sorting
	{
		struct FNameSortItemSortOp
		{
			FORCEINLINE bool operator()(const TSharedPtr<FAttributeData>& A, const TSharedPtr<FAttributeData>& B) const
			{
				return (A->AttributeName.ToString().Compare(B->AttributeName.ToString()) < 0);
			}
		};
		ShownAttributes.Sort(FNameSortItemSortOp());

		SortColumn = FColumnIds::AttributeName;
		SortMode = EColumnSortMode::Type::Ascending;
	}

	// Rebuild list view
	NameListView->RequestListRefresh();
}

void SAnimAttributePicker::HandleFilterTextChanged(const FText& InFilterText)
{
	FilterText = InFilterText.ToString();

	FilterAvailableAttributes();
}


EColumnSortMode::Type SAnimAttributePicker::GetColumnSortMode(FName ColumnId) const
{
    return (SortColumn == ColumnId) ? SortMode : EColumnSortMode::None;
}

void SAnimAttributePicker::OnSortModeChanged(EColumnSortPriority::Type Priority, const FName& ColumnId, EColumnSortMode::Type NewSortMode)
{
	SortColumn = ColumnId;
	SortMode = NewSortMode;

	ShownAttributes.Sort([this](const TSharedPtr<FAttributeData>& A, const TSharedPtr<FAttributeData>& B) -> bool
		{
			const bool bAscending = (SortMode == EColumnSortMode::Ascending);

			if (SortColumn == "AttributeName")
			{
				return bAscending
					? A->AttributeName.ToString() < B->AttributeName.ToString()
					: A->AttributeName.ToString() > B->AttributeName.ToString();
			}
			else if (SortColumn == "BoneName")
			{
				return bAscending
					? A->BoneName.ToString() < B->BoneName.ToString()
					: A->BoneName.ToString() > B->BoneName.ToString();
			}
			else if (SortColumn == "AttributeType")
			{
				return bAscending
					? A->AttributeType.ToString() < B->AttributeType.ToString()
					: A->AttributeType.ToString() > B->AttributeType.ToString();
			}

			return true;
		});

	NameListView->RequestListRefresh();
}

#undef LOCTEXT_NAMESPACE