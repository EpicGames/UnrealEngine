// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimData/AttributeIdentifier.h"
#include "CoreMinimal.h"
#include "Delegates/DelegateCombinations.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Views/STableRow.h"
#include "PersonaDelegates.h"

class SAnimAttributePicker : public SCompoundWidget
{
public:
	struct FColumnIds
	{
		static const FName AttributeName;
		static const FName BoneName;
		static const FName AttributeType;
	};

	/** Virtual destructor. */
	virtual ~SAnimAttributePicker() override = default;

	SLATE_BEGIN_ARGS(SAnimAttributePicker)
		: _bEnableMultiSelect(false)
		, _bCanShowOtherSkeletonAttributes(false)
	{}

	SLATE_EVENT(FOnAttributesPicked, OnAttributesPicked)

	SLATE_ARGUMENT(bool, bEnableMultiSelect)

	SLATE_ARGUMENT(bool, bCanShowOtherSkeletonAttributes)
	
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TObjectPtr<const USkeleton> InSkeleton);

private:
	void RefreshListItems();

	void FilterAvailableAttributes();

public:
	struct FAttributeData
	{
		FName AttributeName;

		FName BoneName;

		int32 BoneIndex;

		FName AttributeType;

		FAttributeData(const FAnimationAttributeIdentifier& InAttributeId)
			: AttributeName(InAttributeId.GetName())
			, BoneName(InAttributeId.GetBoneName())
			, BoneIndex(InAttributeId.GetBoneIndex())
			, AttributeType(FName(InAttributeId.GetScriptStructPath().ToString()))
		{
		}

		bool operator==(const FAttributeData& Other) const
		{
			return AttributeName == Other.AttributeName;
		}

		inline friend uint32 GetTypeHash(const FAttributeData& Attribute)
		{
			return GetTypeHash(Attribute.AttributeName);
		}
	};

private:
	TSharedRef<ITableRow> HandleGenerateRow(TSharedPtr<FAttributeData> InItem, const TSharedRef<STableViewBase>& InOwnerTable);

	void HandleFilterTextChanged(const FText& InFilterText);

	EColumnSortMode::Type GetColumnSortMode(FName ColumnId) const;

	void OnSortModeChanged(EColumnSortPriority::Type Priority, const FName& ColumnId, EColumnSortMode::Type NewSortMode);

private:
	FOnAttributesPicked OnAttributesPicked;

	TWeakObjectPtr<const USkeleton> Skeleton;

	TArray<TSharedPtr<FAttributeData>> ShownAttributes;

	TSet<FAttributeData> FoundUniqueAttributes;

	FString FilterText;

	TSharedPtr<SListView<TSharedPtr<FAttributeData>>> NameListView;

	TSharedPtr<SSearchBox> SearchBox;

	bool bShowOtherSkeletonAttributes = false;

	TArray<FAssetData> OldAssets;

	FName SortColumn;

	EColumnSortMode::Type SortMode = EColumnSortMode::None;
};