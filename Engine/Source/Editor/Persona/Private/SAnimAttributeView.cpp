// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAnimAttributeView.h"
#include "AnimPreviewInstance.h"
#include "IPersonaPreviewScene.h"
#include "Animation/BuiltInAttributeTypes.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "IStructureDetailsView.h"
#include "Animation/AnimBlueprintGeneratedClass.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Views/SListView.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Styling/AppStyle.h"
#include "Widgets/Layout/SScrollBox.h"
#include "SPoseWatchPicker.h"
#include "Engine/PoseWatch.h"

#define LOCTEXT_NAMESPACE "SAnimAttributeView"

static const FName ColumnId_AnimAttributeName( TEXT("AttributeName") );
static const FName ColumnId_AnimAttributeBoneName( TEXT("BoneName") );
static const FName ColumnId_AnimAttributeTypeName( TEXT("TypeName") );
static const FName ColumnId_AnimAttributeSnapshotName( TEXT("SnapshotName") );

static const int32 MinimalNumColumns = 3;

static FName GetAnimAttributeTypeName(const UScriptStruct* InType)
{
	return *(InType->GetName().Replace(TEXT("AnimationAttribute"), TEXT("")));
}

TSharedRef<IStructureDetailsView> SAnimAttributeView::CreateValueViewWidget()
{
	FDetailsViewArgs DetailsViewArgs;
	{
		DetailsViewArgs.bLockable = false;
		DetailsViewArgs.bUpdatesFromSelection = false;
		DetailsViewArgs.bAllowSearch = false;
		DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::ENameAreaSettings::HideNameArea;
		DetailsViewArgs.bAllowMultipleTopLevelObjects = false;
		DetailsViewArgs.bShowPropertyMatrixButton = false;
	}

	const FStructureDetailsViewArgs StructureViewArgs;
	
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	TSharedRef<IStructureDetailsView> ValueView = PropertyEditorModule.CreateStructureDetailView(DetailsViewArgs, StructureViewArgs, nullptr);
	ValueView->GetDetailsView()->SetIsPropertyEditingEnabledDelegate(
		FIsPropertyEditingEnabled::CreateLambda( []() { return false; }));

	return ValueView;
}

TSharedRef<ITableRow> SAnimAttributeView::MakeTableRowWidget(
	TSharedPtr<FAnimAttributeEntry> InItem,
	const TSharedRef<STableViewBase>& InOwnerTable)
{
	return InItem->MakeTableRowWidget(InOwnerTable);
}


TSharedRef<FAnimAttributeEntry> FAnimAttributeEntry::MakeEntry(
	const FAnimationAttributeIdentifier& InIdentifier,
	const FName& InSnapshotDisplayName)
{
	return MakeShareable(new FAnimAttributeEntry(InIdentifier, InSnapshotDisplayName)); 
}

FName SAnimAttributeView::GetSnapshotColumnDisplayName(const TArray<FName>& InSnapshotNames)
{
	if (InSnapshotNames.Num() == 0)
	{
		return NAME_None;
	}
	
	if (InSnapshotNames.Num() == 1)
	{
		return InSnapshotNames[0];
	}
	return *(InSnapshotNames[0].ToString() + " - " + InSnapshotNames.Last().ToString());
}


FAnimAttributeEntry::FAnimAttributeEntry(const FAnimationAttributeIdentifier& InIdentifier, const FName& InSnapshotDisplayName)
{
	Identifier = InIdentifier;

	SnapshotDisplayName = InSnapshotDisplayName;
	
	CachedTypeName = GetAnimAttributeTypeName(InIdentifier.GetType());
}

TSharedRef<ITableRow> FAnimAttributeEntry::MakeTableRowWidget(const TSharedRef<STableViewBase>& InOwnerTable)
{
	return SNew(SAnimAttributeEntry, InOwnerTable, SharedThis(this));
}

FName FAnimAttributeEntry::GetDisplayName() const
{
	return GetName();
}

void SAnimAttributeEntry::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable, TSharedRef<FAnimAttributeEntry> InEntry)
{
	Entry = InEntry;
	SMultiColumnTableRow< TSharedPtr<FAnimAttributeEntry> >::Construct(
		FSuperRowType::FArguments(),
		InOwnerTable );
}

TSharedRef<SWidget> SAnimAttributeEntry::GenerateWidgetForColumn(const FName& ColumnName)
{
	if (ColumnName == ColumnId_AnimAttributeName)
	{
		return
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(4)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(this, &SAnimAttributeEntry::GetEntryName)	
			];
	}
	else if (ColumnName == ColumnId_AnimAttributeBoneName)
	{
		return
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(4)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(this, &SAnimAttributeEntry::GetEntryBoneName)	
			];
	}
	else if (ColumnName == ColumnId_AnimAttributeTypeName)
	{
		return
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(4)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(this, &SAnimAttributeEntry::GetEntryTypeName)	
			];
	}
	else if (ColumnName == ColumnId_AnimAttributeSnapshotName)
	{
		return
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(4)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(this, &SAnimAttributeEntry::GetEntrySnapshotDisplayName)	
			];
	}

	return SNullWidget::NullWidget;
}

FText SAnimAttributeEntry::GetEntryName() const
{
	return FText::FromName(Entry.Pin()->GetName());
}

FText SAnimAttributeEntry::GetEntryBoneName() const
{
	return FText::FromName(Entry.Pin()->GetBoneName());
}

FText SAnimAttributeEntry::GetEntryTypeName() const
{
	return FText::FromName(Entry.Pin()->GetTypeName());
}

FText SAnimAttributeEntry::GetEntrySnapshotDisplayName() const
{
	return FText::FromName(Entry.Pin()->GetSnapshotDisplayName());
}

SAnimAttributeView::SAnimAttributeView()
{
	bShouldRefreshListView = false;
	bShouldRefreshValueView = false;
	CachedNumSnapshots = 0;
	ColumnIdToSort = ColumnId_AnimAttributeSnapshotName;
	ActiveSortMode = EColumnSortMode::Ascending;
}

void SAnimAttributeView::Construct(const FArguments& InArgs)
{
	OnGetAttributeSnapshotColumnDisplayName = InArgs._OnGetAttributeSnapshotColumnDisplayName;
	SnapshotColumnLabelOverride = InArgs._SnapshotColumnLabelOverride;

	SExpandableArea::FArguments ExpandableAreaArgs;
	ExpandableAreaArgs.AreaTitleFont(FAppStyle::GetFontStyle("DetailsView.CategoryFontStyle"));
	
	ChildSlot
	[
		SNew(SSplitter)
		.Orientation(Orient_Vertical)
		+SSplitter::Slot()
		.Value(0.6)
		.MinSize(80.0f)
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			.VAlign(VAlign_Top)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.Padding(3.0f, 1.0f)
				[
					SNew(SSearchBox)
					.OnTextChanged(this, &SAnimAttributeView::OnFilterTextChanged)
				]
			]
			+SVerticalBox::Slot()
			[
				SAssignNew(AttributeListView, SListView<TSharedPtr<FAnimAttributeEntry>>)
				.ListItemsSource(&FilteredAttributeEntries)
				.OnGenerateRow_Static(&SAnimAttributeView::MakeTableRowWidget)
				.OnSelectionChanged(this, &SAnimAttributeView::OnSelectionChanged)
				.HeaderRow
				(
					SAssignNew(HeaderRow, SHeaderRow)
					+ SHeaderRow::Column( ColumnId_AnimAttributeName)
					.FillWidth(1.f)
					.DefaultLabel( LOCTEXT( "AnimAttributeNameLabel", "Name" ) )
					.SortMode_Raw(this, &SAnimAttributeView::GetSortModeForColumn, ColumnId_AnimAttributeName)
					.OnSort_Raw(this, &SAnimAttributeView::OnSortAttributeEntries)

					+ SHeaderRow::Column( ColumnId_AnimAttributeBoneName)
					.FillWidth(1.f)
					.DefaultLabel( LOCTEXT( "AnimAttributeBoneNameLabel", "Bone" ) )
					.SortMode_Raw(this, &SAnimAttributeView::GetSortModeForColumn, ColumnId_AnimAttributeBoneName)
					.OnSort_Raw(this, &SAnimAttributeView::OnSortAttributeEntries)
					
					+ SHeaderRow::Column( ColumnId_AnimAttributeTypeName)
					.FillWidth(1.f)
					.DefaultLabel( LOCTEXT( "AnimAttributeTypeNameLabel", "Type" ) )
					.SortMode_Raw(this, &SAnimAttributeView::GetSortModeForColumn, ColumnId_AnimAttributeTypeName)
					.OnSort_Raw(this, &SAnimAttributeView::OnSortAttributeEntries)
				)
			]
		]
		+SSplitter::Slot()
		.Value(0.4)
		.MinSize(30.0f)
		[
			SAssignNew(ValueViewBox, SScrollBox)
		]	
	];
}

void SAnimAttributeView::OnSelectionChanged(TSharedPtr<FAnimAttributeEntry> InEntry, ESelectInfo::Type InSelectType)
{
	if (InEntry.IsValid())
	{
		SelectedAttribute = *InEntry.Get();
	}
	else
	{
		SelectedAttribute.Reset();
	}

	RefreshValueView();
}

void SAnimAttributeView::OnFilterTextChanged(const FText& InText)
{
	if (FilterText == InText.ToString())
	{
		return;
	}
	
	FilterText = InText.ToString();

	RefreshFilteredAttributeEntries();
}

EColumnSortMode::Type SAnimAttributeView::GetSortModeForColumn(FName InColumnId) const
{
	if (ColumnIdToSort == InColumnId)
	{
		return ActiveSortMode;
	}
		
	return EColumnSortMode::None;
}

void SAnimAttributeView::OnSortAttributeEntries(
   EColumnSortPriority::Type InPriority,
   const FName& InColumnId,
   EColumnSortMode::Type InSortMode)
{
	ColumnIdToSort = InColumnId;
	ActiveSortMode = InSortMode;
	
	ExecuteSort();
}

void SAnimAttributeView::ExecuteSort()
{
	static const TArray<FName> ColumnIds =
	{
		ColumnId_AnimAttributeSnapshotName,
		ColumnId_AnimAttributeName,
		ColumnId_AnimAttributeBoneName,
		ColumnId_AnimAttributeTypeName
	};

	TArray<FName> ColumnIdsBySortOrder = {ColumnIdToSort};
	for (const FName& Id : ColumnIds)
	{
		if (Id != ColumnIdToSort)
		{
			ColumnIdsBySortOrder.Add(Id);
		}
	}

	FilteredAttributeEntries.Sort(
		[ColumnIdsBySortOrder, this](const TSharedPtr<FAnimAttributeEntry>& Left, const TSharedPtr<FAnimAttributeEntry>& Right)
		{
			int32 CompareResult = 0;
			for (const FName& ColumnId : ColumnIdsBySortOrder)
			{
				if (ColumnId == ColumnId_AnimAttributeSnapshotName)
				{
					CompareResult = Left->GetSnapshotDisplayName().Compare(Right->GetSnapshotDisplayName());	
				}
				if (ColumnId == ColumnId_AnimAttributeName)
				{
					CompareResult = Left->GetName().Compare(Right->GetName());
				}
				else if (ColumnId == ColumnId_AnimAttributeBoneName)
				{
					CompareResult = Left->GetBoneName().Compare(Right->GetBoneName());
				}
				else if (ColumnId == ColumnId_AnimAttributeTypeName)
				{
					CompareResult = Left->GetTypeName().Compare(Right->GetTypeName());
				}
				
				if (CompareResult != 0)
				{
					// we have a winner
					return ActiveSortMode == EColumnSortMode::Ascending ? CompareResult < 0 : CompareResult > 0;
				}
			}

			// keep the original order if two entries are the same (though we should never have identical entries)
			return ActiveSortMode == EColumnSortMode::Ascending ? true : false;
		});

	bShouldRefreshListView = true;
}

void SAnimAttributeView::RefreshFilteredAttributeEntries()
{
	FilteredAttributeEntries.Reset(CachedAttributeSnapshotMap.Num());
	
	for (const TTuple<FAnimationAttributeIdentifier, TArray<FName>>& Identifier : CachedAttributeSnapshotMap)
	{
		FName SnapshotDisplayName;
		if (OnGetAttributeSnapshotColumnDisplayName.IsBound())
		{
			SnapshotDisplayName = OnGetAttributeSnapshotColumnDisplayName.Execute(Identifier.Value);	
		}
		else
		{
			SnapshotDisplayName = GetSnapshotColumnDisplayName(Identifier.Value);
		}
		
		if 	(FilterText.IsEmpty() ||
			Identifier.Key.GetName().ToString().Contains(FilterText) ||
			Identifier.Key.GetBoneName().ToString().Contains(FilterText) ||
			GetAnimAttributeTypeName(Identifier.Key.GetType()).ToString().Contains(FilterText) ||
			SnapshotDisplayName.ToString().Contains(FilterText))
		{
			FilteredAttributeEntries.Add(FAnimAttributeEntry::MakeEntry(Identifier.Key, SnapshotDisplayName));
		}
	}

	ExecuteSort();
	
	// delay the refresh to until tick since this function
	// can be invoked from animation thread
	bShouldRefreshListView = true;

	bool bSelectedAttributeStillValid = false;
	for (const TSharedPtr<FAnimAttributeEntry>& Entry : FilteredAttributeEntries)
	{
		if (SelectedAttribute.IsSet())
		{
			if (*(Entry) == SelectedAttribute.GetValue())
			{
				bSelectedAttributeStillValid = true;
				break;
			}
		}	
	}

	if (!bSelectedAttributeStillValid)
	{
		SelectedAttribute.Reset();
		bShouldRefreshValueView = true;
	}
}

void SAnimAttributeView::RefreshValueView()
{
	for (const FAttributeValueView& ValueView : SelectedAttributeSnapshotValueViews)
	{
		ValueViewBox->RemoveSlot(ValueView.ViewWidget->GetWidget().ToSharedRef());
	}
	
	SelectedAttributeSnapshotValueViews.Reset();
	
	if (SelectedAttribute.IsSet())
	{
		const FAnimationAttributeIdentifier& Identifier = SelectedAttribute->GetAnimationAttributeIdentifier();

		if (const TArray<FName>* SnapshotNames = CachedAttributeSnapshotMap.Find(Identifier))
		{
			for (const FName& SnapshotName : *SnapshotNames)
			{
				SelectedAttributeSnapshotValueViews.Add(
					FAttributeValueView(
							SnapshotName,
							SelectedAttribute.GetValue()));
			}
		}
	}

	
	// for (int32 ViewIndex = SelectedAttributeSnapshotValueViews.Num() - 1; ViewIndex >= 0; ViewIndex--)
	for (int32 ViewIndex = 0; ViewIndex < SelectedAttributeSnapshotValueViews.Num(); ViewIndex++)
	{
		const FAttributeValueView& ValueView = SelectedAttributeSnapshotValueViews[ViewIndex];
		// slots are added in reverse order
		ValueViewBox->AddSlot()
		[
			ValueView.ViewWidget->GetWidget().ToSharedRef()
		];	
	}
}

void SAnimAttributeView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	
	if (bShouldRefreshListView)
	{
		if (CachedNumSnapshots == 1 && HeaderRow->GetColumns().Num() > MinimalNumColumns)
		{
			HeaderRow->RemoveColumn(ColumnId_AnimAttributeSnapshotName);
		}
		else if (CachedNumSnapshots > 1 && HeaderRow->GetColumns().Num() <= MinimalNumColumns)
		{
			SHeaderRow::FColumn::FArguments	ColumnArgs;

			ColumnArgs
				.ColumnId(ColumnId_AnimAttributeSnapshotName)
				.FillWidth(1.f)
				.DefaultLabel( SnapshotColumnLabelOverride )
				.SortMode_Raw(this, &SAnimAttributeView::GetSortModeForColumn, ColumnId_AnimAttributeSnapshotName)
				.OnSort_Raw(this, &SAnimAttributeView::OnSortAttributeEntries);
			
			HeaderRow->AddColumn(ColumnArgs);
		}
		
		AttributeListView->RequestListRefresh();
		bShouldRefreshListView = false;
	}

	if (bShouldRefreshValueView)
	{
		RefreshValueView();
		bShouldRefreshValueView = false;
	}
}

SAnimAttributeView::FAttributeValueView::FAttributeValueView(FName InSnapshotName,
	const FAnimAttributeEntry& InSelectedAttribute)
{
	SubjectAttribute = InSelectedAttribute;
	SnapshotName = InSnapshotName;
	StructData = MakeShareable(new FStructOnScope(InSelectedAttribute.GetScriptStruct()));
	ViewWidget = CreateValueViewWidget();

	const FString DisplayName = InSelectedAttribute.GetDisplayName().ToString() + TEXT(" - ") + InSnapshotName.ToString(); 
	
	ViewWidget->SetCustomName(FText::FromString(DisplayName));
	ViewWidget->SetStructureData(StructData);
}

void SAnimAttributeViewer::Construct(const FArguments& InArgs, const TSharedRef<IPersonaPreviewScene>& InPreviewScene)
{
	PreviewScenePtr = InPreviewScene;

	SAssignNew(AttributeView, SAnimAttributeView);
	SAssignNew(PoseWatchPicker, SPoseWatchPicker)
	.AnimBlueprintGeneratedClass_Lambda([this]() -> const UAnimBlueprintGeneratedClass*
			{
				if(const UAnimInstance* AnimInstance = GetAnimInstance())
				{
					return Cast<UAnimBlueprintGeneratedClass>(AnimInstance->GetClass());
				}
				
				return nullptr;
			})
	.DefaultEntryDisplayText(LOCTEXT("DefaultEntryDisplayText", "Output Attributes"));

	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			PoseWatchPicker.ToSharedRef()
		]
		+SVerticalBox::Slot()
		.FillHeight(1.f)
		[
			AttributeView.ToSharedRef()
		]
	];
}

UAnimInstance* SAnimAttributeViewer::GetAnimInstance() const
{
	UDebugSkelMeshComponent* MeshComponent = PreviewScenePtr.Pin()->GetPreviewMeshComponent();
	UAnimInstance* AnimInstance = MeshComponent->GetAnimInstance();
	if (AnimInstance)
	{
		// Look at the debugged anim instance if we are targeting one
		if(AnimInstance == MeshComponent->PreviewInstance.Get())
		{
			UAnimPreviewInstance* AnimPreviewInstance = MeshComponent->PreviewInstance;
			if(USkeletalMeshComponent* DebuggedComponent = AnimPreviewInstance->GetDebugSkeletalMeshComponent())
			{
				AnimInstance = DebuggedComponent->GetAnimInstance();
			}
		}
	}

	return AnimInstance;
}

void SAnimAttributeViewer::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{	
	const UAnimInstance* AnimInstance = GetAnimInstance();
	if(AnimInstance && PoseWatchPicker.IsValid())
	{
		// Find if we want to use a pose watch
		if(UPoseWatchPoseElement* PoseWatchPoseElement = PoseWatchPicker->GetCurrentPoseWatch())
		{
			if(UAnimBlueprintGeneratedClass* AnimClass = Cast<UAnimBlueprintGeneratedClass>(AnimInstance->GetClass()))
			{
				// We have to grab our pose watches from the root class as no pose watches can be set on child anim BPs
				if(const UAnimBlueprintGeneratedClass* RootClass = Cast<UAnimBlueprintGeneratedClass>(AnimClass->GetRootClass()))
				{
					const FAnimBlueprintDebugData& DebugData = RootClass->AnimBlueprintDebugData;
					for(const FAnimNodePoseWatch& AnimNodePoseWatch : DebugData.AnimNodePoseWatch)
					{
						if(AnimNodePoseWatch.PoseWatchPoseElement == PoseWatchPoseElement)
						{
							const UE::Anim::FMeshAttributeContainer& OutputSnapshot = AnimNodePoseWatch.GetAttributes();
							const TArray<TTuple<FName, const UE::Anim::FMeshAttributeContainer&>> Snapshots =
							{
								{*AnimNodePoseWatch.PoseWatch->GetLabel().ToString(), OutputSnapshot}
							};

							AttributeView->DisplayNewAttributeContainerSnapshots(Snapshots, PreviewScenePtr.Pin()->GetPreviewMeshComponent());

							break;
						}
					}
				}
			}
		}
		else if (UDebugSkelMeshComponent* MeshComponent = PreviewScenePtr.Pin()->GetPreviewMeshComponent())
		{
			const UE::Anim::FMeshAttributeContainer& OutputSnapshot = MeshComponent->GetCustomAttributes();		
			const TArray<TTuple<FName, const UE::Anim::FMeshAttributeContainer&>> Snapshots =
			{
				{TEXT("Evaluation"), OutputSnapshot }
			};

			AttributeView->DisplayNewAttributeContainerSnapshots(Snapshots, MeshComponent);
		}
	}
}

#undef LOCTEXT_NAMESPACE
