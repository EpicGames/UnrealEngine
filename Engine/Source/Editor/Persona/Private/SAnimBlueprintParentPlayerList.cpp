// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "SAnimBlueprintParentPlayerList.h"
#include "Widgets/Text/STextBlock.h"
#include "PropertyEditorModule.h"
#include "Widgets/Layout/SBox.h"
#include "Animation/AnimBlueprint.h"
#include "Animation/EditorParentPlayerListObj.h"
#include "AnimGraphNodeDetails.h"
#include "Widgets/Views/SListView.h"
#include "Animation/AnimBlueprintGeneratedClass.h"

SAnimBlueprintParentPlayerList::SAnimBlueprintParentPlayerList()
	: RootBlueprint(NULL)
{

}

SAnimBlueprintParentPlayerList::~SAnimBlueprintParentPlayerList()
{
	// Unregister delegates
	if(RootBlueprint)
	{
		RootBlueprint->OnChanged().RemoveAll(this);
		RootBlueprint->OnCompiled().RemoveAll(this);
	}

	if(CurrentBlueprint)
	{
		TArray<UBlueprint*> BlueprintHierarchy;
		CurrentBlueprint->GetBlueprintHierarchyFromClass(CurrentBlueprint->GetAnimBlueprintGeneratedClass(), BlueprintHierarchy);
		// Start from 1 as 0 is our current BP
		for(int32 Idx = 1; Idx < BlueprintHierarchy.Num() ; ++Idx)
		{
			if(UAnimBlueprint* AnimBlueprint = Cast<UAnimBlueprint>(BlueprintHierarchy[Idx]))
			{
				AnimBlueprint->UnregisterOnOverrideChanged(this);
			}
		}

		CurrentBlueprint->OnChanged().RemoveAll(this);
		CurrentBlueprint->OnCompiled().RemoveAll(this);
	}
}

void SAnimBlueprintParentPlayerList::Construct(const FArguments& InArgs, const TSharedRef<FBlueprintEditor>& InBlueprintEditor, FSimpleMulticastDelegate& InOnPostUndo)
{
	// Register a refresh on post undo to grab the blueprint assets again
	InOnPostUndo.Add(FSimpleDelegate::CreateSP(this, &SAnimBlueprintParentPlayerList::RefreshDetailView));

	UEditorParentPlayerListObj* EditorObject = Cast<UEditorParentPlayerListObj>(ObjectTracker.GetEditorObjectForClass(UEditorParentPlayerListObj::StaticClass()));
	CurrentBlueprint = CastChecked<UAnimBlueprint>(InBlueprintEditor->GetBlueprintObj());
	check(CurrentBlueprint);
	CurrentParentClass = CurrentBlueprint->ParentClass;

	EditorObject->InitialiseFromBlueprint(CurrentBlueprint);
	
	if(EditorObject->Overrides.Num() > 0)
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		FDetailsViewArgs ViewArgs;
		ViewArgs.bAllowSearch = false;
		ViewArgs.bHideSelectionTip = true;

		DetailView = PropertyModule.CreateDetailView(ViewArgs);

		DetailView->RegisterInstancedCustomPropertyLayout(UEditorParentPlayerListObj::StaticClass(), FOnGetDetailCustomizationInstance::CreateStatic(&FAnimGraphParentPlayerDetails::MakeInstance, InBlueprintEditor));

		this->ChildSlot
		[
			DetailView->AsShared()
		];

		DetailView->SetObject(EditorObject, true);

		// Register delegate to refresh the view when a node changes in the root blueprint animgraph;
		RootBlueprint = UAnimBlueprint::FindRootAnimBlueprint(CurrentBlueprint);
		if(RootBlueprint != CurrentBlueprint)
		{
			RootBlueprint->OnChanged().AddSP(this, &SAnimBlueprintParentPlayerList::OnRootBlueprintChanged);
			RootBlueprint->OnCompiled().AddSP(this, &SAnimBlueprintParentPlayerList::OnRootBlueprintChanged);
		}
		else
		{
			RootBlueprint = NULL;
		}

		// Register delegate to handle parents in the hierarchy changing their overrides.
		TArray<UBlueprint*> BlueprintHierarchy;
		CurrentBlueprint->GetBlueprintHierarchyFromClass(CurrentBlueprint->GetAnimBlueprintGeneratedClass(), BlueprintHierarchy);
		// Start from 1 as 0 is our current BP
		for(int32 Idx = 1; Idx < BlueprintHierarchy.Num() ; ++Idx)
		{
			if(UAnimBlueprint* AnimBlueprint = Cast<UAnimBlueprint>(BlueprintHierarchy[Idx]))
			{
				AnimBlueprint->RegisterOnOverrideChanged(UAnimBlueprint::FOnOverrideChanged::CreateSP(this, &SAnimBlueprintParentPlayerList::OnHierarchyOverrideChanged));
			}
		}

		// Register a delegate for the current BP changing; so we can look for a root change.
		CurrentBlueprint->OnChanged().AddSP(this, &SAnimBlueprintParentPlayerList::OnCurrentBlueprintChanged);
		CurrentBlueprint->OnCompiled().AddSP(this, &SAnimBlueprintParentPlayerList::OnCurrentBlueprintChanged);
	}
	else
	{
		this->ChildSlot
			[
				SNew(SBox)
				.Padding(FMargin(10.0f, 10.0f))
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Content()
				[
					SNew(STextBlock)
					.WrapTextAt(300.0f)
					.Text(NSLOCTEXT("ParentPlayerList", "NoOverrides", "No possible overrides found. Either there are no nodes referencing assets in the parent class or this is not a derived blueprint."))
				]
			];
	}
}

void SAnimBlueprintParentPlayerList::OnRootBlueprintChanged(UBlueprint* InBlueprint)
{
	RefreshDetailView();
}


void SAnimBlueprintParentPlayerList::OnCurrentBlueprintChanged(UBlueprint* InBlueprint)
{
	UAnimBlueprint* NewRoot = UAnimBlueprint::FindRootAnimBlueprint(CurrentBlueprint);
	if(NewRoot != RootBlueprint)
	{
		// The blueprint has been reparented in a way which has changed its root,
		// The overrides we have stored are no longer valid.
		RootBlueprint = NewRoot;
		CurrentParentClass = CurrentBlueprint->ParentClass;
		CurrentBlueprint->ParentAssetOverrides.Empty();
		RefreshDetailView();
	}
	else if(CurrentBlueprint->ParentClass != CurrentParentClass)
	{
		// The blueprint has been reparented to another blueprint with the same root.
		CurrentParentClass = CurrentBlueprint->ParentClass;
		RefreshDetailView();
	}
}


void SAnimBlueprintParentPlayerList::OnHierarchyOverrideChanged(FGuid NodeGuid, UAnimationAsset* NewAsset)
{
	RefreshDetailView();
}

void SAnimBlueprintParentPlayerList::RefreshDetailView()
{
	if(DetailView.IsValid())
	{
		UEditorParentPlayerListObj* EditorObject = Cast<UEditorParentPlayerListObj>(ObjectTracker.GetEditorObjectForClass(UEditorParentPlayerListObj::StaticClass()));
		EditorObject->InitialiseFromBlueprint(CurrentBlueprint);
		DetailView->SetObject(EditorObject, true);
	}
}
