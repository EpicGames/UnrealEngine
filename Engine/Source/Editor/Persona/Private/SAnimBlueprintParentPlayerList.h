// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "BlueprintEditor.h"
#include "IDetailsView.h"
#include "EditorObjectsTracker.h"

class UAnimationAsset;
class UAnimBlueprint;

class SAnimBlueprintParentPlayerList : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAnimBlueprintParentPlayerList)
	{}

	SLATE_END_ARGS()

	SAnimBlueprintParentPlayerList();
	~SAnimBlueprintParentPlayerList();

	void Construct(const FArguments& InArgs, const TSharedRef<FBlueprintEditor>& InBlueprintEditor, FSimpleMulticastDelegate& InOnPostUndo);

private:

	// Called when the root blueprint is changed. depending on the action
	// we need to refresh the list of available overrides incase we need
	// to remove or add some
	void OnRootBlueprintChanged(UBlueprint* InBlueprint);

	// Called when the current blueprint changes. Used to detect reparenting
	// So the data and UI can be updated accordingly
	void OnCurrentBlueprintChanged(UBlueprint* InBlueprint);

	// Called when an override is changed on a less derived blueprint in the
	// current blueprint's hierarchy so we can copy them if we haven't overridden
	// the same asset
	void OnHierarchyOverrideChanged(FGuid NodeGuid, UAnimationAsset* NewAsset);

	void RefreshDetailView();

	// Object tracker to maintain single instance of editor objects
	FEditorObjectTracker ObjectTracker;

	// The blueprint currently in use
	UAnimBlueprint* CurrentBlueprint;

	// Parent class of the current blueprint, used to detect reparenting
	UClass* CurrentParentClass;

	// Root blueprint if one exists
	UAnimBlueprint* RootBlueprint;

	// The details view for the list
	TSharedPtr<IDetailsView> DetailView;
};
