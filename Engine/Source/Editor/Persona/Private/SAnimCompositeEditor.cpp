// Copyright Epic Games, Inc. All Rights Reserved.


#include "SAnimCompositeEditor.h"
#include "Animation/EditorAnimBaseObj.h"
#include "IDocumentation.h"

#include "SAnimNotifyPanel.h"
#include "Editor.h"
#include "AnimTimeline/AnimModel_AnimComposite.h"
#include "AnimTimeline/SAnimTimeline.h"

//////////////////////////////////////////////////////////////////////////
// SAnimCompositeEditor

TSharedRef<SWidget> SAnimCompositeEditor::CreateDocumentAnchor()
{
	return IDocumentation::Get()->CreateAnchor(TEXT("Engine/Animation/AnimationComposite"));
}

void SAnimCompositeEditor::Construct(const FArguments& InArgs, const TSharedRef<class IPersonaPreviewScene>& InPreviewScene, const TSharedRef<class IEditableSkeleton>& InEditableSkeleton, const TSharedRef<FUICommandList>& InCommandList)
{
	CompositeObj = InArgs._Composite;
	check(CompositeObj);

	AnimModel = MakeShared<FAnimModel_AnimComposite>(InPreviewScene, InEditableSkeleton, InCommandList, CompositeObj);
	AnimModel->OnSelectObjects = FOnObjectsSelected::CreateSP(this, &SAnimEditorBase::OnSelectionChanged);
	AnimModel->OnInvokeTab = InArgs._OnInvokeTab;

	AnimModel->OnEditCurves = FOnEditCurves::CreateLambda([this, InOnEditCurves = InArgs._OnEditCurves](UAnimSequenceBase* InAnimSequence, const TArray<IAnimationEditor::FCurveEditInfo>& InCurveInfo, const TSharedPtr<ITimeSliderController>& InExternalTimeSliderController)
	{
		InOnEditCurves.ExecuteIfBound(InAnimSequence, InCurveInfo, TimelineWidget->GetTimeSliderController());
	});

	AnimModel->Initialize();

	SAnimEditorBase::Construct( SAnimEditorBase::FArguments()
		.OnObjectsSelected(InArgs._OnObjectsSelected)
		.AnimModel(AnimModel), 
		InPreviewScene);

	if (GEditor)
	{
		GEditor->RegisterForUndo(this);
	}
}

SAnimCompositeEditor::~SAnimCompositeEditor()
{
	if (GEditor)
	{
		GEditor->UnregisterForUndo(this);
	}
}

void SAnimCompositeEditor::PostUndo( bool bSuccess )
{
	PostUndoRedo();
}

void SAnimCompositeEditor::PostRedo( bool bSuccess )
{
	PostUndoRedo();
}

void SAnimCompositeEditor::PostUndoRedo()
{
	GetPreviewScene()->SetPreviewAnimationAsset(CompositeObj);

	AnimModel->RefreshTracks();
}
