// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "SAnimEditorBase.h"
#include "Animation/AnimComposite.h"
#include "SAnimCompositePanel.h"

class SAnimNotifyPanel;
class FAnimModel_AnimComposite;

//////////////////////////////////////////////////////////////////////////
// SAnimCompositeEditor

/** Overall animation composite editing widget. This mostly contains functions for editing the UAnimComposite.

	SAnimCompositeEditor will create the SAnimCompositePanel which is mostly responsible for setting up the UI 
	portion of the composite tool and registering callbacks to the SAnimCompositeEditor to do the actual editing.
	
*/
class SAnimCompositeEditor : public SAnimEditorBase, public FEditorUndoClient
{
public:
	SLATE_BEGIN_ARGS( SAnimCompositeEditor )
		: _Composite(NULL)
		{}

		SLATE_ARGUMENT( UAnimComposite*, Composite)
		SLATE_EVENT(FOnObjectsSelected, OnObjectsSelected)
		SLATE_EVENT(FOnInvokeTab, OnInvokeTab)
		SLATE_EVENT(FOnEditCurves, OnEditCurves)

	SLATE_END_ARGS()

private:
	TSharedPtr<FAnimModel_AnimComposite> AnimModel;

public:
	~SAnimCompositeEditor();

	void Construct(const FArguments& InArgs, const TSharedRef<class IPersonaPreviewScene>& InPreviewScene, const TSharedRef<class IEditableSkeleton>& InEditableSkeleton, const TSharedRef<FUICommandList>& InCommandList);

	/** Return the animation composite being edited */
	UAnimComposite* GetCompositeObj() const { return CompositeObj; }
	virtual UAnimationAsset* GetEditorObject() const override { return GetCompositeObj(); }

private:
	/** Pointer to the animation composite being edited */
	UAnimComposite* CompositeObj;

public:
	//~ Begin SAnimEditorBase Interface
	virtual TSharedRef<SWidget> CreateDocumentAnchor() override;
	//~ End SAnimEditorBase Interface

	/** FEditorUndoClient interface */
	virtual void PostUndo( bool bSuccess ) override;
	virtual void PostRedo( bool bSuccess ) override;

	/** Post undo **/
	void PostUndoRedo();
};
