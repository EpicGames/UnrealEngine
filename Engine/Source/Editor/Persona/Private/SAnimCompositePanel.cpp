// Copyright Epic Games, Inc. All Rights Reserved.


#include "SAnimCompositePanel.h"
#include "Widgets/Layout/SBorder.h"
#include "Animation/EditorAnimCompositeSegment.h"

#include "SAnimSegmentsPanel.h"
#include "SAnimCompositeEditor.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "AnimTimeline/AnimModel.h"
#include "AssetToolsModule.h"
#include "IAssetTypeActions.h"
#include "Animation/AnimationSettings.h"
#include "Animation/AnimSequence.h"
#include "Animation/AnimMontage.h"

#define LOCTEXT_NAMESPACE "AnimCompositePanel"

//////////////////////////////////////////////////////////////////////////
// SAnimCompositePanel

void SAnimCompositePanel::Construct(const FArguments& InArgs, const TSharedRef<FAnimModel>& InModel)
{
	if(GEditor)
	{
		GEditor->RegisterForUndo(this);
	}

	SAnimTrackPanel::Construct( SAnimTrackPanel::FArguments()
		.WidgetWidth(InArgs._WidgetWidth)
		.ViewInputMin(InArgs._ViewInputMin)
		.ViewInputMax(InArgs._ViewInputMax)
		.InputMin(InArgs._InputMin)
		.InputMax(InArgs._InputMax)
		.OnSetInputViewRange(InArgs._OnSetInputViewRange));

	WeakModel = InModel;
	Composite = InArgs._Composite;
	bIsActiveTimerRegistered = false;
	bIsSelecting = false;

	InModel->OnHandleObjectsSelected().AddSP(this, &SAnimCompositePanel::HandleObjectsSelected);

	this->ChildSlot
	[
		SAssignNew( PanelArea, SBorder )
		.BorderImage( FAppStyle::GetBrush("NoBorder") )
		.Padding(0.0f)
		.ColorAndOpacity( FLinearColor::White )
	];

	Update();

	CollapseComposite();
}

SAnimCompositePanel::~SAnimCompositePanel()
{
	if(GEditor)
	{
		GEditor->UnregisterForUndo(this);
	}
}

void SAnimCompositePanel::Update()
{
	ClearSelected();
	if ( Composite != NULL )
	{		
		TSharedPtr<SVerticalBox> CompositeSlots;
		PanelArea->SetContent(
			SAssignNew( CompositeSlots, SVerticalBox )
			);

		CompositeSlots->AddSlot()
			.AutoHeight()
			.VAlign(VAlign_Center)
			[
				SNew(SAnimSegmentsPanel)
				.AnimTrack(&Composite->AnimationTrack)
				.NodeSelectionSet(&SelectionSet)
				.ViewInputMin(ViewInputMin)
				.ViewInputMax(ViewInputMax)
				.OnGetNodeColor(this,  &SAnimCompositePanel::HandleGetNodeColor)
				.TrackMaxValue(Composite->GetPlayLength())
				.TrackNumDiscreteValues(Composite->GetNumberOfSampledKeys())
				.bChildAnimMontage(false)
				.OnAnimSegmentNodeClicked( this, &SAnimCompositePanel::ShowSegmentInDetailsView )
				.OnPreAnimUpdate( this, &SAnimCompositePanel::PreAnimUpdate )
				.OnPostAnimUpdate( this, &SAnimCompositePanel::PostAnimUpdate )
				.OnIsAnimAssetValid(this, &SAnimCompositePanel::OnIsAnimAssetValid)
			];
	}
}

void SAnimCompositePanel::OnCompositeChange(class UObject *EditorAnimBaseObj, bool bRebuild)
{
	if ( Composite != nullptr )
	{
		if(bRebuild && !bIsActiveTimerRegistered)
		{
			// sometimes crashes because the timer delay but animation still renders, so invalidating here before calling timer
			Composite->InvalidateRecursiveAsset();
			bIsActiveTimerRegistered = true;
			RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateSP(this, &SAnimCompositePanel::TriggerRebuildPanel));
		} 
		else
		{
			CollapseComposite();
		}

		Composite->MarkPackageDirty();
	}
}

EActiveTimerReturnType SAnimCompositePanel::TriggerRebuildPanel(double InCurrentTime, float InDeltaTime)
{
	// we should not update any property related within PostEditChange, 
	// so this is deferred to Tick, when it needs to rebuild, just mark it and this will update in the next tick
	SortAndUpdateComposite();

	bIsActiveTimerRegistered = false;
	return EActiveTimerReturnType::Stop;
}

void SAnimCompositePanel::ShowSegmentInDetailsView(int32 SegmentIndex)
{
	if(!bIsSelecting)
	{
		TGuardValue<bool> GuardValue(bIsSelecting, true);

		UEditorAnimCompositeSegment *Obj = Cast<UEditorAnimCompositeSegment>(WeakModel.Pin()->ShowInDetailsView(UEditorAnimCompositeSegment::StaticClass()));
		if(Obj != nullptr)
		{
			Obj->InitFromAnim(Composite, FOnAnimObjectChange::CreateSP( this, &SAnimCompositePanel::OnCompositeChange ));
			Obj->InitAnimSegment(SegmentIndex);
		}
	}
}

void SAnimCompositePanel::ClearSelected()
{
	if(!bIsSelecting)
	{
		TGuardValue<bool> GuardValue(bIsSelecting, true);

		SelectionSet.Empty();
		WeakModel.Pin()->ClearDetailsView();
	}
}

void SAnimCompositePanel::PreAnimUpdate()
{
	Composite->Modify();
}

void SAnimCompositePanel::PostAnimUpdate()
{
	Composite->MarkPackageDirty();
	SortAndUpdateComposite();
}

void SAnimCompositePanel::SortAndUpdateComposite()
{
	if (Composite == nullptr)
	{
		return;
	}

	Composite->AnimationTrack.SortAnimSegments();

	Composite->UpdateCommonTargetFrameRate();

	WeakModel.Pin()->RecalculateSequenceLength();

	// Update view (this will recreate everything)
	Update();

	// Range may have changed
	WeakModel.Pin()->UpdateRange();
}

void SAnimCompositePanel::CollapseComposite()
{
	if ( Composite == nullptr )
	{
		return;
	}

	Composite->AnimationTrack.CollapseAnimSegments();

	WeakModel.Pin()->RecalculateSequenceLength();
}

void SAnimCompositePanel::PostUndo( bool bSuccess )
{
	PostUndoRedo();
}

void SAnimCompositePanel::PostRedo( bool bSuccess )
{
	PostUndoRedo();
}

void SAnimCompositePanel::PostUndoRedo()
{
	if (!bIsActiveTimerRegistered)
	{
		bIsActiveTimerRegistered = true;
		RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateSP(this, &SAnimCompositePanel::TriggerRebuildPanel));
	}
}

FLinearColor SAnimCompositePanel::HandleGetNodeColor(const FAnimSegment& InSegment) const
{
	static const FLinearColor DisabledColor(64, 64, 64);

	const FSlateColor OrangeAccent = FAppStyle::Get().GetSlateColor("Colors.AccentOrange");
	FLinearColor OutOfDateNodeColor = OrangeAccent.GetSpecifiedColor();

    if (InSegment.IsPlayLengthOutOfDate())
    {
	    return OutOfDateNodeColor;
    }

	if(const TObjectPtr<UAnimSequenceBase> AnimReference = InSegment.GetAnimReference())
	{
		FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>(TEXT("AssetTools"));
		TWeakPtr<IAssetTypeActions> AssetTypeActions = AssetToolsModule.Get().GetAssetTypeActionsForClass(AnimReference->GetClass());
		check(AssetTypeActions.IsValid());
		return AssetTypeActions.Pin()->GetTypeColor().ReinterpretAsLinear();
	}

	return DisabledColor;
}

void SAnimCompositePanel::HandleObjectsSelected(const TArray<UObject*>& InObjects)
{
	if(!bIsSelecting)
	{
		ClearSelected();
	}
}

bool SAnimCompositePanel::OnIsAnimAssetValid(const UAnimSequenceBase* AnimSequenceBase, FText* OutReason)
{
	if (AnimSequenceBase)
	{
		if (UAnimationSettings::Get()->bEnforceSupportedFrameRates)
		{
			const FFrameRate AssetFrameRate = AnimSequenceBase->GetSamplingFrameRate();
			const UAnimComposite* AnimComposite = WeakModel.Pin()->GetAsset<UAnimComposite>();
			const FFrameRate CompositeFrameRate = AnimComposite->GetCommonTargetFrameRate();
			const bool bContainsSegments = AnimComposite->AnimationTrack.AnimSegments.Num() != 0;
			if (CompositeFrameRate.IsValid() && bContainsSegments && !AssetFrameRate.IsMultipleOf(CompositeFrameRate) && !AssetFrameRate.IsFactorOf(CompositeFrameRate))
			{
				if (OutReason)
				{
					*OutReason = FText::Format(LOCTEXT("InvalidFrameRate", "Animation Asset {0} its framerate {1} is incompatible with the Anim Composite's {2}"), FText::FromString(AnimSequenceBase->GetName()), AssetFrameRate.ToPrettyText(), CompositeFrameRate.ToPrettyText());
				}
			
				return false;
			}
		}
		
		return true;	
	}
	return false;
}

#undef LOCTEXT_NAMESPACE
