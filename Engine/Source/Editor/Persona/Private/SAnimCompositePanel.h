// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "SAnimTrackPanel.h"
#include "STrack.h"

class SAnimCompositeEditor;
class SBorder;
class FAnimModel;
struct FAnimSegment;

//////////////////////////////////////////////////////////////////////////
// SAnimCompositePanel
//	This is the main montage editing widget that is responsible for setting up
//	a set of generic widgets (STrack and STrackNodes) for editing an anim composite.
//
//	SAnimCompositePanel will usually not edit the montage directly but just setup the 
//  callbacks to SAnimCompositeEditor.
//
//////////////////////////////////////////////////////////////////////////

class SAnimCompositePanel : public SAnimTrackPanel, public FEditorUndoClient
{
public:
	SLATE_BEGIN_ARGS( SAnimCompositePanel )
		: _Composite()
		, _ViewInputMin()
		, _ViewInputMax()
		, _InputMin()
		, _InputMax()
		, _OnSetInputViewRange()
	{}

	SLATE_ARGUMENT( class UAnimComposite*, Composite)
	SLATE_ARGUMENT( float, WidgetWidth )
	SLATE_ATTRIBUTE( float, ViewInputMin )
	SLATE_ATTRIBUTE( float, ViewInputMax )
	SLATE_ATTRIBUTE( float, InputMin )
	SLATE_ATTRIBUTE( float, InputMax )
	SLATE_EVENT( FOnSetInputViewRange, OnSetInputViewRange )
	SLATE_END_ARGS()

	~SAnimCompositePanel();

	void Construct(const FArguments& InArgs, const TSharedRef<FAnimModel>& InModel);

	/** Recreates the editor panel to reflect changes to the composite */
	void Update();

	/** Clears the selected segment from the details panel */
	void ClearSelected();

	/** Handlers for when the user clicks on a anim sequence node*/
	void ShowSegmentInDetailsView(int32 SegmentIndex);

	/** Delegate handlers for when the composite is being edited */
	void PreAnimUpdate();
	void PostAnimUpdate();

	/** This will sort all components of the composite and update (recreate) the UI */
	void SortAndUpdateComposite();

	/** Handler for when composite is edited in details view */
	void OnCompositeChange(class UObject *EditorAnimBaseObj, bool bRebuild);

	/** One-off active timer to trigger a panel rebuild */
	EActiveTimerReturnType TriggerRebuildPanel(double InCurrentTime, float InDeltaTime);

	/** FEditorUndoClient interface */
	virtual void PostUndo( bool bSuccess ) override;
	virtual void PostRedo( bool bSuccess ) override;
	void PostUndoRedo();

	void CollapseComposite();

	/** Get a color for an anim segment */
	FLinearColor HandleGetNodeColor(const FAnimSegment& InSegment) const;

	void HandleObjectsSelected(const TArray<UObject*>& InObjects);

	bool OnIsAnimAssetValid(const UAnimSequenceBase* AnimSequenceBase, FText* OutReason);

private:
	/** Reference to our editor model */
	TWeakPtr<FAnimModel> WeakModel;

	/** The composite we are currently editing */
	class UAnimComposite*		Composite;

	/** Is populated by the Update method with the panels UI */
	TSharedPtr<SBorder> PanelArea;

	/** Passed to the anim segments panel to handle selection */
	STrackNodeSelectionSet SelectionSet;

	/** Whether the active timer to trigger a panel rebuild is currently registered */
	bool bIsActiveTimerRegistered;

	/** Recursion guard for selection */
	bool bIsSelecting;
};
