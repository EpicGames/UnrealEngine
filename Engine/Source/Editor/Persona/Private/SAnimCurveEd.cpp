// Copyright Epic Games, Inc. All Rights Reserved.


#include "SAnimCurveEd.h"

#include "Animation/AnimTypes.h"
#include "GenericPlatform/ICursor.h"
#include "HAL/PlatformCrt.h"
#include "Input/Events.h"
#include "Layout/Clipping.h"
#include "Layout/Geometry.h"
#include "Layout/PaintGeometry.h"
#include "Math/Color.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/Optional.h"
#include "Rendering/DrawElements.h"
#include "Rendering/RenderingCommon.h"
#include "SScrubWidget.h"

class FSlateRect;
class FWidgetStyle;

#define LOCTEXT_NAMESPACE "AnimCurveEd"

//////////////////////////////////////////////////////////////////////////
//  SAnimCurveEd : anim curve editor

float SAnimCurveEd::GetTimeStep(FTrackScaleInfo &ScaleInfo)const
{
	if(NumberOfKeys.Get())
	{
		int32 Divider = SScrubWidget::GetDivider(ViewMinInput.Get(), ViewMaxInput.Get(), ScaleInfo.WidgetSize, TimelineLength.Get(), NumberOfKeys.Get());
		const FAnimKeyHelper Helper(TimelineLength.Get(), NumberOfKeys.Get());
		float TimePerKey = Helper.TimePerKey();
		return TimePerKey * Divider;
	}

	return 0.0f;
}

int32 SAnimCurveEd::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	int32 NewLayerId = SCurveEditor::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled) + 1;

	float Value = 0.f;

	if(OnGetScrubValue.IsBound())
	{
		Value = OnGetScrubValue.Execute();
	}

	FPaintGeometry MyGeometry = AllottedGeometry.ToPaintGeometry();

	// scale info
	FTrackScaleInfo ScaleInfo(ViewMinInput.Get(), ViewMaxInput.Get(), 0.f, 0.f, AllottedGeometry.GetLocalSize());
	float XPos = ScaleInfo.InputToLocalX(Value);

	TArray<FVector2D> LinePoints;
	LinePoints.Add(FVector2D(XPos, 0.f));
	LinePoints.Add(FVector2D(XPos, AllottedGeometry.GetLocalSize().Y));


	FSlateDrawElement::MakeLines(
		OutDrawElements,
		NewLayerId,
		MyGeometry,
		LinePoints,
		ESlateDrawEffect::None,
		FLinearColor::Red
		);

	// now draw scrub with new layer ID + 1;
	return NewLayerId;
}

FReply SAnimCurveEd::OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	const float ZoomDelta = -0.1f * MouseEvent.GetWheelDelta();

	const FVector2D WidgetSpace = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
	const float ZoomRatio = FMath::Clamp(static_cast<float>(WidgetSpace.X / MyGeometry.GetLocalSize().X), 0.f, 1.f);

	const float InputViewSize = ViewMaxInput.Get() - ViewMinInput.Get();
	const float InputChange = InputViewSize * ZoomDelta;

	const float NewViewMinInput = ViewMinInput.Get() - (InputChange * ZoomRatio);
	const float NewViewMaxInput = ViewMaxInput.Get() + (InputChange * (1.f - ZoomRatio));

	SetInputMinMax(NewViewMinInput, NewViewMaxInput);

	return FReply::Handled();
}

FCursorReply SAnimCurveEd::OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const
{
	if(ViewMinInput.Get() > 0.f || ViewMaxInput.Get() < TimelineLength.Get())
	{
		return FCursorReply::Cursor(EMouseCursor::GrabHand);
	}

	return FCursorReply::Unhandled();
}

void SAnimCurveEd::Construct(const FArguments& InArgs)
{
	OnGetScrubValue = InArgs._OnGetScrubValue;
	NumberOfKeys = InArgs._NumberOfKeys;
	SetClipping(EWidgetClipping::ClipToBounds);

	SCurveEditor::Construct(SCurveEditor::FArguments()
		.ViewMinInput(InArgs._ViewMinInput)
		.ViewMaxInput(InArgs._ViewMaxInput)
		.DataMinInput(InArgs._DataMinInput)
		.DataMaxInput(InArgs._DataMaxInput)
		.ViewMinOutput(0.f)
		.ViewMaxOutput(1.f)
		.ZoomToFitVertical(true)
		.ZoomToFitHorizontal(false)
		.TimelineLength(InArgs._TimelineLength)
		.DrawCurve(InArgs._DrawCurve)
		.HideUI(InArgs._HideUI)
		.AllowZoomOutput(false)
		.DesiredSize(InArgs._DesiredSize)
		.OnSetInputViewRange(InArgs._OnSetInputViewRange));
}

void SAnimCurveEd::SetDefaultOutput(const float MinZoomRange)
{
	const float NewMinOutput = (ViewMinOutput.Get());
	const float NewMaxOutput = (ViewMaxOutput.Get() + MinZoomRange);

	SetOutputMinMax(NewMinOutput, NewMaxOutput);
}
#undef LOCTEXT_NAMESPACE
