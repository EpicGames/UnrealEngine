// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "Input/CursorReply.h"
#include "Input/Reply.h"
#include "Math/Vector2D.h"
#include "Misc/Attribute.h"
#include "Misc/OptionalFwd.h"
#include "SCurveEditor.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class FPaintArgs;
class FSlateRect;
class FSlateWindowElementList;
class FWidgetStyle;
class UObject;
struct FGeometry;
struct FPointerEvent;

DECLARE_DELEGATE_OneParam(FOnSelectionChanged, const TArray<UObject*>&)
DECLARE_DELEGATE(FOnUpdatePanel)
DECLARE_DELEGATE_RetVal(float, FOnGetScrubValue)

//////////////////////////////////////////////////////////////////////////
//  SAnimCurveEd : anim curve editor

class SAnimCurveEd: public SCurveEditor
{
public:
	SLATE_BEGIN_ARGS(SAnimCurveEd)
		: _ViewMinInput(0.0f)
		, _ViewMaxInput(10.0f)
		, _TimelineLength(5.0f)
		, _DrawCurve(true)
		, _HideUI(true)
		, _OnGetScrubValue()
	{}

	SLATE_ATTRIBUTE(float, ViewMinInput)
	SLATE_ATTRIBUTE(float, ViewMaxInput)
	SLATE_ATTRIBUTE(TOptional<float>, DataMinInput)
	SLATE_ATTRIBUTE(TOptional<float>, DataMaxInput)
	SLATE_ATTRIBUTE(float, TimelineLength)
	SLATE_ATTRIBUTE(int32, NumberOfKeys)
	SLATE_ATTRIBUTE(FVector2D, DesiredSize)
	SLATE_ARGUMENT(bool, DrawCurve)
	SLATE_ARGUMENT(bool, HideUI)
	SLATE_EVENT(FOnGetScrubValue, OnGetScrubValue)
	SLATE_EVENT(FOnSetInputViewRange, OnSetInputViewRange)
SLATE_END_ARGS()

void Construct(const FArguments& InArgs);

protected:
	// SWidget interface
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	virtual FReply OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FCursorReply OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const override;
	// SWidget interface

	// SCurveEditor interface
	virtual void SetDefaultOutput(const float MinZoomRange) override;
	virtual float GetTimeStep(FTrackScaleInfo &ScaleInfo) const override;
	// SCurveEditor interface

private:
	// scrub value grabber
	FOnGetScrubValue	OnGetScrubValue;
	// @todo redo this code, all mess and dangerous
	TAttribute<int32>	NumberOfKeys;
};

