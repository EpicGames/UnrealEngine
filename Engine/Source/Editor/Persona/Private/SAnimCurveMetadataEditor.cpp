// Copyright Epic Games, Inc. All Rights Reserved.


#include "SAnimCurveMetadataEditor.h"

#include "AnimAssetFindReplaceCurves.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Styling/AppStyle.h"
#include "Widgets/Input/SSpinBox.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Input/STextEntryPopup.h"
#include "Animation/AnimSingleNodeInstance.h"
#include "IEditableSkeleton.h"
#include "Framework/Commands/GenericCommands.h"
#include "CurveViewerCommands.h"
#include "PersonaTabs.h"
#include "SAnimAssetFindReplace.h"
#include "Animation/EditorAnimCurveBoneLinks.h"
#include "HAL/PlatformApplicationMisc.h"
#include "ScopedTransaction.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Filters/GenericFilter.h"
#include "Filters/SBasicFilterBar.h"
#include "Misc/ScopedSlowTask.h"
#include "SPositiveActionButton.h"
#include "Widgets/Docking/SDockTab.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SAnimCurveMetadataEditor)

#define LOCTEXT_NAMESPACE "SAnimCurveMetadataEditor"

namespace CurveMetadataEditorColumns
{
static const FName AnimCurveNameLabel( "Curve Name" );
static const FName AnimCurveTypeLabel("Type");
static const FName AnimCurveNumBoneLabel("Num Bones");
static const FName AnimCurveMaxLODLabel("Max LOD");
}

//////////////////////////////////////////////////////////////////////////
// SAnimCurveMetadataEditorRow

typedef TSharedPtr< FAnimCurveMetadataEditorItem > FAnimCurveMetadataEditorItemPtr;


// This is a flag that is used to filter UI part
enum class EAnimCurveMetadataEditorFilterFlags : uint8 
{
	// Show all curves
	None			= 0,
	// Show morph target curves
	MorphTarget		= 0x01, 
	// Show material curves
	Material		= 0x02, 
};

ENUM_CLASS_FLAGS(EAnimCurveMetadataEditorFilterFlags);

class FAnimCurveMetadataEditorFilter : public FGenericFilter<EAnimCurveMetadataEditorFilterFlags>
{
public:
	FAnimCurveMetadataEditorFilter(EAnimCurveMetadataEditorFilterFlags InFlags, const FString& InName, const FText& InDisplayName, const FText& InToolTipText, FLinearColor InColor, TSharedPtr<FFilterCategory> InCategory)
		: FGenericFilter<EAnimCurveMetadataEditorFilterFlags>(InCategory, InName, InDisplayName, FGenericFilter<EAnimCurveMetadataEditorFilterFlags>::FOnItemFiltered())
		, Flags(InFlags)
	{
		ToolTip = InToolTipText;
		Color = InColor;
	}

	bool IsActive() const
	{
		return bIsActive;
	}

	EAnimCurveMetadataEditorFilterFlags GetFlags() const
	{
		return Flags;
	}
	
private:
	// FFilterBase interface
	virtual void ActiveStateChanged(bool bActive) override
	{
		bIsActive = bActive;
	}

	virtual bool PassesFilter(EAnimCurveMetadataEditorFilterFlags InItem) const override
	{
		return EnumHasAnyFlags(InItem, Flags);
	}
	
private:
	EAnimCurveMetadataEditorFilterFlags Flags;
	bool bIsActive = false;
};

void SAnimCurveMetadataEditorRow::Construct( const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, const TSharedRef<IPersonaPreviewScene>& InPreviewScene)
{
	Item = InArgs._Item;
	AnimCurveViewerPtr = InArgs._AnimCurveViewerPtr;
	PreviewScenePtr = InPreviewScene;

	check( Item.IsValid() );

	SMultiColumnTableRow< TSharedPtr<FAnimCurveMetadataEditorItem> >::Construct( FSuperRowType::FArguments(), InOwnerTableView );
}

TSharedRef< SWidget > SAnimCurveMetadataEditorRow::GenerateWidgetForColumn( const FName& ColumnName )
{
	if ( ColumnName == CurveMetadataEditorColumns::AnimCurveNameLabel )
	{
		TSharedPtr<SAnimCurveMetadataEditor> AnimCurveViewer = AnimCurveViewerPtr.Pin();
		if (AnimCurveViewer.IsValid())
		{
			return
				SNew(SVerticalBox)
				.ToolTipText(this, &SAnimCurveMetadataEditorRow::GetItemName)

				+ SVerticalBox::Slot()
				.FillHeight(1.0f)
				.Padding(4)
				.VAlign(VAlign_Center)
				[
					SAssignNew(Item->EditableText, SInlineEditableTextBlock)
					.OnTextCommitted(AnimCurveViewer.Get(), &SAnimCurveMetadataEditor::OnNameCommitted, Item)
					.Font(FAppStyle::Get().GetFontStyle("SmallFont"))
					.IsSelected(this, &SAnimCurveMetadataEditorRow::IsSelected)
					.Text(this, &SAnimCurveMetadataEditorRow::GetItemName)
					.HighlightText(this, &SAnimCurveMetadataEditorRow::GetFilterText)
				];
		}
	}
	else if (ColumnName == CurveMetadataEditorColumns::AnimCurveTypeLabel)
	{
		TSharedPtr<SAnimCurveMetadataEditor> AnimCurveViewer = AnimCurveViewerPtr.Pin();
		if (AnimCurveViewer.IsValid())
		{
			return
				SNew(SVerticalBox)
				.ToolTipText(LOCTEXT("AnimCurveTypeTooltip", "The type of the curve (e.g. morph target, material)."))

				+ SVerticalBox::Slot()
				.FillHeight(1.0f)
				.VAlign(VAlign_Center)
				[
					GetCurveTypeWidget()
				];
		}
	}
	else if(ColumnName == CurveMetadataEditorColumns::AnimCurveNumBoneLabel)
	{
		return
			SNew(SVerticalBox)
			.ToolTipText(LOCTEXT("AnimCurveBonesTooltip", "The number of bones linked to this curve."))
		
			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			.Padding(0.0f, 1.0f)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(this, &SAnimCurveMetadataEditorRow::GetNumConnectedBones)
				.TextStyle(&FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>( "SmallText" ))
			];
	}
	else if(ColumnName == CurveMetadataEditorColumns::AnimCurveMaxLODLabel)
	{
		return
			SNew(SVerticalBox)
			.ToolTipText(LOCTEXT("AnimCurveLODTooltip", "The max LOD this curve is used with."))
		
			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			.Padding(0.0f, 1.0f)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(this, &SAnimCurveMetadataEditorRow::GetMaxLOD)
				.TextStyle(&FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>( "SmallText" ))
			];
	}

	return SNullWidget::NullWidget;
}

FText SAnimCurveMetadataEditorRow::GetNumConnectedBones() const
{
	if(Item->AnimCurveMetaData.IsValid())
	{
		const FCurveMetaData* CurveMetaData = Item->AnimCurveMetaData->GetCurveMetaData(Item->CurveName);
		if (CurveMetaData)
		{
			return FText::AsNumber(CurveMetaData->LinkedBones.Num());
		}
	}

	return FText::AsNumber(0);
}

FText SAnimCurveMetadataEditorRow::GetMaxLOD() const
{
	if(Item->AnimCurveMetaData.IsValid())
	{
		const FCurveMetaData* CurveMetaData = Item->AnimCurveMetaData->GetCurveMetaData(Item->CurveName);
		if (CurveMetaData)
		{
			return FText::AsNumber(CurveMetaData->MaxLOD);
		}
	}

	return FText::AsNumber(0);
}

TSharedRef< SWidget > SAnimCurveMetadataEditorRow::GetCurveTypeWidget()
{
	return SNew(SHorizontalBox)

		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(0.f, 1.f, 1.f, 1.f)
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Center)
		[
			SNew(SCheckBox)
			.OnCheckStateChanged(this, &SAnimCurveMetadataEditorRow::OnAnimCurveTypeBoxChecked, true)
			.IsChecked(this, &SAnimCurveMetadataEditorRow::IsAnimCurveTypeBoxChangedChecked, true)
			.CheckedImage(FAppStyle::GetBrush("AnimCurveViewer.MorphTargetOn"))
			.CheckedPressedImage(FAppStyle::GetBrush("AnimCurveViewer.MorphTargetOn"))
			.UncheckedImage(FAppStyle::GetBrush("AnimCurveViewer.MorphTargetOff"))
			.CheckedHoveredImage(FAppStyle::GetBrush("AnimCurveViewer.MorphTargetOn"))
			.UncheckedHoveredImage(FAppStyle::GetBrush("AnimCurveViewer.MorphTargetOff"))
			.ToolTipText(LOCTEXT("CurveTypeMorphTarget_Tooltip", "MorphTarget"))
			.ForegroundColor(FAppStyle::GetSlateColor("DefaultForeground"))
		]

		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(0.f, 1.f, 1.f, 1.f)
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Center)
		[
			SNew(SCheckBox)
			.OnCheckStateChanged(this, &SAnimCurveMetadataEditorRow::OnAnimCurveTypeBoxChecked, false)
			.IsChecked(this, &SAnimCurveMetadataEditorRow::IsAnimCurveTypeBoxChangedChecked, false)
			.CheckedImage(FAppStyle::GetBrush("AnimCurveViewer.MaterialOn"))
			.CheckedPressedImage(FAppStyle::GetBrush("AnimCurveViewer.MaterialOn"))
			.UncheckedImage(FAppStyle::GetBrush("AnimCurveViewer.MaterialOff"))
			.CheckedHoveredImage(FAppStyle::GetBrush("AnimCurveViewer.MaterialOn"))
			.UncheckedHoveredImage(FAppStyle::GetBrush("AnimCurveViewer.MaterialOff"))
			.ToolTipText(LOCTEXT("CurveTypeMaterial_Tooltip", "Material"))
			.ForegroundColor(FAppStyle::GetSlateColor("DefaultForeground"))
		];

}

void SAnimCurveMetadataEditorRow::OnAnimCurveTypeBoxChecked(ECheckBoxState InState, bool bMorphTarget)
{
	if(Item->AnimCurveMetaData.IsValid())
	{
		bool bNewData = (InState == ECheckBoxState::Checked);
		if (bMorphTarget)
		{
			Item->AnimCurveMetaData->SetCurveMetaDataMorphTarget(Item->CurveName, bNewData);
		}
		else
		{
			Item->AnimCurveMetaData->SetCurveMetaDataMaterial(Item->CurveName, bNewData);
		}
	}
	
	UAnimInstance* AnimInstance = PreviewScenePtr.Pin()->GetPreviewMeshComponent()->GetAnimInstance();
	if (AnimInstance)
	{
		AnimInstance->RecalcRequiredCurves(UE::Anim::FCurveFilterSettings());
	}

	TSharedPtr<SAnimCurveMetadataEditor> AnimCurveViewer = AnimCurveViewerPtr.Pin();
	if (AnimCurveViewer.IsValid())
	{
		AnimCurveViewer->RefreshCurveList(false);
	}
}

ECheckBoxState SAnimCurveMetadataEditorRow::IsAnimCurveTypeBoxChangedChecked(bool bMorphTarget) const
{
	bool bData = false;

	if(Item->AnimCurveMetaData.IsValid())
	{
		const FCurveMetaData* CurveMetaData = Item->AnimCurveMetaData->GetCurveMetaData(Item->CurveName);

		if (CurveMetaData)
		{
			if (bMorphTarget)
			{
				bData = CurveMetaData->Type.bMorphtarget != 0;
			}
			else
			{
				bData = CurveMetaData->Type.bMaterial != 0;
			}
		}
	}

	return (bData)? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

FText SAnimCurveMetadataEditorRow::GetItemName() const
{
	return FText::FromName(Item->CurveName);
}

FText SAnimCurveMetadataEditorRow::GetFilterText() const
{
	TSharedPtr<SAnimCurveMetadataEditor> AnimCurveViewer = AnimCurveViewerPtr.Pin();
	if (AnimCurveViewer.IsValid())
	{
		return AnimCurveViewer->GetFilterText();
	}
	else
	{
		return FText::GetEmpty();
	}
}

//////////////////////////////////////////////////////////////////////////
// SAnimCurveMetadataEditor

void SAnimCurveMetadataEditor::Construct(const FArguments& InArgs, UObject* InAnimCurveMetaDataHost, const TSharedRef<IPersonaPreviewScene>& InPreviewScene, FOnObjectsSelected InOnObjectsSelected)
{
	AnimCurveMetaDataHost = Cast<IInterface_AssetUserData>(InAnimCurveMetaDataHost);

	// If the host doesnt already have it's asset user data set up 
	if(AnimCurveMetaDataHost.IsValid() && AnimCurveMetaDataHost->GetAssetUserData<UAnimCurveMetaData>() == nullptr)
	{
		UAnimCurveMetaData* NewMetadata = NewObject<UAnimCurveMetaData>(InAnimCurveMetaDataHost, NAME_None, RF_Transactional);
		AnimCurveMetaDataHost->AddAssetUserData(NewMetadata);
	}

	// Disable the widget if metadata is no longer hosted
	SetEnabled(MakeAttributeLambda([this]()
	{
		return GetAnimCurveMetaData() != nullptr;
	}));

	OnObjectsSelected = InOnObjectsSelected;

	EditorObjectTracker.SetAllowOnePerClass(false);

	PreviewScenePtr = InPreviewScene;

	InPreviewScene->RegisterOnPreviewMeshChanged(FOnPreviewMeshChanged::CreateSP(this, &SAnimCurveMetadataEditor::OnPreviewMeshChanged));
	InPreviewScene->RegisterOnAnimChanged(FOnAnimChanged::CreateSP(this, &SAnimCurveMetadataEditor::OnPreviewAssetChanged));

	if(UAnimCurveMetaData* AnimCurveMetaData = GetAnimCurveMetaData())
	{
		CurveMetaDataChangedHandle = AnimCurveMetaData->RegisterOnCurveMetaDataChanged(FSimpleMulticastDelegate::FDelegate::CreateSP(this, &SAnimCurveMetadataEditor::HandleCurveMetaDataChange));
	}

	// Register and bind all our menu commands
	FCurveViewerCommands::Register();
	BindCommands();

	CurrentCurveFlag = EAnimCurveMetadataEditorFilterFlags::None;

	TSharedPtr<FFilterCategory> FilterCategory = MakeShared<FFilterCategory>(LOCTEXT("CurveFiltersLabel", "Curve Filters"), LOCTEXT("CurveFiltersToolTip", "Filter what kind fo curves can be displayed."));

	Filters.Add(MakeShared<FAnimCurveMetadataEditorFilter>(
		EAnimCurveMetadataEditorFilterFlags::MorphTarget,
		"MorphTarget",
		LOCTEXT("MorphTargetLabel", "Morph Target"),
		LOCTEXT("MorphTargetTooltip", "Show morph target curves"),
		FLinearColor::Red,
		FilterCategory
		));

	Filters.Add(MakeShared<FAnimCurveMetadataEditorFilter>(
		EAnimCurveMetadataEditorFilterFlags::Material,
		"Material",
		LOCTEXT("MaterialLabel", "Material"),
		LOCTEXT("MaterialTooltip", "Show material curves"),
		FLinearColor::Green,
		FilterCategory
		));

	TSharedRef<SBasicFilterBar<EAnimCurveMetadataEditorFilterFlags>> FilterBar = SNew(SBasicFilterBar<EAnimCurveMetadataEditorFilterFlags>)
	.CustomFilters(Filters)
	.UseSectionsForCategories(true)
	.OnFilterChanged_Lambda([this]()
	{
		CurrentCurveFlag = EAnimCurveMetadataEditorFilterFlags::None;

		for(const TSharedRef<FFilterBase<EAnimCurveMetadataEditorFilterFlags>>& Filter : Filters)
		{
			TSharedRef<FAnimCurveMetadataEditorFilter> AnimCurveFilter = StaticCastSharedRef<FAnimCurveMetadataEditorFilter>(Filter);
			if(AnimCurveFilter->IsActive())
			{
				CurrentCurveFlag |= AnimCurveFilter->GetFlags();
			}
		}

		RefreshCurveList(false);
	});

	ChildSlot
	[
		SNew( SVerticalBox )
		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Left)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.0f)
			[
				SNew(SPositiveActionButton)
				.Text(LOCTEXT("AddCurveButton", "Add Curve"))
				.ToolTipText(LOCTEXT("AddCurveButtonToolTip", "Add a new curve metadata entry"))
				.OnClicked_Lambda([this]()
				{
					OnAddClicked();
					return FReply::Handled();
				})
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.0f)
			[
				SNew(SSimpleButton)
				.Text(LOCTEXT("FindReplaceCurvesButton", "Find/Replace Curves..."))
				.ToolTipText(LOCTEXT("FindReplaceCurvesButtonTooltip", "Find and replace curves across multiple assets"))
				.Icon(FAppStyle::GetBrush("Kismet.Tabs.FindResults"))
				.OnClicked_Lambda([this]()
				{
					FindReplaceCurves();
					return FReply::Handled();
				})
			]
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0,2)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.0f,0.0f)
			[
				SBasicFilterBar<EAnimCurveMetadataEditorFilterFlags>::MakeAddFilterButton(FilterBar)
			]
			// Filter entry
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.Padding(2.0f,0.0f)
			[
				SAssignNew( NameFilterBox, SSearchBox )
				.SelectAllTextWhenFocused( true )
				.OnTextChanged( this, &SAnimCurveMetadataEditor::OnFilterTextChanged )
				.OnTextCommitted( this, &SAnimCurveMetadataEditor::OnFilterTextCommitted )
			]
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			FilterBar
		]
		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			SAssignNew( AnimCurveListView, SListView< TSharedPtr<FAnimCurveMetadataEditorItem> > )
			.ListItemsSource( &AnimCurveList )
			.OnGenerateRow( this, &SAnimCurveMetadataEditor::GenerateAnimCurveRow )
			.OnContextMenuOpening( this, &SAnimCurveMetadataEditor::OnGetContextMenuContent )
			.SelectionMode(ESelectionMode::Multi)
			.OnSelectionChanged( this, &SAnimCurveMetadataEditor::OnSelectionChanged )
			.HeaderRow
			(
				SNew( SHeaderRow )
				+ SHeaderRow::Column(CurveMetadataEditorColumns::AnimCurveNameLabel)
				.FillWidth(0.65f)
				.DefaultLabel( LOCTEXT( "AnimCurveNameLabel", "Curve Name" ) )

				+ SHeaderRow::Column(CurveMetadataEditorColumns::AnimCurveTypeLabel)
				.FixedWidth(48.0f)
				.DefaultLabel(LOCTEXT("AnimCurveTypeLabel", "Type"))

				+ SHeaderRow::Column(CurveMetadataEditorColumns::AnimCurveNumBoneLabel)
				.FillWidth(0.17f)
				.DefaultLabel(LOCTEXT("AnimCurveNumBoneLabel", "Bones"))
				
				+SHeaderRow::Column(CurveMetadataEditorColumns::AnimCurveMaxLODLabel)
				.FillWidth(0.17f)
				.DefaultLabel(LOCTEXT("AnimCurveMaxLODLabel", "Max LOD"))	
			)
		]
	];

	RefreshCurveList(true);
}

SAnimCurveMetadataEditor::~SAnimCurveMetadataEditor()
{
	if (PreviewScenePtr.IsValid() )
	{
		PreviewScenePtr.Pin()->UnregisterOnPreviewMeshChanged(this);
		PreviewScenePtr.Pin()->UnregisterOnAnimChanged(this);
	}

	if(UAnimCurveMetaData* AnimCurveMetaData = GetAnimCurveMetaData())
	{
		AnimCurveMetaData->UnregisterOnCurveMetaDataChanged(CurveMetaDataChangedHandle);
	}
}

FReply SAnimCurveMetadataEditor::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (UICommandList.IsValid() && UICommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

void SAnimCurveMetadataEditor::BindCommands()
{
	// This should not be called twice on the same instance
	check(!UICommandList.IsValid());

	UICommandList = MakeShareable(new FUICommandList);

	FUICommandList& CommandList = *UICommandList;

	// Grab the list of menu commands to bind...
	const FCurveViewerCommands& MenuActions = FCurveViewerCommands::Get();

	// ...and bind them all

	CommandList.MapAction(
		FGenericCommands::Get().Rename,
		FExecuteAction::CreateSP(this, &SAnimCurveMetadataEditor::OnRenameClicked),
		FCanExecuteAction::CreateSP(this, &SAnimCurveMetadataEditor::CanRename));

	CommandList.MapAction(
		FGenericCommands::Get().Delete,
		FExecuteAction::CreateSP(this, &SAnimCurveMetadataEditor::OnDeleteNameClicked),
		FCanExecuteAction::CreateSP(this, &SAnimCurveMetadataEditor::CanDelete));

	CommandList.MapAction(
		FGenericCommands::Get().Copy,
		FExecuteAction::CreateSP(this, &SAnimCurveMetadataEditor::OnCopyClicked),
		FCanExecuteAction::CreateSP(this, &SAnimCurveMetadataEditor::CanCopy));

	CommandList.MapAction(
		FGenericCommands::Get().Paste,
		FExecuteAction::CreateSP(this, &SAnimCurveMetadataEditor::OnPasteClicked),
		FCanExecuteAction::CreateSP(this, &SAnimCurveMetadataEditor::CanPaste));
	
	CommandList.MapAction(
		MenuActions.AddCurve,
		FExecuteAction::CreateSP(this, &SAnimCurveMetadataEditor::OnAddClicked),
		FCanExecuteAction());

	CommandList.MapAction(
		MenuActions.FindCurveUses,
		FExecuteAction::CreateSP(this, &SAnimCurveMetadataEditor::OnFindCurveUsesClicked),
		FCanExecuteAction::CreateSP(this, &SAnimCurveMetadataEditor::CanFindCurveUses));
}

void SAnimCurveMetadataEditor::OnPreviewMeshChanged(class USkeletalMesh* OldPreviewMesh, class USkeletalMesh* NewPreviewMesh)
{
	RefreshCurveList(true);
}

void SAnimCurveMetadataEditor::OnFilterTextChanged( const FText& SearchText )
{
	FilterText = SearchText;

	RefreshCurveList(false);
}

void SAnimCurveMetadataEditor::OnCurvesChanged()
{
	RefreshCurveList(true);
}

void SAnimCurveMetadataEditor::OnFilterTextCommitted( const FText& SearchText, ETextCommit::Type CommitInfo )
{
	// Just do the same as if the user typed in the box
	OnFilterTextChanged( SearchText );
}

TSharedRef<ITableRow> SAnimCurveMetadataEditor::GenerateAnimCurveRow(TSharedPtr<FAnimCurveMetadataEditorItem> InInfo, const TSharedRef<STableViewBase>& OwnerTable)
{
	check( InInfo.IsValid() );

	return
		SNew( SAnimCurveMetadataEditorRow, OwnerTable, PreviewScenePtr.Pin().ToSharedRef() )
		.Item( InInfo )
		.AnimCurveViewerPtr( SharedThis(this) );
}

TSharedPtr<SWidget> SAnimCurveMetadataEditor::OnGetContextMenuContent() const
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder( bShouldCloseWindowAfterMenuSelection, UICommandList);

	const FCurveViewerCommands& Actions = FCurveViewerCommands::Get();

	MenuBuilder.BeginSection("AnimCurveAction", LOCTEXT( "CurveAction", "Curve Actions" ) );

	MenuBuilder.AddMenuEntry(FGenericCommands::Get().Rename, NAME_None, LOCTEXT("RenameSmartNameLabel", "Rename Curve"), LOCTEXT("RenameSmartNameToolTip", "Rename the selected curve"));
	MenuBuilder.AddMenuEntry(FGenericCommands::Get().Delete, NAME_None, LOCTEXT("DeleteSmartNameLabel", "Delete Curve"), LOCTEXT("DeleteSmartNameToolTip", "Delete the selected curve"));
	MenuBuilder.AddMenuEntry(FGenericCommands::Get().Copy);
	MenuBuilder.AddMenuEntry(FGenericCommands::Get().Paste);
	MenuBuilder.AddMenuEntry(Actions.AddCurve);
	MenuBuilder.AddMenuEntry(Actions.FindCurveUses);

	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void SAnimCurveMetadataEditor::OnRenameClicked()
{
	TArray< TSharedPtr<FAnimCurveMetadataEditorItem> > SelectedItems = AnimCurveListView->GetSelectedItems();

	SelectedItems[0]->EditableText->EnterEditingMode();
}

bool SAnimCurveMetadataEditor::CanRename()
{
	return AnimCurveListView->GetNumItemsSelected() == 1;
}

void SAnimCurveMetadataEditor::OnAddClicked()
{
	TSharedRef<STextEntryPopup> TextEntry =
		SNew(STextEntryPopup)
		.Label(LOCTEXT("NewSmartnameLabel", "New Name"))
		.OnTextCommitted(this, &SAnimCurveMetadataEditor::CreateNewNameEntry);

	FSlateApplication& SlateApp = FSlateApplication::Get();
	SlateApp.PushMenu(
		AsShared(),
		FWidgetPath(),
		TextEntry,
		SlateApp.GetCursorPos(),
		FPopupTransitionEffect::TypeInPopup
		);
}

void SAnimCurveMetadataEditor::OnFindCurveUsesClicked()
{
	FindReplaceCurves();
}

bool SAnimCurveMetadataEditor::CanFindCurveUses()
{
	return AnimCurveListView->GetNumItemsSelected() == 1;
}

void SAnimCurveMetadataEditor::FindReplaceCurves()
{
	FName CurveName = NAME_None;
	bool bMorphTarget = false;
	bool bMaterial = false;
	TArray<TSharedPtr<FAnimCurveMetadataEditorItem>> SelectedItems = AnimCurveListView->GetSelectedItems();
	if(SelectedItems.Num() > 0)
	{
		CurveName = SelectedItems[0]->CurveName;
		bMorphTarget = EnumHasAnyFlags(SelectedItems[0]->Flags, EAnimCurveMetadataEditorFilterFlags::MorphTarget);
		bMaterial = EnumHasAnyFlags(SelectedItems[0]->Flags, EAnimCurveMetadataEditorFilterFlags::Material);
	}

	if(TSharedPtr<SDockTab> ActiveTab = FGlobalTabmanager::Get()->GetActiveTab())
	{
		if(TSharedPtr<FTabManager> TabManager = ActiveTab->GetTabManagerPtr())
		{
			if(TSharedPtr<SDockTab> Tab = TabManager->TryInvokeTab(FPersonaTabs::FindReplaceID))
			{
				TSharedRef<IAnimAssetFindReplace> FindReplaceWidget = StaticCastSharedRef<IAnimAssetFindReplace>(Tab->GetContent());
				FindReplaceWidget->SetCurrentProcessor(UAnimAssetFindReplaceCurves::StaticClass());
				if(CurveName != NAME_None)
				{
					if(UAnimAssetFindReplaceCurves* Processor = FindReplaceWidget->GetProcessor<UAnimAssetFindReplaceCurves>())
					{
						Processor->SetFindString(CurveName.ToString());
						Processor->SetFindWholeWord(true);
						Processor->SetSearchMaterials(bMaterial);
						Processor->SetSearchMorphTargets(bMorphTarget);
					}
				}
			}
		}
	}
}

void SAnimCurveMetadataEditor::CreateNewNameEntry(const FText& CommittedText, ETextCommit::Type CommitType)
{
	FSlateApplication::Get().DismissAllMenus();

	if(UAnimCurveMetaData* AnimCurveMetaData = GetAnimCurveMetaData())
	{
		if (!CommittedText.IsEmpty() && CommitType == ETextCommit::OnEnter)
		{
			FName NewName = FName(*CommittedText.ToString());
			if (AnimCurveMetaData->AddCurveMetaData(NewName))
			{
				// Successfully added
				RefreshCurveList(true);
			}
		}
	}
}

UAnimInstance* SAnimCurveMetadataEditor::GetAnimInstance() const
{
	return PreviewScenePtr.Pin()->GetPreviewMeshComponent()->GetAnimInstance();
}

void SAnimCurveMetadataEditor::CreateAnimCurveList( const FString& SearchText, bool bInFullRefresh )
{
	if(UAnimCurveMetaData* AnimCurveMetaData = GetAnimCurveMetaData())
	{
		bool bDirty = bInFullRefresh;

		AnimCurveList.Reset();

		if(bInFullRefresh)
		{
			AllSeenAnimCurvesMap.Reset();
		}

		auto AddCurve = [this, AnimCurveMetaData](FName InCurveName, EAnimCurveMetadataEditorFilterFlags InFlags)
		{
			// Only add if the curve doesnt exist
			TSharedPtr<FAnimCurveMetadataEditorItem>* ExistingItem = AllSeenAnimCurvesMap.Find(InCurveName);
			if(ExistingItem == nullptr)
			{
				UEditorAnimCurveBoneLinks* EditorMirrorObj = Cast<UEditorAnimCurveBoneLinks> (EditorObjectTracker.GetEditorObjectForClass(UEditorAnimCurveBoneLinks::StaticClass()));
				EditorMirrorObj->Initialize(AnimCurveMetaData, InCurveName, FOnAnimCurveBonesChange::CreateSP(this, &SAnimCurveMetadataEditor::ApplyCurveBoneLinks));
				TSharedRef<FAnimCurveMetadataEditorItem> NewInfo = FAnimCurveMetadataEditorItem::Make(AnimCurveMetaData, InCurveName, InFlags, EditorMirrorObj);

				AllSeenAnimCurvesMap.Add(InCurveName, NewInfo);
			}
			else
			{
				(*ExistingItem)->Flags = InFlags;
			}
		};
		
		// Add curve items from metadata
		AnimCurveMetaData->ForEachCurveMetaData([&AddCurve](FName InCurveName, const FCurveMetaData& InCurveMetaData)
		{
			EAnimCurveMetadataEditorFilterFlags Flags = EAnimCurveMetadataEditorFilterFlags::None;
			if(InCurveMetaData.Type.bMaterial)
			{
				Flags |= EAnimCurveMetadataEditorFilterFlags::Material; 
			}
			if(InCurveMetaData.Type.bMorphtarget)
			{
				Flags |= EAnimCurveMetadataEditorFilterFlags::MorphTarget; 
			}
			AddCurve(InCurveName, Flags);
		});

		// Iterate through all curves that have been seen
		for (const TPair<FName, TSharedPtr<FAnimCurveMetadataEditorItem>>& CurveNameValuePair : AllSeenAnimCurvesMap)
		{
			TSharedPtr<FAnimCurveMetadataEditorItem> Item = CurveNameValuePair.Value;
			
			bool bAddToList = true;

			// See if we pass the search filter
			if (!FilterText.IsEmpty())
			{
				if (!CurveNameValuePair.Key.ToString().Contains(*FilterText.ToString()))
				{
					bAddToList = false;
				}
			}

			if(CurrentCurveFlag != EAnimCurveMetadataEditorFilterFlags::None)
			{
				bAddToList = EnumHasAnyFlags(Item->Flags, CurrentCurveFlag);
			}
			
			if(Item->bShown != bAddToList)
			{
				Item->bShown = bAddToList;
				bDirty = true;
			}
			
			// If we still want to add
			if (bAddToList)
			{
				AnimCurveList.Add(Item);
			}
		}

		if(bDirty)
		{
			// Sort final list
			struct FSortSmartNamesAlphabetically
			{
				bool operator()(const TSharedPtr<FAnimCurveMetadataEditorItem>& A, const TSharedPtr<FAnimCurveMetadataEditorItem>& B) const
				{
					return (A.Get()->CurveName.Compare(B.Get()->CurveName) < 0);
				}
			};

			AnimCurveList.Sort(FSortSmartNamesAlphabetically());

			AnimCurveListView->RequestListRefresh();
		}
	}
}

void SAnimCurveMetadataEditor::PostUndoRedo()
{
	RefreshCurveList(true);
}

void SAnimCurveMetadataEditor::OnPreviewAssetChanged(class UAnimationAsset* NewAsset)
{
	OverrideCurves.Empty();
	RefreshCurveList(true);
}

void SAnimCurveMetadataEditor::RefreshCurveList(bool bInFullRefresh)
{
	CreateAnimCurveList(FilterText.ToString(), bInFullRefresh);
}

void SAnimCurveMetadataEditor::OnNameCommitted(const FText& InNewName, ETextCommit::Type, TSharedPtr<FAnimCurveMetadataEditorItem> Item)
{
	if(UAnimCurveMetaData* AnimCurveMetaData = GetAnimCurveMetaData())
	{
		FName NewName(*InNewName.ToString());
		if (NewName == Item->CurveName)
		{
			// Do nothing if trying to rename to existing name...
		}
		else if (NewName != NAME_None)
		{
			if(!AnimCurveMetaData->RenameCurveMetaData(Item->CurveName, NewName))
			{
				FFormatNamedArguments Args;
				Args.Add(TEXT("InvalidName"), FText::FromName(NewName) );
				FNotificationInfo Info(FText::Format(LOCTEXT("AnimCurveRenamed", "The name \"{InvalidName}\" is invalid or already used."), Args));

				Info.bUseLargeFont = false;
				Info.ExpireDuration = 5.0f;

				TSharedPtr<SNotificationItem> Notification = FSlateNotificationManager::Get().AddNotification(Info);
				if (Notification.IsValid())
				{
					Notification->SetCompletionState(SNotificationItem::CS_Fail);
				}
			}
			else
			{
				AllSeenAnimCurvesMap.Remove(Item->CurveName);
				AnimCurveList.Remove(Item);
			}
		}
	}
}

void SAnimCurveMetadataEditor::OnDeleteNameClicked()
{
	TArray< TSharedPtr<FAnimCurveMetadataEditorItem> > SelectedItems = AnimCurveListView->GetSelectedItems();
	TArray<FName> SelectedNames;

	for (TSharedPtr<FAnimCurveMetadataEditorItem> Item : SelectedItems)
	{
		SelectedNames.Add(Item->CurveName);
	}

	if(UAnimCurveMetaData* AnimCurveMetaData = GetAnimCurveMetaData())
	{
		AnimCurveMetaData->RemoveCurveMetaData(SelectedNames);
	}
}

bool SAnimCurveMetadataEditor::CanDelete()
{
	return AnimCurveListView->GetNumItemsSelected() > 0;
}

static const TCHAR* ClipboardHeader = TEXT("AnimCurveViewer");

void SAnimCurveMetadataEditor::OnCopyClicked()
{
	if(UAnimCurveMetaData* AnimCurveMetaData = GetAnimCurveMetaData())
	{
		TArray<TSharedPtr<FAnimCurveMetadataEditorItem>> SelectedItems = AnimCurveListView->GetSelectedItems();

		FAnimCurveMetadataEditorClipboard Clipboard;

		for (TSharedPtr<FAnimCurveMetadataEditorItem> Item : SelectedItems)
		{
			FAnimCurveMetadataEditorClipboardEntry Entry;
			Entry.CurveName = Item->CurveName;

			if(const FCurveMetaData* CurveMetaData = AnimCurveMetaData->GetCurveMetaData(Item->CurveName))
			{
				Entry.MetaData = *CurveMetaData;
			}

			Clipboard.Entries.Add(Entry);
		}

		FString ClipboardString = ClipboardHeader;
		FAnimCurveMetadataEditorClipboard::StaticStruct()->ExportText(ClipboardString, &Clipboard, nullptr, nullptr, PPF_None, nullptr);
		FPlatformApplicationMisc::ClipboardCopy(*ClipboardString);
	}
}

bool SAnimCurveMetadataEditor::CanCopy() const
{
	return AnimCurveListView->GetNumItemsSelected() > 0;
}

void SAnimCurveMetadataEditor::OnPasteClicked()
{
	if(UAnimCurveMetaData* AnimCurveMetaData = GetAnimCurveMetaData())
	{
		FScopedTransaction Transaction(LOCTEXT("PasteCurves", "Paste Curves"));
		
		FString TextToImport;
		FPlatformApplicationMisc::ClipboardPaste(TextToImport);
		TextToImport.RemoveFromStart(ClipboardHeader);

		FAnimCurveMetadataEditorClipboard Clipboard;
		FAnimCurveMetadataEditorClipboard::StaticStruct()->ImportText(*TextToImport, &Clipboard, nullptr, PPF_None, GLog, FAnimCurveMetadataEditorClipboard::StaticStruct()->GetName());

		for(const FAnimCurveMetadataEditorClipboardEntry& Entry : Clipboard.Entries)
		{
			AnimCurveMetaData->AddCurveMetaData(Entry.CurveName);
			AnimCurveMetaData->SetCurveMetaDataBoneLinks(Entry.CurveName, Entry.MetaData.LinkedBones, Entry.MetaData.MaxLOD, GetSkeleton());
			AnimCurveMetaData->SetCurveMetaDataMaterial(Entry.CurveName, Entry.MetaData.Type.bMaterial);
			AnimCurveMetaData->SetCurveMetaDataMorphTarget(Entry.CurveName, Entry.MetaData.Type.bMorphtarget);

			RefreshCurveList(true);
		}
	}
}

bool SAnimCurveMetadataEditor::CanPaste() const
{
	FString TextToImport;
	FPlatformApplicationMisc::ClipboardPaste(TextToImport);
	return TextToImport.StartsWith(ClipboardHeader);
}

void SAnimCurveMetadataEditor::OnSelectionChanged(TSharedPtr<FAnimCurveMetadataEditorItem> InItem, ESelectInfo::Type SelectInfo)
{
	if(SelectInfo != ESelectInfo::Direct)
	{

		if(UAnimCurveMetaData* AnimCurveMetaData = GetAnimCurveMetaData())
		{
			// make sure the currently selected ones are refreshed if it's first time
			TArray<UObject*> SelectedObjects;

			TArray< TSharedPtr< FAnimCurveMetadataEditorItem > > SelectedRows = AnimCurveListView->GetSelectedItems();
			for (auto ItemIt = SelectedRows.CreateIterator(); ItemIt; ++ItemIt)
			{
				TSharedPtr< FAnimCurveMetadataEditorItem > RowItem = (*ItemIt);
				UEditorAnimCurveBoneLinks* EditorMirrorObj = RowItem->EditorMirrorObject;
				if (RowItem == InItem)
				{
					// first time selected, refresh
					TArray<FBoneReference> BoneLinks;
					FName CurrentName = RowItem->CurveName;
					const FCurveMetaData* CurveMetaData = AnimCurveMetaData->GetCurveMetaData(CurrentName);
					uint8 MaxLOD = 0xFF;
					if (CurveMetaData)
					{
						BoneLinks = CurveMetaData->LinkedBones;
						MaxLOD = CurveMetaData->MaxLOD;
					}

					EditorMirrorObj->Refresh(CurrentName, BoneLinks, MaxLOD);
				}

				SelectedObjects.Add(EditorMirrorObj);
			}

			OnObjectsSelected.ExecuteIfBound(SelectedObjects);
		}
	}
}

void SAnimCurveMetadataEditor::ApplyCurveBoneLinks(UEditorAnimCurveBoneLinks* EditorObj)
{
	if(UAnimCurveMetaData* AnimCurveMetaData = GetAnimCurveMetaData())
	{
		if (EditorObj && !GIsTransacting)
		{
			AnimCurveMetaData->SetCurveMetaDataBoneLinks(EditorObj->CurveName, EditorObj->ConnectedBones, EditorObj->MaxLOD, GetSkeleton());
		}
	}
}

void SAnimCurveMetadataEditor::HandleCurveMetaDataChange()
{
	AnimCurveList.Empty();
	RefreshCurveList(true);
}

UAnimCurveMetaData* SAnimCurveMetadataEditor::GetAnimCurveMetaData() const
{
	if(IInterface_AssetUserData* AssetUserData = AnimCurveMetaDataHost.Get())
	{
		return AssetUserData->GetAssetUserData<UAnimCurveMetaData>();
	}

	return nullptr;
}

USkeleton* SAnimCurveMetadataEditor::GetSkeleton() const
{
	if(USkeleton* Skeleton = Cast<USkeleton>(AnimCurveMetaDataHost.Get()))
	{
		return Skeleton;
	}
	else if(USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(AnimCurveMetaDataHost.Get()))
	{
		return SkeletalMesh->GetSkeleton();
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE

