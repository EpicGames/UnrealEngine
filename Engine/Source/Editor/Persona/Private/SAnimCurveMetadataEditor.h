// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "Styling/SlateColor.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SBoxPanel.h"
#include "Animation/SmartName.h"
#include "IPersonaPreviewScene.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"
#include "Animation/AnimInstance.h"
#include "EditorObjectsTracker.h"
#include "PersonaDelegates.h"
#include "EditorUndoClient.h"
#include "Filters/FilterBase.h"
#include "UObject/WeakInterfacePtr.h"
#include "SAnimCurveMetadataEditor.generated.h"

class FUICommandList;
class IEditableSkeleton;
class SAnimCurveMetadataEditor;
class SAnimCurveMetadataEditorTypeList;
class UEditorAnimCurveBoneLinks;
enum class EAnimCurveMetadataEditorFilterFlags : uint8;

/** Clipboard contents for anim curves */
USTRUCT()
struct FAnimCurveMetadataEditorClipboardEntry
{
	GENERATED_BODY()
	
	UPROPERTY()
	FName CurveName;

	UPROPERTY()
	FCurveMetaData MetaData;
};

USTRUCT()
struct FAnimCurveMetadataEditorClipboard
{
	GENERATED_BODY()
	
	UPROPERTY()
	TArray<FAnimCurveMetadataEditorClipboardEntry> Entries;
};

//////////////////////////////////////////////////////////////////////////
// FAnimCurveMetadataEditorItem

class FAnimCurveMetadataEditorItem
{
public:
	FName CurveName;
	TWeakObjectPtr<UAnimCurveMetaData> AnimCurveMetaData;		// The asset user data we are associated with
	TSharedPtr<SInlineEditableTextBlock> EditableText;	// The editable text box in the list, used to focus from the context menu
	UEditorAnimCurveBoneLinks* EditorMirrorObject;
	EAnimCurveMetadataEditorFilterFlags Flags;
	bool bShown;
	
	/** Static function for creating a new item, but ensures that you can only have a TSharedRef to one */
	static TSharedRef<FAnimCurveMetadataEditorItem> Make(UAnimCurveMetaData* InAnimCurveMetaData, const FName& InCurveName, EAnimCurveMetadataEditorFilterFlags InFlags, UEditorAnimCurveBoneLinks* InEditorMirrorObject)
	{
		return MakeShareable(new FAnimCurveMetadataEditorItem(InAnimCurveMetaData, InCurveName, InFlags, InEditorMirrorObject));
	}

protected:
	/** Hidden constructor, always use Make above */
	FAnimCurveMetadataEditorItem(UAnimCurveMetaData* InAnimCurveMetaData, const FName& InCurveName, EAnimCurveMetadataEditorFilterFlags InFlags, UEditorAnimCurveBoneLinks* InEditorMirrorObject)
		: CurveName(InCurveName)
		, AnimCurveMetaData(InAnimCurveMetaData)
		, EditorMirrorObject(InEditorMirrorObject)
		, Flags(InFlags)
		, bShown(false)
	{}
};

//////////////////////////////////////////////////////////////////////////
// SAnimCurveMetadataEditorRow

class SAnimCurveMetadataEditorRow : public SMultiColumnTableRow< TSharedPtr<FAnimCurveMetadataEditorItem> >
{
public:

	SLATE_BEGIN_ARGS(SAnimCurveMetadataEditorRow) {}

		/** The item for this row **/
		SLATE_ARGUMENT(TSharedPtr<FAnimCurveMetadataEditorItem>, Item)

		/* The SAnimCurveMetadataEditor that we push the morph target weights into */
		SLATE_ARGUMENT(class TWeakPtr<SAnimCurveMetadataEditor>, AnimCurveViewerPtr)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTableView, const TSharedRef<class IPersonaPreviewScene>& InPreviewScene);

	/** Overridden from SMultiColumnTableRow.  Generates a widget for this column of the tree row. */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

private:
	/* Curve Flag checks for morphtarget or material */
	void OnAnimCurveTypeBoxChecked(ECheckBoxState InState, bool bMorphTarget);
	ECheckBoxState IsAnimCurveTypeBoxChangedChecked(bool bMorphTarget) const;
	
	/** Returns name of this curve */
	FText GetItemName() const;
	/** Get text we are filtering for */
	FText GetFilterText() const;
	
	/* The SAnimCurveMetadataEditor that we push the morph target weights into */
	TWeakPtr<SAnimCurveMetadataEditor> AnimCurveViewerPtr;

	/** The name and weight of the morph target */
	TSharedPtr<FAnimCurveMetadataEditorItem>	Item;

	/** Preview scene used to update on scrub */
	TWeakPtr<class IPersonaPreviewScene> PreviewScenePtr;
	/** Returns curve type widget constructed */
	TSharedRef<SWidget> GetCurveTypeWidget();

	/** returns display text for number of connected joint setting */
	FText GetNumConnectedBones() const;

	/** returns display text for max LOD setting */
	FText GetMaxLOD() const;
};

//////////////////////////////////////////////////////////////////////////
// SAnimCurveMetadataEditor

class SAnimCurveMetadataEditor : public SCompoundWidget, public FSelfRegisteringEditorUndoClient
{
public:
	SLATE_BEGIN_ARGS( SAnimCurveMetadataEditor )
	{}
	
	SLATE_END_ARGS()

	/**
	* Slate construction function
	*
	* @param InArgs - Arguments passed from Slate
	*
	*/
	void Construct( const FArguments& InArgs, UObject* InAnimCurveMetaDataHost, const TSharedRef<class IPersonaPreviewScene>& InPreviewScene, FOnObjectsSelected InOnObjectsSelected);

	/**
	* Destructor - resets the animation curve
	*
	*/
	virtual ~SAnimCurveMetadataEditor();

	/** SWidget interface */
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent);

	/**
	* Is registered with Persona to handle when its preview mesh is changed.
	*
	* @param NewPreviewMesh - The new preview mesh being used by Persona
	*
	*/
	void OnPreviewMeshChanged(class USkeletalMesh* OldPreviewMesh, class USkeletalMesh* NewPreviewMesh);

	/**
	* Is registered with Persona to handle when its preview asset is changed.
	*
	* Pose Asset will have to add curve manually
	*/
	void OnPreviewAssetChanged(class UAnimationAsset* NewPreviewAsset);

	/** Is registered with Persona to handle when curves change. */
	void OnCurvesChanged();

	/**
	* Filters the SListView when the user changes the search text box (NameFilterBox)
	*
	* @param SearchText - The text the user has typed
	*
	*/
	void OnFilterTextChanged( const FText& SearchText );

	/**
	* Filters the SListView when the user hits enter or clears the search box
	* Simply calls OnFilterTextChanged
	*
	* @param SearchText - The text the user has typed
	* @param CommitInfo - Not used
	*
	*/
	void OnFilterTextCommitted( const FText& SearchText, ETextCommit::Type CommitInfo );

	/**
	* Create a widget for an entry in the tree from an info
	*
	* @param InInfo - Shared pointer to the morph target we're generating a row for
	* @param OwnerTable - The table that owns this row
	*
	* @return A new Slate widget, containing the UI for this row
	*/
	TSharedRef<ITableRow> GenerateAnimCurveRow(TSharedPtr<FAnimCurveMetadataEditorItem> InInfo, const TSharedRef<STableViewBase>& OwnerTable);

	/**
	* Accessor so our rows can grab the filtertext for highlighting
	*
	*/
	FText& GetFilterText() { return FilterText; }

	// FSelfRegisteringEditorUndoClient interface
	virtual void PostUndo(bool bSuccess) override { PostUndoRedo(); }
	virtual void PostRedo(bool bSuccess) override { PostUndoRedo(); }

	/**
	 * Refreshes the morph target list after an undo
	 */
	void PostUndoRedo();

	void RefreshCurveList(bool bInFullRefresh);

	// When a name is committed after being edited in the list
	virtual void OnNameCommitted(const FText& NewName, ETextCommit::Type CommitType, TSharedPtr<FAnimCurveMetadataEditorItem> Item);

private:

	void BindCommands();

	/** Handler for context menus */
	TSharedPtr<SWidget> OnGetContextMenuContent() const;
	void OnSelectionChanged(TSharedPtr<FAnimCurveMetadataEditorItem> InItem, ESelectInfo::Type SelectInfo);

	/**
	* Clears and rebuilds the table, according to an optional search string
	*
	* @param SearchText - Optional search string
	*
	*/
	void CreateAnimCurveList( const FString& SearchText = FString(), bool bInFullRefresh = false );

	void OnDeleteNameClicked();
	bool CanDelete();

	void OnRenameClicked();
	bool CanRename();

	void OnCopyClicked();
	bool CanCopy() const;

	void OnPasteClicked();
	bool CanPaste() const;

	void OnAddClicked();

	void OnFindCurveUsesClicked();
	bool CanFindCurveUses();
	
	// Adds a new curve metadata name entry to the skeleton
	void CreateNewNameEntry(const FText& CommittedText, ETextCommit::Type CommitType);

	/** Handle curve meta data removal */
	void HandleCurveMetaDataChange();

	/** Get the metadata to edit */
	UAnimCurveMetaData* GetAnimCurveMetaData() const;

	/** Get the skeleton associated with any bone references */
	USkeleton* GetSkeleton() const;
	
	/** Get the anim instance we are viewing */
	UAnimInstance* GetAnimInstance() const;

	/** Open the find/replace tab to process curves. Automatically supplies the name of the selected curve, if any */
	void FindReplaceCurves();

	/** Pointer to the preview scene we are bound to */
	TWeakPtr<class IPersonaPreviewScene> PreviewScenePtr;

	/** Pointer to the metada interface */
	TWeakInterfacePtr<IInterface_AssetUserData> AnimCurveMetaDataHost;

	/** Box to filter to a specific morph target name */
	TSharedPtr<SSearchBox>	NameFilterBox;

	/** A list of animation curve. Used by the AnimCurveListView. */
	TArray< TSharedPtr<FAnimCurveMetadataEditorItem> > AnimCurveList;

	/** Tracking map of anim curves */
	TMap< FName, TSharedPtr<FAnimCurveMetadataEditorItem> > AllSeenAnimCurvesMap;
	
	/** The skeletal mesh that we grab the animation curve from */
	UAnimInstance* CachedPreviewInstance;						

	/** Widget used to display the list of animation curve */
	TSharedPtr<SListView< TSharedPtr<FAnimCurveMetadataEditorItem> >> AnimCurveListView;

	/** Current text typed into NameFilterBox */
	FText FilterText;

	EAnimCurveMetadataEditorFilterFlags CurrentCurveFlag;

	TMap<FName, float> OverrideCurves;

	/** Commands that are bound to delegates*/
	TSharedPtr<FUICommandList> UICommandList;

	friend SAnimCurveMetadataEditorRow;
	friend SAnimCurveMetadataEditorTypeList;

	/** Tracks objects created for the details panel */
	FEditorObjectTracker EditorObjectTracker;

	/** Delegate called to select objects */
	FOnObjectsSelected OnObjectsSelected;

	/** apply curve bone links from editor mirror object to skeleton */
	void ApplyCurveBoneLinks(class UEditorAnimCurveBoneLinks* EditorObj);

	/** Delegate handle for HandleCurveMetaDataChange callback */
	FDelegateHandle CurveMetaDataChangedHandle;

	/** All filters that can be applied to the widget's display */
	TArray<TSharedRef<FFilterBase<EAnimCurveMetadataEditorFilterFlags>>> Filters;
};