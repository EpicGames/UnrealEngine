// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAnimCurvePicker.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "IEditableSkeleton.h"
#include "Animation/AnimationAsset.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "SListViewSelectorDropdownMenu.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Layout/SMenuOwner.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Input/SCheckBox.h"
#include "Engine/SkeletalMesh.h"

#define LOCTEXT_NAMESPACE "SAnimCurvePicker"

SAnimCurvePicker::~SAnimCurvePicker()
{
}

void SAnimCurvePicker::Construct(const FArguments& InArgs, const USkeleton* InSkeleton)
{
	OnCurvesPicked = InArgs._OnCurvesPicked;
	IsCurveNameMarkedForExclusion = InArgs._IsCurveNameMarkedForExclusion;
	Skeleton = InSkeleton;
	bShowOtherSkeletonCurves = false;

	SAssignNew(SearchBox, SSearchBox)
	.HintText(LOCTEXT("SearchBoxHint", "Search"))
	.OnTextChanged(this, &SAnimCurvePicker::HandleFilterTextChanged);

	SAssignNew(NameListView, SListView<TSharedPtr<FName>>)
	.SelectionMode(InArgs._bEnableMultiselect ? ESelectionMode::Multi : ESelectionMode::Single)
	.OnSelectionChanged_Lambda([this, InArgs](TSharedPtr<FName> InNameItem, ESelectInfo::Type SelectInfo)
	{
		if (InNameItem && !InArgs._bEnableMultiselect)
		{
			OnCurvesPicked.ExecuteIfBound(TArray<FName>{ *InNameItem });
		}
	})
	.ListItemsSource(&CurveNames)
	.OnGenerateRow(this, &SAnimCurvePicker::HandleGenerateRow);

	const float HorizontalPadding = 8.0f;
	const float VerticalPadding = 2.0f;
	const float WeightOverride = 300.0f;

	TSharedPtr<SVerticalBox> VerticalBox;

	ChildSlot
	[
		SNew(SMenuOwner)
		[
			SNew(SListViewSelectorDropdownMenu<TSharedPtr<FName>>, SearchBox, NameListView)
			[
				SAssignNew(VerticalBox, SVerticalBox)
				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(HorizontalPadding, VerticalPadding)
				[
					SearchBox.ToSharedRef()
				]
				+SVerticalBox::Slot()
				.FillHeight(1.0f)
				.Padding(HorizontalPadding, VerticalPadding)
				[
					SNew(SBox)
					.WidthOverride(WeightOverride)
					.HeightOverride(WeightOverride)
					[
						SNew(SOverlay)
						+SOverlay::Slot()
						[
							SNew(SBorder)
							.BorderImage(FAppStyle::GetBrush("Graph.StateNode.Body"))
							.BorderBackgroundColor(FAppStyle::Get().GetSlateColor("Colors.Input"))
						]
						+SOverlay::Slot()
						[
							SNew(SScrollBox)
							.Orientation(EOrientation::Orient_Vertical)
							+SScrollBox::Slot()
							.HAlign(HAlign_Fill)
							.VAlign(VAlign_Fill)
							[
								NameListView.ToSharedRef()
							]
						]
					]
				]
			]
		]
	];

	if(Skeleton.IsValid())
	{
		VerticalBox->AddSlot()
			.AutoHeight()
			.Padding(HorizontalPadding, VerticalPadding)
			[
				SNew(SCheckBox)
				.ToolTipText(LOCTEXT("ShowOtherSkeletonsTooltip", "Whether to show all curves or just the curves from the current skeleton"))
				.IsChecked_Lambda([this]()
				{
					return bShowOtherSkeletonCurves ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				})
				.OnCheckStateChanged_Lambda([this](ECheckBoxState InState)
				{
					bShowOtherSkeletonCurves = (InState == ECheckBoxState::Checked);
					RefreshListItems();
				})
				.Content()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("ShowOtherSkeletons", "Show curves from other skeletons"))
				]
			];
	}

	if (InArgs._bEnableMultiselect)
	{
		VerticalBox->AddSlot()
			.AutoHeight()
			.Padding(HorizontalPadding, VerticalPadding)
			[
				SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), "PrimaryButton")
					.Text(LOCTEXT("ConfirmButton", "Confirm"))
					.IsEnabled_Lambda([this]()
					{
						return NameListView->GetNumItemsSelected() > 0;
					})
					.OnClicked_Lambda([this]()
					{
						TArray<TSharedPtr<FName>> SelectionPtrs;
						NameListView->GetSelectedItems(SelectionPtrs);
						check(SelectionPtrs.Num() > 0);

						TArray<FName> Selection;
						for (TSharedPtr<FName> Selected : SelectionPtrs)
						{
							Selection.Add(*Selected);
						}

						OnCurvesPicked.ExecuteIfBound(Selection);
					
						return FReply::Handled();
					})
			];
	}

	RefreshListItems();
}

TSharedRef<ITableRow> SAnimCurvePicker::HandleGenerateRow(TSharedPtr<FName> InItem, const TSharedRef<STableViewBase>& InOwnerTable)
{
	return 
		SNew(STableRow<TSharedPtr<FName>>, InOwnerTable)
		.Padding(FMargin(8.0f, 0.0f))
		[
			SNew(SBox)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(FText::FromName(*InItem))
				.HighlightText_Lambda([this]() { return FText::FromString(FilterText); })
			]
		];
}

void SAnimCurvePicker::RefreshListItems()
{
	const USkeleton* CurrentSkeleton = Skeleton.Get();
	FString CurrentSkeletonName;
	if(CurrentSkeleton)
	{
		CurrentSkeletonName = FAssetData(CurrentSkeleton).GetExportTextName();
	}

	CurveNames.Reset();
	UniqueCurveNames.Reset();

	{
		// First check skeleton metadata
		if(CurrentSkeleton)
		{
			CurrentSkeleton->ForEachCurveMetaData([this](FName InCurveName, const FCurveMetaData& InMetaData)
			{
				UniqueCurveNames.Add(InCurveName);
			});
		}

		// We use the asset registry to query all assets (optionally with the supplied skeleton) and accumulate their curve names
		const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");

		FARFilter Filter;
		Filter.bRecursiveClasses = true;
		Filter.ClassPaths.Append({ UAnimationAsset::StaticClass()->GetClassPathName(), USkeletalMesh::StaticClass()->GetClassPathName(), USkeleton::StaticClass()->GetClassPathName() } );

		TArray<FAssetData> FoundAssetData;
		AssetRegistryModule.Get().GetAssets(Filter, FoundAssetData);

		// Build set of unique curve smart names
		for (FAssetData& AssetData : FoundAssetData)
		{
			if(!bShowOtherSkeletonCurves && AssetData.GetClass() != USkeleton::StaticClass())
			{
				if(!CurrentSkeletonName.IsEmpty())
				{
					// Check skeleton tag
					const FString SkeletonName = AssetData.GetTagValueRef<FString>(USkeleton::StaticClass()->GetFName());
					if(SkeletonName != CurrentSkeletonName)
					{
						continue;
					}
				}
			}
			
			const FString TagValue = AssetData.GetTagValueRef<FString>(USkeleton::CurveNameTag);
			if (!TagValue.IsEmpty())
			{
				TArray<FString> AssetCurveNames;
				if (TagValue.ParseIntoArray(AssetCurveNames, *USkeleton::CurveTagDelimiter, true) > 0)
				{
					for (const FString& CurveNameString : AssetCurveNames)
					{
						FName CurveName = FName(*CurveNameString);

						if(CurveName == NAME_None)
						{
							continue;
						}

						if (IsCurveNameMarkedForExclusion.IsBound() && IsCurveNameMarkedForExclusion.Execute(CurveName))
						{
							continue;
						}

						UniqueCurveNames.Add(CurveName);
					}
				}
			}
		}
	}

	FilterAvailableCurves();
}

void SAnimCurvePicker::FilterAvailableCurves()
{
	CurveNames.Reset();
	
	// Exact filtering
	for (const FName& UniqueCurveName : UniqueCurveNames)
	{
		if (FilterText.IsEmpty() || UniqueCurveName.ToString().Contains(FilterText))
		{
			CurveNames.Add(MakeShared<FName>(UniqueCurveName));
		}
	}

	// Alphabetical sorting
	{
		struct FNameSortItemSortOp
		{
			FORCEINLINE bool operator()( const TSharedPtr<FName>& A, const TSharedPtr<FName>& B ) const
			{
				return (A->ToString().Compare(B->ToString()) < 0);
			}
		};
		CurveNames.Sort(FNameSortItemSortOp());
	}

	// Rebuild list view
	NameListView->RequestListRefresh();
}

void SAnimCurvePicker::HandleFilterTextChanged(const FText& InFilterText)
{
	FilterText = InFilterText.ToString();

	FilterAvailableCurves();
}

#undef LOCTEXT_NAMESPACE