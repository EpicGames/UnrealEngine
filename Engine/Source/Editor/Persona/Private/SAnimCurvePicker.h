// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableRow.h"
#include "Animation/SmartName.h"
#include "Widgets/Input/SSearchBox.h"
#include "PersonaDelegates.h"

class IEditableSkeleton;

class SAnimCurvePicker : public SCompoundWidget
{
public:

	/** Virtual destructor. */
	virtual ~SAnimCurvePicker() override;
	
	SLATE_BEGIN_ARGS(SAnimCurvePicker)
		: _bEnableMultiselect(false)
	{}

	FOnCurvesPicked ConvertOnCurvePickedFn(const FOnCurvePicked& LegacyDelegate)
	{
		return FOnCurvesPicked::CreateLambda([LegacyDelegate](const TConstArrayView<FName> PickedCurves)
		{
			check(PickedCurves.Num() == 1)
			LegacyDelegate.ExecuteIfBound(PickedCurves[0]);
		});
	}

	SLATE_EVENT_DEPRECATED(5.6, "Use FOnCurvesPicked instead", FOnCurvePicked, OnCurvePicked, OnCurvesPicked, ConvertOnCurvePickedFn)

	SLATE_EVENT(FOnCurvesPicked, OnCurvesPicked)

	SLATE_EVENT(FIsCurveNameMarkedForExclusion, IsCurveNameMarkedForExclusion)

	SLATE_ARGUMENT(bool, bEnableMultiselect)
	
	SLATE_END_ARGS()

	/**
	 * Construct this widget.
	 * @param InArgs - The declaration data for this widget
	 * @param InSkeleton - The skeleton from which the widget extracts curve information 
	 */
	void Construct(const FArguments& InArgs, const USkeleton* InSkeleton);

private:
	/** Refresh the list of available curves */
	void RefreshListItems();

	/** Filter available curves */
	void FilterAvailableCurves();

	/** UI handlers */
	TSharedRef<ITableRow> HandleGenerateRow(TSharedPtr<FName> InItem, const TSharedRef<STableViewBase>& InOwnerTable);
	void HandleFilterTextChanged(const FText& InFilterText);

private:
	/** Delegate fired when the curves selection is confirmed */
	FOnCurvesPicked OnCurvesPicked;

	/* Filter predicate to determine if curve should be excluded from the picker's curve list */
	FIsCurveNameMarkedForExclusion IsCurveNameMarkedForExclusion;

	/** The editable skeleton we use to grab curves from */
	TWeakObjectPtr<const USkeleton> Skeleton;

	/** The names of the curves we are displaying */
	TArray<TSharedPtr<FName>> CurveNames;

	/** All the unique curve names we can find */
	TSet<FName> UniqueCurveNames;

	/** The string we use to filter curve names */
	FString FilterText;

	/** The list view used to display names */
	TSharedPtr<SListView<TSharedPtr<FName>>> NameListView;

	/** The search box used to filter curves */
	TSharedPtr<SSearchBox> SearchBox;

	/** Whether we should show other skeleton's curves */
	bool bShowOtherSkeletonCurves;
};