// Copyright Epic Games, Inc. All Rights Reserved.


#include "SAnimCurveViewer.h"

#include "AnimAssetFindReplaceCurves.h"
#include "AnimationEditorUtils.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Styling/AppStyle.h"
#include "Widgets/Input/SSpinBox.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "Widgets/Input/SSearchBox.h"
#include "Animation/AnimSingleNodeInstance.h"
#include "IEditableSkeleton.h"
#include "Widgets/Colors/SColorBlock.h"
#include "CurveViewerCommands.h"
#include "Animation/AnimBlueprintGeneratedClass.h"
#include "Animation/EditorAnimCurveBoneLinks.h"
#include "HAL/PlatformApplicationMisc.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Misc/ScopedSlowTask.h"
#include "Engine/PoseWatch.h"
#include "Filters/GenericFilter.h"
#include "Filters/SBasicFilterBar.h"
#include "Widgets/Input/STextComboBox.h"
#include "AnimPreviewInstance.h"
#include "PersonaTabs.h"
#include "SAnimAssetFindReplace.h"
#include "Widgets/Docking/SDockTab.h"
#include "SPoseWatchPicker.h"

#define LOCTEXT_NAMESPACE "SAnimCurveViewer"

namespace CurveViewerColumns
{
static const FName AnimCurveNameLabel( "Curve Name" );
static const FName AnimCurveTypeLabel("Type");
static const FName AnimCurveWeightLabel( "Weight" );
static const FName AnimCurveEditLabel( "Edit" );
}

//////////////////////////////////////////////////////////////////////////
// SAnimCurveListRow

typedef TSharedPtr< FDisplayedAnimCurveInfo > FDisplayedAnimCurveInfoPtr;

// This is a flag that is used to filter UI part
enum class EAnimCurveViewerFilterFlags : uint8 
{
	// Show all
	ShowAll			= 0, 
	// Show active curves
	Active			= 0x01, 
	// Show morph target curves
	MorphTarget		= 0x02, 
	// Show material curves
	Material		= 0x04, 
};

ENUM_CLASS_FLAGS(EAnimCurveViewerFilterFlags);

class FAnimCurveViewerFilter : public FGenericFilter<EAnimCurveViewerFilterFlags>
{
public:
	FAnimCurveViewerFilter(EAnimCurveViewerFilterFlags InFlags, const FString& InName, const FText& InDisplayName, const FText& InToolTipText, FLinearColor InColor, TSharedPtr<FFilterCategory> InCategory)
		: FGenericFilter<EAnimCurveViewerFilterFlags>(InCategory, InName, InDisplayName, FGenericFilter<EAnimCurveViewerFilterFlags>::FOnItemFiltered())
		, Flags(InFlags)
	{
		ToolTip = InToolTipText;
		Color = InColor;
	}

	bool IsActive() const
	{
		return bIsActive;
	}

	EAnimCurveViewerFilterFlags GetFlags() const
	{
		return Flags;
	}
	
private:
	// FFilterBase interface
	virtual void ActiveStateChanged(bool bActive) override
	{
		bIsActive = bActive;
	}

	virtual bool PassesFilter(EAnimCurveViewerFilterFlags InItem) const override
	{
		return EnumHasAnyFlags(InItem, Flags);
	}
	
private:
	EAnimCurveViewerFilterFlags Flags;
	bool bIsActive = false;
};

bool FDisplayedAnimCurveInfo::GetActiveFlag(const TSharedPtr<SAnimCurveViewer>& InAnimCurveViewer, bool bInMorphTarget) const
{
	const UAnimInstance* AnimInstance = InAnimCurveViewer->GetAnimInstance();
	if (AnimInstance)
	{
		auto FindMorphOrMatCurveOnAnimInstance = [AnimInstance](const FName& CurveName, bool bIsMorphTarget) {
			const TMap<FName, float>& CurveList = AnimInstance->GetAnimationCurveList(bIsMorphTarget ? EAnimCurveType::MorphTargetCurve : EAnimCurveType::MaterialCurve);
			return CurveList.Find(CurveName) != nullptr;
		};

		// Find if we want to use a pose watch
		if (InAnimCurveViewer->PoseWatchPicker.IsValid())
		{
			if (UPoseWatchPoseElement* PoseWatchPoseElement = InAnimCurveViewer->PoseWatchPicker->GetCurrentPoseWatch())
			{
				if (UAnimBlueprintGeneratedClass* AnimClass = Cast<UAnimBlueprintGeneratedClass>(AnimInstance->GetClass()))
				{
					// We have to grab our pose watches from the root class as no pose watches can be set on child anim BPs
					if (const UAnimBlueprintGeneratedClass* RootClass = Cast<UAnimBlueprintGeneratedClass>(AnimClass->GetRootClass()))
					{
						const FAnimBlueprintDebugData& DebugData = RootClass->AnimBlueprintDebugData;
						for (const FAnimNodePoseWatch& AnimNodePoseWatch : DebugData.AnimNodePoseWatch)
						{
							if (AnimNodePoseWatch.PoseWatchPoseElement == PoseWatchPoseElement)
							{
								UE::Anim::ECurveElementFlags Flags = AnimNodePoseWatch.GetCurves().GetFlags(CurveName);
								return EnumHasAnyFlags(Flags, bInMorphTarget ? UE::Anim::ECurveElementFlags::MorphTarget : UE::Anim::ECurveElementFlags::Material);
							}
						}
					}
				}
			}
			else
			{
				// We are just displaying the output curves, so find if the curve exists on the anim instance
				return FindMorphOrMatCurveOnAnimInstance(CurveName, bInMorphTarget);
			}
		}
		else
		{
			// We aren't using pose watch so check on the anim instance and the skeleton
			if (FindMorphOrMatCurveOnAnimInstance(CurveName, bInMorphTarget))
			{
				return true;
			}

			if (TSharedPtr<IEditableSkeleton> EditableSkeleton = InAnimCurveViewer->EditableSkeletonPtr.Pin())
			{
				if (const FCurveMetaData* CurveMetaData = EditableSkeleton->GetSkeleton().GetCurveMetaData(CurveName))
				{
					return bInMorphTarget ? CurveMetaData->Type.bMorphtarget : CurveMetaData->Type.bMaterial;
				}
			}
		}
	}

	return false;
}

void SAnimCurveListRow::Construct( const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, const TSharedRef<IPersonaPreviewScene>& InPreviewScene)
{
	Item = InArgs._Item;
	AnimCurveViewerPtr = InArgs._AnimCurveViewerPtr;
	PreviewScenePtr = InPreviewScene;

	check( Item.IsValid() );

	SMultiColumnTableRow< TSharedPtr<FDisplayedAnimCurveInfo> >::Construct( FSuperRowType::FArguments(), InOwnerTableView );
}

TSharedRef< SWidget > SAnimCurveListRow::GenerateWidgetForColumn( const FName& ColumnName )
{
	if ( ColumnName == CurveViewerColumns::AnimCurveNameLabel )
	{
		TSharedPtr<SAnimCurveViewer> AnimCurveViewer = AnimCurveViewerPtr.Pin();
		if (AnimCurveViewer.IsValid())
		{
			return
				SNew(SVerticalBox)
				.ToolTipText(this, &SAnimCurveListRow::GetItemName)
			
				+ SVerticalBox::Slot()
				.FillHeight(1.0f)
				.Padding(4)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Font(this, &SAnimCurveListRow::GetItemFont)
					.Text(this, &SAnimCurveListRow::GetItemName)
					.HighlightText(this, &SAnimCurveListRow::GetFilterText)
				];
		}
	}
	else if (ColumnName == CurveViewerColumns::AnimCurveTypeLabel)
	{
		TSharedPtr<SAnimCurveViewer> AnimCurveViewer = AnimCurveViewerPtr.Pin();
		if (AnimCurveViewer.IsValid())
		{
			return
				SNew(SVerticalBox)
				.ToolTipText(LOCTEXT("AnimCurveTypeTooltip", "The type of the curve (e.g. morph target, material)."))

				+ SVerticalBox::Slot()
				.FillHeight(1.0f)
				.VAlign(VAlign_Center)
				[
					GetCurveTypeWidget()
				];
		}
	}
	else if ( ColumnName == CurveViewerColumns::AnimCurveWeightLabel )
	{
		// Encase the SSpinbox in an SVertical box so we can apply padding. Setting ItemHeight on the containing SListView has no effect :-(
		return
			SNew( SVerticalBox )
			.ToolTipText(LOCTEXT("AnimCurveWeightTooltip", "The current weight of the curve."))
		
			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			.Padding( 0.0f, 1.0f )
			.VAlign( VAlign_Center )
			[
				SNew( SSpinBox<float> )
				.Font(FAppStyle::Get().GetFontStyle("SmallFont"))
				.Value(this, &SAnimCurveListRow::GetWeight)
				.IsEnabled(this, &SAnimCurveListRow::CanBeEdited)
				.MinSliderValue(this, &SAnimCurveListRow::GetMinWeight)
				.MaxSliderValue(this, &SAnimCurveListRow::GetMaxWeight)
				.OnValueChanged( this, &SAnimCurveListRow::OnAnimCurveWeightChanged )
				.OnValueCommitted( this, &SAnimCurveListRow::OnAnimCurveWeightValueCommitted )
			];
	}
	else if ( ColumnName == CurveViewerColumns::AnimCurveEditLabel)
	{
		return
			SNew(SVerticalBox)
			.ToolTipText(LOCTEXT("OverrideTooltip", "Whether the value of this curve is being overriden, or populated from the debugged data."))
		
			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			.Padding(0.0f, 1.0f)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			[
				SNew(SCheckBox)
				.OnCheckStateChanged(this, &SAnimCurveListRow::OnAnimCurveOverrideChecked)
				.IsChecked(this, &SAnimCurveListRow::IsAnimCurveOverrideChecked)
			];
	}
	
	return SNullWidget::NullWidget;
}

TSharedRef< SWidget > SAnimCurveListRow::GetCurveTypeWidget()
{
	return SNew(SHorizontalBox)

		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(0.f, 1.f, 1.f, 1.f)
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Center)
		[
			SNew(SImage)
			.ToolTipText(LOCTEXT("CurveTypeMorphTarget_Tooltip", "MorphTarget"))
			.Image_Lambda([this]()
			{
				bool bHasCurveType = GetActiveFlag(true);
				if(bHasCurveType)
				{
					return FAppStyle::GetBrush("AnimCurveViewer.MorphTargetOn");
				}
				else
				{
					return FAppStyle::GetBrush("AnimCurveViewer.MorphTargetOff");
				}
			})
		]

		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(0.f, 1.f, 1.f, 1.f)
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Center)
		[
			SNew(SImage)
			.ToolTipText(LOCTEXT("CurveTypeMaterial_Tooltip", "Material"))
			.Image_Lambda([this]()
			{
				bool bHasCurveType = GetActiveFlag(false);
				if(bHasCurveType)
				{
					return FAppStyle::GetBrush("AnimCurveViewer.MaterialOn");
				}
				else
				{
					return FAppStyle::GetBrush("AnimCurveViewer.MaterialOff");
				}
			})
		];

}

bool SAnimCurveListRow::GetActiveFlag(bool bMorphTarget) const
{
	// If anim viewer
	TSharedPtr<SAnimCurveViewer> AnimCurveViewer = AnimCurveViewerPtr.Pin();
	if (AnimCurveViewer.IsValid())
	{
		return Item->GetActiveFlag(AnimCurveViewer, bMorphTarget);
	}

	return false;
}

ECheckBoxState SAnimCurveListRow::IsAnimCurveTypeBoxChangedChecked(bool bMorphTarget) const
{
	bool bHasCurveType = GetActiveFlag(bMorphTarget);
	
	return (bHasCurveType) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SAnimCurveListRow::OnAnimCurveOverrideChecked(ECheckBoxState InState)
{
	Item->bOverrideData = InState == ECheckBoxState::Checked;

	TSharedPtr<SAnimCurveViewer> AnimCurveViewer = AnimCurveViewerPtr.Pin();
	if (AnimCurveViewer.IsValid())
	{
		if (Item->bOverrideData)
		{
			AnimCurveViewer->AddAnimCurveOverride(Item->CurveName, Item->Weight);
		}
		else
		{
			// clear value so that it can be filled up
			AnimCurveViewer->RemoveAnimCurveOverride(Item->CurveName);
		}
	}
}

ECheckBoxState SAnimCurveListRow::IsAnimCurveOverrideChecked() const
{
	return (Item->bOverrideData) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SAnimCurveListRow::OnAnimCurveWeightChanged( float NewWeight )
{
	Item->Weight = NewWeight;
	Item->bOverrideData = true;

	TSharedPtr<SAnimCurveViewer> AnimCurveViewer = AnimCurveViewerPtr.Pin();
	if (AnimCurveViewer.IsValid())
	{
		// If we try to slide an entry that is not selected, we select just it
		bool bItemIsSelected = AnimCurveViewer->AnimCurveListView->IsItemSelected(Item);
		if (!bItemIsSelected)
		{
			AnimCurveViewer->AnimCurveListView->SetSelection(Item, ESelectInfo::Direct);
		}

		// Add override
		AnimCurveViewer->AddAnimCurveOverride(Item->CurveName, Item->Weight);

		// ...then any selected rows need changing by the same delta
		TArray< TSharedPtr< FDisplayedAnimCurveInfo > > SelectedRows = AnimCurveViewer->AnimCurveListView->GetSelectedItems();
		for (auto ItemIt = SelectedRows.CreateIterator(); ItemIt; ++ItemIt)
		{
			TSharedPtr< FDisplayedAnimCurveInfo > RowItem = (*ItemIt);

			if (RowItem != Item) // Don't do "this" row again if it's selected
			{
				RowItem->Weight = NewWeight;
				RowItem->bOverrideData = true;
				AnimCurveViewer->AddAnimCurveOverride(RowItem->CurveName, RowItem->Weight);
			}
		}

		if(PreviewScenePtr.IsValid())
		{
			PreviewScenePtr.Pin()->InvalidateViews();
		}
	}
}

void SAnimCurveListRow::OnAnimCurveWeightValueCommitted( float NewWeight, ETextCommit::Type CommitType)
{
	if (CommitType == ETextCommit::OnEnter || CommitType == ETextCommit::OnUserMovedFocus)
	{
		OnAnimCurveWeightChanged(NewWeight);
	}
}

FText SAnimCurveListRow::GetItemName() const
{
	return FText::FromName(Item->CurveName);
}

FText SAnimCurveListRow::GetFilterText() const
{
	TSharedPtr<SAnimCurveViewer> AnimCurveViewer = AnimCurveViewerPtr.Pin();
	if (AnimCurveViewer.IsValid())
	{
		return AnimCurveViewer->GetFilterText();
	}
	else
	{
		return FText::GetEmpty();
	}
}


bool SAnimCurveListRow::GetActiveWeight(float& OutWeight) const
{
	// If anim viewer
	TSharedPtr<SAnimCurveViewer> AnimCurveViewer = AnimCurveViewerPtr.Pin();
	if (AnimCurveViewer.IsValid())
	{
		const UAnimInstance* AnimInstance = AnimCurveViewer->GetAnimInstance();
		if(AnimInstance)
		{
			// If we're dealing with a pose watch, get the data from that
			if (AnimCurveViewer->PoseWatchPicker.IsValid())
			{
				// Find if we want to use a pose watch
				if (UPoseWatchPoseElement* PoseWatchPoseElement = AnimCurveViewer->PoseWatchPicker->GetCurrentPoseWatch())
				{
					if (UAnimBlueprintGeneratedClass* AnimClass = Cast<UAnimBlueprintGeneratedClass>(AnimInstance->GetClass()))
					{
						// We have to grab our pose watches from the root class as no pose watches can be set on child anim BPs
						if (const UAnimBlueprintGeneratedClass* RootClass = Cast<UAnimBlueprintGeneratedClass>(AnimClass->GetRootClass()))
						{
							const FAnimBlueprintDebugData& DebugData = RootClass->AnimBlueprintDebugData;
							for (const FAnimNodePoseWatch& AnimNodePoseWatch : DebugData.AnimNodePoseWatch)
							{
								if (AnimNodePoseWatch.PoseWatchPoseElement == PoseWatchPoseElement)
								{
									bool bHasElement = false;
									float CurrentValue = AnimNodePoseWatch.GetCurves().Get(Item->CurveName, bHasElement);
									if (bHasElement)
									{
										OutWeight = CurrentValue;
										return true;
									}
									break;
								}
							}
						}
					}

					// didn't find the curve on the pose watch element
					return false;
				}
			}

			// We aren't dealing with a pose watch so go directly to the anim instance curves
			const TMap<FName, float>& CurveList = AnimInstance->GetAnimationCurveList(EAnimCurveType::AttributeCurve);
			const float* CurrentValue = CurveList.Find(Item->CurveName);
			if (CurrentValue)
			{
				OutWeight = *CurrentValue;
				return true;
			}
		}
	}

	return false;
}

bool SAnimCurveListRow::CanBeEdited() const
{
	if (const TSharedPtr<SAnimCurveViewer> AnimCurveViewer = AnimCurveViewerPtr.Pin())
	{
		return !AnimCurveViewer->bIsReadOnlyViewer;
	}

	return false;
}

FSlateFontInfo SAnimCurveListRow::GetItemFont() const
{
	// Show bright if active
	float Weight = 0.f;
	const bool bItemActive = GetActiveWeight(Weight);

	return bItemActive ? FAppStyle::Get().GetFontStyle("AnimCurveViewer.ActiveCurveFont") : FAppStyle::Get().GetFontStyle("SmallFont");
}

float SAnimCurveListRow::GetWeight() const
{
	float Weight = 0.0f;
	bool bItemActive = false;
	if (!Item->bOverrideData)
	{
		bItemActive = GetActiveWeight(Weight);
		MinWeight = FMath::Min(MinWeight, Weight);
		MaxWeight = FMath::Max(MaxWeight, Weight);
	}

	if(!bItemActive)
	{
		Weight = Item->Weight;
	}

	return Weight;
}

TOptional<float> SAnimCurveListRow::GetMinWeight() const 
{
	return MinWeight;
}

TOptional<float> SAnimCurveListRow::GetMaxWeight() const 
{
	return MaxWeight;
}

//////////////////////////////////////////////////////////////////////////
// SAnimCurveViewer

void SAnimCurveViewer::Construct(const FArguments& InArgs,  const TSharedRef<IPersonaPreviewScene>& InPreviewScene, FOnObjectsSelected InOnObjectsSelected)
{
	OnObjectsSelected = InOnObjectsSelected;

	PreviewScenePtr = InPreviewScene;
	EditableSkeletonPtr = InArgs._EditableSkeleton;
	bIsReadOnlyViewer = InArgs._bIsReadOnly;

	InPreviewScene->RegisterOnPreviewMeshChanged(FOnPreviewMeshChanged::CreateSP(this, &SAnimCurveViewer::OnPreviewMeshChanged));
	InPreviewScene->RegisterOnAnimChanged(FOnAnimChanged::CreateSP(this, &SAnimCurveViewer::OnPreviewAssetChanged));

	// Register and bind all our menu commands
	FCurveViewerCommands::Register();
	BindCommands();

	CurrentCurveFlag = EAnimCurveViewerFilterFlags::Active;
	TArray<FName> HiddenColumns;
	if (bIsReadOnlyViewer)
	{
		HiddenColumns.Add(CurveViewerColumns::AnimCurveEditLabel);
	}

	TSharedPtr<FFilterCategory> FilterCategory = MakeShared<FFilterCategory>(LOCTEXT("CurveFiltersLabel", "Curve Filters"), LOCTEXT("CurveFiltersToolTip", "Filter what kind fo curves can be displayed."));

	Filters.Add(MakeShared<FAnimCurveViewerFilter>(
		EAnimCurveViewerFilterFlags::Active,
		"Active",
		LOCTEXT("ShowActiveLabel", "Active"),
		LOCTEXT("ShowActiveTooltip", "Show only active curves"),
		FLinearColor::Yellow,
		FilterCategory
		));

	Filters.Add(MakeShared<FAnimCurveViewerFilter>(
		EAnimCurveViewerFilterFlags::MorphTarget,
		"MorphTarget",
		LOCTEXT("MorphTargetLabel", "Morph Target"),
		LOCTEXT("MorphTargetTooltip", "Show morph target curves"),
		FLinearColor::Red,
		FilterCategory
		));

	Filters.Add(MakeShared<FAnimCurveViewerFilter>(
		EAnimCurveViewerFilterFlags::Material,
		"Material",
		LOCTEXT("MaterialLabel", "Material"),
		LOCTEXT("MaterialTooltip", "Show material curves"),
		FLinearColor::Green,
		FilterCategory
		));
	
	TSharedRef<SBasicFilterBar<EAnimCurveViewerFilterFlags>> FilterBar = SNew(SBasicFilterBar<EAnimCurveViewerFilterFlags>)
	.CustomFilters(Filters)
	.bPinAllFrontendFilters(true)
	.UseSectionsForCategories(true)
	.OnFilterChanged_Lambda([this]()
	{
		CurrentCurveFlag = EAnimCurveViewerFilterFlags::ShowAll;

		for(const TSharedRef<FFilterBase<EAnimCurveViewerFilterFlags>>& Filter : Filters)
		{
			TSharedRef<FAnimCurveViewerFilter> AnimCurveFilter = StaticCastSharedRef<FAnimCurveViewerFilter>(Filter);
			if(AnimCurveFilter->IsActive())
			{
				CurrentCurveFlag |= AnimCurveFilter->GetFlags();
			}
		}

		RefreshCurveList(true);
	});

	Filters[0]->SetActive(true);

	ChildSlot
	[
		SNew( SVerticalBox )
		+SVerticalBox::Slot()
		.Padding(2.0f,2.0f)
		.AutoHeight()
		.HAlign(HAlign_Left)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SSimpleButton)
				.Text(LOCTEXT("RefreshButton", "Refresh"))
				.ToolTipText(LOCTEXT("RefreshButtonTooltip", "Refresh the displayed curves, clearing out any curves that are not currently active"))
				.Icon(FAppStyle::GetBrush("Icons.Refresh"))
				.OnClicked_Lambda([this]()
				{
					AllSeenAnimCurvesMap.Reset();
					RefreshCurveList(true);
					return FReply::Handled();
				})
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SSimpleButton)
				.Text(LOCTEXT("FindReplaceCurvesButton", "Find/Replace Curves..."))
				.ToolTipText(LOCTEXT("FindReplaceCurvesButtonTooltip", "Find and replace curves across multiple assets"))
				.Icon(FAppStyle::GetBrush("Kismet.Tabs.FindResults"))
				.OnClicked_Lambda([this]()
				{
					FindReplaceCurves();
					return FReply::Handled();
				})
			]
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.0f,2.0f)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.0f,0.0f)
			[
				SBasicFilterBar<EAnimCurveViewerFilterFlags>::MakeAddFilterButton(FilterBar)
			]
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.Padding(2.0f,0.0f)
			[
				SAssignNew( NameFilterBox, SSearchBox )
				.SelectAllTextWhenFocused( true )
				.OnTextChanged( this, &SAnimCurveViewer::OnFilterTextChanged )
				.OnTextCommitted( this, &SAnimCurveViewer::OnFilterTextCommitted )
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.0f,0.0f)
			[
				SNew(SBox)
				.MaxDesiredWidth(200.0f)
				[
					CreateCurveSourceSelector()
				]
			]
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			FilterBar
		]
		+SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			SAssignNew(AnimCurveListView, SAnimCurveListType)
			.OnContextMenuOpening( this, &SAnimCurveViewer::OnGetContextMenuContent )
			.ListItemsSource( &AnimCurveList )
			.OnGenerateRow( this, &SAnimCurveViewer::GenerateAnimCurveRow )
			.SelectionMode(ESelectionMode::Multi)
			.HeaderRow
			(
				SNew( SHeaderRow )
				.HiddenColumnsList(HiddenColumns)
				+ SHeaderRow::Column(CurveViewerColumns::AnimCurveNameLabel)
				.FillWidth(1.f)
				.DefaultLabel( LOCTEXT( "AnimCurveNameLabel", "Curve Name" ) )
				.DefaultTooltip(LOCTEXT("AnimCurveNameTooltip", "The name of the curve."))
				
				+ SHeaderRow::Column(CurveViewerColumns::AnimCurveTypeLabel)
				.FixedWidth(48.0f)
				.DefaultLabel(LOCTEXT("AnimCurveTypeLabel", "Type"))
				.DefaultTooltip(LOCTEXT("AnimCurveTypeTooltip", "The type of the curve (e.g. morph target, material)."))

				+ SHeaderRow::Column(CurveViewerColumns::AnimCurveWeightLabel )
				.FillWidth(1.f)
				.DefaultLabel( LOCTEXT( "AnimCurveWeightLabel", "Weight" ) )
				.DefaultTooltip(LOCTEXT("AnimCurveWeightTooltip", "The current weight of the curve."))

				+ SHeaderRow::Column(CurveViewerColumns::AnimCurveEditLabel)
				.FixedWidth(24.0f)
				.DefaultLabel(FText::GetEmpty())
				.DefaultTooltip(LOCTEXT("OverrideTooltip", "Whether the value of this curve is being overriden, or populated from the debugged data."))
			)
		]
	];

	RefreshCurveList(true);
}

SAnimCurveViewer::~SAnimCurveViewer()
{
	if (PreviewScenePtr.IsValid() )
	{
		PreviewScenePtr.Pin()->UnregisterOnPreviewMeshChanged(this);
		PreviewScenePtr.Pin()->UnregisterOnAnimChanged(this);
	}

	AnimationEditorUtils::OnPoseWatchesChanged().RemoveAll(this);
}

void SAnimCurveViewer::OnPreviewMeshChanged(class USkeletalMesh* OldPreviewMesh, class USkeletalMesh* NewPreviewMesh)
{
	RefreshCurveList(true);
}

void SAnimCurveViewer::OnFilterTextChanged( const FText& SearchText )
{
	FilterText = SearchText;

	RefreshCurveList(false);
}

void SAnimCurveViewer::OnFilterTextCommitted( const FText& SearchText, ETextCommit::Type CommitInfo )
{
	// Just do the same as if the user typed in the box
	OnFilterTextChanged( SearchText );
}

TSharedRef<ITableRow> SAnimCurveViewer::GenerateAnimCurveRow(TSharedPtr<FDisplayedAnimCurveInfo> InInfo, const TSharedRef<STableViewBase>& OwnerTable)
{
	check( InInfo.IsValid() );

	return
		SNew( SAnimCurveListRow, OwnerTable, PreviewScenePtr.Pin().ToSharedRef() )
		.Item( InInfo )
		.AnimCurveViewerPtr( SharedThis(this) );
}

void SAnimCurveViewer::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (GetAnimInstance())
	{
		// We refresh when ticking each time as curve flags can potentially vary
		RefreshCurveList(false);
	}
}

FReply SAnimCurveViewer::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (UICommandList.IsValid() && UICommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

UAnimInstance* SAnimCurveViewer::GetAnimInstance() const
{
	UDebugSkelMeshComponent* MeshComponent = PreviewScenePtr.Pin()->GetPreviewMeshComponent();
	UAnimInstance* AnimInstance = MeshComponent->GetAnimInstance();
	if (AnimInstance)
	{
		// Look at the debugged anim instance if we are targeting one
		if(AnimInstance == MeshComponent->PreviewInstance.Get())
		{
			UAnimPreviewInstance* AnimPreviewInstance = MeshComponent->PreviewInstance;
			if(USkeletalMeshComponent* DebuggedComponent = AnimPreviewInstance->GetDebugSkeletalMeshComponent())
			{
				AnimInstance = DebuggedComponent->GetAnimInstance();
			}
		}
	}

	return AnimInstance;
}

void SAnimCurveViewer::CreateAnimCurveList( const FString& SearchText, bool bInFullRefresh )
{
	if(!AnimCurveListView.IsValid())
	{
		return;
	}
	
	bool bDirty = bInFullRefresh;

	AnimCurveList.Reset();

	auto AddCurve = [this](FName InCurveName, UE::Anim::ECurveElementFlags InFlags)
	{
		// Only add if the curve doesnt exist
		TSharedPtr<FDisplayedAnimCurveInfo>* ExistingItem = AllSeenAnimCurvesMap.Find(InCurveName);
		if(ExistingItem == nullptr)
		{
			TSharedRef<FDisplayedAnimCurveInfo> NewInfo = FDisplayedAnimCurveInfo::Make(InCurveName);

			float Weight = 0.f;
			const bool bOverride = GetAnimCurveOverride(InCurveName, Weight);
			NewInfo->bOverrideData = bOverride;
			NewInfo->Weight = Weight;
			NewInfo->bMaterial = EnumHasAnyFlags(InFlags, UE::Anim::ECurveElementFlags::Material);
			NewInfo->bMorphTarget = EnumHasAnyFlags(InFlags, UE::Anim::ECurveElementFlags::MorphTarget);

			AllSeenAnimCurvesMap.Add(InCurveName, NewInfo);
		}
		else
		{
			(*ExistingItem)->bMaterial = EnumHasAnyFlags(InFlags, UE::Anim::ECurveElementFlags::Material);
			(*ExistingItem)->bMorphTarget = EnumHasAnyFlags(InFlags, UE::Anim::ECurveElementFlags::MorphTarget);
		}
	};

	// Add curve items from skeleton metadata
	if(TSharedPtr<IEditableSkeleton> EditableSkeleton = EditableSkeletonPtr.Pin())
	{
		EditableSkeleton->GetSkeleton().ForEachCurveMetaData([&AddCurve](FName InCurveName, const FCurveMetaData& InCurveMetaData)
		{
			UE::Anim::ECurveElementFlags Flags = UE::Anim::ECurveElementFlags::None;
			if(InCurveMetaData.Type.bMaterial)
			{
				Flags |= UE::Anim::ECurveElementFlags::Material; 
			}
			if(InCurveMetaData.Type.bMorphtarget)
			{
				Flags |= UE::Anim::ECurveElementFlags::MorphTarget; 
			}
			AddCurve(InCurveName, Flags);
		});
	}

	// Add active curves if required
	TSet<FName> ActiveCurves;
	const UAnimInstance* AnimInstance = GetAnimInstance();

	if(AnimInstance)
	{
		bool bIsPoseWatchActive = false;
		if (PoseWatchPicker.IsValid())
		{
			// Find if we want to use a pose watch
			if (UPoseWatchPoseElement* PoseWatchPoseElement = PoseWatchPicker->GetCurrentPoseWatch())
			{
				bIsPoseWatchActive = true;
				if (UAnimBlueprintGeneratedClass* AnimClass = Cast<UAnimBlueprintGeneratedClass>(AnimInstance->GetClass()))
				{
					// We have to grab our pose watches from the root class as no pose watches can be set on child anim BPs
					if (const UAnimBlueprintGeneratedClass* RootClass = Cast<UAnimBlueprintGeneratedClass>(AnimClass->GetRootClass()))
					{
						const FAnimBlueprintDebugData& DebugData = RootClass->AnimBlueprintDebugData;
						for (const FAnimNodePoseWatch& AnimNodePoseWatch : DebugData.AnimNodePoseWatch)
						{
							if (AnimNodePoseWatch.PoseWatchPoseElement == PoseWatchPoseElement)
							{
								AnimNodePoseWatch.GetCurves().ForEachElement([this, &AddCurve, &ActiveCurves](const UE::Anim::FCurveElement& InElement)
									{
										if (EnumHasAnyFlags(CurrentCurveFlag, EAnimCurveViewerFilterFlags::MorphTarget))
										{
											if (EnumHasAnyFlags(InElement.Flags, UE::Anim::ECurveElementFlags::MorphTarget))
											{
												AddCurve(InElement.Name, InElement.Flags);
												ActiveCurves.Add(InElement.Name);
											}
										}
										if (EnumHasAnyFlags(CurrentCurveFlag, EAnimCurveViewerFilterFlags::Material))
										{
											if (EnumHasAnyFlags(InElement.Flags, UE::Anim::ECurveElementFlags::Material))
											{
												AddCurve(InElement.Name, InElement.Flags);
												ActiveCurves.Add(InElement.Name);
											}
										}

										// If we arent filtering by curve type, just show all curves
										if (!EnumHasAnyFlags(CurrentCurveFlag, EAnimCurveViewerFilterFlags::MorphTarget | EAnimCurveViewerFilterFlags::Material))
										{
											AddCurve(InElement.Name, InElement.Flags);
											ActiveCurves.Add(InElement.Name);
										}
									});
								break;
							}
						}
					}
				}
			}
		}

		// If there isn't an active pose watch, grab the curve list directly from the anim instance
		if (!bIsPoseWatchActive)
		{
			if (EnumHasAnyFlags(CurrentCurveFlag, EAnimCurveViewerFilterFlags::MorphTarget))
			{
				for (const TPair<FName, float>& CurveValuePair : AnimInstance->GetAnimationCurveList(EAnimCurveType::MorphTargetCurve))
				{
					AddCurve(CurveValuePair.Key, UE::Anim::ECurveElementFlags::MorphTarget);
					ActiveCurves.Add(CurveValuePair.Key);
				}
			}
			if (EnumHasAnyFlags(CurrentCurveFlag, EAnimCurveViewerFilterFlags::Material))
			{
				for (const TPair<FName, float>& CurveValuePair : AnimInstance->GetAnimationCurveList(EAnimCurveType::MaterialCurve))
				{
					AddCurve(CurveValuePair.Key, UE::Anim::ECurveElementFlags::Material);
					ActiveCurves.Add(CurveValuePair.Key);
				}
			}

			// If we arent filtering by curve type, show 'attribute' curves
			if (!EnumHasAnyFlags(CurrentCurveFlag, EAnimCurveViewerFilterFlags::MorphTarget | EAnimCurveViewerFilterFlags::Material))
			{
				for (const TPair<FName, float>& CurveValuePair : AnimInstance->GetAnimationCurveList(EAnimCurveType::AttributeCurve))
				{
					AddCurve(CurveValuePair.Key, UE::Anim::ECurveElementFlags::None);
					ActiveCurves.Add(CurveValuePair.Key);
				}
			}
		}
	}
	
	// Iterate through all curves that have been seen
	for (const TPair<FName, TSharedPtr<FDisplayedAnimCurveInfo>>& CurveNameValuePair : AllSeenAnimCurvesMap)
	{
		bool bAddToList = true;

		// See if we pass the search filter
		if (!FilterText.IsEmpty())
		{
			if (!CurveNameValuePair.Key.ToString().Contains(*FilterText.ToString()))
			{
				bAddToList = false;
			}
		}

		// Look at whether we need to filter out any curves that aren't morphs or materials
		if (bAddToList)
		{
			if ((EnumHasAnyFlags(CurrentCurveFlag, EAnimCurveViewerFilterFlags::MorphTarget) && !CurveNameValuePair.Value->bMorphTarget)
				|| (EnumHasAnyFlags(CurrentCurveFlag, EAnimCurveViewerFilterFlags::Material) && !CurveNameValuePair.Value->bMaterial))
			{
				bAddToList = false;
			}
		}

		// If we passed that, see if we are filtering to only active
		if (bAddToList && EnumHasAnyFlags(CurrentCurveFlag, EAnimCurveViewerFilterFlags::Active))
		{
			bAddToList = ActiveCurves.Contains(CurveNameValuePair.Key);
		}

		if(CurveNameValuePair.Value->bShown != bAddToList)
		{
			CurveNameValuePair.Value->bShown = bAddToList;
			bDirty = true;
		}
		
		// If we still want to add
		if (bAddToList)
		{
			AnimCurveList.Add(CurveNameValuePair.Value);
		}
	}

	if (bDirty)
	{
		// Sort final list
		struct FSortSmartNamesAlphabetically
		{
			bool operator()(const TSharedPtr<FDisplayedAnimCurveInfo>& A, const TSharedPtr<FDisplayedAnimCurveInfo>& B) const
			{
				return (A.Get()->CurveName.Compare(B.Get()->CurveName) < 0);
			}
		};

		AnimCurveList.Sort(FSortSmartNamesAlphabetically());

		AnimCurveListView->RequestListRefresh();
	}
}

TSharedRef<SWidget> SAnimCurveViewer::CreateCurveSourceSelector()
{
	bool bShowPoseWatches = false;
	if(const UAnimInstance* AnimInstance = GetAnimInstance())
	{
		if(UAnimBlueprintGeneratedClass* AnimClass = Cast<UAnimBlueprintGeneratedClass>(AnimInstance->GetClass()))
		{
			bShowPoseWatches = true;
		}
	}

	if(bShowPoseWatches)
	{
		PoseWatchPicker = SNew(SPoseWatchPicker)
		.AnimBlueprintGeneratedClass_Lambda([this]() -> const UAnimBlueprintGeneratedClass*
		{
			if(const UAnimInstance* AnimInstance = GetAnimInstance())
			{
				return Cast<UAnimBlueprintGeneratedClass>(AnimInstance->GetClass());
			}
			
			return nullptr;
		})
		.DefaultEntryDisplayText(LOCTEXT("PoseWatchOutputCurves", "Output Curves"));

		return PoseWatchPicker.ToSharedRef();
	}

	return SNullWidget::NullWidget;
}

void SAnimCurveViewer::AddAnimCurveOverride( FName& Name, float Weight)
{
	float& Value = OverrideCurves.FindOrAdd(Name);
	Value = Weight;

	UAnimSingleNodeInstance* SingleNodeInstance = Cast<UAnimSingleNodeInstance>(GetAnimInstance());
	if (SingleNodeInstance)
	{
		SingleNodeInstance->SetPreviewCurveOverride(Name, Value, false);
	}
}

void SAnimCurveViewer::RemoveAnimCurveOverride(FName& Name)
{
	OverrideCurves.Remove(Name);

	UAnimSingleNodeInstance* SingleNodeInstance = Cast<UAnimSingleNodeInstance>(GetAnimInstance());
	if (SingleNodeInstance)
	{
		SingleNodeInstance->SetPreviewCurveOverride(Name, 0.f, true);
	}
}

bool SAnimCurveViewer::GetAnimCurveOverride(FName& Name, float& Weight)
{
	Weight = 0.f;
	float* WeightPtr = OverrideCurves.Find(Name);
	if (WeightPtr)
	{
		Weight = *WeightPtr;
		return true;
	}
	else
	{
		return false;
	}
}

void SAnimCurveViewer::PostUndoRedo()
{
	RefreshCurveList(true);
}

void SAnimCurveViewer::OnPreviewAssetChanged(class UAnimationAsset* NewAsset)
{
	OverrideCurves.Empty();
	RefreshCurveList(true);
}

void SAnimCurveViewer::ApplyCustomCurveOverride(UAnimInstance* AnimInstance) const
{
	for (auto Iter = OverrideCurves.CreateConstIterator(); Iter; ++Iter)
	{ 
		// @todo we might want to save original curve flags? or just change curve to apply flags only
		AnimInstance->AddCurveValue(Iter.Key(), Iter.Value());
	}
}

void SAnimCurveViewer::RefreshCurveList(bool bInFullRefresh)
{
	CreateAnimCurveList(FilterText.ToString(), bInFullRefresh);
}

void SAnimCurveViewer::HandleCurveMetaDataChange()
{
	AnimCurveList.Empty();
	RefreshCurveList(true);
}

void SAnimCurveViewer::BindCommands()
{
	// This should not be called twice on the same instance
	check(!UICommandList.IsValid());

	UICommandList = MakeShared<FUICommandList>();

	FUICommandList& CommandList = *UICommandList;

	// Grab the list of menu commands to bind...
	const FCurveViewerCommands& MenuActions = FCurveViewerCommands::Get();

	// ...and bind them all
	CommandList.MapAction(
		MenuActions.FindCurveUses,
		FExecuteAction::CreateSP(this, &SAnimCurveViewer::OnFindCurveUsesClicked),
		FCanExecuteAction::CreateSP(this, &SAnimCurveViewer::CanFindCurveUses));
}

TSharedPtr<SWidget> SAnimCurveViewer::OnGetContextMenuContent() const
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder( bShouldCloseWindowAfterMenuSelection, UICommandList);

	const FCurveViewerCommands& Actions = FCurveViewerCommands::Get();

	MenuBuilder.BeginSection("AnimCurveAction", LOCTEXT( "CurveAction", "Curve Actions" ) );
	MenuBuilder.AddMenuEntry(Actions.FindCurveUses);
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void SAnimCurveViewer::OnFindCurveUsesClicked()
{
	FindReplaceCurves();
}

bool SAnimCurveViewer::CanFindCurveUses()
{
	return AnimCurveListView->GetNumItemsSelected() == 1;
}

void SAnimCurveViewer::FindReplaceCurves()
{
	FName CurveName = NAME_None;
	bool bMorphTarget = false;
	bool bMaterial = false;
	TArray<TSharedPtr<FDisplayedAnimCurveInfo>> SelectedItems = AnimCurveListView->GetSelectedItems();
	if(SelectedItems.Num() > 0)
	{
		CurveName = SelectedItems[0]->CurveName;
		bMorphTarget = SelectedItems[0]->GetActiveFlag(SharedThis(this), true);
		bMaterial = SelectedItems[0]->GetActiveFlag(SharedThis(this), false);
	}

	if(TSharedPtr<SDockTab> ActiveTab = FGlobalTabmanager::Get()->GetActiveTab())
	{
		if(TSharedPtr<FTabManager> TabManager = ActiveTab->GetTabManagerPtr())
		{
			if(TSharedPtr<SDockTab> Tab = TabManager->TryInvokeTab(FPersonaTabs::FindReplaceID))
			{
				TSharedRef<IAnimAssetFindReplace> FindReplaceWidget = StaticCastSharedRef<IAnimAssetFindReplace>(Tab->GetContent());
				FindReplaceWidget->SetCurrentProcessor(UAnimAssetFindReplaceCurves::StaticClass());
				if(CurveName != NAME_None)
				{
					if(UAnimAssetFindReplaceCurves* Processor = FindReplaceWidget->GetProcessor<UAnimAssetFindReplaceCurves>())
					{
						Processor->SetFindString(CurveName.ToString());
						Processor->SetFindWholeWord(true);
						Processor->SetSearchMaterials(bMaterial);
						Processor->SetSearchMorphTargets(bMorphTarget);
					}
				}
			}
		}
	}
}


#undef LOCTEXT_NAMESPACE

