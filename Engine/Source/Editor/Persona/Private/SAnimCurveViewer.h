// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "Styling/SlateColor.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SBoxPanel.h"
#include "Animation/SmartName.h"
#include "IPersonaPreviewScene.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"
#include "Animation/AnimInstance.h"
#include "PersonaDelegates.h"
#include "EditorUndoClient.h"
#include "Filters/FilterBase.h"
#include "Widgets/Input/SComboBox.h"

class SPoseWatchPicker;

namespace UE::Anim
{
enum class ECurveElementFlags : uint8;
}

class FUICommandList;
class IEditableSkeleton;
class SAnimCurveViewer;
class UEditorAnimCurveBoneLinks;
class UPoseWatchPoseElement;
class UAnimBlueprint;
class UEdGraphNode;
enum class EAnimCurveViewerFilterFlags: uint8;

//////////////////////////////////////////////////////////////////////////
// FDisplayedAnimCurveInfo

class FDisplayedAnimCurveInfo
{
public:
	FName CurveName;
	float Weight;
	bool bOverrideData;
	bool bShown;
	bool bMorphTarget;
	bool bMaterial;
	
	/** Static function for creating a new item, but ensures that you can only have a TSharedRef to one */
	static TSharedRef<FDisplayedAnimCurveInfo> Make(const FName& InCurveName)
	{
		return MakeShareable(new FDisplayedAnimCurveInfo(InCurveName));
	}

	// Get the active morph/material flag for this curve 
	bool GetActiveFlag(const TSharedPtr<SAnimCurveViewer>& InAnimCurveViewer, bool bMorphTarget) const;

protected:
	/** Hidden constructor, always use Make above */
	FDisplayedAnimCurveInfo(const FName& InCurveName)
		: CurveName(InCurveName)
		, Weight(0.0f)
		, bOverrideData(false)
		, bShown(false)
		, bMorphTarget(false)
		, bMaterial(false)
	{}
};

typedef SListView< TSharedPtr<FDisplayedAnimCurveInfo> > SAnimCurveListType;

//////////////////////////////////////////////////////////////////////////
// SAnimCurveListRow

class SAnimCurveListRow : public SMultiColumnTableRow< TSharedPtr<FDisplayedAnimCurveInfo> >
{
public:

	SLATE_BEGIN_ARGS(SAnimCurveListRow) {}

		/** The item for this row **/
		SLATE_ARGUMENT(TSharedPtr<FDisplayedAnimCurveInfo>, Item)

		/* The SAnimCurveViewer that we push the morph target weights into */
		SLATE_ARGUMENT(class TWeakPtr<SAnimCurveViewer>, AnimCurveViewerPtr)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTableView, const TSharedRef<class IPersonaPreviewScene>& InPreviewScene);

	/** Overridden from SMultiColumnTableRow.  Generates a widget for this column of the tree row. */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

private:

	/**
	* Called when the user changes the value of the SSpinBox
	*
	* @param NewWeight - The new number the SSpinBox is set to
	*
	*/
	void OnAnimCurveWeightChanged(float NewWeight);
	/**
	* Called when the user types the value and enters
	*
	* @param NewWeight - The new number the SSpinBox is set to
	*
	*/
	void OnAnimCurveWeightValueCommitted(float NewWeight, ETextCommit::Type CommitType);

	/** Override check call back functions */
	void OnAnimCurveOverrideChecked(ECheckBoxState InState);
	ECheckBoxState IsAnimCurveOverrideChecked() const;

	/** Check for an active morph target or material flag */
	bool GetActiveFlag(bool bMorphTarget) const;
	
	/* Curve Flag checks for morphtarget or material */
	ECheckBoxState IsAnimCurveTypeBoxChangedChecked(bool bMorphTarget) const;

	/** Returns the weight of this curve */
	float GetWeight() const;
	/** Returns the min seen weight of this curve */
	TOptional<float> GetMinWeight() const;
	/** Returns the max seen weight of this curve */
	TOptional<float> GetMaxWeight() const;
	/** Returns name of this curve */
	FText GetItemName() const;
	/** Get text we are filtering for */
	FText GetFilterText() const;
	/** Return font for text of item */
	FSlateFontInfo GetItemFont() const;

	/** Get current active weight. Returns false if not currently active */
	bool GetActiveWeight(float& OutWeight) const;

	/** If the values of the curve represented by this row can be edited */
	bool CanBeEdited() const;

	/* The SAnimCurveViewer that we push the morph target weights into */
	TWeakPtr<SAnimCurveViewer> AnimCurveViewerPtr;

	/** The name and weight of the morph target */
	TSharedPtr<FDisplayedAnimCurveInfo>	Item;
	
	/** Preview scene used to update on scrub */
	TWeakPtr<class IPersonaPreviewScene> PreviewScenePtr;

	/** The min weight we have seen */
	mutable float MinWeight = -1.0f;

	/** The max weight we have seen */
	mutable float MaxWeight = 1.0f;
	
	/** Returns curve type widget constructed */
	TSharedRef<SWidget> GetCurveTypeWidget();
};

//////////////////////////////////////////////////////////////////////////
// SAnimCurveViewer

class SAnimCurveViewer : public SCompoundWidget, public FSelfRegisteringEditorUndoClient
{
public:
	SLATE_BEGIN_ARGS( SAnimCurveViewer ) {}

	SLATE_ARGUMENT(TSharedPtr<IEditableSkeleton>, EditableSkeleton)
	SLATE_ARGUMENT(bool, bIsReadOnly)
	SLATE_END_ARGS()

	/**
	* Slate construction function
	*
	* @param InArgs - Arguments passed from Slate
	*
	*/
	void Construct( const FArguments& InArgs, const TSharedRef<class IPersonaPreviewScene>& InPreviewScene, FOnObjectsSelected InOnObjectsSelected);

	/**
	* Destructor - resets the animation curve
	*
	*/
	virtual ~SAnimCurveViewer();
	
	/**
	* Is registered with Persona to handle when its preview mesh is changed.
	*
	* @param NewPreviewMesh - The new preview mesh being used by Persona
	*
	*/
	void OnPreviewMeshChanged(class USkeletalMesh* OldPreviewMesh, class USkeletalMesh* NewPreviewMesh);

	/**
	* Is registered with Persona to handle when its preview asset is changed.
	*
	* Pose Asset will have to add curve manually
	*/
	void OnPreviewAssetChanged(class UAnimationAsset* NewPreviewAsset);

	/**
	* Filters the SListView when the user changes the search text box (NameFilterBox)
	*
	* @param SearchText - The text the user has typed
	*
	*/
	void OnFilterTextChanged( const FText& SearchText );

	/**
	* Filters the SListView when the user hits enter or clears the search box
	* Simply calls OnFilterTextChanged
	*
	* @param SearchText - The text the user has typed
	* @param CommitInfo - Not used
	*
	*/
	void OnFilterTextCommitted( const FText& SearchText, ETextCommit::Type CommitInfo );

	/**
	* Create a widget for an entry in the tree from an info
	*
	* @param InInfo - Shared pointer to the morph target we're generating a row for
	* @param OwnerTable - The table that owns this row
	*
	* @return A new Slate widget, containing the UI for this row
	*/
	TSharedRef<ITableRow> GenerateAnimCurveRow(TSharedPtr<FDisplayedAnimCurveInfo> InInfo, const TSharedRef<STableViewBase>& OwnerTable);

	/**
	* Adds a curve override or updates the weight for an existing one
	*
	* @param Name - Name of the curve we want to override
	* @param Weight - How much of this curve to apply (0.0 - 1.0)
	*
	*/
	void AddAnimCurveOverride( FName& Name, float Weight);

	/** Removes an existing curve override */
	void RemoveAnimCurveOverride(FName& Name);

	/** Is there currently an override for this curve, and if so, what is its weight */
	bool GetAnimCurveOverride(FName& Name, float& Weight);

	/**
	* Accessor so our rows can grab the filtertext for highlighting
	*
	*/
	FText& GetFilterText() { return FilterText; }

	// FSelfRegisteringEditorUndoClient interface
	virtual void PostUndo(bool bSuccess) override { PostUndoRedo(); }
	virtual void PostRedo(bool bSuccess) override { PostUndoRedo(); }

	/**
	 * Refreshes the morph target list after an undo
	 */
	void PostUndoRedo();

	/** SWidget interface */
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

	void RefreshCurveList(bool bInFullRefresh);

private:

	/**
	* Clears and rebuilds the table, according to an optional search string
	*
	* @param SearchText - Optional search string
	*
	*/
	void CreateAnimCurveList( const FString& SearchText = FString(), bool bInFullRefresh = false );

	void ApplyCustomCurveOverride(UAnimInstance* AnimInstance) const;

	/** Handle curve meta data removal */
	void HandleCurveMetaDataChange();

	/** Get the anim instance we are viewing */
	UAnimInstance* GetAnimInstance() const;

	TSharedRef<SWidget> CreateCurveSourceSelector();
	
	void BindCommands();
	
	/** Build context menu */
	TSharedPtr<SWidget> OnGetContextMenuContent() const;
	
	void OnFindCurveUsesClicked();
	bool CanFindCurveUses();
	void FindReplaceCurves();
	
	/** Pointer to the preview scene we are bound to */
	TWeakPtr<class IPersonaPreviewScene> PreviewScenePtr;

	/** Pointer to the editable skeleton */
	TWeakPtr<class IEditableSkeleton> EditableSkeletonPtr;

	/** Box to filter to a specific morph target name */
	TSharedPtr<SSearchBox>	NameFilterBox;

	/** A list of animation curve. Used by the AnimCurveListView. */
	TArray< TSharedPtr<FDisplayedAnimCurveInfo> > AnimCurveList;

	/** Tracking map of anim curves */
	TMap< FName, TSharedPtr<FDisplayedAnimCurveInfo> > AllSeenAnimCurvesMap;

	/** Widget used to display the list of animation curve */
	TSharedPtr<SAnimCurveListType> AnimCurveListView;

	/** Current text typed into NameFilterBox */
	FText FilterText;

	EAnimCurveViewerFilterFlags CurrentCurveFlag;
	
	/** Whatever this viewer is read only or allows editing of its values */
	bool bIsReadOnlyViewer = false;

	TMap<FName, float> OverrideCurves;

	/** Commands that are bound to delegates*/
	TSharedPtr<FUICommandList> UICommandList;

	friend class SAnimCurveListRow;
	friend class SAnimCurveTypeList;
	friend class FDisplayedAnimCurveInfo;

	/** Delegate called to select objects */
	FOnObjectsSelected OnObjectsSelected;

	/** Names of each pose watch that can be selected */
	TArray<TSharedPtr<TWeakObjectPtr<UPoseWatchPoseElement>>> PoseWatches;
	
	/** Combobox used to select pose watch */
	TSharedPtr<SComboBox<TSharedPtr<TWeakObjectPtr<UPoseWatchPoseElement>>>> PoseWatchCombo;

	TSharedPtr<SPoseWatchPicker> PoseWatchPicker;

	/** All filters that can be applied to the widget's display */
	TArray<TSharedRef<FFilterBase<EAnimCurveViewerFilterFlags>>> Filters;
};