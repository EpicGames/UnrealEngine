// Copyright Epic Games, Inc. All Rights Reserved.


#include "SAnimEditorBase.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "Animation/EditorAnimBaseObj.h"
#include "Animation/AnimCompositeBase.h"
#include "IDocumentation.h"

#include "AnimPreviewInstance.h"
#include "Animation/BlendSpace.h"
#include "AnimTimeline/AnimModel.h"
#include "AnimTimeline/SAnimTimeline.h"

#define LOCTEXT_NAMESPACE "AnimEditorBase"

//////////////////////////////////////////////////////////////////////////
// SAnimEditorBase

TSharedRef<SWidget> SAnimEditorBase::CreateDocumentAnchor()
{
	return IDocumentation::Get()->CreateAnchor(TEXT("Engine/Animation/Sequences"));
}

void SAnimEditorBase::Construct(const FArguments& InArgs, const TSharedPtr<class IPersonaPreviewScene>& InPreviewScene)
{
	PreviewScenePtr = InPreviewScene;
	OnObjectsSelected = InArgs._OnObjectsSelected;

	SetInputViewRange(0, GetSequenceLength());

	TSharedPtr<SVerticalBox> AnimVerticalBox;

	TSharedPtr<SWidget> TimelineToUse;
	if (InArgs._DisplayAnimTimeline)
	{
		check(InArgs._AnimModel.IsValid());
		TimelineToUse = SAssignNew(TimelineWidget, SAnimTimeline, InArgs._AnimModel.ToSharedRef());
	}
	else
	{
		TimelineToUse = SNullWidget::NullWidget;
	}

	ChildSlot
	[
		SAssignNew(AnimVerticalBox, SVerticalBox)
		+SVerticalBox::Slot()
		.FillHeight(1)
		[
			SNew(SOverlay)
			+SOverlay::Slot()
			[
				SAssignNew(NonScrollEditorPanels, SVerticalBox)
			]
			+SOverlay::Slot()
			[
				TimelineToUse.ToSharedRef()
			]
		]
	];

	if (InArgs._DisplayAnimScrubBar)
	{
		// If we are an anim sequence, add scrub panel as well
		AnimVerticalBox->AddSlot()
		.AutoHeight()
		.VAlign(VAlign_Bottom)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.FillWidth(1)
			[
				ConstructAnimScrubPanel(InArgs._DisplayAnimScrubBarEditing)
			]
		];
	}
}

TSharedRef<SWidget> SAnimEditorBase::ConstructAnimScrubPanel(bool bDisplayAnimScrubBarEditing)
{
	if(PreviewScenePtr.IsValid())
	{
		return SAssignNew( AnimScrubPanel, SAnimationScrubPanel, PreviewScenePtr.Pin().ToSharedRef() )
			.LockedSequence(Cast<UAnimSequenceBase>(GetEditorObject()))
			.ViewInputMin(this, &SAnimEditorBase::GetViewMinInput)
			.ViewInputMax(this, &SAnimEditorBase::GetViewMaxInput)
			.bDisplayAnimScrubBarEditing(bDisplayAnimScrubBarEditing)
			.OnSetInputViewRange(this, &SAnimEditorBase::SetInputViewRange)
			.bAllowZoom(true);
	}

	return SNullWidget::NullWidget;
}

void SAnimEditorBase::AddReferencedObjects( FReferenceCollector& Collector )
{
	EditorObjectTracker.AddReferencedObjects(Collector);
}

UObject* SAnimEditorBase::ShowInDetailsView( UClass* EdClass )
{
	check(GetEditorObject()!=NULL);

	UObject *Obj = EditorObjectTracker.GetEditorObjectForClass(EdClass);

	if(Obj != NULL)
	{
		if(Obj->IsA(UEditorAnimBaseObj::StaticClass()))
		{
			UEditorAnimBaseObj *EdObj = Cast<UEditorAnimBaseObj>(Obj);
			InitDetailsViewEditorObject(EdObj);

			TArray<UObject*> Objects;
			Objects.Add(EdObj);
			OnObjectsSelected.ExecuteIfBound(Objects);
		}
	}
	return Obj;
}

void SAnimEditorBase::ClearDetailsView()
{
	TArray<UObject*> Objects;
	OnObjectsSelected.ExecuteIfBound(Objects);
}

FText SAnimEditorBase::GetEditorObjectName() const
{
	if (GetEditorObject() != NULL)
	{
		return FText::FromString(GetEditorObject()->GetName());
	}
	else
	{
		return LOCTEXT("NoEditorObject", "No Editor Object");
	}
}

void SAnimEditorBase::OnSelectionChanged(const TArray<UObject*>& SelectedItems)
{
	OnObjectsSelected.ExecuteIfBound(SelectedItems);
}

class UAnimSingleNodeInstance* SAnimEditorBase::GetPreviewInstance() const
{
	return (GetPreviewScene()->GetPreviewMeshComponent()) ? GetPreviewScene()->GetPreviewMeshComponent()->PreviewInstance : nullptr;
}

float SAnimEditorBase::GetScrubValue() const
{
	UAnimSingleNodeInstance * PreviewInstance = GetPreviewInstance();
	if (PreviewInstance)
	{
		float CurTime = PreviewInstance->GetCurrentTime();
		return (CurTime); 
	}
	else
	{
		return 0.f;
	}
}

void SAnimEditorBase::SetInputViewRange(float InViewMinInput, float InViewMaxInput)
{
	ViewMaxInput = FMath::Min<float>(InViewMaxInput, GetSequenceLength());
	ViewMinInput = FMath::Max<float>(InViewMinInput, 0.f);
}

float SAnimEditorBase::GetSequenceLength() const
{
	if (UAnimSequenceBase* AnimSeqBase = Cast<UAnimSequenceBase>(GetEditorObject()))
	{
		return AnimSeqBase->GetPlayLength();
	}
	else if (UBlendSpace* BlendSpaceBase = Cast<UBlendSpace>(GetEditorObject()))
	{
		// Blendspaces use normalized time, so we just return 1 here
		return 1.0f;
	}
	
	return 0.f;
}
#undef LOCTEXT_NAMESPACE
