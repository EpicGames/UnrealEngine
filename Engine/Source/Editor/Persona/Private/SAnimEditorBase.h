// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "UObject/GCObject.h"
#include "IPersonaPreviewScene.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "PersonaDelegates.h"
#include "SAnimationScrubPanel.h"
#include "EditorObjectsTracker.h"

class FAnimModel;
class SAnimTimeline;

//////////////////////////////////////////////////////////////////////////
// FTrackMarkerBar

// Simple collection class to store data about a track marker (section, branch point etc)
class FTrackMarkerBar
{
public:
	float Time;
	FLinearColor DrawColour;
};

//////////////////////////////////////////////////////////////////////////
// SAnimEditorBase

/** Common functionality for all anim editor widgets */
class SAnimEditorBase : public SCompoundWidget, public FGCObject
{
public:
	SLATE_BEGIN_ARGS(SAnimEditorBase)
		: _DisplayAnimTimeline(true)
		, _DisplayAnimScrubBar(false)
	{}

	SLATE_EVENT(FOnObjectsSelected, OnObjectsSelected)

	SLATE_ARGUMENT( bool, DisplayAnimTimeline )

	SLATE_ARGUMENT( bool, DisplayAnimScrubBar )

	SLATE_ARGUMENT( bool, DisplayAnimScrubBarEditing )

	SLATE_ARGUMENT( TSharedPtr<FAnimModel>, AnimModel )

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedPtr<class IPersonaPreviewScene>& InPreviewScene);

	/** Accessors to the current viewed Min/Max input range of the editor */
	float GetViewMinInput() const { return ViewMinInput; }
	float GetViewMaxInput() const { return ViewMaxInput; }

	/** Accessors to the total Min/Max input range of the current object being edited */
	float GetMinInput() const		{ return 0.f; }
	float GetMaxInput() const		{ return GetSequenceLength(); }

	void SetInputViewRange(float InViewMinInput, float InViewMaxInput);

	/** Scrolls the input view based on the supplied ScreenDelta */
	void ScrollInputViewRange(float ScreenDelta, FVector2D ScreenViewSize);

	/** Delegate when selection changed **/
	void OnSelectionChanged(const TArray<UObject*>& SelectedItems);

	/** Get Current Scrub Value **/
	float GetScrubValue() const;

	/** Creates an editor object from the given type to be used in a details panel */
	UObject* ShowInDetailsView( UClass* EdClass );

	/** Clears the detail view of whatever we displayed last */
	void ClearDetailsView();

	// FGCObject interface start
	virtual void AddReferencedObjects( FReferenceCollector& Collector ) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("SAnimEditorBase");
	}
	// FGCObject interface end

	/** Creates a anchor widget for each animation editor type */
	virtual TSharedRef<SWidget> CreateDocumentAnchor();

protected:
	/** Preview scene reference **/
	TWeakPtr<IPersonaPreviewScene> PreviewScenePtr;

	TSharedRef<IPersonaPreviewScene> GetPreviewScene() const { return PreviewScenePtr.Pin().ToSharedRef(); }

	/** Allows derived classes to create different animation scrub panel */
	virtual TSharedRef<SWidget> ConstructAnimScrubPanel(bool bDisplayAnimScrubBarEditing);

	/** Allows derived classes to init newly created editor objects */
	virtual void InitDetailsViewEditorObject(class UEditorAnimBaseObj* EdObj) {};

	/** Get the length of the current sequence */
	float GetSequenceLength() const;

	/** Get the sequence that is currently being edited */
	virtual UAnimationAsset* GetEditorObject() const = 0;

	/** Get Name of Object being edited **/
	FText GetEditorObjectName() const;

	/** Get Min/Max Input of value **/
	float ViewMinInput;
	float ViewMaxInput;

	/** To get scrub value, get preview instance **/
	class UAnimSingleNodeInstance* GetPreviewInstance() const;

	/** The slate container that the editor panels are placed in */
	TSharedPtr<SVerticalBox>	 EditorPanels;

	/** Slate container used to add controls that are not embedded in a scroll box */
	TSharedPtr<SVerticalBox> NonScrollEditorPanels;

	/** The editors Animation Scrub Panel */
	TSharedPtr<class SAnimationScrubPanel> AnimScrubPanel;

	/** Tracks objects created for the details panel */
	FEditorObjectTracker EditorObjectTracker;

	/** Delegate called to select objects */
	FOnObjectsSelected OnObjectsSelected;

	/** The timeline widget */
	TSharedPtr<SAnimTimeline> TimelineWidget;

private:
	float GetPercentageInternal() const;
};
