// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAnimMontagePanel.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SCheckBox.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Animation/EditorAnimSegment.h"
#include "SAnimSegmentsPanel.h"
#include "Widgets/Input/STextEntryPopup.h"
#include "Widgets/Input/STextComboBox.h"
#include "SAnimTimingPanel.h"
#include "TabSpawners.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Styling/CoreStyle.h"
#include "IEditableSkeleton.h"
#include "Editor.h"
#include "AnimTimeline/AnimModel.h"
#include "AnimPreviewInstance.h"
#include "ScopedTransaction.h"
#include "Animation/AnimationSettings.h"
#include "Misc/MessageDialog.h"
#include "Animation/EditorAnimCompositeSegment.h"
#include "Factories/AnimMontageFactory.h"
#include "Animation/EditorCompositeSection.h"
#include "AnimTimeline/AnimModel_AnimMontage.h"
#include "Subsystems/AssetEditorSubsystem.h"

#define LOCTEXT_NAMESPACE "AnimMontagePanel"

//////////////////////////////////////////////////////////////////////////
// SAnimMontagePanel

void SAnimMontagePanel::Construct(const FArguments& InArgs, const TSharedRef<FAnimModel_AnimMontage>& InModel)
{
	SAnimTrackPanel::Construct( SAnimTrackPanel::FArguments()
		.WidgetWidth(InArgs._WidgetWidth)
		.ViewInputMin(InArgs._ViewInputMin)
		.ViewInputMax(InArgs._ViewInputMax)
		.InputMin(InArgs._InputMin)
		.InputMax(InArgs._InputMax)
		.OnSetInputViewRange(InArgs._OnSetInputViewRange));

	WeakModel = InModel;
	Montage = InArgs._Montage;
	OnInvokeTab = InArgs._OnInvokeTab;
	SectionTimingNodeVisibility = InArgs._SectionTimingNodeVisibility;
	CurrentPreviewSlot = 0;
	bIsSelecting = false;

	InModel->OnHandleObjectsSelected().AddSP(this, &SAnimMontagePanel::HandleObjectsSelected);

	bChildAnimMontage = InArgs._bChildAnimMontage;

	//TrackStyle = TRACK_Double;
	CurrentPosition = InArgs._CurrentPosition;

	bIsActiveTimerRegistered = false;

	EnsureStartingSection();
	EnsureSlotNode();

	if(GEditor)
	{
		GEditor->RegisterForUndo(this);
	}

	this->ChildSlot
	[
		SAssignNew( PanelArea, SBorder )
		.Padding(0.0f)
		.BorderImage( FAppStyle::GetBrush("NoBorder") )
		.ColorAndOpacity( FLinearColor::White )
	];

	InModel->GetEditableSkeleton()->RegisterOnNotifiesChanged(FSimpleDelegate::CreateSP(this, &SAnimMontagePanel::Update));
	InModel->OnTracksChanged().Add(FSimpleDelegate::CreateSP(this, &SAnimMontagePanel::Update));

	CollapseMontage();

	Update();
}

SAnimMontagePanel::~SAnimMontagePanel()
{
	if(GEditor)
	{
		GEditor->UnregisterForUndo(this);
	}
}

UAnimPreviewInstance* SAnimMontagePanel::GetPreviewInstance() const
{
	UDebugSkelMeshComponent* PreviewMeshComponent = WeakModel.Pin()->GetPreviewScene()->GetPreviewMeshComponent();
	return PreviewMeshComponent && PreviewMeshComponent->IsPreviewOn()? PreviewMeshComponent->PreviewInstance : nullptr;
}

bool SAnimMontagePanel::OnIsAnimAssetValid(const UAnimSequenceBase* AnimSequenceBase, FText* OutReason)
{
	if (AnimSequenceBase)
	{
		if (UAnimationSettings::Get()->bEnforceSupportedFrameRates)
		{
			const FFrameRate AssetFrameRate = AnimSequenceBase->GetSamplingFrameRate();
			
			const UAnimMontage* AnimMontage = WeakModel.Pin()->GetAsset<UAnimMontage>();
			const FFrameRate MontageFrameRate = WeakModel.Pin()->GetAsset<UAnimMontage>()->GetCommonTargetFrameRate();
			const bool bContainsSegments = AnimMontage->SlotAnimTracks.Num() != 0 && AnimMontage->SlotAnimTracks[0].AnimTrack.AnimSegments.Num() != 0;
			if (MontageFrameRate.IsValid() && bContainsSegments && !AssetFrameRate.IsMultipleOf(MontageFrameRate) && !AssetFrameRate.IsFactorOf(MontageFrameRate))
			{
				if (OutReason)
				{
					*OutReason = FText::Format(LOCTEXT("InvalidFrameRate", "Animation Asset {0} its framerate {1} is incompatible with the Anim Montage's {2}"), FText::FromString(AnimSequenceBase->GetName()), AssetFrameRate.ToPrettyText(), MontageFrameRate.ToPrettyText());
				}				
				
				return false;
			}
		}
		
		return true;	
	}
	return false;
}

FReply SAnimMontagePanel::OnFindParentClassInContentBrowserClicked()
{
	if (Montage != nullptr)
	{
		UObject* ParentClass = Montage->ParentAsset;
		if (ParentClass != nullptr)
		{
			TArray< UObject* > ParentObjectList;
			ParentObjectList.Add(ParentClass);
			GEditor->SyncBrowserToObjects(ParentObjectList);
		}
	}

	return FReply::Handled();
}

FReply SAnimMontagePanel::OnEditParentClassClicked()
{
	if (Montage != nullptr)
	{
		UObject* ParentClass = Montage->ParentAsset;
		if (ParentClass != nullptr)
		{
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(ParentClass);
		}
	}

	return FReply::Handled();
}

void SAnimMontagePanel::OnSetMontagePreviewSlot(int32 SlotIndex)
{
	UAnimSingleNodeInstance * PreviewInstance = GetPreviewInstance();
	if (PreviewInstance && Montage->SlotAnimTracks.IsValidIndex(SlotIndex))
	{
		const FName SlotName = Montage->SlotAnimTracks[SlotIndex].SlotName;
		PreviewInstance->SetMontagePreviewSlot(SlotName);
	}
}

bool SAnimMontagePanel::ValidIndexes(int32 AnimSlotIndex, int32 AnimSegmentIndex) const
{
	return (Montage != nullptr && Montage->SlotAnimTracks.IsValidIndex(AnimSlotIndex) && Montage->SlotAnimTracks[AnimSlotIndex].AnimTrack.AnimSegments.IsValidIndex(AnimSegmentIndex) );
}

bool SAnimMontagePanel::ValidSection(int32 SectionIndex) const
{
	return (Montage != nullptr && Montage->CompositeSections.IsValidIndex(SectionIndex));
}

void SAnimMontagePanel::HandleNotifiesChanged()
{
	WeakModel.Pin()->OnTracksChanged().Broadcast();
}


bool SAnimMontagePanel::GetSectionTime( int32 SectionIndex, float &OutTime ) const
{
	if (Montage != nullptr && Montage->CompositeSections.IsValidIndex(SectionIndex))
	{
		OutTime = Montage->CompositeSections[SectionIndex].GetTime();
		return true;
	}

	return false;
}

TArray<FString>	SAnimMontagePanel::GetSectionNames() const
{
	TArray<FString> Names;
	if (Montage != nullptr)
	{
		for( int32 I=0; I < Montage->CompositeSections.Num(); I++)
		{
			Names.Add(Montage->CompositeSections[I].SectionName.ToString());
		}
	}
	return Names;
}

TArray<float> SAnimMontagePanel::GetSectionStartTimes() const
{
	TArray<float>	Times;
	if (Montage != nullptr)
	{
		for( int32 I=0; I < Montage->CompositeSections.Num(); I++)
		{
			Times.Add(Montage->CompositeSections[I].GetTime());
		}
	}
	return Times;
}

TArray<FTrackMarkerBar> SAnimMontagePanel::GetMarkerBarInformation() const
{
	TArray<FTrackMarkerBar> MarkerBars;
	if (Montage != nullptr)
	{
		for( int32 I=0; I < Montage->CompositeSections.Num(); I++)
		{
			FTrackMarkerBar Bar;
			Bar.Time = Montage->CompositeSections[I].GetTime();
			Bar.DrawColour = FLinearColor(0.f,1.f,0.f);
			MarkerBars.Add(Bar);
		}
	}
	return MarkerBars;
}

TArray<float> SAnimMontagePanel::GetAnimSegmentStartTimes() const
{
	TArray<float>	Times;
	if (Montage != nullptr)
	{
		for ( int32 i=0; i < Montage->SlotAnimTracks.Num(); i++)
		{
			for (int32 j=0; j < Montage->SlotAnimTracks[i].AnimTrack.AnimSegments.Num(); j++)
			{
				Times.Add( Montage->SlotAnimTracks[i].AnimTrack.AnimSegments[j].StartPos );
			}
		}
	}
	return Times;
}

void SAnimMontagePanel::PreAnimUpdate()
{
	Montage->Modify();
}

void SAnimMontagePanel::OnMontageModified()
{
	Montage->PostEditChange();
	Montage->MarkPackageDirty();
}

void SAnimMontagePanel::PostAnimUpdate()
{
	SortAndUpdateMontage();
	OnMontageModified();
}

bool SAnimMontagePanel::IsDiffererentFromParent(FName SlotName, int32 SegmentIdx, const FAnimSegment& Segment)
{
	// if it doesn't hare parent asset, no reason to come here
	if (Montage && ensureAlways(Montage->ParentAsset))
	{
		// find correct source asset from parent
		UAnimMontage* ParentMontage = Cast<UAnimMontage>(Montage->ParentAsset);
		if (ParentMontage->IsValidSlot(SlotName))
		{
			const FAnimTrack* ParentTrack = ParentMontage->GetAnimationData(SlotName);

			if (ParentTrack && ParentTrack->AnimSegments.IsValidIndex(SegmentIdx))
			{
				UAnimSequenceBase* SourceAsset = ParentTrack->AnimSegments[SegmentIdx].GetAnimReference();
				return (SourceAsset != Segment.GetAnimReference());
			}
		}
	}

	// if something doesn't match, we assume they're different, so default feedback  is to return true
	return true;
}

void SAnimMontagePanel::ReplaceAnimationMapping(FName SlotName, int32 SegmentIdx, UAnimSequenceBase* OldSequenceBase, UAnimSequenceBase* NewSequenceBase)
{
	// if it doesn't hare parent asset, no reason to come here
	if (Montage && ensureAlways(Montage->ParentAsset))
	{
		// find correct source asset from parent
		UAnimMontage* ParentMontage = Cast<UAnimMontage>(Montage->ParentAsset);
		if (ParentMontage->IsValidSlot(SlotName))
		{
			const FAnimTrack* ParentTrack = ParentMontage->GetAnimationData(SlotName);

			if (ParentTrack && ParentTrack->AnimSegments.IsValidIndex(SegmentIdx))
			{
				UAnimSequenceBase* SourceAsset = ParentTrack->AnimSegments[SegmentIdx].GetAnimReference();
				if (Montage->RemapAsset(SourceAsset, NewSequenceBase))
				{
					// success
					return;
				}
			}
		}
	}

	// failed to do the process, check if the animation is correct or if the same type of animation
	// print error
	FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("FailedToRemap", "Make sure the target animation is valid. If source is additive, target animation has to be additive also."));
}

void SAnimMontagePanel::RebuildMontagePanel(bool bNotifyAsset /*= true*/)
{
	SortAndUpdateMontage();
	WeakModel.Pin()->OnSectionsChanged.Execute();

	if (bNotifyAsset)
	{
		OnMontageModified();
	}
}


/** This will remove empty spaces in the montage's anim segment but not resort. e.g. - all cached indexes remains valid. UI IS NOT REBUILT after this */
void SAnimMontagePanel::CollapseMontage()
{
	if (Montage==nullptr)
	{
		return;
	}

	for (int32 i=0; i < Montage->SlotAnimTracks.Num(); i++)
	{
		Montage->SlotAnimTracks[i].AnimTrack.CollapseAnimSegments();
	}

	Montage->UpdateLinkableElements();

	WeakModel.Pin()->RecalculateSequenceLength();
}

/** This will sort all components of the montage and update (recreate) the UI */
void SAnimMontagePanel::SortAndUpdateMontage()
{
	if (Montage==nullptr)
	{
		return;
	}
	
	SortAnimSegments();

	Montage->UpdateLinkableElements();

	WeakModel.Pin()->RecalculateSequenceLength();

	SortSections();

	StaticCastSharedPtr<FAnimModel_AnimMontage>(WeakModel.Pin())->RefreshNotifyTriggerOffsets();

	// Update view (this will recreate everything)
// 	AnimMontagePanel->Update();
// 	AnimMontageSectionsPanel->Update();
// 	AnimTimingPanel->Update();

	WeakModel.Pin()->RefreshTracks();

	// Restart the preview instance of the montage
	RestartPreview();
}


/** Make sure all Sections and Notifies are clamped to NewEndTime (called before NewEndTime is set to SequenceLength) */
bool SAnimMontagePanel::ClampToEndTime(float NewEndTime)
{
	float SequenceLength = GetSequenceLength();

	bool bClampingNeeded = (SequenceLength > 0.f && NewEndTime < SequenceLength);
	if(bClampingNeeded)
	{
		float ratio = NewEndTime / Montage->GetPlayLength();

		for(int32 i=0; i < Montage->CompositeSections.Num(); i++)
		{
			if(Montage->CompositeSections[i].GetTime() > NewEndTime)
			{
				float CurrentTime = Montage->CompositeSections[i].GetTime();
				Montage->CompositeSections[i].SetTime(CurrentTime * ratio);
			}
		}

		for(int32 i=0; i < Montage->Notifies.Num(); i++)
		{
			FAnimNotifyEvent& Notify = Montage->Notifies[i];
			float NotifyTime = Notify.GetTime();

			if(NotifyTime >= NewEndTime)
			{
				Notify.SetTime(NotifyTime * ratio);
				Notify.TriggerTimeOffset = GetTriggerTimeOffsetForType(Montage->CalculateOffsetForNotify(Notify.GetTime()));
			}
		}
	}

	return bClampingNeeded;
}

void SAnimMontagePanel::AddNewSection(float StartTime, FString SectionName)
{
	if ( Montage != nullptr )
	{
		const FScopedTransaction Transaction( LOCTEXT("AddNewSection", "Add New Section") );
		Montage->Modify();

		if (Montage->AddAnimCompositeSection(FName(*SectionName), StartTime) != INDEX_NONE)
		{
			RebuildMontagePanel();
		}
		OnMontageModified();
		WeakModel.Pin()->OnSectionsChanged.Execute();
	}
}

void SAnimMontagePanel::RemoveSection(int32 SectionIndex)
{
	if(ValidSection(SectionIndex))
	{
		const FScopedTransaction Transaction( LOCTEXT("DeleteSection", "Delete Section") );
		Montage->Modify();

		// Unlink section name that is being removed from other section's NextSection.
		const FName SectionNameToRemove = Montage->CompositeSections[SectionIndex].SectionName;
		for (FCompositeSection& CompositeSection : Montage->CompositeSections)
		{
			if(CompositeSection.NextSectionName == SectionNameToRemove)
			{
				CompositeSection.NextSectionName = NAME_None;
			}
		}

		// Remove the section.
		Montage->CompositeSections.RemoveAt(SectionIndex);
		EnsureStartingSection();
		OnMontageModified();
		WeakModel.Pin()->RefreshTracks();
		WeakModel.Pin()->OnSectionsChanged.Execute();
		RestartPreview();
	}
}

FString	SAnimMontagePanel::GetSectionName(int32 SectionIndex) const
{
	if (ValidSection(SectionIndex))
	{
		return Montage->GetSectionName(SectionIndex).ToString();
	}
	return FString();
}

void SAnimMontagePanel::RenameSlotNode(int32 SlotIndex, FString NewSlotName)
{
	if(Montage->SlotAnimTracks.IsValidIndex(SlotIndex))
	{
		FName NewName(*NewSlotName);
		if(Montage->SlotAnimTracks[SlotIndex].SlotName != NewName)
		{
			const FScopedTransaction Transaction( LOCTEXT("RenameSlot", "Rename Slot") );
			WeakModel.Pin()->GetEditableSkeleton()->RegisterSlotNode(NewName);

			Montage->Modify();

			Montage->SlotAnimTracks[SlotIndex].SlotName = NewName;
			OnMontageModified();
		}
	}
}

void SAnimMontagePanel::AddNewMontageSlot( FName NewSlotName )
{
	if ( Montage != nullptr )
	{
		const FScopedTransaction Transaction( LOCTEXT("AddSlot", "Add Slot") );

		WeakModel.Pin()->GetEditableSkeleton()->RegisterSlotNode(NewSlotName);

		Montage->Modify();

		Montage->AddSlot(NewSlotName);

		OnMontageModified();

		Update();

		WeakModel.Pin()->RefreshTracks();
	}
}

FText SAnimMontagePanel::GetMontageSlotName(int32 SlotIndex) const
{
	if(Montage->SlotAnimTracks.IsValidIndex(SlotIndex) && Montage->SlotAnimTracks[SlotIndex].SlotName != NAME_None)
	{
		return FText::FromName( Montage->SlotAnimTracks[SlotIndex].SlotName );
	}	
	return FText::GetEmpty();
}

void SAnimMontagePanel::RemoveMontageSlot(int32 AnimSlotIndex)
{
	if ( Montage != nullptr && Montage->SlotAnimTracks.IsValidIndex( AnimSlotIndex ) )
	{
		const FScopedTransaction Transaction( LOCTEXT("RemoveSlot", "Remove Slot") );
		Montage->Modify();

		Montage->SlotAnimTracks.RemoveAt(AnimSlotIndex);
		OnMontageModified();
		Update();

		// Iterate the notifies and relink anything that is now invalid
		for(FAnimNotifyEvent& Event : Montage->Notifies)
		{
			Event.ConditionalRelink();
		}

		// Do the same for sections
		for(FCompositeSection& Section : Montage->CompositeSections)
		{
			Section.ConditionalRelink();
		}

		WeakModel.Pin()->RefreshTracks();
	}
}

bool SAnimMontagePanel::CanRemoveMontageSlot(int32 AnimSlotIndex)
{
	return (Montage != nullptr) && (Montage->SlotAnimTracks.Num()) > 1;
}

void SAnimMontagePanel::DuplicateMontageSlot(int32 AnimSlotIndex)
{
	if (Montage != nullptr && Montage->SlotAnimTracks.IsValidIndex(AnimSlotIndex))
	{
		const FScopedTransaction Transaction(LOCTEXT("DuplicateSlot", "Duplicate Slot"));
		Montage->Modify();

		FSlotAnimationTrack& NewTrack = Montage->AddSlot(FAnimSlotGroup::DefaultSlotName); 
		NewTrack.AnimTrack = Montage->SlotAnimTracks[AnimSlotIndex].AnimTrack;

		OnMontageModified();

		Update();

		WeakModel.Pin()->RefreshTracks();
	}
}

void SAnimMontagePanel::RestartPreview()
{
	if (UDebugSkelMeshComponent* MeshComponent = WeakModel.Pin()->GetPreviewScene()->GetPreviewMeshComponent())
	{
		if (UAnimPreviewInstance* Preview = MeshComponent->PreviewInstance)
		{
			Preview->MontagePreview_PreviewNormal(INDEX_NONE, Preview->IsPlaying());
		}
	}
}

void SAnimMontagePanel::RestartPreviewPlayAllSections()
{
	if (UDebugSkelMeshComponent* MeshComponent = WeakModel.Pin()->GetPreviewScene()->GetPreviewMeshComponent())
	{
		if (UAnimPreviewInstance* Preview = MeshComponent->PreviewInstance)
		{
			Preview->MontagePreview_PreviewAllSections(Preview->IsPlaying());
		}
	}
}

void SAnimMontagePanel::MakeDefaultSequentialSections()
{
	check( Montage != nullptr );
	SortSections();
	for(int32 SectionIdx=0; SectionIdx < Montage->CompositeSections.Num(); SectionIdx++)
	{
		Montage->CompositeSections[SectionIdx].NextSectionName = Montage->CompositeSections.IsValidIndex(SectionIdx+1) ? Montage->CompositeSections[SectionIdx+1].SectionName : NAME_None;
	}
	RestartPreview();
}

void SAnimMontagePanel::ClearSquenceOrdering()
{
	check( Montage != nullptr );
	SortSections();
	for(int32 SectionIdx=0; SectionIdx < Montage->CompositeSections.Num(); SectionIdx++)
	{
		Montage->CompositeSections[SectionIdx].NextSectionName = NAME_None;
	}
	RestartPreview();
}

void SAnimMontagePanel::PostUndo( bool bSuccess )
{
	PostRedoUndo();
}

void SAnimMontagePanel::PostRedo( bool bSuccess )
{
	PostRedoUndo();
}

void SAnimMontagePanel::PostRedoUndo()
{
	// When we rebuild the montage panel as part of a undo/redo it also restarts the montage from the start
	// To facilitate a smoother workflow we try to restore the montage position afterwards
	UAnimPreviewInstance* Preview = nullptr;
	if (UDebugSkelMeshComponent* MeshComponent = WeakModel.Pin()->GetPreviewScene()->GetPreviewMeshComponent())
	{
		Preview = MeshComponent->PreviewInstance;
	}
	const float PrevTime = Preview ? Preview->GetCurrentTime() : 0.0f;

	RebuildMontagePanel(); //Rebuild here, undoing adds can cause slate to crash later on if we don't (using dummy args since they aren't used by the method

	if (Preview)
	{
		Preview->SetPosition(PrevTime, false);
	}
}

void SAnimMontagePanel::Update()
{
	if ( Montage != nullptr )
	{
		CurrentPreviewSlot = FMath::Clamp(CurrentPreviewSlot, 0, Montage->SlotAnimTracks.Num() - 1);
		OnSetMontagePreviewSlot(CurrentPreviewSlot);

		int32 ColorIdx=0;

		const FSlateColor GreenAccent = FAppStyle::Get().GetSlateColor("Colors.AccentGreen");
		FLinearColor NodeColor = GreenAccent.GetSpecifiedColor();

		const FSlateColor OrangeAccent = FAppStyle::Get().GetSlateColor("Colors.AccentOrange");
		FLinearColor OutOfDateNodeColor = OrangeAccent.GetSpecifiedColor();

		TSharedPtr<SVerticalBox> MontageSlots;
		PanelArea->SetContent(
			SAssignNew( MontageSlots, SVerticalBox )
			);

		MontageSlots->ClearChildren();

		RefreshTimingNodes();

		// ===================================
		// Anim Segment Tracks
		// ===================================
		{
			int32 NumAnimTracks = Montage->SlotAnimTracks.Num();

			SlotNameComboBoxes.Empty(NumAnimTracks);
			SlotNameComboSelectedNames.Empty(NumAnimTracks);
			SlotWarningImages.Empty(NumAnimTracks);
			SlotNameComboBoxes.AddZeroed(NumAnimTracks);
			SlotNameComboSelectedNames.AddZeroed(NumAnimTracks);
			SlotWarningImages.AddZeroed(NumAnimTracks);

			RefreshComboLists();
			check(SlotNameComboBoxes.Num() == NumAnimTracks);
			check(SlotNameComboSelectedNames.Num() == NumAnimTracks);

			for (int32 SlotAnimIdx = 0; SlotAnimIdx < NumAnimTracks; SlotAnimIdx++)
			{			
				int32 FoundIndex = SlotNameList.Find(SlotNameComboSelectedNames[SlotAnimIdx]);
				TSharedPtr<FString> ComboItem = SlotNameComboListItems[FoundIndex];

				if (bChildAnimMontage)
				{
					MontageSlots->AddSlot()
					.AutoHeight()
					.VAlign(VAlign_Center)
					[
						SNew(SAnimSegmentsPanel)
						.AnimTrack(&Montage->SlotAnimTracks[SlotAnimIdx].AnimTrack)
						.SlotName(Montage->SlotAnimTracks[SlotAnimIdx].SlotName)
						.NodeSelectionSet(&SelectionSet)
						.ViewInputMin(ViewInputMin)
						.ViewInputMax(ViewInputMax)
						.bChildAnimMontage(bChildAnimMontage)
						.OnGetNodeColor_Lambda([NodeColor, OutOfDateNodeColor](const FAnimSegment& InSegment)
						{
							if (InSegment.IsPlayLengthOutOfDate())
							{
								return OutOfDateNodeColor;
							}
							
							return NodeColor;
						})
						.OnPreAnimUpdate(this, &SAnimMontagePanel::PreAnimUpdate)
						.OnPostAnimUpdate(this, &SAnimMontagePanel::PostAnimUpdate)
						.OnAnimReplaceMapping(this, &SAnimMontagePanel::ReplaceAnimationMapping)
						.OnDiffFromParentAsset(this, &SAnimMontagePanel::IsDiffererentFromParent)
						.TrackMaxValue(this, &SAnimMontagePanel::GetSequenceLength)
						.TrackNumDiscreteValues(Montage->GetNumberOfSampledKeys())
						.OnIsAnimAssetValid(this, &SAnimMontagePanel::OnIsAnimAssetValid)
					];

				}
				else
				{
					MontageSlots->AddSlot()
					.AutoHeight()
					.VAlign(VAlign_Center)
					[
						SNew(SAnimSegmentsPanel)
						.AnimTrack(&Montage->SlotAnimTracks[SlotAnimIdx].AnimTrack)
						.SlotName(Montage->SlotAnimTracks[SlotAnimIdx].SlotName)
						.NodeSelectionSet(&SelectionSet)
						.ViewInputMin(ViewInputMin)
						.ViewInputMax(ViewInputMax)
						.bChildAnimMontage(bChildAnimMontage)
						.OnGetNodeColor_Lambda([NodeColor, OutOfDateNodeColor](const FAnimSegment& InSegment)
						{
							if (InSegment.IsPlayLengthOutOfDate())
							{
								return OutOfDateNodeColor;
							}
							
							return NodeColor;
						})
						.TrackMaxValue(this, &SAnimMontagePanel::GetSequenceLength)
						.TrackNumDiscreteValues(Montage->GetNumberOfSampledKeys())
						.OnAnimSegmentNodeClicked(this, &SAnimMontagePanel::ShowSegmentInDetailsView, SlotAnimIdx)
						.OnPreAnimUpdate(this, &SAnimMontagePanel::PreAnimUpdate)
						.OnPostAnimUpdate(this, &SAnimMontagePanel::PostAnimUpdate)
						.OnAnimSegmentRemoved(this, &SAnimMontagePanel::OnAnimSegmentRemoved, SlotAnimIdx)
						.OnTrackRightClickContextMenu(this, &SAnimMontagePanel::SummonTrackContextMenu, static_cast<int>(SlotAnimIdx))
						.OnIsAnimAssetValid(this, &SAnimMontagePanel::OnIsAnimAssetValid)
					];
				}
			}
		}
	}
}

FReply SAnimMontagePanel::OnMouseButtonDown( const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent )
{
	FReply Reply = SAnimTrackPanel::OnMouseButtonDown( InMyGeometry, InMouseEvent );

	ClearSelected();

	return Reply;
}

void SAnimMontagePanel::SummonTrackContextMenu( FMenuBuilder& MenuBuilder, float DataPosX, int32 SectionIndex, int32 AnimSlotIndex )
{
	FUIAction UIAction;

	// Sections
	MenuBuilder.BeginSection("AnimMontageSections", LOCTEXT("Sections", "Sections"));
	{
		// New Action as we have a CanExecuteAction defined
		UIAction.ExecuteAction.BindRaw(this, &SAnimMontagePanel::OnNewSectionClicked, static_cast<float>(DataPosX));
		UIAction.CanExecuteAction.BindRaw(this, &SAnimMontagePanel::CanAddNewSection);
		MenuBuilder.AddMenuEntry(LOCTEXT("NewMontageSection", "New Montage Section"), LOCTEXT("NewMontageSectionToolTip", "Adds a new Montage Section"), FSlateIcon(), UIAction);
		UIAction.CanExecuteAction.Unbind();
	}
	MenuBuilder.EndSection();

	// Slots
	MenuBuilder.BeginSection("AnimMontageSlots", LOCTEXT("Slots", "Slots") );
	{
		MenuBuilder.AddSubMenu(
			LOCTEXT("NewSlot", "New Slot"),
			LOCTEXT("NewSlotToolTip", "Adds a new Slot"), 
			FNewMenuDelegate::CreateSP(this, &SAnimMontagePanel::BuildNewSlotMenu)
		);

		if(AnimSlotIndex != INDEX_NONE)
		{
			UIAction.ExecuteAction.BindRaw(this, &SAnimMontagePanel::RemoveMontageSlot, AnimSlotIndex);
			UIAction.CanExecuteAction.BindRaw(this, &SAnimMontagePanel::CanRemoveMontageSlot, AnimSlotIndex);
			MenuBuilder.AddMenuEntry(LOCTEXT("DeleteSlot", "Delete Slot"), LOCTEXT("DeleteSlotToolTip", "Deletes Slot"), FSlateIcon(), UIAction);
			UIAction.CanExecuteAction.Unbind();

			UIAction.ExecuteAction.BindRaw(this, &SAnimMontagePanel::DuplicateMontageSlot, AnimSlotIndex);
			MenuBuilder.AddMenuEntry(LOCTEXT("DuplicateSlot", "Duplicate Slot"), LOCTEXT("DuplicateSlotToolTip", "Duplicates the slected slot"), FSlateIcon(), UIAction);
		}
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("AnimMontageElementBulkActions", LOCTEXT("BulkLinkActions", "Bulk Link Actions"));
	{
		MenuBuilder.AddSubMenu(LOCTEXT("SetElementLink_SubMenu", "Set Elements to..."), LOCTEXT("SetElementLink_TooTip", "Sets all montage elements (Sections, Notifies) to a chosen link type."), FNewMenuDelegate::CreateSP(this, &SAnimMontagePanel::FillElementSubMenuForTimes));

		if(Montage->SlotAnimTracks.Num() > 1)
		{
			MenuBuilder.AddSubMenu(LOCTEXT("SetToSlotMenu", "Link all Elements to Slot..."), LOCTEXT("SetToSlotMenuToolTip", "Link all elements to a selected slot"), FNewMenuDelegate::CreateSP(this, &SAnimMontagePanel::FillSlotSubMenu));
		}
	}
	MenuBuilder.EndSection();

	LastContextHeading.Empty();
}

void SAnimMontagePanel::FillElementSubMenuForTimes(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.AddMenuEntry(LOCTEXT("SubLinkAbs", "Absolute"), LOCTEXT("SubLinkAbsTooltip", "Set all elements to absolute link"), FSlateIcon(), FUIAction(FExecuteAction::CreateSP(this, &SAnimMontagePanel::OnSetElementsToLinkMode, EAnimLinkMethod::Absolute)));
	MenuBuilder.AddMenuEntry(LOCTEXT("SubLinkRel", "Relative"), LOCTEXT("SubLinkRelTooltip", "Set all elements to relative link"), FSlateIcon(), FUIAction(FExecuteAction::CreateSP(this, &SAnimMontagePanel::OnSetElementsToLinkMode, EAnimLinkMethod::Relative)));
	MenuBuilder.AddMenuEntry(LOCTEXT("SubLinkPro", "Proportional"), LOCTEXT("SubLinkProTooltip", "Set all elements to proportional link"), FSlateIcon(), FUIAction(FExecuteAction::CreateSP(this, &SAnimMontagePanel::OnSetElementsToLinkMode, EAnimLinkMethod::Proportional)));
}

void SAnimMontagePanel::FillSlotSubMenu(FMenuBuilder& Menubuilder)
{
	for(int32 SlotIdx = 0 ; SlotIdx < Montage->SlotAnimTracks.Num() ; ++SlotIdx)
	{
		FSlotAnimationTrack& Slot = Montage->SlotAnimTracks[SlotIdx];
		Menubuilder.AddMenuEntry(FText::Format(LOCTEXT("SubSlotMenuNameEntry", "{SlotName}"), FText::FromString(Slot.SlotName.ToString())), LOCTEXT("SubSlotEntry", "Set to link to this slot"), FSlateIcon(), FUIAction(FExecuteAction::CreateSP(this, &SAnimMontagePanel::OnSetElementsToSlot, SlotIdx)));
	}
}

/** Slots */

void SAnimMontagePanel::BuildNewSlotMenu(FMenuBuilder& InMenuBuilder)
{
	USkeleton* Skeleton = Montage->GetSkeleton();
	FName CurrentSlotGroupName = FAnimSlotGroup::DefaultGroupName;
	if (Montage->SlotAnimTracks.Num() > 0)
	{
		FName CurrentSlotName = Montage->SlotAnimTracks[0].SlotName;
		CurrentSlotGroupName = Skeleton->GetSlotGroupName(CurrentSlotName);
	}
		
	if (FAnimSlotGroup* SlotGroup = Skeleton->FindAnimSlotGroup(CurrentSlotGroupName))
	{
		InMenuBuilder.BeginSection("AnimMontageAvailableAddSlots", FText::FromString(SlotGroup->GroupName.ToString()));
		{
			for (const FName& SlotName : SlotGroup->SlotNames)
			{
				FText SlotItemText = FText::FromString(*SlotName.ToString());

				FText Tooltip = CanCreateNewSlot(SlotName) ? FText::Format(LOCTEXT("SlotTooltipFormat", "Add new Slot '{0}'"), SlotItemText) :
				                                             FText::Format(LOCTEXT("SlotUnavailableTooltipFormat", "Slot '{0}' already has a track in this Montage"), SlotItemText);

				InMenuBuilder.AddMenuEntry(
					SlotItemText,
					Tooltip,
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateSP(this, &SAnimMontagePanel::AddNewMontageSlot, SlotName),
						FCanExecuteAction::CreateSP(this, &SAnimMontagePanel::CanCreateNewSlot, SlotName)
					));
			}
		}
		InMenuBuilder.EndSection();
	}
}

bool SAnimMontagePanel::CanCreateNewSlot(FName InName) const
{
	for (auto &Track : Montage->SlotAnimTracks)
	{
		if (Track.SlotName == InName)
		{
			return false;
		}
	}

	return true;
}

void SAnimMontagePanel::CreateNewSlot(const FText& NewSlotName, ETextCommit::Type CommitInfo)
{
	if (CommitInfo == ETextCommit::OnEnter)
	{
		AddNewMontageSlot(*NewSlotName.ToString());
	}

	FSlateApplication::Get().DismissAllMenus();
}

/** Sections */
void SAnimMontagePanel::OnNewSectionClicked(float DataPosX)
{
	// Show dialog to enter new track name
	TSharedRef<STextEntryPopup> TextEntry =
		SNew(STextEntryPopup)
		.Label( LOCTEXT("NewSectionNameLabel", "Section Name") )
		.OnTextCommitted( this, &SAnimMontagePanel::CreateNewSection, DataPosX );


	// Show dialog to enter new event name
	FSlateApplication::Get().PushMenu(
		AsShared(), // Menu being summoned from a menu that is closing: Parent widget should be k2 not the menu thats open or it will be closed when the menu is dismissed
		FWidgetPath(),
		TextEntry,
		FSlateApplication::Get().GetCursorPos(),
		FPopupTransitionEffect( FPopupTransitionEffect::TypeInPopup )
		);
}

bool SAnimMontagePanel::CanAddNewSection()
{
	// Can't add sections if there isn't a montage, or that montage is of zero length
	return Montage && Montage->GetPlayLength() > 0.0f;
}

void SAnimMontagePanel::CreateNewSection(const FText& NewSectionName, ETextCommit::Type CommitInfo, float StartTime)
{
	if (CommitInfo == ETextCommit::OnEnter)
	{
		AddNewSection(StartTime,NewSectionName.ToString());
	}
	FSlateApplication::Get().DismissAllMenus();
}

void SAnimMontagePanel::ShowSegmentInDetailsView(int32 AnimSegmentIndex, int32 AnimSlotIndex)
{
	if(!bIsSelecting)
	{
		TGuardValue<bool> GuardValue(bIsSelecting, true);

		UEditorAnimSegment *Obj = Cast<UEditorAnimSegment>(WeakModel.Pin()->ShowInDetailsView(UEditorAnimSegment::StaticClass()));
		if(Obj != nullptr)
		{
			Obj->InitFromAnim(Montage, FOnAnimObjectChange::CreateSP(this, &SAnimMontagePanel::OnMontageChange));
			Obj->InitAnimSegment(AnimSlotIndex,AnimSegmentIndex);
		}
	}
}

EActiveTimerReturnType SAnimMontagePanel::TriggerRebuildMontagePanel(double InCurrentTime, float InDeltaTime)
{
	RebuildMontagePanel();

	bIsActiveTimerRegistered = false;
	return EActiveTimerReturnType::Stop;
}

void SAnimMontagePanel::OnMontageChange(UObject* EditorAnimBaseObj, bool bRebuild)
{
	if ( Montage != nullptr )
	{
		float PreviousSeqLength = GetSequenceLength();

		if(bRebuild && !bIsActiveTimerRegistered)
		{
			bIsActiveTimerRegistered = true;
			RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateSP(this, &SAnimMontagePanel::TriggerRebuildMontagePanel));
		} 
		else
		{
			CollapseMontage();
		}

		// if animation length changed, we might be out of range, let's restart
		if (GetSequenceLength() != PreviousSeqLength)
		{
			// this might not be safe
			RestartPreview();
		}

		OnMontageModified();
	}
}

void SAnimMontagePanel::ClearSelected()
{
	if(!bIsSelecting)
	{
		TGuardValue<bool> GuardValue(bIsSelecting, true);

		SelectionSet.Empty();
		WeakModel.Pin()->ClearDetailsView();
	}
}

void SAnimMontagePanel::OnSlotNameChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo, int32 AnimSlotIndex)
{
	// if it's set from code, we did that on purpose
	if (SelectInfo != ESelectInfo::Direct)
	{
		int32 ItemIndex = SlotNameComboListItems.Find(NewSelection);
		if (ItemIndex != INDEX_NONE)
		{
			FName NewSlotName = SlotNameList[ItemIndex];

			SlotNameComboSelectedNames[AnimSlotIndex] = NewSlotName;
			if (SlotNameComboBoxes[AnimSlotIndex].IsValid())
			{
				SlotNameComboBoxes[AnimSlotIndex]->SetToolTipText(FText::FromString(*NewSelection));
			}

			if (Montage->GetSkeleton()->ContainsSlotName(NewSlotName))
			{
				RenameSlotNode(AnimSlotIndex, NewSlotName.ToString());
			}
			
			// Clear selection, so Details panel for AnimNotifies doesn't show outdated information.
			ClearSelected();
		}
	}
}

bool SAnimMontagePanel::IsSlotPreviewed(int32 SlotIndex) const
{
	return (SlotIndex == CurrentPreviewSlot);
}

void SAnimMontagePanel::OnSlotPreviewedChanged(int32 SlotIndex)
{
	CurrentPreviewSlot = SlotIndex;
	OnSetMontagePreviewSlot(CurrentPreviewSlot);
}

void SAnimMontagePanel::OnSlotListOpening(int32 AnimSlotIndex)
{
	// Refresh Slot Names, in case we used the Anim Slot Manager to make changes.
	RefreshComboLists(true);
}

void SAnimMontagePanel::OnOpenAnimSlotManager()
{
	OnInvokeTab.ExecuteIfBound(FPersonaTabs::SkeletonSlotNamesID);
}

void SAnimMontagePanel::RefreshComboLists(bool bOnlyRefreshIfDifferent /*= false*/)
{
	// Make sure all slots defined in the montage are registered in our skeleton.
	int32 NumAnimTracks = Montage->SlotAnimTracks.Num();
	for (int32 TrackIndex = 0; TrackIndex < NumAnimTracks; TrackIndex++)
	{
		FName TrackSlotName = Montage->SlotAnimTracks[TrackIndex].SlotName;
		WeakModel.Pin()->GetEditableSkeleton()->RegisterSlotNode(TrackSlotName);
		SlotNameComboSelectedNames[TrackIndex] = TrackSlotName;
	}

	// Refresh Slot Names
	{
		TArray<TSharedPtr<FString>>	NewSlotNameComboListItems;
		TArray<FName> NewSlotNameList;

		bool bIsSlotNameListDifferent = false;

		const TArray<FAnimSlotGroup>& SlotGroups = Montage->GetSkeleton()->GetSlotGroups();
		for (auto SlotGroup : SlotGroups)
		{
			int32 Index = 0;
			for (auto SlotName : SlotGroup.SlotNames)
			{
				NewSlotNameList.Add(SlotName);

				FString ComboItemString = FString::Printf(TEXT("%s.%s"), *SlotGroup.GroupName.ToString(), *SlotName.ToString());
				NewSlotNameComboListItems.Add(MakeShareable(new FString(ComboItemString)));

				bIsSlotNameListDifferent = bIsSlotNameListDifferent || (!SlotNameComboListItems.IsValidIndex(Index) || (SlotNameComboListItems[Index] != NewSlotNameComboListItems[Index]));
				Index++;
			}
		}

		// Refresh if needed
		if (bIsSlotNameListDifferent || !bOnlyRefreshIfDifferent || (NewSlotNameComboListItems.Num() == 0))
		{
			SlotNameComboListItems = NewSlotNameComboListItems;
			SlotNameList = NewSlotNameList;

			// Update Combo Boxes
			for (int32 TrackIndex = 0; TrackIndex < NumAnimTracks; TrackIndex++)
			{
				if (SlotNameComboBoxes[TrackIndex].IsValid())
				{
					FName SelectedSlotName = SlotNameComboSelectedNames[TrackIndex];
					if (Montage->GetSkeleton()->ContainsSlotName(SelectedSlotName))
					{
						int32 FoundIndex = SlotNameList.Find(SelectedSlotName);
						TSharedPtr<FString> ComboItem = SlotNameComboListItems[FoundIndex];

						SlotNameComboBoxes[TrackIndex]->SetSelectedItem(ComboItem);
						SlotNameComboBoxes[TrackIndex]->SetToolTipText(FText::FromString(*ComboItem));
					}
					SlotNameComboBoxes[TrackIndex]->RefreshOptions();
				}
			}
		}
	}
}

void SAnimMontagePanel::OnSetElementsToLinkMode(EAnimLinkMethod::Type NewLinkMethod)
{
	TArray<FAnimLinkableElement*> Elements;
	CollectLinkableElements(Elements);

	for(FAnimLinkableElement* Element : Elements)
	{
		Element->ChangeLinkMethod(NewLinkMethod);
	}

	// Handle notify state links
	for(FAnimNotifyEvent& Notify : Montage->Notifies)
	{
		if(Notify.GetDuration() > 0.0f)
		{
			// Always keep link methods in sync between notifies and duration links
			if(Notify.GetLinkMethod() != Notify.EndLink.GetLinkMethod())
			{
				Notify.EndLink.ChangeLinkMethod(Notify.GetLinkMethod());
			}
		}
	}
}

void SAnimMontagePanel::OnSetElementsToSlot(int32 SlotIndex)
{
	TArray<FAnimLinkableElement*> Elements;
	CollectLinkableElements(Elements);

	for(FAnimLinkableElement* Element : Elements)
	{
		Element->ChangeSlotIndex(SlotIndex);
	}

	// Handle notify state links
	for(FAnimNotifyEvent& Notify : Montage->Notifies)
	{
		if(Notify.GetDuration() > 0.0f)
		{
			// Always keep link methods in sync between notifies and duration links
			if(Notify.GetSlotIndex() != Notify.EndLink.GetSlotIndex())
			{
				Notify.EndLink.ChangeSlotIndex(Notify.GetSlotIndex());
			}
		}
	}
}

void SAnimMontagePanel::CollectLinkableElements(TArray<FAnimLinkableElement*> &Elements)
{
	for(auto& Composite : Montage->CompositeSections)
	{
		FAnimLinkableElement* Element = &Composite;
		Elements.Add(Element);
	}

	for(auto& Notify : Montage->Notifies)
	{
		FAnimLinkableElement* Element = &Notify;
		Elements.Add(Element);
	}
}

void SAnimMontagePanel::OnAnimSegmentRemoved(int32 SegmentIndex, int32 SlotIndex)
{
	TArray<FAnimLinkableElement*> LinkableElements;
	CollectLinkableElements(LinkableElements);

	for(FAnimNotifyEvent& Notify : Montage->Notifies)
	{
		if(Notify.NotifyStateClass)
		{
			LinkableElements.Add(&Notify.EndLink);
		}
	}

	// Go through the linkable elements and fix the indices.
	// BG TODO: Once we can identify moved segments, remove
	for(FAnimLinkableElement* Element : LinkableElements)
	{
		if(Element->GetSlotIndex() == SlotIndex)
		{
			if(Element->GetSegmentIndex() == SegmentIndex)
			{
				Element->Clear();
			}
			else if(Element->GetSegmentIndex() > SegmentIndex)
			{
				Element->SetSegmentIndex(Element->GetSegmentIndex() - 1);
			}
		}
	}
}

float SAnimMontagePanel::GetSequenceLength() const
{
	if(Montage != nullptr)
	{
		return Montage->GetPlayLength();
	}
	return 0.0f;
}

void SAnimMontagePanel::RefreshTimingNodes()
{
	if(SectionNameTrack.IsValid())
	{
		// Clear current nodes
		SectionNameTrack->ClearTrack();
		// Add section timing widgets
		TArray<TSharedPtr<FTimingRelevantElementBase>> TimingElements;
		TArray<TSharedRef<SAnimTimingNode>> SectionTimingNodes;
		SAnimTimingPanel::GetTimingRelevantElements(Montage, TimingElements);
		for(int32 Idx = 0 ; Idx < TimingElements.Num() ; ++Idx)
		{
			TSharedPtr<FTimingRelevantElementBase>& Element = TimingElements[Idx];
			if(Element->GetType() == ENotifyTimingElementType::Section)
			{
				TSharedRef<SAnimTimingTrackNode> Node = SNew(SAnimTimingTrackNode)
					.ViewInputMin(ViewInputMin)
					.ViewInputMax(ViewInputMax)
					.DataStartPos(Element->GetElementTime())
					.Element(Element)
					.bUseTooltip(true);

				Node->SetVisibility(SectionTimingNodeVisibility);

				SectionNameTrack->AddTrackNode
				(
					Node
				);
			}
		}
	}
}

void SAnimMontagePanel::SortSections()
{
	StaticCastSharedPtr<FAnimModel_AnimMontage>(WeakModel.Pin())->SortSections();
}

void SAnimMontagePanel::EnsureStartingSection()
{
	StaticCastSharedPtr<FAnimModel_AnimMontage>(WeakModel.Pin())->EnsureStartingSection();
}

void SAnimMontagePanel::EnsureSlotNode()
{
	if (Montage && Montage->SlotAnimTracks.Num() == 0)
	{
		AddNewMontageSlot(FAnimSlotGroup::DefaultSlotName);
		OnMontageModified();
	}
}

void SAnimMontagePanel::SortAnimSegments()
{
	for (int32 I=0; I < Montage->SlotAnimTracks.Num(); I++)
	{
		Montage->SlotAnimTracks[I].AnimTrack.SortAnimSegments();
	}
}

void SAnimMontagePanel::HandleObjectsSelected(const TArray<UObject*>& InObjects)
{
	if(!bIsSelecting)
	{
		ClearSelected();
	}
}

void SAnimMontagePanel::SetSectionTime(int32 SectionIndex, float NewTime)
{
	if(Montage && Montage->CompositeSections.IsValidIndex(SectionIndex))
	{
		const FScopedTransaction Transaction(LOCTEXT("EditSection", "Edit Section Start Time"));
		Montage->Modify();
	
		FCompositeSection& Section = Montage->CompositeSections[SectionIndex];
		Section.SetTime(NewTime);
		Section.Link(Montage, NewTime);

		SortAndUpdateMontage();
	}
}

#undef LOCTEXT_NAMESPACE
