// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "Misc/Attribute.h"
#include "Input/Reply.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "SAnimTrackPanel.h"
#include "SMontageEditor.h"
#include "Animation/EditorAnimBaseObj.h"
#include "STrack.h"
#include "EditorUndoClient.h"
#include "SAnimCurveEd.h"

class FMenuBuilder;
class SBorder;
class SImage;
class STextBlock;
class FAnimModel_AnimMontage;
class UAnimPreviewInstance;

DECLARE_DELEGATE( FOnMontageLengthChange )
DECLARE_DELEGATE( FOnMontagePropertyChanged )
DECLARE_DELEGATE_OneParam( FOnMontageSetPreviewSlot, int32)

//////////////////////////////////////////////////////////////////////////
// SAnimMontagePanel
//	This is the main montage editing widget that is responsible for setting up
//	a set of generic widgets (STrack and STrackNodes) for editing an anim montage.
//
//	SAnimMontagePanel will usually not edit the montage directly but just setup the 
//  callbacks to SMontageEditor.
//
//////////////////////////////////////////////////////////////////////////

class SAnimMontagePanel : public SAnimTrackPanel, public FEditorUndoClient
{
public:
	SLATE_BEGIN_ARGS( SAnimMontagePanel )
		: _Montage()
		, _CurrentPosition()
		, _ViewInputMin()
		, _ViewInputMax()
		, _InputMin()
		, _InputMax()
		, _OnSetInputViewRange()
		, _OnGetScrubValue()
		, _OnMontageChange()
		, _OnInvokeTab()
		, _bChildAnimMontage(false)
	{}

	SLATE_ARGUMENT( class UAnimMontage*, Montage)
	SLATE_ARGUMENT( float, WidgetWidth )
	SLATE_ATTRIBUTE( float, CurrentPosition )
	SLATE_ATTRIBUTE( float, ViewInputMin )
	SLATE_ATTRIBUTE( float, ViewInputMax )
	SLATE_ATTRIBUTE( float, InputMin )
	SLATE_ATTRIBUTE( float, InputMax )
	SLATE_ATTRIBUTE(EVisibility, SectionTimingNodeVisibility)
	SLATE_EVENT( FOnSetInputViewRange, OnSetInputViewRange )
	SLATE_EVENT( FOnGetScrubValue, OnGetScrubValue )
	SLATE_EVENT( FOnAnimObjectChange, OnMontageChange )
	SLATE_EVENT( FOnInvokeTab, OnInvokeTab )
	SLATE_EVENT(FOnMontageSetPreviewSlot, OnSetMontagePreviewSlot)
	SLATE_ARGUMENT(bool, bChildAnimMontage)
	SLATE_END_ARGS()

	~SAnimMontagePanel();

	void Construct(const FArguments& InArgs, const TSharedRef<FAnimModel_AnimMontage>& InModel);

	/** Handler for when the preview slot is changed */
	void OnSetMontagePreviewSlot(int32 SlotIndex);

	// SWidget interface
	virtual FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;

	void RestartPreview();
	void RestartPreviewFromSection(int32 FromSectionIdx = INDEX_NONE);
	void RestartPreviewPlayAllSections();

	/** This is the main function that creates the UI widgets for the montage tool.*/
	void Update();

	// Functions used as delegates that build the context menus for the montage tracks
	void SummonTrackContextMenu( FMenuBuilder& MenuBuilder, float DataPosX, int32 SectionIndex, int32 AnimSlotIndex );
	void SummonSectionContextMenu( FMenuBuilder& MenuBuilder, int32 SectionIndex );
	void FillElementSubMenuForTimes( FMenuBuilder& MenuBuilder );
	void FillSlotSubMenu( FMenuBuilder& Menubuilder );

	void BuildNewSlotMenu(FMenuBuilder& InMenuBuilder);
	void CreateNewSlot(FName InName);
	bool CanCreateNewSlot(FName InName) const;

	void OnNewSectionClicked(float DataPosX);
	bool CanAddNewSection();

	void ShowSegmentInDetailsView(int32 AnimSegmentIndex, int32 AnimSlotIndex);
	void ShowSectionInDetailsView(int32 SectionIndex);

	void ClearSelected();

	// helper method to check whether the slot name is empty or not. If empty, shows an error message to provide a valid name
	void CheckSlotName(const FText& SlotName, int32 SlotNodeIndex, bool bShouldCheckCollapsed = false) const;

	// check the slot name whether valid or not while the user is typing
	void OnSlotNameChanged(const FText& NewText, int32 SlotNodeIndex);

	// get slot name from a montage editor and check the slot name whether valid or not
	FText GetMontageSlotName(int32 SlotIndex) const;
	
	// Handlers for preview slot checkbox UI
	bool IsSlotPreviewed(int32 SlotIndex) const;
	void OnSlotPreviewedChanged(int32 SlotIndex);

	// Context menu callback to set all elements in the montage to a given link method
	void OnSetElementsToLinkMode(EAnimLinkMethod::Type NewLinkMethod);

	// Fills the given array with all linkable elements in the montage
	void CollectLinkableElements(TArray<FAnimLinkableElement*>& Elements);

	// Context menu callback to set all elements in the montage to a given slot
	void OnSetElementsToSlot(int32 SlotIndex);

	// Clears the name track of timing nodes and rebuilds them
	void RefreshTimingNodes();

	/** Sort Composite Sections by Start Time */
	void SortSections();

	/** Ensure there is at least one section in the montage and that the first section starts at T=0.f */
	void EnsureStartingSection();

	/** Ensure there is at least one slot node track */
	void EnsureSlotNode();

	/** Sort Segments by starting time */
	void SortAnimSegments();

	void SortAndUpdateMontage();
	void CollapseMontage();

	bool ClampToEndTime(float NewEndTime);

	/** FEditorUndoClient interface */
	virtual void PostUndo( bool bSuccess ) override;
	virtual void PostRedo( bool bSuccess ) override;

	void PostRedoUndo();
	
	bool GetSectionTime( int32 SectionIndex, float &OutTime ) const;

	bool ValidIndexes(int32 AnimSlotIndex, int32 AnimSegmentIndex) const;
	bool ValidSection(int32 SectionIndex) const;

	/** Updates Notify trigger offsets to take into account current montage state */
	void RefreshNotifyTriggerOffsets();

	/** Handle notifies changing, so we rebuild timing */
	void HandleNotifiesChanged();

	/** One-off active timer to trigger a montage panel rebuild */
	EActiveTimerReturnType TriggerRebuildMontagePanel(double InCurrentTime, float InDeltaTime);

	/** Rebuilds the montage panel */
	void RebuildMontagePanel(bool bNotifyAsset=true);

	UAnimPreviewInstance* GetPreviewInstance() const;

	bool OnIsAnimAssetValid(const UAnimSequenceBase* AnimSequenceBase, FText* OutReason);
public:

	/** These are meant to be callbacks used by the montage editing UI widgets */
	void					OnMontageChange(class UObject *EditorAnimBaseObj, bool Rebuild);

	void					SetSectionTime(int32 SectionIndex, float NewTime);

	TArray<float>			GetSectionStartTimes() const;
	TArray<FTrackMarkerBar>	GetMarkerBarInformation() const;
	TArray<FString>			GetSectionNames() const;
	TArray<float>			GetAnimSegmentStartTimes() const;

	void					AddNewSection(float StartTime, FString SectionName);
	void					RemoveSection(int32 SectionIndex);
	FString					GetSectionName(int32 SectionIndex) const;

	void					RenameSlotNode(int32 SlotIndex, FString NewSlotName);

	// UI Slot Action handlers
	void					AddNewMontageSlot(FName NewSlotName);
	void					RemoveMontageSlot(int32 AnimSlotIndex);
	bool					CanRemoveMontageSlot(int32 AnimSlotIndex);
	void					DuplicateMontageSlot(int32 AnimSlotIndex);

	void					MakeDefaultSequentialSections();
	void					ClearSquenceOrdering();

	/** Delegete handlers for when the editor UI is changing the montage */
	void			PreAnimUpdate();
	void			PostAnimUpdate();
	void			OnMontageModified();
	void			ReplaceAnimationMapping(FName SlotName, int32 SegmentIdx, UAnimSequenceBase* OldSequenceBase, UAnimSequenceBase* NewSequenceBase);
	bool			IsDiffererentFromParent(FName SlotName, int32 SegmentIdx, const FAnimSegment& Segment);

	/** Delegate fired when montage sections have changed */
	FSimpleDelegate			OnSectionsChanged;

	FReply	OnFindParentClassInContentBrowserClicked();
	FReply	OnEditParentClassClicked();

	void HandleObjectsSelected(const TArray<UObject*>& InObjects);

	void OnOpenAnimSlotManager();

private:
	TWeakPtr<FAnimModel_AnimMontage> WeakModel;
	TSharedPtr<SBorder> PanelArea;
	class UAnimMontage* Montage;
	TAttribute<float> CurrentPosition;

	TArray<TSharedPtr<class SMontageEdTrack>>	TrackList;

	FString LastContextHeading;

	STrackNodeSelectionSet SelectionSet;

	TArray< TSharedPtr< class STextComboBox> >	SlotNameComboBoxes;
	TArray< FName > SlotNameComboSelectedNames;

	TArray< TSharedPtr< FString > > SlotNameComboListItems;
	TArray< FName > SlotNameList;

	TArray< TSharedPtr<SImage> > SlotWarningImages;

	TSharedPtr<STrack> SectionNameTrack;
	TAttribute<EVisibility> SectionTimingNodeVisibility;

	/** Member data to allow use to set preview slot from editor */
	int32					 CurrentPreviewSlot;

	/** If previewing section, it is section used to restart previewing when play button is pushed */
	int32 PreviewingStartSectionIdx;

	/** Recursion guard for selection */
	bool bIsSelecting;

	/** If currently previewing all or selected section */
	bool bPreviewingAllSections : 1;

	/** If currently previewing tracks instead of sections */
	bool bPreviewingTracks : 1;

	/** If the active timer to trigger a montage panel rebuild is currently registered */
	bool bIsActiveTimerRegistered : 1;

  	/* 
	 * Child Anim Montage: Child Anim Montage only can replace name of animations, and no other meaningful edits 
	 * as it will derive every data from Parent. There might be some other data that will allow to be replaced, but for now, it is
	 * not. 
	 */
	bool bChildAnimMontage : 1;

	/************************************************************************/
	/* Status Bar                                                           */
	/************************************************************************/
	TSharedPtr<STextBlock> StatusBarTextBlock;
	TSharedPtr<SImage> StatusBarWarningImage;

	void OnSlotNameChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo, int32 AnimSlotIndex);
	void OnSlotListOpening(int32 AnimSlotIndex);

	void RefreshComboLists(bool bOnlyRefreshIfDifferent = false);
	
	void CreateNewSlot(const FText& NewSlotName, ETextCommit::Type CommitInfo);
	void CreateNewSection(const FText& NewSectionName, ETextCommit::Type CommitInfo, float StartTime);

	/** Called when a segment is removed from a track, so we can adjust the indices in linkable elements
	 *	@param SegmentIndex - Index of the removed segment
	 */
	void OnAnimSegmentRemoved(int32 SegmentIndex, int32 SlotIndex);

	/** Gets the length of the montage we are currently editing
	 */
	virtual float GetSequenceLength() const override;

	/** Delegate used to invoke a tab in the containing editor */
	FOnInvokeTab OnInvokeTab;
};
