// Copyright Epic Games, Inc. All Rights Reserved.
#include "SAnimMontageScrubPanel.h"
#include "AnimPreviewInstance.h"

// keep localization same
#define LOCTEXT_NAMESPACE "AnimationScrubPanel"

void SAnimMontageScrubPanel::Construct( const SAnimMontageScrubPanel::FArguments& InArgs, const TSharedRef<IPersonaPreviewScene>& InPreviewScene)
{
	MontageEditor = InArgs._MontageEditor;
	SAnimationScrubPanel::Construct( SAnimationScrubPanel::FArguments()
		.LockedSequence(InArgs._LockedSequence)
		.ViewInputMin(InArgs._ViewInputMin)
		.ViewInputMax(InArgs._ViewInputMax)
		.OnSetInputViewRange(InArgs._OnSetInputViewRange)
		.OnCropAnimSequence(InArgs._OnCropAnimSequence)
		.OnReZeroAnimSequence(InArgs._OnReZeroAnimSequence)
		.bAllowZoom(InArgs._bAllowZoom),
		InPreviewScene
		);
}

FReply SAnimMontageScrubPanel::OnClick_ToggleLoop()
{
	if (UAnimPreviewInstance* AnimPreviewInstance = Cast<UAnimPreviewInstance>(GetPreviewInstance()))
	{
		bool bIsLooping = AnimPreviewInstance->IsLooping();
		AnimPreviewInstance->MontagePreview_SetLooping(!bIsLooping);
	}
	return FReply::Handled();
}

FReply SAnimMontageScrubPanel::OnClick_Backward()
{
	if (UAnimPreviewInstance* AnimPreviewInstance = Cast<UAnimPreviewInstance>(GetPreviewInstance()))
	{
		if (! AnimPreviewInstance->IsPlaying() || ! AnimPreviewInstance->IsReverse())
		{
			// check if there is montage being played, if not, start it, if there is, just unpause
			if (! AnimPreviewInstance->IsPlayingMontage())
			{
				AnimPreviewInstance->MontagePreview_SetReverse(true);
				AnimPreviewInstance->MontagePreview_Restart();
			}
			else
			{
				AnimPreviewInstance->MontagePreview_SetReverse(true);
				AnimPreviewInstance->MontagePreview_SetPlaying(true);
			}
		}
		else
		{
			AnimPreviewInstance->MontagePreview_SetPlaying(false);
		}
		return FReply::Handled();
	}

	return SAnimationScrubPanel::OnClick_Backward();
}

FReply SAnimMontageScrubPanel::OnClick_Forward()
{
	if (UAnimPreviewInstance* AnimPreviewInstance = Cast<UAnimPreviewInstance>(GetPreviewInstance()))
	{
		if (! AnimPreviewInstance->IsPlaying() || AnimPreviewInstance->IsReverse())
		{
			// check if there is montage being played, if not, start it, if there is, just unpause
			if (! AnimPreviewInstance->IsPlayingMontage())
			{
				AnimPreviewInstance->MontagePreview_SetReverse(false);
				AnimPreviewInstance->MontagePreview_Restart();
			}
			else
			{
				AnimPreviewInstance->MontagePreview_SetReverse(false);
				AnimPreviewInstance->MontagePreview_SetPlaying(true);
			}
		}
		else
		{
			AnimPreviewInstance->MontagePreview_SetPlaying(false);
		}
		return FReply::Handled();
	}

	return SAnimationScrubPanel::OnClick_Forward();
}

FReply SAnimMontageScrubPanel::OnClick_Backward_End()
{
	if (UAnimPreviewInstance* AnimPreviewInstance = Cast<UAnimPreviewInstance>(GetPreviewInstance()))
	{
		AnimPreviewInstance->MontagePreview_JumpToStart();
	}
	return FReply::Handled();
}

FReply SAnimMontageScrubPanel::OnClick_Forward_End()
{
	if (UAnimPreviewInstance* AnimPreviewInstance = Cast<UAnimPreviewInstance>(GetPreviewInstance()))
	{
		AnimPreviewInstance->MontagePreview_JumpToEnd();
	}
	return FReply::Handled();
}

FReply SAnimMontageScrubPanel::OnClick_Backward_Step()
{
	if (UAnimPreviewInstance* AnimPreviewInstance = Cast<UAnimPreviewInstance>(GetPreviewInstance()))
	{
		AnimPreviewInstance->MontagePreview_StepBackward();
	}
	return FReply::Handled();
}

FReply SAnimMontageScrubPanel::OnClick_Forward_Step()
{
	if (UAnimPreviewInstance* AnimPreviewInstance = Cast<UAnimPreviewInstance>(GetPreviewInstance()))
	{
		AnimPreviewInstance->MontagePreview_StepForward();
	}
	return FReply::Handled();
}

void SAnimMontageScrubPanel::OnValueChanged(float NewValue, bool bFireNotifies)
{
	if (UAnimPreviewInstance* AnimPreviewInstance = Cast<UAnimPreviewInstance>(GetPreviewInstance()))
	{
		AnimPreviewInstance->MontagePreview_JumpToPosition(NewValue);
	}
}

#undef LOCTEXT_NAMESPACE
