// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "IPersonaPreviewScene.h"
#include "SAnimationScrubPanel.h"
#include "SMontageEditor.h"

class SAnimMontageScrubPanel : public SAnimationScrubPanel
{
public:

	SLATE_BEGIN_ARGS(SAnimMontageScrubPanel)
		: _MontageEditor()
		, _LockedSequence()
	{}
		SLATE_ARGUMENT( TWeakPtr<class SMontageEditor>, MontageEditor)
		/** If you'd like to lock to one asset for this scrub control, give this**/
		SLATE_ARGUMENT(UAnimSequenceBase*, LockedSequence)
		/** View Input range **/
		SLATE_ATTRIBUTE( float, ViewInputMin )
		SLATE_ATTRIBUTE( float, ViewInputMax )
		SLATE_EVENT( FOnSetInputViewRange, OnSetInputViewRange )
		/** Called when an anim sequence is cropped before/after a selected frame */
		SLATE_EVENT( FOnCropAnimSequence, OnCropAnimSequence )
		/** Called to zero out selected frame's translation from origin */
		SLATE_EVENT( FSimpleDelegate, OnReZeroAnimSequence )
		SLATE_ARGUMENT( bool, bAllowZoom )
	SLATE_END_ARGS()

	/**
	 * Construct the widget
	 * 
	 * @param InArgs   A declaration from which to construct the widget
	 */
	void Construct( const FArguments& InArgs, const TSharedRef<IPersonaPreviewScene>& InPreviewScene );
protected:
	// notifiers 
	virtual FReply OnClick_Backward_End() override;
	virtual FReply OnClick_Forward_End() override;

	virtual FReply OnClick_Backward_Step() override;
	virtual FReply OnClick_Forward_Step() override;

	virtual FReply OnClick_Forward() override;
	virtual FReply OnClick_Backward() override;

	virtual FReply OnClick_ToggleLoop() override;

	virtual void OnValueChanged(float NewValue, bool bFireNotifies = true) override;

private:
	TWeakPtr<SMontageEditor> MontageEditor;
};
