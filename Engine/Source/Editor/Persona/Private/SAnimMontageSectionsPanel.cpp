// Copyright Epic Games, Inc. All Rights Reserved.


#include "SAnimMontageSectionsPanel.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "SMontageEditor.h"
#include "Widgets/Input/SButton.h"

#include "ScopedTransaction.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "IPersonaToolkit.h"
#include "PersonaTabs.h"
#include "IDocumentation.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/Text/STextBlock.h"
#include "EditorFontGlyphs.h"
#include "Preferences/PersonaOptions.h"
#include "Widgets/Input/SComboButton.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "AnimPreviewInstance.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "Widgets/Layout/SScrollBox.h"

#define LOCTEXT_NAMESPACE "AnimMontageSectionsPanel"

FAnimMontageSectionsSummoner::FAnimMontageSectionsSummoner(TSharedPtr<class FAssetEditorToolkit> InHostingApp, const TSharedRef<IPersonaToolkit>& InPersonaToolkit, FSimpleMulticastDelegate& InOnSectionsChanged)
	: FWorkflowTabFactory(FPersonaTabs::AnimMontageSectionsID, InHostingApp)
	, PersonaToolkit(InPersonaToolkit)
	, OnSectionsChanged(InOnSectionsChanged)
{
	TabLabel = LOCTEXT("MontageSectionsTabTitle", "Montage Sections");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Persona.Tabs.AnimSlotManager");

	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("MontageSectionsMenu", "Montage Sections");
	ViewMenuTooltip = LOCTEXT("MontageSectionsMenu_ToolTip", "Manage montage's sections.");
}

TSharedRef<SWidget> FAnimMontageSectionsSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	return SNew(SAnimMontageSectionsPanel, PersonaToolkit.Pin().ToSharedRef(), OnSectionsChanged);
}

TSharedPtr<SToolTip> FAnimMontageSectionsSummoner::CreateTabToolTipWidget(const FWorkflowTabSpawnInfo& Info) const
{
	return IDocumentation::Get()->CreateToolTip(LOCTEXT("WindowTooltip", "This tab lets you modify a montage's sections"), NULL, TEXT("Shared/Editors/Persona"), TEXT("AnimMontageSections_Window"));
}

//////////////////////////////////////////////////////////////////////////
// SAnimMontagePanel

void SAnimMontageSectionsPanel::Construct(const FArguments& InArgs, const TSharedRef<IPersonaToolkit>& InPersonaToolkit, FSimpleMulticastDelegate& InOnSectionsChanged)
{
	WeakPersonaToolkit = InPersonaToolkit;

	InOnSectionsChanged.Add(FSimpleDelegate::CreateSP(this, &SAnimMontageSectionsPanel::Update));

	UAnimMontage* Montage = Cast<UAnimMontage>(WeakPersonaToolkit.Pin()->GetAnimationAsset());
	if(Montage)
	{
		Montage->RegisterOnMontageChanged(UAnimMontage::FOnMontageChanged::CreateSP(this, &SAnimMontageSectionsPanel::Update));
	}

	if(GEditor)
	{
		GEditor->RegisterForUndo(this);
	}

	this->ChildSlot
	[
		SAssignNew( PanelArea, SBorder )
		.BorderImage( FAppStyle::GetBrush("NoBorder") )
		.ColorAndOpacity( FLinearColor::White )
	];

	Update();
}

SAnimMontageSectionsPanel::~SAnimMontageSectionsPanel()
{
	if(TSharedPtr<IPersonaToolkit> PersonaToolkit = WeakPersonaToolkit.Pin())
	{
		UAnimMontage* Montage = Cast<UAnimMontage>(PersonaToolkit->GetAnimationAsset());
		if(Montage)
		{
			Montage->UnregisterOnMontageChanged(this);
		}
	}

	if(GEditor)
	{
		GEditor->RegisterForUndo(this);
	}
}

void SAnimMontageSectionsPanel::Update()
{
	int32 ColorIdx=0;
	FLinearColor Colors[] = { FLinearColor(0.9f, 0.9f, 0.9f, 0.9f), FLinearColor(0.5f, 0.5f, 0.5f) };
	FLinearColor NodeColor = GetDefault<UPersonaOptions>()->SectionTimingNodeColor;
	FLinearColor SelectedColor = FLinearColor(1.0f,0.65f,0.0f);
	FLinearColor LoopColor = FLinearColor(0.0f, 0.25f, 0.25f, 0.5f);

	UAnimMontage* Montage = Cast<UAnimMontage>(WeakPersonaToolkit.Pin()->GetAnimationAsset());
	if ( Montage != nullptr )
	{
		bool bChildAnimMontage = Montage->HasParentAsset();

		TSharedPtr<STrack> Track;
		TSharedPtr<SVerticalBox> ContentArea;
		PanelArea->SetContent(
			SAssignNew( ContentArea, SVerticalBox )
			);

		ContentArea->ClearChildren();
		
		TArray<bool>	Used;
		Used.AddZeroed(Montage->CompositeSections.Num());

		int RowIdx=0;

		/** Create Buttons for reseting/creating default section ordering */
		ContentArea->AddSlot()
			.AutoHeight()
			.VAlign(VAlign_Center)
			[
				SNew(SBorder)
				.BorderImage(FCoreStyle::Get().GetBrush("ToolPanel.GroupBorder"))
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(2.0f)
					[				
						SNew(SButton)
						.ButtonStyle(FAppStyle::Get(), "FlatButton.Success")
						.IsEnabled(!bChildAnimMontage)
						.ToolTipText( LOCTEXT("CreateDefaultToolTip", "Reconstructs section ordering based on start time") )
						.OnClicked(this, &SAnimMontageSectionsPanel::MakeDefaultSequence)
						[
							SNew(SHorizontalBox)
							+SHorizontalBox::Slot()
							.VAlign(VAlign_Center)
							.Padding(0.0f, 0.0f, 2.0f, 0.0f)
							.AutoWidth()
							[
								SNew(STextBlock)
								.Text(FEditorFontGlyphs::File)
								.TextStyle(FAppStyle::Get(), "FlatButton.DefaultTextStyle")
								.Font(FAppStyle::Get().GetFontStyle("FontAwesome.11"))
							]
							+SHorizontalBox::Slot()
							.VAlign(VAlign_Center)
							.AutoWidth()
							[
								SNew(STextBlock)
								.Text( LOCTEXT("CreateDefault", "Create Default") )
								.TextStyle(FAppStyle::Get(), "FlatButton.DefaultTextStyle")
							]
						]
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(2.0f)
					[				
						SNew(SButton)
						.ButtonStyle(FAppStyle::Get(), "FlatButton.Danger")
						.TextStyle(FAppStyle::Get(), "FlatButton.DefaultTextStyle")
						.IsEnabled(!bChildAnimMontage)
						.ToolTipText( LOCTEXT("ClearToolTip", "Resets section orderings") )
						.OnClicked(this, &SAnimMontageSectionsPanel::ClearSequence)
						[
							SNew(SHorizontalBox)
							+SHorizontalBox::Slot()
							.VAlign(VAlign_Center)
							.Padding(0.0f, 0.0f, 2.0f, 0.0f)
							.AutoWidth()
							[
								SNew(STextBlock)
								.Text(FEditorFontGlyphs::Times)
								.TextStyle(FAppStyle::Get(), "FlatButton.DefaultTextStyle")
								.Font(FAppStyle::Get().GetFontStyle("FontAwesome.11"))
							]
							+SHorizontalBox::Slot()
							.VAlign(VAlign_Center)
							.AutoWidth()
							[
								SNew(STextBlock)
								.Text( LOCTEXT("Clear", "Clear") )
								.TextStyle(FAppStyle::Get(), "FlatButton.DefaultTextStyle")
							]
						]
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(2.0f)
					[				
						SNew(SButton)
						.ButtonStyle(FAppStyle::Get(), "FlatButton")
						.ToolTipText( LOCTEXT("PreviewAllToolTip", "Preview all sections in their specified order") )
						.OnClicked(this, &SAnimMontageSectionsPanel::PreviewAllSectionsClicked)
						[
							SNew(SHorizontalBox)
							+SHorizontalBox::Slot()
							.VAlign(VAlign_Center)
							.Padding(0.0f, 0.0f, 2.0f, 0.0f)
							.AutoWidth()
							[
								SNew(STextBlock)
								.Text(FEditorFontGlyphs::Play)
								.TextStyle(FAppStyle::Get(), "FlatButton.DefaultTextStyle")
								.Font(FAppStyle::Get().GetFontStyle("FontAwesome.11"))
							]
							+SHorizontalBox::Slot()
							.VAlign(VAlign_Center)
							.AutoWidth()
							[
								SNew(STextBlock)
								.Text( LOCTEXT("PreviewAll", "Preview All") )
								.TextStyle(FAppStyle::Get(), "FlatButton.DefaultTextStyle")
							]
						]
					]
				]
			];

		TSharedPtr<SVerticalBox> MontageSlots;
		ContentArea->AddSlot()
		[
			SNew(SScrollBox)
			+SScrollBox::Slot()
			[
				SAssignNew(MontageSlots, SVerticalBox)
			]
		];

		/** Create as many tracks as necessary to show each section at least once
		  * -Each track represents one chain of sections (section->next->next)
		  */
		TSharedPtr<SWrapBox> WrapBox;

		while(true)
		{
			int32 SectionIdx = 0;
			TArray<bool>	UsedInThisRow;
			UsedInThisRow.AddZeroed(Montage->CompositeSections.Num());
			
			/** Find first section we haven't shown yet */
			for(;SectionIdx < Montage->CompositeSections.Num(); SectionIdx++)
			{
				if(!Used[SectionIdx])
					break;
			}

			if(SectionIdx >= Montage->CompositeSections.Num())
			{
				// Ran out of stuff to show - done
				break;
			}

			/** Create new track */
			MontageSlots->AddSlot()
				.AutoHeight()
				.VAlign(VAlign_Center)
				.Padding(0.0f, 2.0f, 0.0f, 0.0f)
				[
					SNew(SBorder)
					.BorderImage(FCoreStyle::Get().GetBrush("ToolPanel.GroupBorder"))
					[
						SAssignNew(WrapBox, SWrapBox)
						.UseAllottedSize(true)
					]
				];
			
			WrapBox->AddSlot()
			.Padding(2.0f)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				[
					SNew(SButton)
					.ToolTipText( LOCTEXT("PreviewToolTip", "Preview this track") )
					.ButtonStyle(FAppStyle::Get(), "FlatButton")
					.TextStyle(FAppStyle::Get(), "FlatButton.DefaultTextStyle")
					.OnClicked(this, &SAnimMontageSectionsPanel::PreviewSectionClicked, SectionIdx)
					[
						SNew(SHorizontalBox)
						+SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.Padding(0.0f, 0.0f, 2.0f, 0.0f)
						.AutoWidth()
						[
							SNew(STextBlock)
							.Text(FEditorFontGlyphs::Play)
							.TextStyle(FAppStyle::Get(), "FlatButton.DefaultTextStyle")
							.Font(FAppStyle::Get().GetFontStyle("FontAwesome.11"))
						]
						+SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.AutoWidth()
						[
							SNew(STextBlock)
							.Text( LOCTEXT("Preview", "Preview") )
							.TextStyle(FAppStyle::Get(), "FlatButton.DefaultTextStyle")
						]
					]
				]
			];

			/** Add each track in this chain to the track we just created */
			while(true)
			{
				/** Add section if it hasn't already been used in this row (if its used in another row, thats ok) */
				if(Montage->IsValidSectionIndex(SectionIdx) && UsedInThisRow[SectionIdx]==false)
				{
					UsedInThisRow[SectionIdx] = true;
					Used[SectionIdx] = true;
				
					int32 NextSectionIdx = Montage->GetSectionIndex( Montage->CompositeSections[SectionIdx].NextSectionName );

					WrapBox->AddSlot()
					.Padding(2.0f)
					[
						SNew(SButton)
						.IsEnabled(!bChildAnimMontage)
						.ButtonColorAndOpacity(IsLoop(SectionIdx) ? LoopColor : NodeColor)
						[
							SNew(SHorizontalBox)
							+SHorizontalBox::Slot()
							.VAlign(VAlign_Center)
							.Padding(0.0f, 0.0f, 2.0f, 0.0f)
							.AutoWidth()
							[
								SNew(STextBlock)
								.Text(FText::FromName(Montage->CompositeSections[SectionIdx].SectionName))
								.TextStyle(FAppStyle::Get(), "FlatButton.DefaultTextStyle")
							]
						]
					];

					FText TextIcon;
					FText ToolTipText;
					if(NextSectionIdx != INDEX_NONE)
					{
						if(NextSectionIdx == SectionIdx)
						{
							TextIcon = FEditorFontGlyphs::Undo;
							ToolTipText = LOCTEXT("NextSectionLoopedTooltip", "Pick the next section (currently looped).");
						}
						else
						{
							if(NextSectionIdx > SectionIdx)
							{
								TextIcon = FEditorFontGlyphs::Arrow_Right;
							}
							else
							{
								TextIcon = FEditorFontGlyphs::Arrow_Left;
							}
							ToolTipText = FText::Format(LOCTEXT("NextSectionWithCurrentTooltip", "Pick the next section (currently {0})."), FText::FromName(Montage->CompositeSections[SectionIdx].NextSectionName));
						}
					}
					else
					{
						TextIcon = FEditorFontGlyphs::Square;
						ToolTipText = LOCTEXT("NextSectionTooltip", "Pick the next section.");
					}

					WrapBox->AddSlot()
					.Padding(2.0f)
					.VAlign(VAlign_Center)
					[
						SNew(SHorizontalBox)
						+SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						[
							SNew(SComboButton)
							.ToolTipText(ToolTipText)
							.IsEnabled(!bChildAnimMontage)
							.ButtonStyle(FAppStyle::Get(), "FlatButton")
							.HasDownArrow(false)
							.OnGetMenuContent(this, &SAnimMontageSectionsPanel::OnGetSectionMenuContent, SectionIdx)
							.ContentPadding(FMargin(4.0, 2.0))
							.ButtonContent()
							[
								SNew(STextBlock)
								.Text(TextIcon)
								.TextStyle(FAppStyle::Get(), "FlatButton.DefaultTextStyle")
								.Font(FAppStyle::Get().GetFontStyle("FontAwesome.11"))
							]
						]
					];

					SectionIdx = NextSectionIdx;

					continue;
				} 
				else
				{
					break;
				}
			}

		}

		RowIdx++;

	}
	else
	{
		PanelArea->ClearContent();
	}
}

TSharedRef<SWidget> SAnimMontageSectionsPanel::OnGetSectionMenuContent(int32 SectionIdx)
{
	UAnimMontage* Montage = Cast<UAnimMontage>(WeakPersonaToolkit.Pin()->GetAnimationAsset());

	FMenuBuilder MenuBuilder(true, nullptr);

	MenuBuilder.BeginSection("Section", LOCTEXT("SectionHeader", "Section"));
	{
		if(Montage->CompositeSections[SectionIdx].NextSectionName != NAME_None)
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("RemoveLink", "Remove Link"),
				LOCTEXT("RemoveLinkTooltip", "Remove the link between this section and the next"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateSP(this, &SAnimMontageSectionsPanel::RemoveLink, SectionIdx))
			);
		}
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("NextSection", LOCTEXT("NExtSectionHeader", "Next Section"));
	{
		for(int32 NextSectionIdx = 0; NextSectionIdx < Montage->CompositeSections.Num(); NextSectionIdx++)
		{
			FText SectionNameText = FText::FromName(Montage->CompositeSections[NextSectionIdx].SectionName);
			MenuBuilder.AddMenuEntry(
				SectionNameText,
				FText::Format(LOCTEXT("SetNextSectionTooltip", "Set {0} to be the next section"), SectionNameText),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateSP(this, &SAnimMontageSectionsPanel::SetNextSectionIndex, SectionIdx, NextSectionIdx))
			);
		}
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void SAnimMontageSectionsPanel::RestartPreview()
{
	if (UDebugSkelMeshComponent* MeshComponent = WeakPersonaToolkit.Pin()->GetPreviewScene()->GetPreviewMeshComponent())
	{
		if (UAnimPreviewInstance* Preview = MeshComponent->PreviewInstance)
		{
			Preview->MontagePreview_PreviewNormal(INDEX_NONE, Preview->IsPlaying());
		}
	}
}

void SAnimMontageSectionsPanel::RestartPreviewPlayAllSections()
{
	if (UDebugSkelMeshComponent* MeshComponent = WeakPersonaToolkit.Pin()->GetPreviewScene()->GetPreviewMeshComponent())
	{
		if (UAnimPreviewInstance* Preview = MeshComponent->PreviewInstance)
		{
			Preview->MontagePreview_PreviewAllSections(Preview->IsPlaying());
		}
	}
}

void SAnimMontageSectionsPanel::RestartPreviewFromSection(int32 FromSectionIdx)
{
	if (UDebugSkelMeshComponent* MeshComponent = WeakPersonaToolkit.Pin()->GetPreviewScene()->GetPreviewMeshComponent())
	{
		if (UAnimPreviewInstance* Preview = MeshComponent->PreviewInstance)
		{
			Preview->MontagePreview_PreviewNormal(FromSectionIdx, Preview->IsPlaying());
		}
	}
}

void SAnimMontageSectionsPanel::SortSections()
{
	UAnimMontage* Montage = Cast<UAnimMontage>(WeakPersonaToolkit.Pin()->GetAnimationAsset());

	struct FCompareSections
	{
		bool operator()( const FCompositeSection &A, const FCompositeSection &B ) const
		{
			return A.GetTime() < B.GetTime();
		}
	};
	if (Montage != nullptr)
	{
		Montage->CompositeSections.Sort(FCompareSections());
	}
}

void SAnimMontageSectionsPanel::MakeDefaultSequentialSections()
{
	UAnimMontage* Montage = Cast<UAnimMontage>(WeakPersonaToolkit.Pin()->GetAnimationAsset());
	if( Montage != nullptr )
	{
		const FScopedTransaction Transaction( LOCTEXT("MakeDefaultSequentialSections", "Make default sequential sections") );
		Montage->Modify();

		SortSections();
		for(int32 SectionIdx=0; SectionIdx < Montage->CompositeSections.Num(); SectionIdx++)
		{
			Montage->CompositeSections[SectionIdx].NextSectionName = Montage->CompositeSections.IsValidIndex(SectionIdx+1) ? Montage->CompositeSections[SectionIdx+1].SectionName : NAME_None;
		}
		RestartPreview();
		Update();

		Montage->PostEditChange();
	}
}

void SAnimMontageSectionsPanel::ClearSequenceOrdering()
{
	UAnimMontage* Montage = Cast<UAnimMontage>(WeakPersonaToolkit.Pin()->GetAnimationAsset());
	if( Montage != nullptr )
	{
		const FScopedTransaction Transaction( LOCTEXT("ClearSectionOrdering", "Clear section ordering") );
		Montage->Modify();

		SortSections();
		for(int32 SectionIdx=0; SectionIdx < Montage->CompositeSections.Num(); SectionIdx++)
		{
			Montage->CompositeSections[SectionIdx].NextSectionName = NAME_None;
		}
		RestartPreview();
	}
}

void SAnimMontageSectionsPanel::SetNextSectionIndex(int32 SectionIdx, int32 NextSectionIndex)
{
	UAnimMontage* Montage = Cast<UAnimMontage>(WeakPersonaToolkit.Pin()->GetAnimationAsset());
	if(Montage && Montage->IsValidSectionIndex(SectionIdx) && Montage->IsValidSectionIndex(NextSectionIndex))
	{
		const FScopedTransaction Transaction( LOCTEXT("SetNextSection", "Set next section") );
		Montage->Modify();

		Montage->CompositeSections[SectionIdx].NextSectionName = Montage->CompositeSections[NextSectionIndex].SectionName;
		Update();
		RestartPreview();
	}
}

FReply SAnimMontageSectionsPanel::PreviewAllSectionsClicked()
{
	RestartPreviewPlayAllSections(); 
	return FReply::Handled();
}

FReply SAnimMontageSectionsPanel::PreviewSectionClicked(int32 SectionIndex)
{
	RestartPreviewFromSection(SectionIndex);
	return FReply::Handled();
}

void SAnimMontageSectionsPanel::RemoveLink(int32 SectionIndex)
{
	UAnimMontage* Montage = Cast<UAnimMontage>(WeakPersonaToolkit.Pin()->GetAnimationAsset());
	if(Montage && Montage->IsValidSectionIndex(SectionIndex))
	{
		const FScopedTransaction Transaction( LOCTEXT("RemoveNextSection", "Remove Next Section from Montage") );
		Montage->Modify();
		
		Montage->CompositeSections[SectionIndex].NextSectionName = NAME_None;
		RestartPreview();
		Update();

		Montage->PostEditChange();
	}
}

FReply SAnimMontageSectionsPanel::MakeDefaultSequence()
{
	UAnimMontage* Montage = Cast<UAnimMontage>(WeakPersonaToolkit.Pin()->GetAnimationAsset());
	if(Montage)
	{
		MakeDefaultSequentialSections();
		Update();
		Montage->PostEditChange();
	}

	return FReply::Handled();
}

FReply SAnimMontageSectionsPanel::ClearSequence()
{
	UAnimMontage* Montage = Cast<UAnimMontage>(WeakPersonaToolkit.Pin()->GetAnimationAsset());
	if(Montage)
	{
		ClearSequenceOrdering();
		Update();
		Montage->PostEditChange();
	}

	return FReply::Handled();
}

bool SAnimMontageSectionsPanel::IsLoop(int32 SectionIdx)
{
	UAnimMontage* Montage = Cast<UAnimMontage>(WeakPersonaToolkit.Pin()->GetAnimationAsset());
	if(Montage && Montage->CompositeSections.IsValidIndex(SectionIdx))
	{
		TArray<bool>	Used;
		Used.AddZeroed(Montage->CompositeSections.Num());
		int32 CurrentIdx = SectionIdx;
		while(true)
		{
			CurrentIdx = Montage->GetSectionIndex(Montage->CompositeSections[CurrentIdx].NextSectionName);
			if(CurrentIdx == INDEX_NONE)
			{
				// End of the chain
				return false;
			}
			if(CurrentIdx == SectionIdx)
			{
				// Hit the starting position, return true
				return true;
			}
			if(Used[CurrentIdx])
			{
				// Hit a loop but the starting section was part of it, so return false
				return false;
			}
			Used[CurrentIdx]=true;
		}
	}
	return false;
}

#undef LOCTEXT_NAMESPACE
