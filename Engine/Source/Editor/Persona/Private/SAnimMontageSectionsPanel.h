// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "SNodePanel.h"
#include "STrack.h"
#include "WorkflowOrientedApp/WorkflowTabFactory.h"
#include "EditorUndoClient.h"

class SBorder;
class SMontageEditor;
class UAnimMontage;

// Forward declatations
class UAnimMontage;
class SMontageEditor;
class IPersonaToolkit;

struct FAnimMontageSectionsSummoner : public FWorkflowTabFactory
{
public:
	FAnimMontageSectionsSummoner(TSharedPtr<class FAssetEditorToolkit> InHostingApp, const TSharedRef<class IPersonaToolkit>& InPersonaToolkit, FSimpleMulticastDelegate& InOnSectionsChanged);

	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;
	virtual TSharedPtr<SToolTip> CreateTabToolTipWidget(const FWorkflowTabSpawnInfo& Info) const override;

public:
	TWeakPtr<class IPersonaToolkit> PersonaToolkit;
	FSimpleMulticastDelegate& OnSectionsChanged;
};

class SAnimMontageSectionsPanel : public SCompoundWidget, public FEditorUndoClient
{
public:
	SLATE_BEGIN_ARGS( SAnimMontageSectionsPanel )
	{}

	SLATE_END_ARGS()

	~SAnimMontageSectionsPanel();

	// FEditorUndoClient interface
	virtual void PostUndo(bool bSuccess) { Update(); };
	virtual void PostRedo(bool bSuccess) { Update(); };

	void Construct(const FArguments& InArgs, const TSharedRef<IPersonaToolkit>& InPersonaToolkit, FSimpleMulticastDelegate& InOnSectionsChanged);

	// Rebuild panel widgets
	void Update();

	// Callback when a section in the upper display is clicked
	// @param SectionIdx - Section that was clicked
	// @param NextSecitonIdx - Section to set as the next section
	void SetNextSectionIndex(int32 SectionIdx, int32 NextSecitonIdx);

	// Unlinks the requested section
	// @param SectionIndex - Section to unlink
	void RemoveLink(int32 SectionIndex);


	// Set section to play sequentially (default behavior)
	FReply MakeDefaultSequence();
	// Completely remove section sequence data
	FReply ClearSequence();

	// Starts playing from the first section
	FReply PreviewAllSectionsClicked();
	// Plays the clicked section
	// @param SectionIndex - Section to play
	FReply PreviewSectionClicked(int32 SectionIndex);

private:

	/** Shows the context menu for a section */
	TSharedRef<SWidget> OnGetSectionMenuContent(int32 SectionIdx);

	/** Restarts the preview animation */
	void RestartPreview();

	/** Restarts the preview animation playing all sections */
	void RestartPreviewPlayAllSections();

	/** Restarts the preview animation from the specified sections */
	void RestartPreviewFromSection(int32 FromSectionIdx);

	void SortSections();

	void MakeDefaultSequentialSections();

	void ClearSequenceOrdering();

	// Returns whether or not the provided section is part of a loop
	// @param SectionIdx - Section to check
	bool IsLoop(int32 SectionIdx);

	// Main panel area widget
	TSharedPtr<SBorder>			PanelArea;

	// Persona toolkit we are hosted in
	TWeakPtr<IPersonaToolkit> WeakPersonaToolkit;
};
