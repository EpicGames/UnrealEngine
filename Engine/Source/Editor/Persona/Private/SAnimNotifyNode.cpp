// Copyright Epic Games, Inc. All Rights Reserved.


#include "SAnimNotifyNode.h"
#include "Rendering/DrawElements.h"
#include "Misc/MessageDialog.h"
#include "Modules/ModuleManager.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"
#include "UObject/PropertyPortFlags.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Animation/AnimSequence.h"
#include "AnimTimeline/AnimTimelineTrack_NotifiesPanel.h"
#include "Fonts/FontMeasure.h"

// AnimNotify Drawing
const float NotifyHeightOffset = 0.f;
const float NotifyHeight = FAnimTimelineTrack_NotifiesPanel::NotificationTrackHeight;
const FVector2D ScrubHandleSize(12.0f, 12.0f);
const FVector2D AlignmentMarkerSize(10.f, 20.f);
const FVector2D TextBorderSize(1.f, 1.f);

#define LOCTEXT_NAMESPACE "AnimNotifyNode"

//////////////////////////////////////////////////////////////////////////
// SAnimNotifyNode

const float SAnimNotifyNode::MinimumStateDuration = (1.0f / 30.0f);

void SAnimNotifyNode::Construct(const FArguments& InArgs)
{
	Font = FCoreStyle::GetDefaultFontStyle("Regular", 10);
	bBeingDragged = false;
	CurrentDragHandle = ENotifyStateHandleHit::None;
	bDrawTooltipToRight = true;
	bSelected = false;
	DragMarkerTransactionIdx = INDEX_NONE;

	SetClipping(EWidgetClipping::ClipToBounds);

	SetToolTipText(TAttribute<FText>(this, &SAnimNotifyNode::GetNodeTooltip));
}

void SAnimNotifyNode::SetSequence(class UAnimSequenceBase* InSequence)
{
	Sequence = InSequence;
}

void SAnimNotifyNode::SetAnimNotify(FAnimNotifyEvent* InAnimNotifyEvent)
{
	MakeNodeInterface<FNotifyNodeInterface>(InAnimNotifyEvent);
	NodeObjectInterface->CacheName();
}

void SAnimNotifyNode::SetAnimSyncMarker(FAnimSyncMarker* InMarker)
{
	MakeNodeInterface<FSyncMarkerNodeInterface>(InMarker);
	NodeObjectInterface->CacheName();
}

void SAnimNotifyNode::SetOnNotifyNodeDragStarted(FOnNotifyNodeDragStarted InDragStarted)
{
	OnNodeDragStarted = InDragStarted;
}

void SAnimNotifyNode::SetOnNotifyStateHandleBeingDragged(FOnNotifyStateHandleBeingDragged InHandleBeingDragged)
{
	OnNotifyStateHandleBeingDragged = InHandleBeingDragged;
}

void SAnimNotifyNode::SetOnUpdatePanel(FOnUpdatePanel InOnUpdatePanel)
{
	OnUpdatePanel = InOnUpdatePanel;
}

void SAnimNotifyNode::SetPanTrackRequest(FPanTrackRequest InPanTrackRequest)
{
	PanTrackRequest = InPanTrackRequest;
}

void SAnimNotifyNode::SetOnTrackSelectionChanged(FOnTrackSelectionChanged InOnTrackSelectionChanged)
{
	OnSelectionChanged = InOnTrackSelectionChanged;
}

void SAnimNotifyNode::SetViewInputMin(TAttribute<float> InViewInputMin)
{
	ViewInputMin = InViewInputMin;
}

void SAnimNotifyNode::SetViewInputMax(TAttribute<float> InViewInputMax)
{
	ViewInputMax = InViewInputMax;
}

void SAnimNotifyNode::SetStateEndTimingNode(TSharedPtr<SAnimTimingNode> InStateEndTimingNode)
{
	if (InStateEndTimingNode.IsValid())
	{
		// The overlay will use the desired size to calculate the notify node size,
		// compute that once here.
		InStateEndTimingNode->SlatePrepass(1.0f);
		SAssignNew(EndMarkerNodeOverlay, SOverlay)
			+ SOverlay::Slot()
			[
				InStateEndTimingNode.ToSharedRef()
			];
	}
}

void SAnimNotifyNode::SetOnSnapPosition(FOnSnapPosition InOnSnapPosition)
{
	OnSnapPosition = InOnSnapPosition;
}

FReply SAnimNotifyNode::OnDragDetected( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	FVector2D ScreenNodePosition = FVector2D(MyGeometry.AbsolutePosition);
	
	// Whether the drag has hit a duration marker
	bool bDragOnMarker = false;
	bBeingDragged = true;

	if(GetDurationSize() > 0.0f)
	{
		// This is a state node, check for a drag on the markers before movement. Use last screen space position before the drag started
		// as using the last position in the mouse event gives us a mouse position after the drag was started.
		ENotifyStateHandleHit::Type MarkerHit = DurationHandleHitTest(LastMouseDownPosition);
		if(MarkerHit == ENotifyStateHandleHit::Start || MarkerHit == ENotifyStateHandleHit::End)
		{
			bDragOnMarker = true;
			bBeingDragged = false;
			CurrentDragHandle = MarkerHit;

			// Modify the owning sequence as we're now dragging the marker and begin a transaction
			check(DragMarkerTransactionIdx == INDEX_NONE);
			DragMarkerTransactionIdx = GEditor->BeginTransaction(NSLOCTEXT("AnimNotifyNode", "StateNodeDragTransation", "Drag State Node Marker"));
			Sequence->Modify();
		}
	}

	return OnNodeDragStarted.Execute(SharedThis(this), MouseEvent, ScreenNodePosition, bDragOnMarker);
}

FLinearColor SAnimNotifyNode::GetNotifyColor() const
{
	TOptional<FLinearColor> Color = NodeObjectInterface->GetEditorColor();
	FLinearColor BaseColor = Color.Get(FLinearColor(1, 1, 0.5f));

	BaseColor.A = 0.67f;

	return BaseColor;
}

FText SAnimNotifyNode::GetNotifyText() const
{
	// Combine comment from notify struct and from function on object
	return FText::FromName( NodeObjectInterface->GetName() );
}

FText SAnimNotifyNode::GetNodeTooltip() const
{
	return NodeObjectInterface->GetNodeTooltip(Sequence);
}

/** @return the Node's position within the graph */
UObject* SAnimNotifyNode::GetObjectBeingDisplayed() const
{
	TOptional<UObject*> Object = NodeObjectInterface->GetObjectBeingDisplayed();
	return Object.Get(Sequence);
}

void SAnimNotifyNode::DropCancelled()
{
	bBeingDragged = false;
}

FVector2D SAnimNotifyNode::ComputeDesiredSize( float ) const
{
	return GetSize();
}

bool SAnimNotifyNode::HitTest(const FGeometry& AllottedGeometry, FVector2D MouseLocalPose) const
{
	FVector2D Position = GetWidgetPosition();
	FVector2D Size = GetSize();

	return MouseLocalPose.ComponentwiseAllGreaterOrEqual(Position)
		&& MouseLocalPose.ComponentwiseAllLessOrEqual(Position + Size);
}

ENotifyStateHandleHit::Type SAnimNotifyNode::DurationHandleHitTest(const FVector2D& CursorTrackPosition) const
{
	ENotifyStateHandleHit::Type MarkerHit = ENotifyStateHandleHit::None;

	// Make sure this node has a duration box (meaning it is a state node)
	if(NotifyDurationSizeX > 0.0f)
	{
		// Test for mouse inside duration box with handles included
		const double ScrubHandleHalfWidth = ScrubHandleSize.X / 2.0f;

		// Position and size of the notify node including the scrub handles
		const FVector2D NotifyNodePosition(NotifyScrubHandleCentre - ScrubHandleHalfWidth, 0.0);
		const FVector2D NotifyNodeSize(NotifyDurationSizeX + ScrubHandleHalfWidth * 2.0, NotifyHeight);

		const FVector2D MouseRelativePosition(CursorTrackPosition - GetWidgetPosition());

		if(MouseRelativePosition.ComponentwiseAllGreaterThan(NotifyNodePosition) &&
			MouseRelativePosition.ComponentwiseAllLessThan(NotifyNodePosition + NotifyNodeSize))
		{
			// Definitely inside the duration box, need to see which handle we hit if any
			if(MouseRelativePosition.X <= (NotifyNodePosition.X + ScrubHandleSize.X))
			{
				// Left Handle
				MarkerHit = ENotifyStateHandleHit::Start;
			}
			else if(MouseRelativePosition.X >= (NotifyNodePosition.X + NotifyNodeSize.X - ScrubHandleSize.X))
			{
				// Right Handle
				MarkerHit = ENotifyStateHandleHit::End;
			}
		}
	}

	return MarkerHit;
}

void SAnimNotifyNode::UpdateSizeAndPosition(const FGeometry& AllottedGeometry)
{
	FTrackScaleInfo ScaleInfo(ViewInputMin.Get(), ViewInputMax.Get(), 0, 0, AllottedGeometry.Size);

	// Cache the geometry information, the alloted geometry is the same size as the track.
	CachedAllotedGeometrySize = AllottedGeometry.Size * AllottedGeometry.Scale;

	NotifyTimePositionX = ScaleInfo.InputToLocalX(NodeObjectInterface->GetTime());
	NotifyDurationSizeX = ScaleInfo.PixelsPerInput * NodeObjectInterface->GetDuration();

	const TSharedRef< FSlateFontMeasure > FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
	TextSize = FontMeasureService->Measure( GetNotifyText(), Font );
	LabelWidth = static_cast<float>(TextSize.X + (TextBorderSize.X * 2.0) + (ScrubHandleSize.X / 2.0));

	const bool bDrawBranchingPoint = NodeObjectInterface->IsBranchingPoint();
	BranchingPointIconSize = FVector2D(TextSize.Y, TextSize.Y);
	if (bDrawBranchingPoint)
	{
		LabelWidth += static_cast<float>(BranchingPointIconSize.X + TextBorderSize.X) * 2.f;
	}

	//Calculate scrub handle box size (the notional box around the scrub handle and the alignment marker)
	const float NotifyHandleBoxWidth = static_cast<float>(FMath::Max(ScrubHandleSize.X, AlignmentMarkerSize.X * 2));

	// Work out where we will have to draw the tool tip
	const float LeftEdgeToNotify = NotifyTimePositionX;
	const float RightEdgeToNotify = static_cast<float>(AllottedGeometry.Size.X) - NotifyTimePositionX;
	bDrawTooltipToRight = NotifyDurationSizeX > 0.0f || ((RightEdgeToNotify > LabelWidth) || (RightEdgeToNotify > LeftEdgeToNotify));

	// Calculate widget width/position based on where we are drawing the tool tip
	WidgetX = bDrawTooltipToRight ? (NotifyTimePositionX - (NotifyHandleBoxWidth / 2.f)) : (NotifyTimePositionX - LabelWidth);
	WidgetSize = bDrawTooltipToRight ? FVector2D((NotifyDurationSizeX > 0.0f ? NotifyDurationSizeX : FMath::Max(LabelWidth, NotifyDurationSizeX)), NotifyHeight) : FVector2D((LabelWidth + NotifyDurationSizeX), NotifyHeight);
	WidgetSize.X += NotifyHandleBoxWidth;
	
	if(EndMarkerNodeOverlay.IsValid())
	{
		const FVector2D OverlaySize = EndMarkerNodeOverlay->GetDesiredSize();
		WidgetSize.X += OverlaySize.X;
	}

	// Widget position of the notify marker
	NotifyScrubHandleCentre = bDrawTooltipToRight ? NotifyHandleBoxWidth / 2.f : LabelWidth;
}

/** @return the Node's position within the track */
FVector2D SAnimNotifyNode::GetWidgetPosition() const
{
	return FVector2D(WidgetX, NotifyHeightOffset);
}

FVector2D SAnimNotifyNode::GetNotifyPosition() const
{
	return FVector2D(NotifyTimePositionX, NotifyHeightOffset);
}

FVector2D SAnimNotifyNode::GetNotifyPositionOffset() const
{
	return GetNotifyPosition() - GetWidgetPosition();
}

FVector2D SAnimNotifyNode::GetSize() const
{
	return WidgetSize;
}

int32 SAnimNotifyNode::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	int32 MarkerLayer = LayerId + 1;
	int32 ScrubHandleID = MarkerLayer + 1;
	int32 TextLayerID = ScrubHandleID + 1;
	int32 BranchPointLayerID = TextLayerID + 1;

	FAnimNotifyEvent* AnimNotifyEvent = NodeObjectInterface->GetNotifyEvent();

	// Paint marker node if we have one
	if(EndMarkerNodeOverlay.IsValid())
	{
		FVector2D MarkerSize = EndMarkerNodeOverlay->GetDesiredSize();
		FVector2D MarkerOffset(NotifyDurationSizeX + MarkerSize.X * 0.5f + 5.0f, (NotifyHeight - MarkerSize.Y) * 0.5f);
		EndMarkerNodeOverlay->Paint(Args.WithNewParent(this), AllottedGeometry.MakeChild(MarkerSize, FSlateLayoutTransform(MarkerOffset)), MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
	}

	const FSlateBrush* StyleInfo = FAppStyle::GetBrush( TEXT("SpecialEditableTextImageNormal") );

	FText Text = GetNotifyText();
	FLinearColor NodeColor = SAnimNotifyNode::GetNotifyColor();
	FLinearColor BoxColor = bSelected ? FAppStyle::GetSlateColor("SelectionColor").GetSpecifiedColor() : SAnimNotifyNode::GetNotifyColor();

	const float HalfScrubHandleWidth = static_cast<float>(ScrubHandleSize.X) / 2.0f;

	// Show duration of AnimNotifyState
	if( NotifyDurationSizeX > 0.f )
	{
		FVector2D DurationBoxSize = FVector2D(NotifyDurationSizeX, TextSize.Y + TextBorderSize.Y * 2.f);
		FVector2D DurationBoxPosition = FVector2D(NotifyScrubHandleCentre, (NotifyHeight - TextSize.Y) * 0.5f);
		FSlateDrawElement::MakeBox( 
			OutDrawElements,
			LayerId, 
			AllottedGeometry.ToPaintGeometry(DurationBoxSize, FSlateLayoutTransform(DurationBoxPosition)), 
			StyleInfo,
			ESlateDrawEffect::None,
			BoxColor);

		DrawScrubHandle(static_cast<float>(DurationBoxPosition.X + DurationBoxSize.X), OutDrawElements, ScrubHandleID, AllottedGeometry, MyCullingRect, NodeColor);
		
		// Render offsets if necessary
		if(AnimNotifyEvent && AnimNotifyEvent->EndTriggerTimeOffset != 0.f) //Do we have an offset to render?
		{
			const float EndTime = AnimNotifyEvent->GetTime() + AnimNotifyEvent->GetDuration();
			if(EndTime != Sequence->GetPlayLength()) //Don't render offset when we are at the end of the sequence, doesnt help the user
			{
				// ScrubHandle
				const float HandleCentre = NotifyDurationSizeX + (static_cast<float>(ScrubHandleSize.X) - 2.0f);
				DrawHandleOffset(AnimNotifyEvent->EndTriggerTimeOffset, HandleCentre, OutDrawElements, MarkerLayer, AllottedGeometry, MyCullingRect, NodeColor);
			}
		}
	}

	// Branching point
	bool bDrawBranchingPoint = AnimNotifyEvent && AnimNotifyEvent->IsBranchingPoint();

	// Background
	FVector2D LabelSize = TextSize + TextBorderSize * 2.f;
	LabelSize.X += HalfScrubHandleWidth + (bDrawBranchingPoint ? (BranchingPointIconSize.X + TextBorderSize.X * 2.f) : 0.f);

	FVector2D LabelPosition(bDrawTooltipToRight ? NotifyScrubHandleCentre : NotifyScrubHandleCentre - LabelSize.X, (NotifyHeight - TextSize.Y) * 0.5f);

	if( NotifyDurationSizeX == 0.f )
	{
		FSlateDrawElement::MakeBox( 
			OutDrawElements,
			LayerId, 
			AllottedGeometry.ToPaintGeometry(LabelSize, FSlateLayoutTransform(LabelPosition)),
			StyleInfo,
			ESlateDrawEffect::None,
			BoxColor);
	}

	// Text
	FVector2D TextPosition = LabelPosition + TextBorderSize;
	if(bDrawTooltipToRight)
	{
		TextPosition.X += HalfScrubHandleWidth;
	}

	FVector2D DrawTextSize;
	DrawTextSize.X = (NotifyDurationSizeX > 0.0f ? FMath::Min(NotifyDurationSizeX - (ScrubHandleSize.X + (bDrawBranchingPoint ? BranchingPointIconSize.X : 0)), TextSize.X) : TextSize.X);
	DrawTextSize.Y = TextSize.Y;

	if (bDrawBranchingPoint)
	{
		TextPosition.X += BranchingPointIconSize.X;
	}

	FPaintGeometry TextGeometry = AllottedGeometry.ToPaintGeometry(DrawTextSize, FSlateLayoutTransform(TextPosition));
	OutDrawElements.PushClip(FSlateClippingZone(TextGeometry));

	FSlateDrawElement::MakeText( 
		OutDrawElements,
		TextLayerID,
		TextGeometry,
		Text,
		Font,
		ESlateDrawEffect::None,
		FLinearColor::Black
		);

	OutDrawElements.PopClip();

	// Draw Branching Point
	if (bDrawBranchingPoint)
	{
		FVector2D BranchPointIconPos = LabelPosition + TextBorderSize;
		if(bDrawTooltipToRight)
		{
			BranchPointIconPos.X += HalfScrubHandleWidth;
		}
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			BranchPointLayerID,
			AllottedGeometry.ToPaintGeometry(BranchingPointIconSize, FSlateLayoutTransform(BranchPointIconPos)),
			FAppStyle::GetBrush(TEXT("AnimNotifyEditor.BranchingPoint")),
			ESlateDrawEffect::None,
			FLinearColor::White
			);
	}	

	DrawScrubHandle(NotifyScrubHandleCentre , OutDrawElements, ScrubHandleID, AllottedGeometry, MyCullingRect, NodeColor);

	if(AnimNotifyEvent && AnimNotifyEvent->TriggerTimeOffset != 0.f) //Do we have an offset to render?
	{
		float NotifyTime = AnimNotifyEvent->GetTime();
		if(NotifyTime != 0.f && NotifyTime != Sequence->GetPlayLength()) //Don't render offset when we are at the start/end of the sequence, doesn't help the user
		{
			float HandleCentre = NotifyScrubHandleCentre;
			float &Offset = AnimNotifyEvent->TriggerTimeOffset;
			
			DrawHandleOffset(AnimNotifyEvent->TriggerTimeOffset, NotifyScrubHandleCentre, OutDrawElements, MarkerLayer, AllottedGeometry, MyCullingRect, NodeColor);
		}
	}

	return TextLayerID;
}

FReply SAnimNotifyNode::OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	// Don't do scrub handle dragging if we haven't captured the mouse.
	if(!this->HasMouseCapture()) return FReply::Unhandled();

	if(CurrentDragHandle == ENotifyStateHandleHit::None)
	{
		// We've had focus taken away - realease the mouse
		FSlateApplication::Get().ReleaseAllPointerCapture();
		return FReply::Unhandled();
	}
	
	FTrackScaleInfo ScaleInfo(ViewInputMin.Get(), ViewInputMax.Get(), 0, 0, CachedAllotedGeometrySize);
	
	const float XPositionInTrack = MyGeometry.AbsolutePosition.X - CachedTrackGeometry.AbsolutePosition.X;
	const float TrackScreenSpaceXPosition = MyGeometry.AbsolutePosition.X - XPositionInTrack;
	const float TrackScreenSpaceOrigin = static_cast<float>(CachedTrackGeometry.LocalToAbsolute(FVector2D(ScaleInfo.InputToLocalX(0.0f), 0.0f)).X);
	const float TrackScreenSpaceLimit = static_cast<float>(CachedTrackGeometry.LocalToAbsolute(FVector2D(ScaleInfo.InputToLocalX(Sequence->GetPlayLength()), 0.0f)).X);

	if(CurrentDragHandle == ENotifyStateHandleHit::Start)
	{
		// Check track bounds
		float OldDisplayTime = NodeObjectInterface->GetTime();

		if(MouseEvent.GetScreenSpacePosition().X >= TrackScreenSpaceXPosition && MouseEvent.GetScreenSpacePosition().X <= TrackScreenSpaceXPosition + CachedAllotedGeometrySize.X)
		{
			float NewDisplayTime = ScaleInfo.LocalXToInput((FVector2f(MouseEvent.GetScreenSpacePosition()) - MyGeometry.AbsolutePosition + XPositionInTrack).X);	// LWC_TODO: Precision loss
			const float NewDuration = NodeObjectInterface->GetDuration() + OldDisplayTime - NewDisplayTime;

			// Check to make sure the duration is not less than the minimum allowed
			if(NewDuration < MinimumStateDuration)
			{
				NewDisplayTime -= MinimumStateDuration - NewDuration;
			}

			NodeObjectInterface->SetTime(FMath::Max(0.0f, NewDisplayTime));
			NodeObjectInterface->SetDuration(NodeObjectInterface->GetDuration() + OldDisplayTime - NodeObjectInterface->GetTime());
		}
		else if(NodeObjectInterface->GetDuration() > MinimumStateDuration)
		{
			float Overflow = HandleOverflowPan(MouseEvent.GetScreenSpacePosition(), TrackScreenSpaceXPosition, TrackScreenSpaceOrigin, TrackScreenSpaceLimit);

			// Update scale info to the new view inputs after panning
			ScaleInfo.ViewMinInput = ViewInputMin.Get();
			ScaleInfo.ViewMaxInput = ViewInputMax.Get();

			float NewDisplayTime = ScaleInfo.LocalXToInput((FVector2f(MouseEvent.GetScreenSpacePosition()) - MyGeometry.AbsolutePosition + XPositionInTrack).X);	// LWC_TODO: Precision loss
			NodeObjectInterface->SetTime(FMath::Max(0.0f, NewDisplayTime));
			NodeObjectInterface->SetDuration(NodeObjectInterface->GetDuration() + OldDisplayTime - NodeObjectInterface->GetTime());

			// Adjust incase we went under the minimum
			if(NodeObjectInterface->GetDuration() < MinimumStateDuration)
			{
				float EndTimeBefore = NodeObjectInterface->GetTime() + NodeObjectInterface->GetDuration();
				NodeObjectInterface->SetTime(NodeObjectInterface->GetTime() + NodeObjectInterface->GetDuration() - MinimumStateDuration);
				NodeObjectInterface->SetDuration(MinimumStateDuration);
				float EndTimeAfter = NodeObjectInterface->GetTime() + NodeObjectInterface->GetDuration();
			}
		}

		// Now we know where the marker should be, look for possible snaps on montage marker bars
		if (FAnimNotifyEvent* AnimNotifyEvent = NodeObjectInterface->GetNotifyEvent())
		{
			float InputStartTime = AnimNotifyEvent->GetTime();
			TOptional<EAnimEventTriggerOffsets::Type> OffsetForSnap;
			float MarkerSnap = GetScrubHandleSnapPosition(InputStartTime, ENotifyStateHandleHit::Start, OffsetForSnap);
			if (MarkerSnap != -1.0f)
			{
				// We're near to a snap bar
				EAnimEventTriggerOffsets::Type Offset = EAnimEventTriggerOffsets::NoOffset;
				if (OffsetForSnap.IsSet())
				{				
					Offset = OffsetForSnap.GetValue();
				}
				else
				{
					Offset = (MarkerSnap < InputStartTime) ? EAnimEventTriggerOffsets::OffsetAfter : EAnimEventTriggerOffsets::OffsetBefore;
				}

				AnimNotifyEvent->TriggerTimeOffset = GetTriggerTimeOffsetForType(Offset);

				// Adjust our start marker
				OldDisplayTime = AnimNotifyEvent->GetTime();
				AnimNotifyEvent->SetTime(MarkerSnap);
				AnimNotifyEvent->SetDuration(AnimNotifyEvent->GetDuration() + OldDisplayTime - AnimNotifyEvent->GetTime());
			}
			else
			{
				AnimNotifyEvent->TriggerTimeOffset = GetTriggerTimeOffsetForType(EAnimEventTriggerOffsets::NoOffset);
			}
		}

		OnNotifyStateHandleBeingDragged.ExecuteIfBound(SharedThis(this), MouseEvent, CurrentDragHandle, NodeObjectInterface->GetTime());
	}
	else
	{
		if(MouseEvent.GetScreenSpacePosition().X >= TrackScreenSpaceXPosition && MouseEvent.GetScreenSpacePosition().X <= TrackScreenSpaceXPosition + CachedAllotedGeometrySize.X)
		{
			float NewDuration = ScaleInfo.LocalXToInput((FVector2f(MouseEvent.GetScreenSpacePosition()) - MyGeometry.AbsolutePosition + XPositionInTrack).X) - NodeObjectInterface->GetTime();	// LWC_TODO: Precision loss

			NodeObjectInterface->SetDuration(FMath::Max(NewDuration, MinimumStateDuration));
		}
		else if(NodeObjectInterface->GetDuration() > MinimumStateDuration)
		{
			float Overflow = HandleOverflowPan(MouseEvent.GetScreenSpacePosition(), TrackScreenSpaceXPosition, TrackScreenSpaceOrigin, TrackScreenSpaceLimit);

			// Update scale info to the new view inputs after panning
			ScaleInfo.ViewMinInput = ViewInputMin.Get();
			ScaleInfo.ViewMaxInput = ViewInputMax.Get();

			float NewDuration = ScaleInfo.LocalXToInput((FVector2f(MouseEvent.GetScreenSpacePosition()) - MyGeometry.AbsolutePosition + XPositionInTrack).X) - NodeObjectInterface->GetTime();	// LWC_TODO: Precision loss
			NodeObjectInterface->SetDuration(FMath::Max(NewDuration, MinimumStateDuration));
		}

		if(NodeObjectInterface->GetTime() + NodeObjectInterface->GetDuration() > Sequence->GetPlayLength())
		{
			NodeObjectInterface->SetDuration(Sequence->GetPlayLength() - NodeObjectInterface->GetTime());
		}

		// Now we know where the scrub handle should be, look for possible snaps on montage marker bars
		if (FAnimNotifyEvent* AnimNotifyEvent = NodeObjectInterface->GetNotifyEvent())
		{
			float InputEndTime = AnimNotifyEvent->GetTime() + AnimNotifyEvent->GetDuration();
			TOptional<EAnimEventTriggerOffsets::Type> OffsetForSnap;
			float MarkerSnap = GetScrubHandleSnapPosition(InputEndTime, ENotifyStateHandleHit::End, OffsetForSnap);
			if (MarkerSnap != -1.0f)
			{
				// We're near to a snap bar
				EAnimEventTriggerOffsets::Type Offset = EAnimEventTriggerOffsets::NoOffset;
				if (OffsetForSnap.IsSet())
				{
					Offset = OffsetForSnap.GetValue();
				}
				else
				{
					Offset = (MarkerSnap < InputEndTime) ? EAnimEventTriggerOffsets::OffsetAfter : EAnimEventTriggerOffsets::OffsetBefore;
				}
				AnimNotifyEvent->EndTriggerTimeOffset = GetTriggerTimeOffsetForType(Offset);

				// Adjust our end marker
				AnimNotifyEvent->SetDuration(MarkerSnap - AnimNotifyEvent->GetTime());
			}
			else
			{
				AnimNotifyEvent->EndTriggerTimeOffset = GetTriggerTimeOffsetForType(EAnimEventTriggerOffsets::NoOffset);
			}
		}

		OnNotifyStateHandleBeingDragged.ExecuteIfBound(SharedThis(this), MouseEvent, CurrentDragHandle, (NodeObjectInterface->GetTime() + NodeObjectInterface->GetDuration()));
	}

	return FReply::Handled();
}

FReply SAnimNotifyNode::OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	bool bLeftButton = MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton;

	if(bLeftButton && CurrentDragHandle != ENotifyStateHandleHit::None)
	{
		// Clear the drag marker and give the mouse back
		CurrentDragHandle = ENotifyStateHandleHit::None;

		// Signal selection changing so details panels get updated
		OnSelectionChanged.ExecuteIfBound();

		// End drag transaction before handing mouse back
		check(DragMarkerTransactionIdx != INDEX_NONE);
		GEditor->EndTransaction();
		DragMarkerTransactionIdx = INDEX_NONE;

		Sequence->PostEditChange();
		Sequence->MarkPackageDirty();

		OnUpdatePanel.ExecuteIfBound();

		return FReply::Handled().ReleaseMouseCapture();
	}

	return FReply::Unhandled();
}

float SAnimNotifyNode::GetScrubHandleSnapPosition( float NotifyInputX, ENotifyStateHandleHit::Type HandleToCheck, TOptional<EAnimEventTriggerOffsets::Type>& OffsetForSnap)
{
	FTrackScaleInfo ScaleInfo(ViewInputMin.Get(), ViewInputMax.Get(), 0, 0, CachedAllotedGeometrySize);

	const float MaxSnapDist = 5.0f;

	if(OnSnapPosition.IsBound() && !FSlateApplication::Get().GetModifierKeys().IsControlDown() && ScaleInfo.PixelsPerInput > 0.0f)
	{
		FName SnapType = NAME_None;
		if(OnSnapPosition.Execute(NotifyInputX, MaxSnapDist / ScaleInfo.PixelsPerInput, TArrayView<const FName>(), SnapType))
		{
			static const FName MontageSectionSnapTypeName("MontageSection");
			if (SnapType == MontageSectionSnapTypeName)
			{
				if (HandleToCheck == ENotifyStateHandleHit::Start)
				{
					OffsetForSnap = EAnimEventTriggerOffsets::OffsetAfter;
				}
				else if (HandleToCheck == ENotifyStateHandleHit::End)
				{
					OffsetForSnap = EAnimEventTriggerOffsets::OffsetBefore;
				}
			}

			return NotifyInputX;
		}
	}	

	return -1.0f;
}

FReply SAnimNotifyNode::OnFocusReceived(const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent)
{
	return FReply::Handled().SetUserFocus(AsShared(), EFocusCause::SetDirectly, true);
}

float SAnimNotifyNode::HandleOverflowPan( const FVector2D &ScreenCursorPos, float TrackScreenSpaceXPosition, float TrackScreenSpaceMin, float TrackScreenSpaceMax )
{
	float Overflow = 0.0f;

	if(ScreenCursorPos.X < TrackScreenSpaceXPosition && TrackScreenSpaceXPosition > TrackScreenSpaceMin - 10.0f)
	{
		// Overflow left edge
		Overflow = FMath::Min(static_cast<float>(ScreenCursorPos.X) - TrackScreenSpaceXPosition, -10.0f);
	}
	else if(ScreenCursorPos.X > CachedAllotedGeometrySize.X && (TrackScreenSpaceXPosition + CachedAllotedGeometrySize.X) < TrackScreenSpaceMax + 10.0f)
	{
		// Overflow right edge
		Overflow = FMath::Max(static_cast<float>(ScreenCursorPos.X) - (TrackScreenSpaceXPosition + static_cast<float>(CachedAllotedGeometrySize.X)), 10.0f);
	}

	PanTrackRequest.ExecuteIfBound(static_cast<int32>(Overflow), CachedAllotedGeometrySize);

	return Overflow;
}

void SAnimNotifyNode::DrawScrubHandle( float ScrubHandleCentre, FSlateWindowElementList& OutDrawElements, int32 ScrubHandleID, const FGeometry &AllottedGeometry, const FSlateRect& MyCullingRect, FLinearColor NodeColour ) const
{
	FVector2D ScrubHandlePosition(ScrubHandleCentre - ScrubHandleSize.X / 2.0f, (NotifyHeight - ScrubHandleSize.Y) / 2.f);
	FSlateDrawElement::MakeBox( 
		OutDrawElements,
		ScrubHandleID, 
		AllottedGeometry.ToPaintGeometry(ScrubHandleSize, FSlateLayoutTransform(ScrubHandlePosition)), 
		FAppStyle::GetBrush( TEXT( "Sequencer.KeyDiamond" ) ),
		ESlateDrawEffect::None,
		NodeColour
		);

	FSlateDrawElement::MakeBox( 
		OutDrawElements,
		ScrubHandleID, 
		AllottedGeometry.ToPaintGeometry(ScrubHandleSize, FSlateLayoutTransform(ScrubHandlePosition)), 
		FAppStyle::GetBrush( TEXT( "Sequencer.KeyDiamondBorder" ) ),
		ESlateDrawEffect::None,
		bSelected ? FAppStyle::GetSlateColor("SelectionColor").GetSpecifiedColor() : FLinearColor::Black
		);
}

void SAnimNotifyNode::DrawHandleOffset( const float& Offset, const float& HandleCentre, FSlateWindowElementList& OutDrawElements, int32 MarkerLayer, const FGeometry &AllottedGeometry, const FSlateRect& MyCullingRect, FLinearColor NodeColor ) const
{
	FVector2D MarkerPosition;
	FVector2D MarkerSize = AlignmentMarkerSize;

	if(Offset < 0.f)
	{
		MarkerPosition.Set( HandleCentre - AlignmentMarkerSize.X, (NotifyHeight - AlignmentMarkerSize.Y) / 2.f);
	}
	else
	{
		MarkerPosition.Set( HandleCentre + AlignmentMarkerSize.X, (NotifyHeight - AlignmentMarkerSize.Y) / 2.f);
		MarkerSize.X = -AlignmentMarkerSize.X;
	}

	FSlateDrawElement::MakeBox( 
		OutDrawElements,
		MarkerLayer, 
		AllottedGeometry.ToPaintGeometry(MarkerSize, FSlateLayoutTransform(MarkerPosition)), 
		FAppStyle::GetBrush( TEXT( "Sequencer.Timeline.NotifyAlignmentMarker" ) ),
		ESlateDrawEffect::None,
		NodeColor
		);
}

void SAnimNotifyNode::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	ScreenPosition = FVector2D(AllottedGeometry.AbsolutePosition);
}

void SAnimNotifyNode::OnFocusLost(const FFocusEvent& InFocusEvent)
{
	if(CurrentDragHandle != ENotifyStateHandleHit::None)
	{
		// Lost focus while dragging a state node, clear the drag and end the current transaction
		CurrentDragHandle = ENotifyStateHandleHit::None;
		
		check(DragMarkerTransactionIdx != INDEX_NONE);
		GEditor->EndTransaction();
		DragMarkerTransactionIdx = INDEX_NONE;
	}
}

bool SAnimNotifyNode::SupportsKeyboardFocus() const
{
	// Need to support focus on the node so we can end drag transactions if the user alt-tabs
	// from the editor while in the proceess of dragging a state notify duration marker.
	return true;
}

FCursorReply SAnimNotifyNode::OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const
{
	// Show resize cursor if the cursor is hoverring over either of the scrub handles of a notify state node
	if(IsHovered() && GetDurationSize() > 0.0f)
	{
		const FVector2D RelMouseLocation = MyGeometry.AbsoluteToLocal(CursorEvent.GetScreenSpacePosition());

		const float HandleHalfWidth = static_cast<float>(ScrubHandleSize.X / 2.0);
		const float DistFromFirstHandle = FMath::Abs(static_cast<float>(RelMouseLocation.X) - NotifyScrubHandleCentre);
		const float DistFromSecondHandle = FMath::Abs(static_cast<float>(RelMouseLocation.X) - (NotifyScrubHandleCentre + NotifyDurationSizeX));

		if(DistFromFirstHandle < HandleHalfWidth || DistFromSecondHandle < HandleHalfWidth || CurrentDragHandle != ENotifyStateHandleHit::None)
		{
			return FCursorReply::Cursor(EMouseCursor::ResizeLeftRight);
		}
	}
	return FCursorReply::Unhandled();
}

#undef LOCTEXT_NAMESPACE
