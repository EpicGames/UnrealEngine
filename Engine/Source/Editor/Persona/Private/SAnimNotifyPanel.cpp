// Copyright Epic Games, Inc. All Rights Reserved.


#include "SAnimNotifyPanel.h"
#include "Rendering/DrawElements.h"
#include "Misc/MessageDialog.h"
#include "Modules/ModuleManager.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"
#include "UObject/PropertyPortFlags.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

#include "Layout/WidgetPath.h"
#include "Framework/Application/MenuStack.h"
#include "Fonts/FontMeasure.h"
#include "Styling/CoreStyle.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScrollBar.h"
#include "Widgets/Input/SButton.h"
#include "Animation/AnimMontage.h"
#include "Animation/EditorNotifyObject.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "ScopedTransaction.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetSelection.h"
#include "Widgets/Input/STextEntryPopup.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "BlueprintActionDatabase.h"
#include "Animation/AnimNotifies/AnimNotifyState.h"
#include "Animation/AnimNotifies/AnimNotify.h"
#include "Animation/BlendSpace.h"
#include "TabSpawners.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Modules/ModuleManager.h"
#include "IEditableSkeleton.h"
#include "ISkeletonEditorModule.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "ClassViewerFilter.h"
#include "ClassViewerModule.h"
#include "SSkeletonAnimNotifies.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "IAnimationEditor.h"
#include "IAnimationSequenceBrowser.h"
#include "AnimTimeline/AnimTimelineTrack_NotifiesPanel.h"
#include "PersonaUtils.h"
#include "AnimAssetFindReplace.h"
#include "AnimAssetFindReplaceSyncMarkers.h"
#include "AnimAssetFindReplaceNotifies.h"
#include "ToolMenus.h"
#include "ToolMenuMisc.h"
#include "AnimNotifyPanelContextMenuContext.h"
#include "SAnimNotifyNode.h"
#include "AnimNotifyNodeFactory.h"

#define LOCTEXT_NAMESPACE "AnimNotifyPanel"

DECLARE_DELEGATE(FCopyNodes)
DECLARE_DELEGATE_FourParams(FPasteNodes, SAnimNotifyTrack*, float, ENotifyPasteMode::Type, ENotifyPasteMultipleMode::Type)
DECLARE_DELEGATE_RetVal_OneParam(EVisibility, FOnGetTimingNodeVisibilityForNode, TSharedPtr<SAnimNotifyNode>)

// Read common info from the clipboard
bool ReadNotifyPasteHeader(FString& OutPropertyString, const TCHAR*& OutBuffer, float& OutOriginalTime, float& OutOriginalLength, int32& OutTrackSpan)
{
	OutBuffer = NULL;
	OutOriginalTime = -1.f;

	FPlatformApplicationMisc::ClipboardPaste(OutPropertyString);

	if (!OutPropertyString.IsEmpty())
	{
		//Remove header text
		const FString HeaderString(TEXT("COPY_ANIMNOTIFYEVENT"));

		//Check for string identifier in order to determine whether the text represents an FAnimNotifyEvent.
		if (OutPropertyString.StartsWith(HeaderString) && OutPropertyString.Len() > HeaderString.Len())
		{
			int32 HeaderSize = HeaderString.Len();
			OutBuffer = *OutPropertyString;
			OutBuffer += HeaderSize;

			FString ReadLine;
			// Read the original time from the first notify
			FParse::Line(&OutBuffer, ReadLine);
			FParse::Value(*ReadLine, TEXT("OriginalTime="), OutOriginalTime);
			FParse::Value(*ReadLine, TEXT("OriginalLength="), OutOriginalLength);
			FParse::Value(*ReadLine, TEXT("TrackSpan="), OutTrackSpan);
			return true;
		}
	}

	return false;
}

class SAnimNotifyPair : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SAnimNotifyPair)
	{}

	SLATE_NAMED_SLOT(FArguments, LeftContent)
	
	SLATE_ARGUMENT(TSharedPtr<SAnimNotifyNode>, Node);
	SLATE_EVENT(FOnGetTimingNodeVisibilityForNode, OnGetTimingNodeVisibilityForNode)

	SLATE_END_ARGS()

		void Construct(const FArguments& InArgs);

	float GetWidgetPaddingLeft();

protected:
	TSharedPtr<SWidget> PairedWidget;
	TSharedPtr<SAnimNotifyNode> NodePtr;
};

void SAnimNotifyPair::Construct(const FArguments& InArgs)
{
	NodePtr = InArgs._Node;
	PairedWidget = InArgs._LeftContent.Widget;
	check(NodePtr.IsValid());
	check(PairedWidget.IsValid());

	float ScaleMult = 1.0f;
	FVector2D NodeSize = NodePtr->ComputeDesiredSize(ScaleMult);
	SetVisibility(EVisibility::SelfHitTestInvisible);

	this->ChildSlot
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SBox)
				.HAlign(EHorizontalAlignment::HAlign_Center)
				.VAlign(EVerticalAlignment::VAlign_Center)
				[
					PairedWidget->AsShared()
				]
			]
			+ SHorizontalBox::Slot()
			[
				NodePtr->AsShared()
			]
		];
}

float SAnimNotifyPair::GetWidgetPaddingLeft()
{
	return static_cast<float>(NodePtr->GetWidgetPosition().X - PairedWidget->GetDesiredSize().X);
}

//////////////////////////////////////////////////////////////////////////
// SAnimNotifyTrack

class SAnimNotifyTrack : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SAnimNotifyTrack )
		: _Sequence(NULL)
		, _ViewInputMin()
		, _ViewInputMax()
		, _TrackIndex()
		, _TrackColor(FLinearColor::White)
		, _OnSelectionChanged()
		, _OnUpdatePanel()
		, _OnGetNotifyBlueprintData()
		, _OnGetNotifyStateBlueprintData()
		, _OnGetNotifyNativeClasses()
		, _OnGetNotifyStateNativeClasses()
		, _OnGetScrubValue()
		, _OnGetDraggedNodePos()
		, _OnNodeDragStarted()
		, _OnNotifyStateHandleBeingDragged()
		, _OnRequestTrackPan()
		, _OnRequestOffsetRefresh()
		, _OnDeleteNotify()
		, _OnGetIsAnimNotifySelectionValidForReplacement()
		, _OnReplaceSelectedWithNotify()
		, _OnReplaceSelectedWithBlueprintNotify()
		, _OnReplaceSelectedWithSyncMarker()
		, _OnDeselectAllNotifies()
		, _OnCopyNodes()
		, _OnPasteNodes()
		, _OnSetInputViewRange()
		{}

		SLATE_ARGUMENT( class UAnimSequenceBase*, Sequence )
		SLATE_ARGUMENT( TArray<FAnimNotifyEvent *>, AnimNotifies )
		SLATE_ARGUMENT( TArray<FAnimSyncMarker *>, AnimSyncMarkers)
		SLATE_ATTRIBUTE( float, ViewInputMin )
		SLATE_ATTRIBUTE( float, ViewInputMax )
		SLATE_EVENT( FOnSnapPosition, OnSnapPosition )
		SLATE_ARGUMENT( int32, TrackIndex )
		SLATE_ARGUMENT( FLinearColor, TrackColor )
		SLATE_ATTRIBUTE(EVisibility, QueuedNotifyTimingNodeVisibility)
		SLATE_ATTRIBUTE(EVisibility, BranchingPointTimingNodeVisibility)
		SLATE_EVENT(FOnTrackSelectionChanged, OnSelectionChanged)
		SLATE_EVENT( FOnUpdatePanel, OnUpdatePanel )
		SLATE_EVENT( FOnGetBlueprintNotifyData, OnGetNotifyBlueprintData )
		SLATE_EVENT( FOnGetBlueprintNotifyData, OnGetNotifyStateBlueprintData )
		SLATE_EVENT( FOnGetNativeNotifyClasses, OnGetNotifyNativeClasses )
		SLATE_EVENT( FOnGetNativeNotifyClasses, OnGetNotifyStateNativeClasses )
		SLATE_EVENT( FOnGetScrubValue, OnGetScrubValue )
		SLATE_EVENT( FOnGetDraggedNodePos, OnGetDraggedNodePos )
		SLATE_EVENT( FOnNotifyNodesDragStarted, OnNodeDragStarted )
		SLATE_EVENT( FOnNotifyStateHandleBeingDragged, OnNotifyStateHandleBeingDragged)
		SLATE_EVENT( FPanTrackRequest, OnRequestTrackPan )
		SLATE_EVENT( FRefreshOffsetsRequest, OnRequestOffsetRefresh )
		SLATE_EVENT( FDeleteNotify, OnDeleteNotify )
		SLATE_EVENT( FOnGetIsAnimNotifySelectionValidForReplacement, OnGetIsAnimNotifySelectionValidForReplacement)
		SLATE_EVENT( FReplaceWithNotify, OnReplaceSelectedWithNotify )
		SLATE_EVENT( FReplaceWithBlueprintNotify, OnReplaceSelectedWithBlueprintNotify)
		SLATE_EVENT( FReplaceWithSyncMarker, OnReplaceSelectedWithSyncMarker)
		SLATE_EVENT( FDeselectAllNotifies, OnDeselectAllNotifies)
		SLATE_EVENT( FCopyNodes, OnCopyNodes )
		SLATE_EVENT(FPasteNodes, OnPasteNodes)
		SLATE_EVENT( FOnSetInputViewRange, OnSetInputViewRange )
		SLATE_EVENT( FOnGetTimingNodeVisibility, OnGetTimingNodeVisibility )
		SLATE_EVENT(FOnInvokeTab, OnInvokeTab)
		SLATE_ARGUMENT(TSharedPtr<FUICommandList>, CommandList)
		SLATE_END_ARGS()
public:

	/** Type used for list widget of tracks */
	void Construct(const FArguments& InArgs);

	// SWidget interface
	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override { UpdateCachedGeometry( AllottedGeometry ); }
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	virtual FCursorReply OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const override;
	virtual bool SupportsKeyboardFocus() const override
	{
		return true;
	}
	// End of SWidget interface

	/**
	 * Update the nodes to match the data that the panel is observing
	 */
	void Update();

	/** Returns the cached rendering geometry of this track */
	const FGeometry& GetCachedGeometry() const { return CachedGeometry; }

	FTrackScaleInfo GetCachedScaleInfo() const { return FTrackScaleInfo(ViewInputMin.Get(), ViewInputMax.Get(), 0.f, 0.f, CachedGeometry.GetLocalSize()); }

	/** Updates sequences when a notify node has been successfully dragged to a new position
	 *	@param Offset - Offset from the widget to the time handle 
	 */
	void HandleNodeDrop(TSharedPtr<SAnimNotifyNode> Node, float Offset = 0.0f);

	// Number of nodes in the track currently selected
	int32 GetNumSelectedNodes() const { return SelectedNodeIndices.Num(); }

	// Index of the track in the notify panel
	int32 GetTrackIndex() const { return TrackIndex; }

	// Time at the position of the last mouseclick
	float GetLastClickedTime() const { return LastClickedTime; }

	// Removes the node widgets from the track and adds them to the provided Array
	void DisconnectSelectedNodesForDrag(TArray<TSharedPtr<SAnimNotifyNode>>& DragNodes);

	// Adds our current selection to the provided set
	void AppendSelectionToSet(FGraphPanelSelectionSet& SelectionSet);
	// Adds our current selection to the provided array
	void AppendSelectionToArray(TArray<INotifyNodeObjectInterface*>& Selection) const;
	// Gets the currently selected SAnimNotifyNode instances
	void AppendSelectedNodeWidgetsToArray(TArray<TSharedPtr<SAnimNotifyNode>>& NodeArray) const;
	// Gets the indices of the selected notifies
	const TArray<int32>& GetSelectedNotifyIndices() const {return SelectedNodeIndices;}

	INotifyNodeObjectInterface* GetNodeObjectInterface(int32 NodeIndex) { return NotifyNodes[NodeIndex]->NodeObjectInterface; }
	/**
	* Deselects all currently selected notify nodes
	* @param bUpdateSelectionSet - Whether we should report a selection change to the panel
	*/
	void DeselectAllNotifyNodes(bool bUpdateSelectionSet = true);

	/** Select all nodes contained in the supplied Guid set. */
	void SelectNodesByGuid(const TSet<FGuid>& InGuids, bool bUpdateSelectionSet);

	/** Get the number of notify nodes we contain */
	int32 GetNumNotifyNodes() const { return NotifyNodes.Num(); }

	/** Check whether a node is selected */
	bool IsNodeSelected(int32 NodeIndex) const { return NotifyNodes[NodeIndex]->bSelected; }

	// get Property Data of one element (NotifyIndex) from Notifies property of Sequence
	static uint8* FindNotifyPropertyData(UAnimSequenceBase* Sequence, int32 NotifyIndex, FArrayProperty*& ArrayProperty);

	// Paste a single Notify into this track from an exported string
	void PasteSingleNotify(FString& NotifyString, float PasteTime);

	// Paste a single Sync Marker into this track from an exported string
	void PasteSingleSyncMarker(FString& MarkerString, float PasteTime);

	// Uses the given track space rect and marquee information to refresh selection information
	void RefreshMarqueeSelectedNodes(const FSlateRect& Rect, FNotifyMarqueeOperation& Marquee);

	// Create new notifies
	FAnimNotifyEvent& CreateNewBlueprintNotify(FString NewNotifyName, FString BlueprintPath, float StartTime);
	FAnimNotifyEvent& CreateNewNotify(FString NewNotifyName, UClass* NotifyClass, float StartTime);

	// Get the Blueprint Class from the path of the Blueprint
	static TSubclassOf<UObject> GetBlueprintClassFromPath(FString BlueprintPath);

	// Get the default Notify Name for a given blueprint notify asset
	FString MakeBlueprintNotifyName(const FString& InNotifyClassName);

	// Need to make sure tool tips are cleared during node clear up so slate system won't
	// call into invalid notify.
	void ClearNodeTooltips();

protected:

	// Build up a "New Notify..." menu
	void FillNewNotifyMenu(FMenuBuilder& MenuBuilderbool, bool bIsReplaceWithMenu = false);
	void FillNewNotifyStateMenu(FMenuBuilder& MenuBuilder, bool bIsReplaceWithMenu  = false);
	void FillNewSyncMarkerMenu(FMenuBuilder& MenuBuilder, bool bIsReplaceWithMenu = false);
	void OnAnimNotifyClassPicked(UClass* NotifyClass, bool bIsReplaceWithMenu = false);

	// New notify functions
	void CreateNewBlueprintNotifyAtCursor(FString NewNotifyName, FString BlueprintPath);
	void CreateNewNotifyAtCursor(FString NewNotifyName, UClass* NotifyClass);
	void CreateNewSyncMarkerAtCursor(FString NewSyncMarkerName);
	void OnNewNotifyClicked();
	void OnNewSyncMarkerClicked();
	void AddNewNotify(const FText& NewNotifyName, ETextCommit::Type CommitInfo);
	void AddNewSyncMarker(const FText& NewMarkerName, ETextCommit::Type CommitInfo);

	// Trigger weight functions
	void OnSetTriggerWeightNotifyClicked(int32 NotifyIndex);
	void SetTriggerWeight(const FText& TriggerWeight, ETextCommit::Type CommitInfo, int32 NotifyIndex);

	// "Replace with... " commands
	void ReplaceSelectedWithBlueprintNotify(FString NewNotifyName, FString BlueprintPath);
	void ReplaceSelectedWithNotify(FString NewNotifyName, UClass* NotifyClass);
	void ReplaceSelectedWithSyncMarker(FString NewSyncMarkerName);
	bool IsValidToPlace(UClass* NotifyClass) const;

	// Whether we have one node selected
	bool IsSingleNodeSelected();
	// Checks the clipboard for an anim notify buffer, and returns whether there's only one notify
	bool IsSingleNodeInClipboard();

	/** Function to check whether it is possible to paste anim notify event */
	bool CanPasteAnimNotify() const;

	/** Handler for context menu paste command */
	void OnPasteNotifyClicked(ENotifyPasteMode::Type PasteMode, ENotifyPasteMultipleMode::Type MultiplePasteType = ENotifyPasteMultipleMode::Absolute);

	/** Handler for popup window asking the user for a paste time */
	void OnPasteNotifyTimeSet(const FText& TimeText, ETextCommit::Type CommitInfo);

	/** Function to paste a previously copied notify */
	void OnPasteNotify(float TimeToPasteAt, ENotifyPasteMultipleMode::Type MultiplePasteType = ENotifyPasteMultipleMode::Absolute);

	/** Provides direct access to the notify menu from the context menu */
	void OnManageNotifies();

	/** Opens the supplied blueprint in an editor */
	void OnOpenNotifySource(UBlueprint* InSourceBlueprint) const;

	/** Filters the asset browser by the selected notify/sync marker */
	void OnFindReferences(FName InName, bool bInIsSyncMarker);

	/**
	 * Selects a node on the track. Supports multi selection
	 * @param TrackNodeIndex - Index of the node to select.
	 * @param Append - Whether to append to to current selection or start a new one.
	 * @param bUpdateSelection - Whether to immediately inform Persona of a selection change
	 */
	void SelectTrackObjectNode(int32 TrackNodeIndex, bool Append, bool bUpdateSelection = true);
	
	/**
	 * Toggles the selection status of a notify node, for example when
	 * Control is held when clicking.
	 * @param NotifyIndex - Index of the notify to toggle the selection status of
	 * @param bUpdateSelection - Whether to immediately inform Persona of a selection change
	 */
	void ToggleTrackObjectNodeSelectionStatus(int32 TrackNodeIndex, bool bUpdateSelection = true);

	/**
	 * Deselects requested notify node
	 * @param NotifyIndex - Index of the notify node to deselect
	 * @param bUpdateSelection - Whether to immediately inform Persona of a selection change
	 */
	void DeselectTrackObjectNode(int32 TrackNodeIndex, bool bUpdateSelection = true);

	int32 GetHitNotifyNode(const FGeometry& MyGeometry, const FVector2D& Position);

	TSharedPtr<SWidget> SummonContextMenu(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

	UToolMenu* CreateContextMenuContent(FName BaseMenuName);

	virtual FVector2D ComputeDesiredSize(float) const override;
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	virtual FReply OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;
	virtual FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;

	float CalculateTime( const FGeometry& MyGeometry, FVector2D NodePos, bool bInputIsAbsolute = true );

	// Handler that is called when the user starts dragging a node
	FReply OnNotifyNodeDragStarted( TSharedRef<SAnimNotifyNode> NotifyNode, const FPointerEvent& MouseEvent, const FVector2D& ScreenNodePosition, const bool bDragOnMarker, int32 NotifyIndex );

	const EVisibility GetTimingNodeVisibility(TSharedPtr<SAnimNotifyNode> NotifyNode);

private:

	// Data structure for bluprint notify context menu entries
	struct BlueprintNotifyMenuInfo
	{
		FString NotifyName;
		FString BlueprintPath;
		UClass* BaseClass;
	};

	// Store the tracks geometry for later use
	void UpdateCachedGeometry(const FGeometry& InGeometry);

	// Returns the padding needed to render the notify in the correct track position
	FMargin GetNotifyTrackPadding(int32 NotifyIndex) const
	{
		const float LeftMargin = NotifyPairs[NotifyIndex]->GetWidgetPaddingLeft();
		const float RightMargin = static_cast<float>(CachedGeometry.GetLocalSize().X - NotifyNodes[NotifyIndex]->GetWidgetPosition().X - NotifyNodes[NotifyIndex]->GetSize().X);
		return FMargin(LeftMargin, 0, RightMargin, 0);
	}

	// Returns the padding needed to render the notify in the correct track position
	FMargin GetSyncMarkerTrackPadding(int32 SyncMarkerIndex) const
	{
		const float LeftMargin = static_cast<float>(NotifyNodes[SyncMarkerIndex]->GetWidgetPosition().X);
		const float RightMargin = static_cast<float>(CachedGeometry.GetLocalSize().X - NotifyNodes[SyncMarkerIndex]->GetWidgetPosition().X - NotifyNodes[SyncMarkerIndex]->GetSize().X);
		return FMargin(LeftMargin, 0, RightMargin, 0);
	}

	// Builds a UObject selection set and calls the OnSelectionChanged delegate
	void SendSelectionChanged()
	{
		OnSelectionChanged.ExecuteIfBound();
	}

protected:
	TWeakPtr<FUICommandList> WeakCommandList;

	float LastClickedTime;

	class UAnimSequenceBase*				Sequence; // need for menu generation of anim notifies - 
	TArray<TSharedPtr<SAnimNotifyNode>>		NotifyNodes;
	TArray<TSharedPtr<SAnimNotifyPair>>		NotifyPairs;
	TArray<FAnimNotifyEvent*>				AnimNotifies;
	TArray<FAnimSyncMarker*>				AnimSyncMarkers;
	TAttribute<float>						ViewInputMin;
	TAttribute<float>						ViewInputMax;
	TAttribute<float>						InputMin;
	TAttribute<float>						InputMax;
	TAttribute<FLinearColor>				TrackColor;
	int32									TrackIndex;
	TAttribute<EVisibility>					NotifyTimingNodeVisibility;
	TAttribute<EVisibility>					BranchingPointTimingNodeVisibility;
	FOnTrackSelectionChanged				OnSelectionChanged;
	FOnUpdatePanel							OnUpdatePanel;
	FOnGetBlueprintNotifyData				OnGetNotifyBlueprintData;
	FOnGetBlueprintNotifyData				OnGetNotifyStateBlueprintData;
	FOnGetNativeNotifyClasses				OnGetNotifyNativeClasses;
	FOnGetNativeNotifyClasses				OnGetNotifyStateNativeClasses;
	FOnGetScrubValue						OnGetScrubValue;
	FOnGetDraggedNodePos					OnGetDraggedNodePos;
	FOnNotifyNodesDragStarted				OnNodeDragStarted;
	FOnNotifyStateHandleBeingDragged		OnNotifyStateHandleBeingDragged;
	FPanTrackRequest						OnRequestTrackPan;
	FDeselectAllNotifies					OnDeselectAllNotifies;
	FCopyNodes							OnCopyNodes;
	FPasteNodes								OnPasteNodes;
	FOnSetInputViewRange					OnSetInputViewRange;
	FOnGetTimingNodeVisibility				OnGetTimingNodeVisibility;

	/** Delegate to call when offsets should be refreshed in a montage */
	FRefreshOffsetsRequest					OnRequestRefreshOffsets;

	/** Delegate to call when deleting notifies */
	FDeleteNotify							OnDeleteNotify;

	/** Delegates to call when replacing notifies */
	FOnGetIsAnimNotifySelectionValidForReplacement OnGetIsAnimNotifySelectionValidforReplacement;
	FReplaceWithNotify						OnReplaceSelectedWithNotify;
	FReplaceWithBlueprintNotify				OnReplaceSelectedWithBlueprintNotify;
	FReplaceWithSyncMarker					OnReplaceSelectedWithSyncMarker;

	FOnInvokeTab							OnInvokeTab;

	TSharedPtr<SBorder>						TrackArea;

	/** Cache the SOverlay used to store all this tracks nodes */
	TSharedPtr<SOverlay> NodeSlots;

	/** Cached for drag drop handling code */
	FGeometry CachedGeometry;

	/** Delegate used to snap when dragging */
	FOnSnapPosition OnSnapPosition;

	/** Nodes that are currently selected */
	TArray<int32> SelectedNodeIndices;
};

//////////////////////////////////////////////////////////////////////////
// 

/** Widget for drawing a single track */
class SNotifyEdTrack : public SCompoundWidget
{
private:
	/** Index of Track in Sequence **/
	int32									TrackIndex;

	/** Anim Sequence **/
	class UAnimSequenceBase*				Sequence;

	/** Pointer to notify panel for drawing*/
	TWeakPtr<SAnimNotifyPanel>			AnimPanelPtr;

public:
	SLATE_BEGIN_ARGS( SNotifyEdTrack )
		: _TrackIndex(INDEX_NONE)
		, _AnimNotifyPanel()
		, _Sequence()
		, _WidgetWidth()
		, _ViewInputMin()
		, _ViewInputMax()
		, _OnSelectionChanged()
		, _OnUpdatePanel()
		, _OnDeleteNotify()
		, _OnDeselectAllNotifies()
		, _OnCopyNodes()
		, _OnSetInputViewRange()
	{}
	SLATE_ARGUMENT( int32, TrackIndex )
	SLATE_ARGUMENT( TSharedPtr<SAnimNotifyPanel>, AnimNotifyPanel)
	SLATE_ARGUMENT( class UAnimSequenceBase*, Sequence )
	SLATE_ARGUMENT( float, WidgetWidth )
	SLATE_ATTRIBUTE( float, ViewInputMin )
	SLATE_ATTRIBUTE( float, ViewInputMax )
	SLATE_EVENT( FOnSnapPosition, OnSnapPosition )
	SLATE_ATTRIBUTE( EVisibility, NotifyTimingNodeVisibility )
	SLATE_ATTRIBUTE( EVisibility, BranchingPointTimingNodeVisibility )
	SLATE_EVENT( FOnTrackSelectionChanged, OnSelectionChanged)
	SLATE_EVENT( FOnGetScrubValue, OnGetScrubValue )
	SLATE_EVENT( FOnGetDraggedNodePos, OnGetDraggedNodePos )
	SLATE_EVENT( FOnUpdatePanel, OnUpdatePanel )
	SLATE_EVENT( FOnGetBlueprintNotifyData, OnGetNotifyBlueprintData )
	SLATE_EVENT( FOnGetBlueprintNotifyData, OnGetNotifyStateBlueprintData )
	SLATE_EVENT( FOnGetNativeNotifyClasses, OnGetNotifyNativeClasses )
	SLATE_EVENT( FOnGetNativeNotifyClasses, OnGetNotifyStateNativeClasses )
	SLATE_EVENT( FOnNotifyNodesDragStarted, OnNodeDragStarted )
	SLATE_EVENT( FOnNotifyStateHandleBeingDragged, OnNotifyStateHandleBeingDragged)
	SLATE_EVENT( FRefreshOffsetsRequest, OnRequestRefreshOffsets )
	SLATE_EVENT( FDeleteNotify, OnDeleteNotify )
	SLATE_EVENT( FDeselectAllNotifies, OnDeselectAllNotifies)
	SLATE_EVENT( FCopyNodes, OnCopyNodes )
	SLATE_EVENT( FPasteNodes, OnPasteNodes )
	SLATE_EVENT( FOnSetInputViewRange, OnSetInputViewRange )
	SLATE_EVENT( FOnGetTimingNodeVisibility, OnGetTimingNodeVisibility )
	SLATE_EVENT(FOnInvokeTab, OnInvokeTab)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	bool CanDeleteTrack();

	/** Pointer to actual anim notify track */
	TSharedPtr<class SAnimNotifyTrack>	NotifyTrack;

	/** Return the tracks name as an FText */
	FText GetTrackName() const
	{
		if(Sequence->AnimNotifyTracks.IsValidIndex(TrackIndex))
		{
			return FText::FromName(Sequence->AnimNotifyTracks[TrackIndex].TrackName);
		}

		/** Should never be possible but better than crashing the editor */
		return LOCTEXT("TrackName_Invalid", "Invalid Track");
	}
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
class FNotifyDragDropOp : public FDragDropOperation
{
public:
	FNotifyDragDropOp(float& InCurrentDragXPosition) : 
		CurrentDragXPosition(InCurrentDragXPosition), 
		SnapTime(-1.f),
		SelectionTimeLength(0.0f)
	{
	}

	struct FTrackClampInfo
	{
		int32 TrackPos;
		int32 TrackSnapTestPos;
		TSharedPtr<SAnimNotifyTrack> NotifyTrack;
	};

	DRAG_DROP_OPERATOR_TYPE(FNotifyDragDropOp, FDragDropOperation)

	virtual void OnDrop( bool bDropWasHandled, const FPointerEvent& MouseEvent ) override
	{
		if ( bDropWasHandled == false )
		{
			int32 NumNodes = SelectedNodes.Num();

			const FScopedTransaction Transaction(NumNodes > 0 ? LOCTEXT("MoveNotifiesEvent", "Move Anim Notifies") : LOCTEXT("MoveNotifyEvent", "Move Anim Notify"));
			Sequence->Modify();
			
			for(int32 CurrentNode = 0 ; CurrentNode < NumNodes ; ++CurrentNode)
			{
				TSharedPtr<SAnimNotifyNode> Node = SelectedNodes[CurrentNode];
				float NodePositionOffset = NodeXOffsets[CurrentNode];
				const FTrackClampInfo& ClampInfo = GetTrackClampInfo(Node->GetScreenPosition());
				ClampInfo.NotifyTrack->HandleNodeDrop(Node, NodePositionOffset);
				Node->DropCancelled();
			}

			Sequence->PostEditChange();
			Sequence->MarkPackageDirty();
			
			OnUpdatePanel.ExecuteIfBound();
		}
		
		FDragDropOperation::OnDrop(bDropWasHandled, MouseEvent);
	}

	virtual void OnDragged( const class FDragDropEvent& DragDropEvent ) override
	{
		// Reset snapped node pointer
		SnappedNode = NULL;

		NodeGroupPosition = DragDropEvent.GetScreenSpacePosition() + DragOffset;

		FTrackClampInfo* SelectionPositionClampInfo = &GetTrackClampInfo(DragDropEvent.GetScreenSpacePosition());
		if((SelectionPositionClampInfo->NotifyTrack->GetTrackIndex() + TrackSpan) >= ClampInfos.Num())
		{
			// Our selection has moved off the bottom of the notify panel, adjust the clamping information to keep it on the panel
			SelectionPositionClampInfo = &ClampInfos[ClampInfos.Num() - TrackSpan - 1];
		}
		
		const FGeometry& TrackGeom = SelectionPositionClampInfo->NotifyTrack->GetCachedGeometry();
		const FTrackScaleInfo& TrackScaleInfo = SelectionPositionClampInfo->NotifyTrack->GetCachedScaleInfo();

		FVector2D SelectionBeginPosition = TrackGeom.LocalToAbsolute(TrackGeom.AbsoluteToLocal(NodeGroupPosition) + SelectedNodes[0]->GetNotifyPositionOffset());
	
		float LocalTrackMin = TrackScaleInfo.InputToLocalX(0.0f);
		float LocalTrackMax = TrackScaleInfo.InputToLocalX(Sequence->GetPlayLength());
		float LocalTrackWidth = LocalTrackMax - LocalTrackMin;

		// Tracks the movement amount to apply to the selection due to a snap.
		float SnapMovement = 0.0f;
		// Clamp the selection into the track
		float SelectionBeginLocalPositionX = static_cast<float>(TrackGeom.AbsoluteToLocal(SelectionBeginPosition).X);
		const float ClampedEnd = FMath::Clamp(SelectionBeginLocalPositionX + NodeGroupWidth, LocalTrackMin, LocalTrackMax);
		const float ClampedBegin = FMath::Clamp(SelectionBeginLocalPositionX, LocalTrackMin, LocalTrackMax);
		if(ClampedBegin > SelectionBeginLocalPositionX)
		{
			SelectionBeginLocalPositionX = ClampedBegin;
		}
		else if(ClampedEnd < SelectionBeginLocalPositionX + NodeGroupWidth)
		{
			SelectionBeginLocalPositionX = ClampedEnd - NodeGroupWidth;
		}

		SelectionBeginPosition.X = TrackGeom.LocalToAbsolute(FVector2D(SelectionBeginLocalPositionX, 0.0f)).X;

		// Handle node snaps
		bool bSnapped = false;
		for(int32 NodeIdx = 0 ; NodeIdx < SelectedNodes.Num() && !bSnapped; ++NodeIdx)
		{
			TSharedPtr<SAnimNotifyNode> CurrentNode = SelectedNodes[NodeIdx];

			// Clear off any snap time currently stored
			CurrentNode->ClearLastSnappedTime();

			const FTrackClampInfo& NodeClamp = GetTrackClampInfo(CurrentNode->GetScreenPosition());

			FVector2D EventPosition = SelectionBeginPosition + FVector2D(TrackScaleInfo.PixelsPerInput * NodeTimeOffsets[NodeIdx], 0.0f);

			// Look for a snap on the first scrub handle
			FVector2D TrackNodePos = TrackGeom.AbsoluteToLocal(EventPosition);
			const FVector2D OriginalNodePosition = TrackNodePos;
			const float SequenceStart = TrackScaleInfo.InputToLocalX(0.f);
			const float SequenceEnd = TrackScaleInfo.InputToLocalX(Sequence->GetPlayLength());

			// Always clamp the Y to the current track
			SelectionBeginPosition.Y = SelectionPositionClampInfo->TrackPos - 1.0f;

			float SnapX = GetSnapPosition(NodeClamp, static_cast<float>(TrackNodePos.X), bSnapped);
			if (FAnimNotifyEvent* CurrentEvent = CurrentNode->NodeObjectInterface->GetNotifyEvent())
			{
				if (bSnapped)
				{
					EAnimEventTriggerOffsets::Type Offset = EAnimEventTriggerOffsets::NoOffset;
					if (SnapX == SequenceStart || SnapX == SequenceEnd)
					{
						Offset = SnapX > SequenceStart ? EAnimEventTriggerOffsets::OffsetBefore : EAnimEventTriggerOffsets::OffsetAfter;
					}
					else
					{
						Offset = (SnapX < TrackNodePos.X) ? EAnimEventTriggerOffsets::OffsetAfter : EAnimEventTriggerOffsets::OffsetBefore;
					}

					CurrentEvent->TriggerTimeOffset = GetTriggerTimeOffsetForType(Offset);
					CurrentNode->SetLastSnappedTime(TrackScaleInfo.LocalXToInput(SnapX));

					if (SnapMovement == 0.0f)
					{
						SnapMovement = SnapX - static_cast<float>(TrackNodePos.X);
						TrackNodePos.X = SnapX;
						SnapTime = TrackScaleInfo.LocalXToInput(SnapX);
						SnappedNode = CurrentNode;
					}
					EventPosition = NodeClamp.NotifyTrack->GetCachedGeometry().LocalToAbsolute(TrackNodePos);
				}
				else
				{
					CurrentEvent->TriggerTimeOffset = GetTriggerTimeOffsetForType(EAnimEventTriggerOffsets::NoOffset);
				}

				if (CurrentNode.IsValid() && CurrentEvent->GetDuration() > 0)
				{
					// If we didn't snap the beginning of the node, attempt to snap the end
					if (!bSnapped)
					{
						const FVector2D TrackNodeEndPos = TrackNodePos + CurrentNode->GetDurationSize();
						SnapX = GetSnapPosition(*SelectionPositionClampInfo, static_cast<float>(TrackNodeEndPos.X), bSnapped);

						// Only attempt to snap if the node will fit on the track
						if (SnapX >= CurrentNode->GetDurationSize())
						{
							EAnimEventTriggerOffsets::Type Offset = EAnimEventTriggerOffsets::NoOffset;
							if (SnapX == SequenceEnd)
							{
								// Only need to check the end of the sequence here; end handle can't hit the beginning
								Offset = EAnimEventTriggerOffsets::OffsetBefore;
							}
							else
							{
								Offset = (SnapX < TrackNodeEndPos.X) ? EAnimEventTriggerOffsets::OffsetAfter : EAnimEventTriggerOffsets::OffsetBefore;
							}
							CurrentEvent->EndTriggerTimeOffset = GetTriggerTimeOffsetForType(Offset);

							if (SnapMovement == 0.0f)
							{
								SnapMovement = SnapX - static_cast<float>(TrackNodeEndPos.X);
								SnapTime = TrackScaleInfo.LocalXToInput(SnapX) - CurrentEvent->GetDuration();
								CurrentNode->SetLastSnappedTime(SnapTime);
								SnappedNode = CurrentNode;
							}
						}
						else
						{
							// Remove any trigger time if we can't fit the node in.
							CurrentEvent->EndTriggerTimeOffset = GetTriggerTimeOffsetForType(EAnimEventTriggerOffsets::NoOffset);
						}
					}
				}
			}
		}

		SelectionBeginPosition.X += SnapMovement;

		CurrentDragXPosition = static_cast<float>(TrackGeom.AbsoluteToLocal(FVector2f(SelectionBeginPosition.X,0.0f)).X);

		CursorDecoratorWindow->MoveWindowTo(TrackGeom.LocalToAbsolute(TrackGeom.AbsoluteToLocal(SelectionBeginPosition) - SelectedNodes[0]->GetNotifyPositionOffset()));
		NodeGroupPosition = SelectionBeginPosition;

		//scroll view
		const float LocalMouseXPos = static_cast<float>(TrackGeom.AbsoluteToLocal(DragDropEvent.GetScreenSpacePosition()).X);
		constexpr float LocalViewportMin = 0.0f;
		const float LocalViewportMax = static_cast<float>(TrackGeom.GetLocalSize().X);
		if(LocalMouseXPos < LocalViewportMin && LocalViewportMin > LocalTrackMin - 10.0f)
		{
			const float ScreenDelta = FMath::Max(LocalMouseXPos - LocalViewportMin, -10.0f);
			RequestTrackPan.Execute(static_cast<int32>(ScreenDelta), FVector2f(LocalTrackWidth, 1.f));
		}
		else if(LocalMouseXPos > LocalViewportMax && LocalViewportMax < LocalTrackMax + 10.0f)
		{
			const float ScreenDelta =  FMath::Max(LocalMouseXPos - LocalViewportMax, 10.0f);
			RequestTrackPan.Execute(static_cast<int32>(ScreenDelta), FVector2f(LocalTrackWidth, 1.f));
		}

		OnNodesBeingDragged.ExecuteIfBound(SelectedNodes, DragDropEvent, CurrentDragXPosition, TrackScaleInfo.LocalXToInput(CurrentDragXPosition));
	}

	float GetSnapPosition(const FTrackClampInfo& ClampInfo, float WidgetSpaceNotifyPosition, bool& bOutSnapped)
	{
		const FTrackScaleInfo& ScaleInfo = ClampInfo.NotifyTrack->GetCachedScaleInfo();

		const float MaxSnapDist = 5.f;

		float CurrentMinSnapDest = MaxSnapDist;
		float SnapPosition = ScaleInfo.LocalXToInput(WidgetSpaceNotifyPosition);
		FName SnapTypeUnused;
		bOutSnapped = OnSnapPosition.IsBound() && !FSlateApplication::Get().GetModifierKeys().IsControlDown() && ScaleInfo.PixelsPerInput > 0.0f && OnSnapPosition.Execute(SnapPosition, MaxSnapDist / ScaleInfo.PixelsPerInput, TArrayView<const FName>(), SnapTypeUnused);
		SnapPosition = ScaleInfo.InputToLocalX(SnapPosition);

		float WidgetSpaceStartPosition = ScaleInfo.InputToLocalX(0.0f);
		float WidgetSpaceEndPosition = ScaleInfo.InputToLocalX(Sequence->GetPlayLength());

		if(!bOutSnapped)
		{
			// Didn't snap to a bar, snap to the track bounds
			float SnapDistBegin = FMath::Abs(WidgetSpaceStartPosition - WidgetSpaceNotifyPosition);
			float SnapDistEnd = FMath::Abs(WidgetSpaceEndPosition - WidgetSpaceNotifyPosition);
			if(SnapDistBegin < CurrentMinSnapDest)
			{
				SnapPosition = WidgetSpaceStartPosition;
				bOutSnapped = true;
			}
			else if(SnapDistEnd < CurrentMinSnapDest)
			{
				SnapPosition = WidgetSpaceEndPosition;
				bOutSnapped = true;
			}
		}

		return SnapPosition;
	}

	FTrackClampInfo& GetTrackClampInfo(const FVector2D NodePos)
	{
		int32 ClampInfoIndex = 0;
		int32 SmallestNodeTrackDist = FMath::Abs(ClampInfos[0].TrackSnapTestPos - static_cast<int32>(NodePos.Y));
		for(int32 i = 0; i < ClampInfos.Num(); ++i)
		{
			const int32 Dist = FMath::Abs(ClampInfos[i].TrackSnapTestPos - static_cast<int32>(NodePos.Y));
			if(Dist < SmallestNodeTrackDist)
			{
				SmallestNodeTrackDist = Dist;
				ClampInfoIndex = i;
			}
		}
		return ClampInfos[ClampInfoIndex];
	}

	class UAnimSequenceBase*			Sequence;				// The owning anim sequence
	FVector2D							DragOffset;				// Offset from the mouse to place the decorator
	TArray<FTrackClampInfo>				ClampInfos;				// Clamping information for all of the available tracks
	float&								CurrentDragXPosition;	// Current X position of the drag operation
	FPanTrackRequest					RequestTrackPan;		// Delegate to request a pan along the edges of a zoomed track
	TArray<float>						NodeTimes;				// Times to drop each selected node at
	float								SnapTime;				// The time that the snapped node was snapped to
	TWeakPtr<SAnimNotifyNode>			SnappedNode;			// The node chosen for the snap
	TArray<TSharedPtr<SAnimNotifyNode>> SelectedNodes;			// The nodes that are in the current selection
	TArray<float>						NodeTimeOffsets;		// Time offsets from the beginning of the selection to the nodes.
	TArray<float>						NodeXOffsets;			// Offsets in X from the widget position to the scrub handle for each node.
	FVector2D							NodeGroupPosition;		// Position of the beginning of the selection
	float								NodeGroupWidth;			// Horizontal size of the selection in track-local pixels
	TSharedPtr<SWidget>					Decorator;				// The widget to display when dragging
	float								SelectionTimeLength;	// Length of time that the selection covers
	int32								TrackSpan;				// Number of tracks that the selection spans
	FOnUpdatePanel						OnUpdatePanel;			// Delegate to redraw the notify panel
	FOnSnapPosition						OnSnapPosition;			// Delegate used to snap times
	FOnNotifyNodesBeingDragged			OnNodesBeingDragged;	// Delegate to notify panel when the mouse was moved during the DragDropOp

	static TSharedRef<FNotifyDragDropOp> New(
		TArray<TSharedPtr<SAnimNotifyNode>>			NotifyNodes, 
		TSharedPtr<SWidget>							Decorator, 
		const TArray<TSharedPtr<SAnimNotifyTrack>>& NotifyTracks, 
		class UAnimSequenceBase*					InSequence, 
		const FVector2D&							CursorPosition, 
		const FVector2D&							SelectionScreenPosition, 
		float										SelectionWidth, 
		float&										CurrentDragXPosition, 
		FPanTrackRequest&							RequestTrackPanDelegate, 
		FOnSnapPosition&							OnSnapPosition,
		FOnUpdatePanel&								UpdatePanel,
		FOnNotifyNodesBeingDragged&					OnNodesBeingDragged
		)
	{
		TSharedRef<FNotifyDragDropOp> Operation = MakeShareable(new FNotifyDragDropOp(CurrentDragXPosition));
		Operation->Sequence = InSequence;
		Operation->RequestTrackPan = RequestTrackPanDelegate;
		Operation->OnUpdatePanel = UpdatePanel;
		Operation->OnNodesBeingDragged = OnNodesBeingDragged;

		Operation->NodeGroupPosition = SelectionScreenPosition;
		Operation->NodeGroupWidth = SelectionWidth;
		Operation->DragOffset = SelectionScreenPosition - CursorPosition;
		Operation->OnSnapPosition = OnSnapPosition;
		Operation->Decorator = Decorator;
		Operation->SelectedNodes = NotifyNodes;
		Operation->TrackSpan = NotifyNodes.Last()->NodeObjectInterface->GetTrackIndex() - NotifyNodes[0]->NodeObjectInterface->GetTrackIndex();
		
		// Caclulate offsets for the selected nodes
		float BeginTime = MAX_flt;
		for(TSharedPtr<SAnimNotifyNode> Node : NotifyNodes)
		{
			float NotifyTime = Node->NodeObjectInterface->GetTime();

			if(NotifyTime < BeginTime)
			{
				BeginTime = NotifyTime;
			}
		}

		// Initialise node data
		for(TSharedPtr<SAnimNotifyNode> Node : NotifyNodes)
		{
			float NotifyTime = Node->NodeObjectInterface->GetTime();

			Node->ClearLastSnappedTime();
			Operation->NodeTimeOffsets.Add(NotifyTime - BeginTime);
			Operation->NodeTimes.Add(NotifyTime);
			Operation->NodeXOffsets.Add(static_cast<float>(Node->GetNotifyPositionOffset().X));

			// Calculate the time length of the selection. Because it is possible to have states
			// with arbitrary durations we need to search all of the nodes and find the furthest
			// possible point
			Operation->SelectionTimeLength = FMath::Max(Operation->SelectionTimeLength, NotifyTime + Node->NodeObjectInterface->GetDuration() - BeginTime);
		}

		Operation->Construct();

		for(int32 i = 0; i < NotifyTracks.Num(); ++i)
		{
			FTrackClampInfo Info;
			Info.NotifyTrack = NotifyTracks[i];
			const FGeometry& CachedGeometry = Info.NotifyTrack->GetCachedGeometry();
			// Use absolute size to keep TrackSnapTestPos in absolute space alongside TrackPos.
			Info.TrackPos = static_cast<int32>(CachedGeometry.AbsolutePosition.Y);
			Info.TrackSnapTestPos = Info.TrackPos + static_cast<int32>(CachedGeometry.GetAbsoluteSize().Y / 2);
			Operation->ClampInfos.Add(Info);
		}

		Operation->CursorDecoratorWindow->SetOpacity(0.5f);
		return Operation;
	}
	
	/** The widget decorator to use */
	virtual TSharedPtr<SWidget> GetDefaultDecorator() const override
	{
		return Decorator;
	}

	FText GetHoverText() const
	{
		FText HoverText = LOCTEXT("Invalid", "Invalid");

		if(SelectedNodes[0].IsValid())
		{
			HoverText = FText::FromName( SelectedNodes[0]->NodeObjectInterface->GetName() );
		}

		return HoverText;
	}
};

//////////////////////////////////////////////////////////////////////////
// SAnimNotifyTrack

void SAnimNotifyTrack::Construct(const FArguments& InArgs)
{
	SetClipping(EWidgetClipping::ClipToBounds);
	
	WeakCommandList = InArgs._CommandList;
	Sequence = InArgs._Sequence;
	ViewInputMin = InArgs._ViewInputMin;
	ViewInputMax = InArgs._ViewInputMax;
	OnSelectionChanged = InArgs._OnSelectionChanged;
	AnimNotifies = InArgs._AnimNotifies;
	AnimSyncMarkers = InArgs._AnimSyncMarkers;
	OnUpdatePanel = InArgs._OnUpdatePanel;
	OnGetNotifyBlueprintData = InArgs._OnGetNotifyBlueprintData;
	OnGetNotifyStateBlueprintData = InArgs._OnGetNotifyStateBlueprintData;
	OnGetNotifyNativeClasses = InArgs._OnGetNotifyNativeClasses;
	OnGetNotifyStateNativeClasses = InArgs._OnGetNotifyStateNativeClasses;
	TrackIndex = InArgs._TrackIndex;
	OnGetScrubValue = InArgs._OnGetScrubValue;
	OnGetDraggedNodePos = InArgs._OnGetDraggedNodePos;
	OnNodeDragStarted = InArgs._OnNodeDragStarted;
	OnNotifyStateHandleBeingDragged = InArgs._OnNotifyStateHandleBeingDragged;
	TrackColor = InArgs._TrackColor;
	OnSnapPosition = InArgs._OnSnapPosition;
	OnRequestTrackPan = InArgs._OnRequestTrackPan;
	OnRequestRefreshOffsets = InArgs._OnRequestOffsetRefresh;
	OnDeleteNotify = InArgs._OnDeleteNotify;
	OnGetIsAnimNotifySelectionValidforReplacement = InArgs._OnGetIsAnimNotifySelectionValidForReplacement;
	OnReplaceSelectedWithNotify = InArgs._OnReplaceSelectedWithNotify;
	OnReplaceSelectedWithBlueprintNotify = InArgs._OnReplaceSelectedWithBlueprintNotify;
	OnReplaceSelectedWithSyncMarker = InArgs._OnReplaceSelectedWithSyncMarker;
	OnDeselectAllNotifies = InArgs._OnDeselectAllNotifies;
	OnCopyNodes = InArgs._OnCopyNodes;
	OnPasteNodes = InArgs._OnPasteNodes;
	OnSetInputViewRange = InArgs._OnSetInputViewRange;
	OnGetTimingNodeVisibility = InArgs._OnGetTimingNodeVisibility;
	OnInvokeTab = InArgs._OnInvokeTab;

	this->ChildSlot
	[
			SAssignNew( TrackArea, SBorder )
			.Visibility(EVisibility::SelfHitTestInvisible)
			.BorderImage( FAppStyle::GetBrush("NoBorder") )
			.Padding( FMargin(0.f, 0.f) )
	];
	Update();

}

FVector2D SAnimNotifyTrack::ComputeDesiredSize( float ) const
{
	FVector2D Size;
	Size.X = 200;
	Size.Y = FAnimTimelineTrack_NotifiesPanel::NotificationTrackHeight;
	return Size;
}

int32 SAnimNotifyTrack::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	const FSlateBrush* StyleInfo = FAppStyle::GetBrush( TEXT( "Persona.NotifyEditor.NotifyTrackBackground" ) );
	FLinearColor Color = TrackColor.Get();

	FPaintGeometry MyGeometry = AllottedGeometry.ToPaintGeometry();

	int32 CustomLayerId = LayerId + 1; 
	FTrackScaleInfo ScaleInfo(ViewInputMin.Get(), ViewInputMax.Get(), 0.f, 0.f, AllottedGeometry.Size);

	bool bAnyDraggedNodes = false;
	for ( int32 I=0; I<NotifyNodes.Num(); ++I )
	{
		if ( NotifyNodes[I].Get()->bBeingDragged == false )
		{
			NotifyNodes[I].Get()->UpdateSizeAndPosition(AllottedGeometry);
		}
		else
		{
			bAnyDraggedNodes = true;
		}
	}

	if(TrackIndex < Sequence->AnimNotifyTracks.Num() - 1)
	{
		// Draw track bottom border
		FSlateDrawElement::MakeLines(
			OutDrawElements,
			CustomLayerId,
			AllottedGeometry.ToPaintGeometry(),
			TArray<FVector2D>({ FVector2D(0.0f, AllottedGeometry.GetLocalSize().Y), FVector2D(AllottedGeometry.GetLocalSize().X, AllottedGeometry.GetLocalSize().Y) }),
			ESlateDrawEffect::None,
			FLinearColor(0.1f, 0.1f, 0.1f, 0.3f)
		);
	}

	++CustomLayerId;

	float Value = 0.f;

	if ( bAnyDraggedNodes && OnGetDraggedNodePos.IsBound() )
	{
		Value = OnGetDraggedNodePos.Execute();

		if(Value >= 0.0f)
		{
			float XPos = Value;
			TArray<FVector2D> LinePoints;
			LinePoints.Add(FVector2D(XPos, 0.f));
			LinePoints.Add(FVector2D(XPos, AllottedGeometry.Size.Y));

			FSlateDrawElement::MakeLines( 
				OutDrawElements,
				CustomLayerId,
				MyGeometry,
				LinePoints,
				ESlateDrawEffect::None,
				FLinearColor(1.0f, 0.5f, 0.0f)
				);
		}
	}

	return SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, CustomLayerId, InWidgetStyle, bParentEnabled);
}

FCursorReply SAnimNotifyTrack::OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const
{
	if (ViewInputMin.Get() > 0.f || ViewInputMax.Get() < Sequence->GetPlayLength())
	{
		return FCursorReply::Cursor(EMouseCursor::GrabHand);
	}

	return FCursorReply::Unhandled();
}

void SAnimNotifyTrack::OnAnimNotifyClassPicked(UClass* NotifyClass, bool bIsReplaceWithMenu /* = false */)
{
	FSlateApplication::Get().DismissAllMenus();

	if (bIsReplaceWithMenu)
	{
		ReplaceSelectedWithNotify(MakeBlueprintNotifyName(NotifyClass->GetName()), NotifyClass);
	}
	else
	{
		CreateNewNotifyAtCursor(MakeBlueprintNotifyName(NotifyClass->GetName()), NotifyClass);
	}
}

void SAnimNotifyTrack::FillNewNotifyStateMenu(FMenuBuilder& MenuBuilder, bool bIsReplaceWithMenu /* = false */)
{
	// MenuBuilder always has a search widget added to it by default, hence if larger then 1 then something else has been added to it
	if (MenuBuilder.GetMultiBox()->GetBlocks().Num() > 1)
	{
		MenuBuilder.AddMenuSeparator();
	}

	TSharedRef<SWidget> Widget = 
		SNew(SBox)
		.WidthOverride(300.0f)
		.HeightOverride(300.0f)
		[	
			PersonaUtils::MakeAnimNotifyStatePicker(Sequence, FOnClassPicked::CreateRaw(this, &SAnimNotifyTrack::OnAnimNotifyClassPicked, bIsReplaceWithMenu))
		];
	MenuBuilder.AddWidget(Widget, FText(), true, false);
}

void SAnimNotifyTrack::FillNewNotifyMenu(FMenuBuilder& MenuBuilder, bool bIsReplaceWithMenu /* = false */)
{
	// now add custom anim notifiers
	USkeleton* SeqSkeleton = Sequence->GetSkeleton();
	if (SeqSkeleton)
	{
		MenuBuilder.BeginSection("AnimNotifySubMenu", LOCTEXT("NewNotifySubMenu", "Notifies"));
		{
			if (!bIsReplaceWithMenu)
			{
				FUIAction UIAction;
				UIAction.ExecuteAction.BindSP(
					this, &SAnimNotifyTrack::OnNewNotifyClicked);
				MenuBuilder.AddMenuEntry(LOCTEXT("NewNotify", "New Notify..."), LOCTEXT("NewNotifyToolTip", "Create a new animation notify"), FSlateIcon(), UIAction);
			}

			ISkeletonEditorModule& SkeletonEditorModule = FModuleManager::LoadModuleChecked<ISkeletonEditorModule>("SkeletonEditor");
			TSharedRef<IEditableSkeleton> EditableSkeleton = SkeletonEditorModule.CreateEditableSkeleton(SeqSkeleton);

			MenuBuilder.AddWidget(
				SNew(SBox)
				.WidthOverride(300.0f)
				.HeightOverride(250.0f)
				[
					SNew(SSkeletonAnimNotifies)
					.IsPicker(true)
					.ShowSyncMarkers(false)
					.ShowNotifies(true)
					.ShowCompatibleSkeletonAssets(true)
					.ShowOtherAssets(true)
					.EditableSkeleton(EditableSkeleton)
					.OnItemSelected_Lambda([this, bIsReplaceWithMenu](const FName& InNotifyName)
					{
						FSlateApplication::Get().DismissAllMenus();

						if (!bIsReplaceWithMenu)
						{
							CreateNewNotifyAtCursor(InNotifyName.ToString(), nullptr);
						}
						else
						{
							ReplaceSelectedWithNotify(InNotifyName.ToString(), nullptr);
						}
					})
				],
				FText(), true, false);
		}
		MenuBuilder.EndSection();
	}

	MenuBuilder.BeginSection("AnimNotifyNotifySubMenu", LOCTEXT("NewNotifySubMenu_Notifies", "Notifies"));
	{
		// Add a notify picker
		TSharedRef<SWidget> Widget = 
			SNew(SBox)
			.WidthOverride(300.0f)
			.HeightOverride(250.0f)
			[
				PersonaUtils::MakeAnimNotifyPicker(Sequence, FOnClassPicked::CreateRaw(this, &SAnimNotifyTrack::OnAnimNotifyClassPicked, bIsReplaceWithMenu))
			];
		MenuBuilder.AddWidget(Widget, FText(), true, false);
	}
	MenuBuilder.EndSection();
}

void SAnimNotifyTrack::FillNewSyncMarkerMenu(FMenuBuilder& MenuBuilder, bool bIsReplaceWithMenu /* = false */)
{
	USkeleton* SeqSkeleton = Sequence->GetSkeleton();
	if (SeqSkeleton)
	{
		MenuBuilder.BeginSection("AnimSyncMarkerSubMenu", LOCTEXT("NewSyncMarkerSubMenu", "Sync Markers"));
		{
			FUIAction UIAction;
			if (!bIsReplaceWithMenu)
			{
				UIAction.ExecuteAction.BindSP(
					this, &SAnimNotifyTrack::OnNewSyncMarkerClicked);
				MenuBuilder.AddMenuEntry(LOCTEXT("NewSyncMarker", "New Sync Marker..."), LOCTEXT("NewSyncMarkerToolTip", "Create a new animation sync marker"), FSlateIcon(), UIAction);
			}

			ISkeletonEditorModule& SkeletonEditorModule = FModuleManager::LoadModuleChecked<ISkeletonEditorModule>("SkeletonEditor");
			TSharedRef<IEditableSkeleton> EditableSkeleton = SkeletonEditorModule.CreateEditableSkeleton(SeqSkeleton);

			MenuBuilder.AddWidget(
				SNew(SBox)
				.WidthOverride(300.0f)
				.HeightOverride(250.0f)
				[
					SNew(SSkeletonAnimNotifies)
					.IsPicker(true)
					.ShowSyncMarkers(true)
					.ShowNotifies(false)
					.ShowCompatibleSkeletonAssets(true)
					.ShowOtherAssets(true)
					.EditableSkeleton(EditableSkeleton)
					.OnItemSelected_Lambda([this, bIsReplaceWithMenu](const FName& InNotifyName)
					{
						FSlateApplication::Get().DismissAllMenus();

						if (!bIsReplaceWithMenu)
						{
							CreateNewSyncMarkerAtCursor(InNotifyName.ToString());
						}
						else
						{
							ReplaceSelectedWithSyncMarker(InNotifyName.ToString());
						}
					})
				],
				FText(), true, false);
		}
		MenuBuilder.EndSection();
	}
}

FAnimNotifyEvent& SAnimNotifyTrack::CreateNewBlueprintNotify(FString NewNotifyName, FString BlueprintPath, float StartTime)
{
	TSubclassOf<UObject> BlueprintClass = GetBlueprintClassFromPath(BlueprintPath);
	check(BlueprintClass);
	return CreateNewNotify(NewNotifyName, BlueprintClass, StartTime);
}

FAnimNotifyEvent& SAnimNotifyTrack::CreateNewNotify(FString NewNotifyName, UClass* NotifyClass, float StartTime)
{
	// Insert a new notify record and spawn the new notify object
	int32 NewNotifyIndex = Sequence->Notifies.Add(FAnimNotifyEvent());
	FAnimNotifyEvent& NewEvent = Sequence->Notifies[NewNotifyIndex];
	NewEvent.NotifyName = FName(*NewNotifyName);
	NewEvent.Guid = FGuid::NewGuid();

	NewEvent.Link(Sequence, StartTime);
	NewEvent.TriggerTimeOffset = GetTriggerTimeOffsetForType(Sequence->CalculateOffsetForNotify(StartTime));
	NewEvent.TrackIndex = TrackIndex;

	if( NotifyClass )
	{
		class UObject* AnimNotifyClass = NewObject<UObject>(Sequence, NotifyClass, NAME_None, RF_Transactional);
		NewEvent.NotifyStateClass = Cast<UAnimNotifyState>(AnimNotifyClass);
		NewEvent.Notify = Cast<UAnimNotify>(AnimNotifyClass);

		if( NewEvent.NotifyStateClass )
		{
			// Set default duration to 1 frame for AnimNotifyState.
			NewEvent.SetDuration(1 / 30.f);
			NewEvent.EndLink.Link(Sequence, NewEvent.EndLink.GetTime());
			NewEvent.TriggerWeightThreshold = NewEvent.NotifyStateClass->GetDefaultTriggerWeightThreshold();
		}
		else if ( NewEvent.Notify )
		{
			NewEvent.TriggerWeightThreshold = NewEvent.Notify->GetDefaultTriggerWeightThreshold();
		}
	}
	else
	{
		NewEvent.Notify = NULL;
		NewEvent.NotifyStateClass = NULL;
	}

	if(NewEvent.Notify)
	{
		TArray<FAssetData> SelectedAssets;
		AssetSelectionUtils::GetSelectedAssets(SelectedAssets);

		for( TFieldIterator<FObjectProperty> PropIt(NewEvent.Notify->GetClass()); PropIt; ++PropIt )
		{
			if(PropIt->GetBoolMetaData(TEXT("ExposeOnSpawn")))
			{
				FObjectProperty* Property = *PropIt;
				const FAssetData* Asset = SelectedAssets.FindByPredicate([Property](const FAssetData& Other)
				{
					return Other.GetAsset()->IsA(Property->PropertyClass);
				});

				if( Asset )
				{
					uint8* Offset = (*PropIt)->ContainerPtrToValuePtr<uint8>(NewEvent.Notify);
					(*PropIt)->ImportText_Direct( *Asset->GetAsset()->GetPathName(), Offset, NewEvent.Notify, 0 );
					break;
				}
			}
		}

		NewEvent.Notify->OnAnimNotifyCreatedInEditor(NewEvent);
	}
	else if (NewEvent.NotifyStateClass)
	{
		NewEvent.NotifyStateClass->OnAnimNotifyCreatedInEditor(NewEvent);
	}

	Sequence->PostEditChange();
	Sequence->MarkPackageDirty();

	return NewEvent;
}

void SAnimNotifyTrack::CreateNewBlueprintNotifyAtCursor(FString NewNotifyName, FString BlueprintPath)
{
	TSubclassOf<UObject> BlueprintClass = GetBlueprintClassFromPath(BlueprintPath);
	check(BlueprintClass);
	CreateNewNotifyAtCursor(NewNotifyName, BlueprintClass);
}

void SAnimNotifyTrack::CreateNewNotifyAtCursor(FString NewNotifyName, UClass* NotifyClass)
{
	const FScopedTransaction Transaction(LOCTEXT("AddNotifyEvent", "Add Anim Notify"));
	Sequence->Modify();
	CreateNewNotify(NewNotifyName, NotifyClass, LastClickedTime);
	OnUpdatePanel.ExecuteIfBound();
}

void SAnimNotifyTrack::CreateNewSyncMarkerAtCursor(FString NewSyncMarkerName)
{
	UAnimSequence* Seq = CastChecked<UAnimSequence>(Sequence);

	FScopedTransaction Transaction(LOCTEXT("AddSyncMarker", "Add Sync Marker"));
	Seq->Modify();
	int32 NewIndex = Seq->AuthoredSyncMarkers.Add(FAnimSyncMarker());
	FAnimSyncMarker& SyncMarker = Seq->AuthoredSyncMarkers[NewIndex];
	SyncMarker.MarkerName = FName(*NewSyncMarkerName);
	SyncMarker.TrackIndex = TrackIndex;
	SyncMarker.Time = LastClickedTime;
	SyncMarker.Guid = FGuid::NewGuid();

	Seq->PostEditChange();
	Seq->MarkPackageDirty();
	OnUpdatePanel.ExecuteIfBound();

	UBlendSpace::UpdateBlendSpacesUsingAnimSequence(Seq);
}

void SAnimNotifyTrack::ReplaceSelectedWithBlueprintNotify(FString NewNotifyName, FString BlueprintPath)
{
	OnReplaceSelectedWithBlueprintNotify.ExecuteIfBound(NewNotifyName, BlueprintPath);
}

void SAnimNotifyTrack::ReplaceSelectedWithNotify(FString NewNotifyName, UClass* NotifyClass)
{
	OnReplaceSelectedWithNotify.ExecuteIfBound(NewNotifyName, NotifyClass);
}

void SAnimNotifyTrack::ReplaceSelectedWithSyncMarker(FString NewNotifyName)
{
	OnReplaceSelectedWithSyncMarker.ExecuteIfBound(NewNotifyName);
}

bool SAnimNotifyTrack::IsValidToPlace(UClass* NotifyClass) const
{
	if (NotifyClass && NotifyClass->IsChildOf(UAnimNotify::StaticClass()))
	{
		UAnimNotify* DefaultNotify = NotifyClass->GetDefaultObject<UAnimNotify>();
		return DefaultNotify->CanBePlaced(Sequence);
	}

	if (NotifyClass && NotifyClass->IsChildOf(UAnimNotifyState::StaticClass()))
	{
		UAnimNotifyState* DefaultNotifyState = NotifyClass->GetDefaultObject<UAnimNotifyState>();
		return DefaultNotifyState->CanBePlaced(Sequence);
	}

	return true;
}

TSubclassOf<UObject> SAnimNotifyTrack::GetBlueprintClassFromPath(FString BlueprintPath)
{
	TSubclassOf<UObject> BlueprintClass = NULL;
	if (!BlueprintPath.IsEmpty())
	{
		UBlueprint* BlueprintLibPtr = LoadObject<UBlueprint>(NULL, *BlueprintPath, NULL, 0, NULL);
		BlueprintClass = BlueprintLibPtr->GeneratedClass;
	}
	return BlueprintClass;
}

FReply SAnimNotifyTrack::OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	bool bLeftMouseButton =  MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton;
	bool bRightMouseButton =  MouseEvent.GetEffectingButton() == EKeys::RightMouseButton;
	bool bShift = MouseEvent.IsShiftDown();
	bool bCtrl = MouseEvent.IsControlDown();

	if ( bRightMouseButton )
	{
		TSharedPtr<SWidget> WidgetToFocus;

		WidgetToFocus = SummonContextMenu(MyGeometry, MouseEvent);

		return (WidgetToFocus.IsValid())
			? FReply::Handled().ReleaseMouseCapture().SetUserFocus(WidgetToFocus.ToSharedRef(), EFocusCause::SetDirectly)
			: FReply::Handled().ReleaseMouseCapture();
	}
	else if ( bLeftMouseButton )
	{
		FVector2D CursorPos = MouseEvent.GetScreenSpacePosition();
		CursorPos = MyGeometry.AbsoluteToLocal(CursorPos);
		int32 NotifyIndex = GetHitNotifyNode(MyGeometry, CursorPos);
		LastClickedTime = CalculateTime(MyGeometry, MouseEvent.GetScreenSpacePosition());

		if(NotifyIndex == INDEX_NONE)
		{
			// Clicked in empty space, clear selection
			OnDeselectAllNotifies.ExecuteIfBound();
		}
		else
		{
			if(bCtrl)
			{
				ToggleTrackObjectNodeSelectionStatus(NotifyIndex);
			}
			else
			{
				SelectTrackObjectNode(NotifyIndex, bShift);
			}
		}

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void SAnimNotifyTrack::SelectTrackObjectNode(int32 TrackNodeIndex, bool Append, bool bUpdateSelection)
{
	if( TrackNodeIndex != INDEX_NONE )
	{
		// Deselect all other notifies if necessary.
		if (Sequence && !Append)
		{
			OnDeselectAllNotifies.ExecuteIfBound();
		}

		// Check to see if we've already selected this node
		if (!SelectedNodeIndices.Contains(TrackNodeIndex))
		{
			// select new one
			if (NotifyNodes.IsValidIndex(TrackNodeIndex))
			{
				TSharedPtr<SAnimNotifyNode> Node = NotifyNodes[TrackNodeIndex];
				Node->bSelected = true;
				SelectedNodeIndices.Add(TrackNodeIndex);

				if(bUpdateSelection)
				{
					SendSelectionChanged();
				}
			}
		}
	}
}

void SAnimNotifyTrack::ToggleTrackObjectNodeSelectionStatus( int32 TrackNodeIndex, bool bUpdateSelection )
{
	check(NotifyNodes.IsValidIndex(TrackNodeIndex));

	bool bSelected = SelectedNodeIndices.Contains(TrackNodeIndex);
	if(bSelected)
	{
		SelectedNodeIndices.Remove(TrackNodeIndex);
	}
	else
	{
		SelectedNodeIndices.Add(TrackNodeIndex);
	}

	TSharedPtr<SAnimNotifyNode> Node = NotifyNodes[TrackNodeIndex];
	Node->bSelected = !Node->bSelected;

	if(bUpdateSelection)
	{
		SendSelectionChanged();
	}
}

void SAnimNotifyTrack::DeselectTrackObjectNode( int32 TrackNodeIndex, bool bUpdateSelection )
{
	check(NotifyNodes.IsValidIndex(TrackNodeIndex));
	TSharedPtr<SAnimNotifyNode> Node = NotifyNodes[TrackNodeIndex];
	Node->bSelected = false;

	int32 ItemsRemoved = SelectedNodeIndices.Remove(TrackNodeIndex);
	check(ItemsRemoved > 0);

	if(bUpdateSelection)
	{
		SendSelectionChanged();
	}
}

void SAnimNotifyTrack::DeselectAllNotifyNodes(bool bUpdateSelectionSet)
{
	for(TSharedPtr<SAnimNotifyNode> Node : NotifyNodes)
	{
		Node->bSelected = false;
	}
	SelectedNodeIndices.Empty();

	if(bUpdateSelectionSet)
	{
		SendSelectionChanged();
	}
}

void SAnimNotifyTrack::SelectNodesByGuid(const TSet<FGuid>& InGuids, bool bUpdateSelectionSet)
{
	SelectedNodeIndices.Empty();

	for(int32 NodeIndex = 0; NodeIndex < NotifyNodes.Num(); ++NodeIndex)
	{
		TSharedPtr<SAnimNotifyNode> Node = NotifyNodes[NodeIndex];
		Node->bSelected = InGuids.Contains(Node->NodeObjectInterface->GetGuid());
		if(Node->bSelected)
		{
			SelectedNodeIndices.Add(NodeIndex);
		}
	}

	if(bUpdateSelectionSet)
	{
		SendSelectionChanged();
	}
}

TSharedPtr<SWidget> SAnimNotifyTrack::SummonContextMenu(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	if (!ToolMenus)
	{
		return SNullWidget::NullWidget;
	}

	UAnimNotifyPanelContextMenuContext* MenuContext = NewObject<UAnimNotifyPanelContextMenuContext>();

	FVector2D CursorPos = MouseEvent.GetScreenSpacePosition();
	LastClickedTime = CalculateTime(MyGeometry, MouseEvent.GetScreenSpacePosition());

	MenuContext->NodeIndex = GetHitNotifyNode(MyGeometry, MyGeometry.AbsoluteToLocal(CursorPos));
	MenuContext->NotifyTrack = SharedThis(this);
	MenuContext->NodeObject = MenuContext->NodeIndex != INDEX_NONE ? NotifyNodes[MenuContext->NodeIndex]->NodeObjectInterface : nullptr;
	MenuContext->NotifyEvent = MenuContext->NodeObject ? MenuContext->NodeObject->GetNotifyEvent() : nullptr;
	MenuContext->NotifyIndex = MenuContext->NotifyEvent ? AnimNotifies.IndexOfByKey(MenuContext->NotifyEvent) : INDEX_NONE;
	MenuContext->MouseEvent = MouseEvent; 

	static const FName BaseMenuName("Persona.AnimNotifyTrackContextMenu");
	if (!ToolMenus->IsMenuRegistered(BaseMenuName))
	{
		CreateContextMenuContent(BaseMenuName);
	}

	FToolMenuContext ToolMenuContext(MenuContext);
	if (WeakCommandList.IsValid())
	{
		ToolMenuContext.AppendCommandList(WeakCommandList.Pin());
	}

	TSharedPtr<SWidget> MenuWidget = ToolMenus->GenerateWidget(BaseMenuName, ToolMenuContext);

	if (MenuWidget.IsValid())
	{
		const FVector2D MouseCursorLocation = FSlateApplication::Get().GetCursorPos();

		FSlateApplication::Get().PushMenu(
			SharedThis(this),
			FWidgetPath(),
			MenuWidget.ToSharedRef(),
			MouseCursorLocation,
			FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu)
		);
	}
	return TSharedPtr<SWidget>();
}

UToolMenu* SAnimNotifyTrack::CreateContextMenuContent(FName BaseMenuName)
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	UToolMenu* Menu = ToolMenus->RegisterMenu(BaseMenuName);

	Menu->AddDynamicSection(NAME_None, FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
		{
			FToolMenuSection& Section = InMenu->AddSection("AnimNotify", LOCTEXT("NotifyHeading", "Notify"));
			UAnimNotifyPanelContextMenuContext* MenuContext = InMenu->FindContext<UAnimNotifyPanelContextMenuContext>();
			TSharedPtr< SAnimNotifyTrack> SourceTrack = MenuContext->NotifyTrack.Pin();
			if (MenuContext->NodeObject)
			{
				if (!SourceTrack->NotifyNodes[MenuContext->NodeIndex]->bSelected)
				{
					SourceTrack->SelectTrackObjectNode(MenuContext->NodeIndex, MenuContext->MouseEvent.IsControlDown());
				}

				if (SourceTrack->IsSingleNodeSelected())
				{
					// Add item to directly set notify time
					TSharedRef<SWidget> TimeWidget =
					SNew(SBox)
					.HAlign(HAlign_Right)
					.ToolTipText(LOCTEXT("SetTimeToolTip", "Set the time of this notify directly"))
					[
						SNew(SBox)
						.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
						.WidthOverride(100.0f)
						[
							SNew(SNumericEntryBox<float>)
							.Font(FAppStyle::GetFontStyle(TEXT("MenuItem.Font")))
							.MinValue(0.0f)
							.MaxValue(SourceTrack->Sequence->GetPlayLength())
							.Value(MenuContext->NodeObject->GetTime())
							.AllowSpin(false)
							.OnValueCommitted_Lambda([SourceTrack, MenuContext](float InValue, ETextCommit::Type InCommitType)
								{
									if (InCommitType == ETextCommit::OnEnter && SourceTrack->NotifyNodes.IsValidIndex(MenuContext->NodeIndex))
									{
										const FScopedTransaction Transaction(LOCTEXT("SetNotifyTimeTransaction", "Set Anim Notify trigger time"));
										SourceTrack->Sequence->Modify();

										INotifyNodeObjectInterface* LocalNodeObject = SourceTrack->NotifyNodes[MenuContext->NodeIndex]->NodeObjectInterface;

										float NewTime = FMath::Clamp(InValue, 0.0f, SourceTrack->Sequence->GetPlayLength() - LocalNodeObject->GetDuration());
										LocalNodeObject->SetTime(NewTime);

										if (FAnimNotifyEvent* Event = LocalNodeObject->GetNotifyEvent())
										{
											Event->RefreshTriggerOffset(SourceTrack->Sequence->CalculateOffsetForNotify(Event->GetTime()));
											if (Event->GetDuration() > 0.0f)
											{
												Event->RefreshEndTriggerOffset(SourceTrack->Sequence->CalculateOffsetForNotify(Event->GetTime() + Event->GetDuration()));
											}
										}
										SourceTrack->OnUpdatePanel.ExecuteIfBound();

										FSlateApplication::Get().DismissAllMenus();
									}
								})
						]
					];

					Section.AddEntry(
						FToolMenuEntry::InitWidget(TEXT("AnimNotifyContextMenuTimeWidget"), TimeWidget, FText::FromString("Notify Begin Time"), true, false)
					);


					// Add item to directly set notify frame
					TSharedRef<SWidget> FrameWidget =
					SNew(SBox)
					.HAlign(HAlign_Right)
					.ToolTipText(LOCTEXT("SetFrameToolTip", "Set the frame of this notify directly"))
					[
						SNew(SBox)
						.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
						.WidthOverride(100.0f)
						[
							SNew(SNumericEntryBox<int32>)
							.Font(FAppStyle::GetFontStyle(TEXT("MenuItem.Font")))
							.MinValue(0)
							.MaxValue(SourceTrack->Sequence->GetNumberOfSampledKeys())
							.Value(SourceTrack->Sequence->GetFrameAtTime(MenuContext->NodeObject->GetTime()))
							.AllowSpin(false)
							.OnValueCommitted_Lambda([SourceTrack, MenuContext](int32 InValue, ETextCommit::Type InCommitType)
								{
									if (InCommitType == ETextCommit::OnEnter && SourceTrack->NotifyNodes.IsValidIndex(MenuContext->NodeIndex))
									{
										const FScopedTransaction Transaction(LOCTEXT("SetNotifyFrameTransaction", "Set Anim Notify trigger frame index"));
										SourceTrack->Sequence->Modify();

										INotifyNodeObjectInterface* LocalNodeObject = SourceTrack->NotifyNodes[MenuContext->NodeIndex]->NodeObjectInterface;

										float NewTime = FMath::Clamp(SourceTrack->Sequence->GetTimeAtFrame(InValue), 0.0f, SourceTrack->Sequence->GetPlayLength() - LocalNodeObject->GetDuration());
										LocalNodeObject->SetTime(NewTime);

										if (FAnimNotifyEvent* Event = LocalNodeObject->GetNotifyEvent())
										{
											Event->RefreshTriggerOffset(SourceTrack->Sequence->CalculateOffsetForNotify(Event->GetTime()));
											if (Event->GetDuration() > 0.0f)
											{
												Event->RefreshEndTriggerOffset(SourceTrack->Sequence->CalculateOffsetForNotify(Event->GetTime() + Event->GetDuration()));
											}
										}
										SourceTrack->OnUpdatePanel.ExecuteIfBound();

										FSlateApplication::Get().DismissAllMenus();
									}
								})
						]
					];
					Section.AddEntry(
						FToolMenuEntry::InitWidget(TEXT("AnimNotifyContextMenuFrameWidget"), FrameWidget, FText::FromString("Notify Frame"), true, false)
					);


					if (MenuContext->NotifyEvent)
					{
						// add menu to get threshold weight for triggering this notify
						TSharedRef<SWidget> ThresholdWeightWidget =
						SNew(SBox)
						.HAlign(HAlign_Right)
						.ToolTipText(LOCTEXT("MinTriggerWeightToolTip", "The minimum weight to trigger this notify"))
						[
							SNew(SBox)
							.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
							.WidthOverride(100.0f)
							[
								SNew(SNumericEntryBox<float>)
								.Font(FAppStyle::GetFontStyle(TEXT("MenuItem.Font")))
								.MinValue(0.0f)
								.MaxValue(1.0f)
								.Value(MenuContext->NotifyEvent->TriggerWeightThreshold)
								.AllowSpin(false)
								.OnValueCommitted_Lambda([SourceTrack, MenuContext](float InValue, ETextCommit::Type InCommitType)
									{
										if (InCommitType == ETextCommit::OnEnter && SourceTrack->AnimNotifies.IsValidIndex(MenuContext->NotifyIndex))
										{
											const FScopedTransaction Transaction(LOCTEXT("SetNotifyWeightTransaction", "Set Anim Notify trigger weight"));
											SourceTrack->Sequence->Modify();

											float NewWeight = FMath::Max(InValue, ZERO_ANIMWEIGHT_THRESH);
											SourceTrack->AnimNotifies[MenuContext->NotifyIndex]->TriggerWeightThreshold = NewWeight;

											FSlateApplication::Get().DismissAllMenus();
										}
									})
							]
						];

						Section.AddEntry(
							FToolMenuEntry::InitWidget(TEXT("AnimNotifyContextMenuTriggerWeightWidget"), ThresholdWeightWidget, FText::FromString("Min Trigger Weight"), true, false)
						);


						// Add menu for changing duration if this is an AnimNotifyState
						if (MenuContext->NotifyEvent->NotifyStateClass)
						{
							TSharedRef<SWidget> NotifyStateDurationWidget =
							SNew(SBox)
							.HAlign(HAlign_Right)
							.ToolTipText(LOCTEXT("SetAnimStateDuration_ToolTip", "The duration of this Anim Notify State"))
							[
								SNew(SBox)
								.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
								.WidthOverride(100.0f)
								[
									SNew(SNumericEntryBox<float>)
									.Font(FAppStyle::GetFontStyle(TEXT("MenuItem.Font")))
									.MinValue(SAnimNotifyNode::MinimumStateDuration)
									.MinSliderValue(SAnimNotifyNode::MinimumStateDuration)
									.MaxSliderValue(100.0f)
									.Value(MenuContext->NotifyEvent->GetDuration())
									.AllowSpin(false)
									.OnValueCommitted_Lambda([SourceTrack, MenuContext](float InValue, ETextCommit::Type InCommitType)
										{
											if (InCommitType == ETextCommit::OnEnter && SourceTrack->AnimNotifies.IsValidIndex(MenuContext->NotifyIndex))
											{
												const FScopedTransaction Transaction(LOCTEXT("SetNotifyDurationSecondsTransaction", "Set Anim Notify State duration in seconds"));
												SourceTrack->Sequence->Modify();

												float NewDuration = FMath::Max(InValue, SAnimNotifyNode::MinimumStateDuration);
												float MaxDuration = SourceTrack->Sequence->GetPlayLength() - SourceTrack->AnimNotifies[MenuContext->NotifyIndex]->GetTime();
												NewDuration = FMath::Min(NewDuration, MaxDuration);
												SourceTrack->AnimNotifies[MenuContext->NotifyIndex]->SetDuration(NewDuration);

												// If we have a delegate bound to refresh the offsets, call it.
												// This is used by the montage editor to keep the offsets up to date.
												SourceTrack->OnRequestRefreshOffsets.ExecuteIfBound();

												FSlateApplication::Get().DismissAllMenus();
											}
										})
								]
							];

							Section.AddEntry(
								FToolMenuEntry::InitWidget(TEXT("AnimNotifyContextStateDurationWidget"), NotifyStateDurationWidget, FText::FromString("Anim Notify State Duration"), true, false)
							);

							TSharedRef<SWidget> NotifyStateDurationFramesWidget =
							SNew(SBox)
							.HAlign(HAlign_Right)
							.ToolTipText(LOCTEXT("SetAnimStateDurationFrames_ToolTip", "The duration of this Anim Notify State in frames"))
							[
								SNew(SBox)
								.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
								.WidthOverride(100.0f)
								[
									SNew(SNumericEntryBox<int32>)
									.Font(FAppStyle::GetFontStyle(TEXT("MenuItem.Font")))
									.MinValue(1)
									.MinSliderValue(1)
									.MaxSliderValue(SourceTrack->Sequence->GetNumberOfSampledKeys())
									.Value(SourceTrack->Sequence->GetFrameAtTime(MenuContext->NotifyEvent->GetDuration()))
									.AllowSpin(false)
									.OnValueCommitted_Lambda([SourceTrack, MenuContext](int32 InValue, ETextCommit::Type InCommitType)
										{
											if (InCommitType == ETextCommit::OnEnter && SourceTrack->AnimNotifies.IsValidIndex(MenuContext->NotifyIndex))
											{
												const FScopedTransaction Transaction(LOCTEXT("SetNotifyDurationFramesTransaction", "Set Anim Notify State duration in frames"));
												SourceTrack->Sequence->Modify();

												float NewDuration = FMath::Max(SourceTrack->Sequence->GetTimeAtFrame(InValue), SAnimNotifyNode::MinimumStateDuration);
												float MaxDuration = SourceTrack->Sequence->GetPlayLength() - SourceTrack->AnimNotifies[MenuContext->NotifyIndex]->GetTime();
												NewDuration = FMath::Min(NewDuration, MaxDuration);
												SourceTrack->AnimNotifies[MenuContext->NotifyIndex]->SetDuration(NewDuration);

												// If we have a delegate bound to refresh the offsets, call it.
												// This is used by the montage editor to keep the offsets up to date.
												SourceTrack->OnRequestRefreshOffsets.ExecuteIfBound();

												FSlateApplication::Get().DismissAllMenus();
											}
										})
								]
							];

							Section.AddEntry(
								FToolMenuEntry::InitWidget(TEXT("AnimNotifyContextMenuStateFramesWidget"), NotifyStateDurationFramesWidget, FText::FromString("Anim Notify State Frames"), true, false)
							);

						}

					}
				}

			}
			else
			{
				Section.AddSubMenu(TEXT("AddNotify"),
					NSLOCTEXT("NewNotifySubMenu", "NewNotifySubMenuAddNotify", "Add Notify..."),
					NSLOCTEXT("NewNotifySubMenu", "NewNotifySubMenuAddNotifyToolTip", "Add AnimNotifyEvent"),
					FNewMenuDelegate::CreateRaw(SourceTrack.Get(), &SAnimNotifyTrack::FillNewNotifyMenu, false),
					false,
					FSlateIcon());

				Section.AddSubMenu(TEXT("AddNotifyState"),
					NSLOCTEXT("NewNotifySubMenu", "NewNotifySubMenuAddNotifyState", "Add Notify State..."),
					NSLOCTEXT("NewNotifySubMenu", "NewNotifySubMenuAddNotifyStateToolTip", "Add AnimNotifyState"),
					FNewMenuDelegate::CreateRaw(SourceTrack.Get(), &SAnimNotifyTrack::FillNewNotifyStateMenu, false),
					false,
					FSlateIcon());


				if (SourceTrack->Sequence->IsA(UAnimSequence::StaticClass()))
				{
					Section.AddSubMenu(TEXT("AddSyncMarker"),
						NSLOCTEXT("NewSyncMarkerSubMenu", "NewSyncMarkerSubMenuAddNotifyState", "Add Sync Marker..."),
						NSLOCTEXT("NewSyncMarkerSubMenu", "NewSyncMarkerSubMenuAddNotifyStateToolTip", "Create a new animation sync marker"),
						FNewMenuDelegate::CreateRaw(SourceTrack.Get(), &SAnimNotifyTrack::FillNewSyncMarkerMenu, false),
						false,
						FSlateIcon());
				}

				Section.AddMenuEntry(TEXT("ManageNotifies"),
					NSLOCTEXT("NewNotifySubMenu", "ManageNotifies", "Manage Notifies..."),
					NSLOCTEXT("NewNotifySubMenu", "ManageNotifiesToolTip", "Opens the Manage Notifies window"),
					FSlateIcon(),
					FUIAction(FExecuteAction::CreateSP(SourceTrack.Get(), &SAnimNotifyTrack::OnManageNotifies)));
			}

			FToolMenuSection& EditSection = InMenu->AddSection("AnimEdit", LOCTEXT("NotifyEditHeading", "Edit"));
			if (MenuContext->NodeObject)
			{
				// copy notify menu item
				EditSection.AddMenuEntry(FAnimNotifyPanelCommands::Get().CopyNotifies);

				// allow it to delete
				EditSection.AddMenuEntry(FAnimNotifyPanelCommands::Get().DeleteNotify);

				// allow it to cut notifies
				EditSection.AddMenuEntry(FAnimNotifyPanelCommands::Get().CutNotifies);
				
				if (MenuContext->NotifyEvent)
				{
					// For the "Replace With..." menu, make sure the current AnimNotify selection is valid for replacement
					if (SourceTrack->OnGetIsAnimNotifySelectionValidforReplacement.IsBound() && SourceTrack->OnGetIsAnimNotifySelectionValidforReplacement.Execute())
					{
						// If this is an AnimNotifyState (has duration) allow it to be replaced with other AnimNotifyStates
						if (MenuContext->NotifyEvent->NotifyStateClass)
						{
							EditSection.AddSubMenu(TEXT("ReplaceWithNotifyState"),
								NSLOCTEXT("NewNotifySubMenu", "NewNotifySubMenuReplaceWithNotifyState", "Replace with Notify State..."),
								NSLOCTEXT("NewNotifySubMenu", "NewNotifySubMenuReplaceWithNotifyStateToolTip", "Replace with AnimNotifyState"),
								FNewMenuDelegate::CreateRaw(SourceTrack.Get(), &SAnimNotifyTrack::FillNewNotifyStateMenu, true),
								false,
								FSlateIcon());
						}
						// If this is a regular AnimNotify (no duration) allow it to be replaced with other AnimNotifies
						else
						{
							EditSection.AddSubMenu(TEXT("ReplaceWithNotify"),
								NSLOCTEXT("NewNotifySubMenu", "NewNotifySubMenuReplaceWithNotify", "Replace with Notify..."),
								NSLOCTEXT("NewNotifySubMenu", "NewNotifySubMenuReplaceWithNotifyToolTip", "Replace with AnimNotifyEvent"),
								FNewMenuDelegate::CreateRaw(SourceTrack.Get(), &SAnimNotifyTrack::FillNewNotifyMenu, true),
								false,
								FSlateIcon()
							);
						}
					}
				}
				else
				{
					if (MenuContext->NodeObject->GetType() == ENotifyNodeObjectTypes::SYNC_MARKER)
					{
						EditSection.AddSubMenu(TEXT("ReplaceSyncMarkers"),
							NSLOCTEXT("NewNotifySubMenu", "NewNotifySubMenuReplaceWithSyncMarker", "Replace Sync Marker(s)..."),
							NSLOCTEXT("NewNotifySubMenu", "NewNotifySubMenuReplaceWithSyncMarkerToolTip", "Replace the selected sync markers"),
							FNewMenuDelegate::CreateRaw(SourceTrack.Get(), &SAnimNotifyTrack::FillNewSyncMarkerMenu, true),
							false,
							FSlateIcon()
						);
					}
				}
			}
			else
			{
				FString PropertyString;
				const TCHAR* Buffer;
				float OriginalTime;
				float OriginalLength;
				int32 TrackSpan;
				FUIAction NewAction;
				//Check whether can we show menu item to paste anim notify event
				if (ReadNotifyPasteHeader(PropertyString, Buffer, OriginalTime, OriginalLength, TrackSpan))
				{
					// paste notify menu item
					if (SourceTrack->IsSingleNodeInClipboard())
					{
						EditSection.AddMenuEntry(FAnimNotifyPanelCommands::Get().PasteNotifies);
					}
					else
					{
						NewAction.ExecuteAction.BindRaw(
							SourceTrack.Get(), &SAnimNotifyTrack::OnPasteNotifyClicked, ENotifyPasteMode::MousePosition, ENotifyPasteMultipleMode::Relative);

						EditSection.AddMenuEntry(TEXT("PasteMultipleRelative"),
							LOCTEXT("PasteMultRel", "Paste Multiple Relative"), 
							LOCTEXT("PasteMultRelToolTip", "Paste multiple notifies beginning at the mouse cursor, maintaining the same relative spacing as the source."),
							FSlateIcon(),
							NewAction);

						EditSection.AddMenuEntry(FAnimNotifyPanelCommands::Get().PasteNotifies,
							LOCTEXT("PasteMultAbs", "Paste Multiple Absolute"),
							LOCTEXT("PasteMultAbsToolTip", "Paste multiple notifies beginning at the mouse cursor, maintaining absolute spacing."));
					}

					if (OriginalTime < SourceTrack->Sequence->GetPlayLength())
					{
						NewAction.ExecuteAction.BindRaw(
							SourceTrack.Get(), &SAnimNotifyTrack::OnPasteNotifyClicked, ENotifyPasteMode::OriginalTime, ENotifyPasteMultipleMode::Absolute);

						FText DisplayText = FText::Format(LOCTEXT("PasteAtOriginalTime", "Paste at original time ({0})"), FText::AsNumber(OriginalTime));

						EditSection.AddMenuEntry(TEXT("Paste at Original Time"),
							DisplayText,
							LOCTEXT("PasteAtOriginalTimeToolTip", "Paste animation notify event at the time it was set to when it was copied"),
							FSlateIcon(), 
							NewAction);
					}

				}
			}

			FToolMenuSection& ViewSection = InMenu->AddSection("AnimView", LOCTEXT("NotifyViewHeading", "View"));
			if (MenuContext->NotifyEvent)
			{
				UObject* NotifyObject = MenuContext->NotifyEvent->Notify;
				NotifyObject = NotifyObject ? NotifyObject : ToRawPtr(MenuContext->NotifyEvent->NotifyStateClass);
				FUIAction NewAction;
				
				if (NotifyObject)
				{
					if (Cast<UBlueprintGeneratedClass>(NotifyObject->GetClass()))
					{
						if (UBlueprint* Blueprint = Cast<UBlueprint>(NotifyObject->GetClass()->ClassGeneratedBy))
						{
							NewAction.ExecuteAction.BindRaw(
								SourceTrack.Get(), &SAnimNotifyTrack::OnOpenNotifySource, Blueprint);
							ViewSection.AddMenuEntry(TEXT("OpenNotifyBlueprint"),
								LOCTEXT("OpenNotifyBlueprint", "Open Notify Blueprint"),
								LOCTEXT("OpenNotifyBlueprintTooltip", "Opens the source blueprint for this notify"),
								FSlateIcon(),
								NewAction);
						}
					}
				}
				else
				{
					// skeleton notify
					NewAction.ExecuteAction.BindRaw(
						SourceTrack.Get(), &SAnimNotifyTrack::OnFindReferences, MenuContext->NodeObject->GetName(), true);
					ViewSection.AddMenuEntry(TEXT("FindReferences"),
						LOCTEXT("FindNotifyReferences", "Find/Replace References..."),
						LOCTEXT("FindNotifyReferencesTooltip",
							"Find, replace and remove references to this  notify in the find/replace tab"),
						FSlateIcon(), 
						NewAction);
				}
			}
			else if (MenuContext->NodeObject && MenuContext->NodeObject->GetType() == ENotifyNodeObjectTypes::SYNC_MARKER)
			{
				FUIAction NewAction;
				NewAction.ExecuteAction.BindRaw(
					SourceTrack.Get(), &SAnimNotifyTrack::OnFindReferences, MenuContext->NodeObject->GetName(), true);
				
				ViewSection.AddMenuEntry(TEXT("FindSyncReferences"),
					LOCTEXT("FindSyncMarkerReferences", "Find/Replace References..."),
					LOCTEXT("FindSyncMarkerReferencesTooltip", "Find, replace and remove references to this sync marker in the find/replace tab"),
					FSlateIcon(),
					NewAction);

			}
		}));
	return Menu; 
}


bool SAnimNotifyTrack::CanPasteAnimNotify() const
{
	FString PropertyString;
	const TCHAR* Buffer;
	float OriginalTime;
	float OriginalLength;
	int32 TrackSpan;
	return ReadNotifyPasteHeader(PropertyString, Buffer, OriginalTime, OriginalLength, TrackSpan);
}

void SAnimNotifyTrack::OnPasteNotifyClicked(ENotifyPasteMode::Type PasteMode, ENotifyPasteMultipleMode::Type MultiplePasteType)
{
	float ClickTime = PasteMode == ENotifyPasteMode::MousePosition ? LastClickedTime : -1.0f;
	OnPasteNodes.ExecuteIfBound(this, ClickTime, PasteMode, MultiplePasteType);
}

void SAnimNotifyTrack::OnManageNotifies()
{
	OnInvokeTab.ExecuteIfBound(FPersonaTabs::SkeletonAnimNotifiesID);
}

void SAnimNotifyTrack::OnOpenNotifySource(UBlueprint* InSourceBlueprint) const
{
	GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(InSourceBlueprint);
}

void SAnimNotifyTrack::OnFindReferences(FName InName, bool bInIsSyncMarker)
{
	IAssetEditorInstance* AssetEditor = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(Sequence, true);
	check(AssetEditor->GetEditorName() == TEXT("AnimationEditor"));
	if (TSharedPtr<SDockTab> Tab = AssetEditor->GetAssociatedTabManager()->TryInvokeTab(FPersonaTabs::FindReplaceID))
	{
		TSharedRef<IAnimAssetFindReplace> FindReplaceWidget = StaticCastSharedRef<IAnimAssetFindReplace>(Tab->GetContent());
		FindReplaceWidget->SetCurrentProcessor(bInIsSyncMarker ? UAnimAssetFindReplaceSyncMarkers::StaticClass() : UAnimAssetFindReplaceNotifies::StaticClass());
		UAnimAssetFindReplaceProcessor_StringBase* Processor = Cast<UAnimAssetFindReplaceProcessor_StringBase>(FindReplaceWidget->GetCurrentProcessor());
		Processor->SetFindString(InName.ToString());
	}
}

bool SAnimNotifyTrack::IsSingleNodeSelected()
{
	return SelectedNodeIndices.Num() == 1;
}

bool SAnimNotifyTrack::IsSingleNodeInClipboard()
{
	FString PropString;
	const TCHAR* Buffer;
	float OriginalTime;
	float OriginalLength;
	int32 TrackSpan;
	uint32 Count = 0;
	if (ReadNotifyPasteHeader(PropString, Buffer, OriginalTime, OriginalLength, TrackSpan))
	{
		// If reading a single line empties the buffer then we only have one notify in there.
		FString TempLine;
		FParse::Line(&Buffer, TempLine);
		return *Buffer == 0;
	}
	return false;
}

void SAnimNotifyTrack::OnNewNotifyClicked()
{
	// Show dialog to enter new track name
	TSharedRef<STextEntryPopup> TextEntry =
		SNew(STextEntryPopup)
		.Label( LOCTEXT("NewNotifyLabel", "Notify Name") )
		.OnTextCommitted( this, &SAnimNotifyTrack::AddNewNotify );

	// Show dialog to enter new event name
	FSlateApplication::Get().PushMenu(
		AsShared(), // Menu being summoned from a menu that is closing: Parent widget should be k2 not the menu thats open or it will be closed when the menu is dismissed
		FWidgetPath(),
		TextEntry,
		FSlateApplication::Get().GetCursorPos(),
		FPopupTransitionEffect( FPopupTransitionEffect::TypeInPopup )
		);
}

void SAnimNotifyTrack::OnNewSyncMarkerClicked()
{
	// Show dialog to enter new track name
	TSharedRef<STextEntryPopup> TextEntry =
		SNew(STextEntryPopup)
		.Label(LOCTEXT("NewSyncMarkerLabel", "Sync Marker Name"))
		.OnTextCommitted(this, &SAnimNotifyTrack::AddNewSyncMarker);

	// Show dialog to enter new event name
	FSlateApplication::Get().PushMenu(
		AsShared(), // Menu being summoned from a menu that is closing: Parent widget should be k2 not the menu thats open or it will be closed when the menu is dismissed
		FWidgetPath(),
		TextEntry,
		FSlateApplication::Get().GetCursorPos(),
		FPopupTransitionEffect(FPopupTransitionEffect::TypeInPopup)
		);
}

void SAnimNotifyTrack::AddNewNotify(const FText& NewNotifyName, ETextCommit::Type CommitInfo)
{
	if (CommitInfo == ETextCommit::OnEnter)
	{
		const FScopedTransaction Transaction( LOCTEXT("AddNewNotifyEvent", "Add New Anim Notify") );
		FName NewName = FName( *NewNotifyName.ToString() );

		CreateNewNotifyAtCursor(NewNotifyName.ToString(), (UClass*)nullptr);

		FBlueprintActionDatabase& ActionDatabase = FBlueprintActionDatabase::Get();
		ActionDatabase.ClearAssetActions(UAnimBlueprint::StaticClass());
		ActionDatabase.RefreshClassActions(UAnimBlueprint::StaticClass());
	}

	FSlateApplication::Get().DismissAllMenus();
}

void SAnimNotifyTrack::AddNewSyncMarker(const FText& NewNotifyName, ETextCommit::Type CommitInfo) 
{
	if (CommitInfo == ETextCommit::OnEnter)
	{
		const FScopedTransaction Transaction(LOCTEXT("AddNewSyncMarker", "Add New Sync Marker"));

		CreateNewSyncMarkerAtCursor(NewNotifyName.ToString());
	}

	FSlateApplication::Get().DismissAllMenus();
}

void SAnimNotifyTrack::Update()
{
	NotifyPairs.Empty();
	NotifyNodes.Empty();

	TrackArea->SetContent(
		SAssignNew( NodeSlots, SOverlay )
		);

	if ( AnimNotifies.Num() > 0 )
	{
		TArray<TSharedPtr<FTimingRelevantElementBase>> TimingElements;
		SAnimTimingPanel::GetTimingRelevantElements(Sequence, TimingElements);
		for (int32 NotifyIndex = 0; NotifyIndex < AnimNotifies.Num(); ++NotifyIndex)
		{
			TSharedPtr<FTimingRelevantElementBase> Element;
			FAnimNotifyEvent* Event = AnimNotifies[NotifyIndex];

			for(int32 Idx = 0 ; Idx < TimingElements.Num() ; ++Idx)
			{
				Element = TimingElements[Idx];

				if(Element->GetType() == ENotifyTimingElementType::NotifyStateBegin
				   || Element->GetType() == ENotifyTimingElementType::BranchPointNotify
				   || Element->GetType() == ENotifyTimingElementType::QueuedNotify)
				{
					// Only the notify type will return the type flags above
					FTimingRelevantElement_Notify* NotifyElement = static_cast<FTimingRelevantElement_Notify*>(Element.Get());
					if(Event == &Sequence->Notifies[NotifyElement->NotifyIndex])
					{
						break;
					}
				}
			}

			TSharedPtr<SAnimNotifyNode> AnimNotifyNode = nullptr;
			TSharedPtr<SAnimNotifyPair> NotifyPair = nullptr;
			TSharedPtr<SAnimTimingNode> TimingNode = nullptr;
			TSharedPtr<SAnimTimingNode> EndTimingNode = nullptr;

			// Create visibility attribute to control timing node visibility for notifies
			TAttribute<EVisibility> TimingNodeVisibility = TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateLambda(
				[this]()
				{
					if(OnGetTimingNodeVisibility.IsBound())
					{
						return OnGetTimingNodeVisibility.Execute(ENotifyTimingElementType::QueuedNotify);
					}
					return EVisibility(EVisibility::Hidden);
				}));

			SAssignNew(TimingNode, SAnimTimingNode)
				.InElement(Element)
				.bUseTooltip(true)
				.Visibility(TimingNodeVisibility);

			if(Event->NotifyStateClass)
			{
				TSharedPtr<FTimingRelevantElementBase>* FoundStateEndElement = TimingElements.FindByPredicate([Event](TSharedPtr<FTimingRelevantElementBase>& ElementToTest)
				{
					if(ElementToTest.IsValid() && ElementToTest->GetType() == ENotifyTimingElementType::NotifyStateEnd)
					{
						FTimingRelevantElement_NotifyStateEnd* StateElement = static_cast<FTimingRelevantElement_NotifyStateEnd*>(ElementToTest.Get());
						return &(StateElement->Sequence->Notifies[StateElement->NotifyIndex]) == Event;
					}
					return false;
				});

				if(FoundStateEndElement)
				{
					// Create an end timing node if we have a state
					SAssignNew(EndTimingNode, SAnimTimingNode)
						.InElement(*FoundStateEndElement)
						.bUseTooltip(true)
						.Visibility(TimingNodeVisibility);
				}
			}

			// Check if there is a factory that wants to create a custom widget. 
			for (const FAnimNotifyNodeFactoryFunction& Factory : FAnimNotifyNodeFactory::GetRegisteredFactories())
			{
				AnimNotifyNode = Factory(Event);
				if (AnimNotifyNode.IsValid())
				{
					break;
				}
			}

			// Otherwise create the default. 
			if (!AnimNotifyNode.IsValid())
			{
				SAssignNew(AnimNotifyNode, SAnimNotifyNode);
			}

			AnimNotifyNode->SetSequence(Sequence);
			AnimNotifyNode->SetAnimNotify(Event);
			AnimNotifyNode->SetOnNotifyNodeDragStarted(FOnNotifyNodeDragStarted::CreateSP(this, &SAnimNotifyTrack::OnNotifyNodeDragStarted, NotifyIndex));
			AnimNotifyNode->SetOnNotifyStateHandleBeingDragged(OnNotifyStateHandleBeingDragged);
			AnimNotifyNode->SetOnUpdatePanel(OnUpdatePanel);
			AnimNotifyNode->SetPanTrackRequest(OnRequestTrackPan);
			AnimNotifyNode->SetViewInputMin(ViewInputMin);
			AnimNotifyNode->SetViewInputMax(ViewInputMax);
			AnimNotifyNode->SetOnSnapPosition(OnSnapPosition);
			AnimNotifyNode->SetOnTrackSelectionChanged(OnSelectionChanged);
			AnimNotifyNode->SetStateEndTimingNode(EndTimingNode);


			SAssignNew(NotifyPair, SAnimNotifyPair)
			.LeftContent()
			[
				TimingNode.ToSharedRef()
			]
			.Node(AnimNotifyNode);

			NodeSlots->AddSlot()
			.Padding(TAttribute<FMargin>::Create(TAttribute<FMargin>::FGetter::CreateSP(this, &SAnimNotifyTrack::GetNotifyTrackPadding, NotifyIndex)))
			[
				NotifyPair->AsShared()
			];

			NotifyNodes.Add(AnimNotifyNode);
			NotifyPairs.Add(NotifyPair);
		}
	}

	for (FAnimSyncMarker* SyncMarker : AnimSyncMarkers)
	{
		TSharedPtr<SAnimNotifyNode> AnimSyncMarkerNode = nullptr;
		TSharedPtr<SAnimTimingNode> EndTimingNode = nullptr;

		const int32 NodeIndex = NotifyNodes.Num();
		SAssignNew(AnimSyncMarkerNode, SAnimNotifyNode);

		AnimSyncMarkerNode->SetSequence(Sequence);
		AnimSyncMarkerNode->SetAnimSyncMarker(SyncMarker);
		AnimSyncMarkerNode->SetOnNotifyNodeDragStarted(FOnNotifyNodeDragStarted::CreateSP(this, &SAnimNotifyTrack::OnNotifyNodeDragStarted, NodeIndex));
		AnimSyncMarkerNode->SetOnUpdatePanel(OnUpdatePanel);
		AnimSyncMarkerNode->SetPanTrackRequest(OnRequestTrackPan);
		AnimSyncMarkerNode->SetViewInputMin(ViewInputMin);
		AnimSyncMarkerNode->SetViewInputMax(ViewInputMax);
		AnimSyncMarkerNode->SetOnSnapPosition(OnSnapPosition);
		AnimSyncMarkerNode->SetOnTrackSelectionChanged(OnSelectionChanged);
		AnimSyncMarkerNode->SetStateEndTimingNode(EndTimingNode);

		NodeSlots->AddSlot()
			.Padding(TAttribute<FMargin>::Create(TAttribute<FMargin>::FGetter::CreateSP(this, &SAnimNotifyTrack::GetSyncMarkerTrackPadding, NodeIndex)))
			[
				AnimSyncMarkerNode->AsShared()
			];

		NotifyNodes.Add(AnimSyncMarkerNode);
	}
}

int32 SAnimNotifyTrack::GetHitNotifyNode(const FGeometry& MyGeometry, const FVector2D& CursorPosition)
{
	for (int32 I = NotifyNodes.Num() - 1; I >= 0; --I) //Run through from 'top most' Notify to bottom
	{
		if (NotifyNodes[I].Get()->HitTest(MyGeometry, CursorPosition))
		{
			return I;
		}
	}

	return INDEX_NONE;
}

FReply SAnimNotifyTrack::OnNotifyNodeDragStarted(TSharedRef<SAnimNotifyNode> NotifyNode, const FPointerEvent& MouseEvent, const FVector2D& ScreenNodePosition, const bool bDragOnMarker, int32 NotifyIndex)
{
	// Check to see if we've already selected the triggering node
	if (!NotifyNode->bSelected)
	{
		SelectTrackObjectNode(NotifyIndex, MouseEvent.IsShiftDown(), false);
	}

	// Sort our nodes so we're acessing them in time order
	SelectedNodeIndices.Sort([this](const int32& A, const int32& B)
	{
		const double TimeA = NotifyNodes[A]->NodeObjectInterface->GetTime();
		const double TimeB = NotifyNodes[B]->NodeObjectInterface->GetTime();
		return TimeA < TimeB;
	});

	// If we're dragging one of the direction markers we don't need to call any further as we don't want the drag drop op
	if (!bDragOnMarker)
	{
		TArray<TSharedPtr<SAnimNotifyNode>> NodesToDrag;
		const TSharedRef<SOverlay> DragBox = SNew(SOverlay);
		for (auto Iter = SelectedNodeIndices.CreateIterator(); Iter; ++Iter)
		{
			const TSharedPtr<SAnimNotifyNode> Node = NotifyNodes[*Iter];
			NodesToDrag.Add(Node);
		}

		FVector2D DecoratorPosition = NodesToDrag[0]->GetWidgetPosition();
		DecoratorPosition = CachedGeometry.LocalToAbsolute(DecoratorPosition);
		return OnNodeDragStarted.Execute(NodesToDrag, DragBox, MouseEvent.GetScreenSpacePosition(), DecoratorPosition, bDragOnMarker);
	}
	else
	{
		// Capture the mouse in the node
		return FReply::Handled().CaptureMouse(NotifyNode).UseHighPrecisionMouseMovement(NotifyNode);
	}
}

FReply SAnimNotifyTrack::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	FVector2D CursorPos = MouseEvent.GetScreenSpacePosition();
	CursorPos = MyGeometry.AbsoluteToLocal(CursorPos);
	int32 HitIndex = GetHitNotifyNode(MyGeometry, CursorPos);

	if (HitIndex != INDEX_NONE)
	{
		if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
		{
			// Hit a node, record the mouse position for use later so we can know when / where a
			// drag happened on the node handles if necessary.
			NotifyNodes[HitIndex]->SetLastMouseDownPosition(CursorPos);

			return FReply::Handled().DetectDrag(NotifyNodes[HitIndex].ToSharedRef(), EKeys::LeftMouseButton);
		}
		else if(MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
		{
			// Hit a node, return handled so we can pop a context menu on mouse up
			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

float SAnimNotifyTrack::CalculateTime(const FGeometry& MyGeometry, FVector2D NodePos, bool bInputIsAbsolute)
{
	if (bInputIsAbsolute)
	{
		NodePos = MyGeometry.AbsoluteToLocal(NodePos);
	}
	FTrackScaleInfo ScaleInfo(ViewInputMin.Get(), ViewInputMax.Get(), 0, 0, MyGeometry.Size);
	return FMath::Clamp<float>(ScaleInfo.LocalXToInput(static_cast<float>(NodePos.X)), 0.f, Sequence->GetPlayLength());
}

FReply SAnimNotifyTrack::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	return FReply::Unhandled();
}

void SAnimNotifyTrack::HandleNodeDrop(TSharedPtr<SAnimNotifyNode> Node, float Offset)
{
	ensure(Node.IsValid());

	if (Node->NodeObjectInterface->GetType() == ENotifyNodeObjectTypes::SYNC_MARKER)
	{
		UBlendSpace::UpdateBlendSpacesUsingAnimSequence(Sequence);
	}

	// Apply the local-space Offset after converting the absolute screen position to local.
	const float LocalX = static_cast<float>(GetCachedGeometry().AbsoluteToLocal(Node->GetScreenPosition()).X + Offset);
	const float SnapTime = Node->GetLastSnappedTime();
	const float Time = SnapTime != -1.0f ? SnapTime : GetCachedScaleInfo().LocalXToInput(LocalX);
	Node->NodeObjectInterface->HandleDrop(Sequence, Time, TrackIndex);
}

void SAnimNotifyTrack::DisconnectSelectedNodesForDrag(TArray<TSharedPtr<SAnimNotifyNode>>& DragNodes)
{
	if(SelectedNodeIndices.Num() == 0)
	{
		return;
	}

	for(auto Iter = SelectedNodeIndices.CreateIterator(); Iter; ++Iter)
	{
		const TSharedPtr<SAnimNotifyNode> Node = NotifyNodes[*Iter];
		if (Node->NodeObjectInterface->GetNotifyEvent())
		{
			const TSharedPtr<SAnimNotifyPair> Pair = NotifyPairs[*Iter];
			NodeSlots->RemoveSlot(Pair->AsShared());
		}
		else
		{
			NodeSlots->RemoveSlot(Node->AsShared());
		}

		DragNodes.Add(Node);
	}
}

void SAnimNotifyTrack::AppendSelectionToSet(FGraphPanelSelectionSet& SelectionSet)
{
	// Add our selection to the provided set
	for(int32 Index : SelectedNodeIndices)
	{
		if (FAnimNotifyEvent* Event = NotifyNodes[Index]->NodeObjectInterface->GetNotifyEvent())
		{
			if (Event->Notify)
			{
				SelectionSet.Add(Event->Notify);
			}
			else if (Event->NotifyStateClass)
			{
				SelectionSet.Add(Event->NotifyStateClass);
			}
		}
	}
}

void SAnimNotifyTrack::AppendSelectionToArray(TArray<INotifyNodeObjectInterface*>& Selection) const
{
	for(int32 Idx : SelectedNodeIndices)
	{
		Selection.Add(NotifyNodes[Idx]->NodeObjectInterface);
	}
}

void SAnimNotifyTrack::PasteSingleNotify(FString& NotifyString, float PasteTime)
{
	int32 NewIdx = Sequence->Notifies.Add(FAnimNotifyEvent());
	FArrayProperty* ArrayProperty = NULL;
	uint8* PropertyData = Sequence->FindNotifyPropertyData(NewIdx, ArrayProperty);

	if(PropertyData && ArrayProperty)
	{
		ArrayProperty->Inner->ImportText_Direct(*NotifyString, PropertyData, NULL, PPF_Copy);

		FAnimNotifyEvent& NewNotify = Sequence->Notifies[NewIdx];

		// We have to link to the montage / sequence again, we need a correct time set and we could be pasting to a new montage / sequence
		int32 NewSlotIndex = 0;
		float NewNotifyTime = PasteTime != 1.0f ? PasteTime : NewNotify.GetTime();
		NewNotifyTime = FMath::Clamp(NewNotifyTime, 0.0f, Sequence->GetPlayLength());

		if(UAnimMontage* Montage = Cast<UAnimMontage>(Sequence))
		{
			// We have a montage, validate slots
			int32 OldSlotIndex = NewNotify.GetSlotIndex();
			if(Montage->SlotAnimTracks.IsValidIndex(OldSlotIndex))
			{
				// Link to the same slot index
				NewSlotIndex = OldSlotIndex;
			}
		}
		NewNotify.Link(Sequence, PasteTime, NewSlotIndex);

		NewNotify.TriggerTimeOffset = GetTriggerTimeOffsetForType(Sequence->CalculateOffsetForNotify(NewNotify.GetTime()));
		NewNotify.TrackIndex = TrackIndex;

		bool bValidNotify = true;
		if(NewNotify.Notify)
		{
			UAnimNotify* NewNotifyObject = Cast<UAnimNotify>(StaticDuplicateObject(NewNotify.Notify, Sequence));
			check(NewNotifyObject);
			bValidNotify = NewNotifyObject->CanBePlaced(Sequence);
			NewNotify.Notify = NewNotifyObject;
		}
		else if(NewNotify.NotifyStateClass)
		{
			UAnimNotifyState* NewNotifyStateObject = Cast<UAnimNotifyState>(StaticDuplicateObject(NewNotify.NotifyStateClass, Sequence));
			check(NewNotifyStateObject);
			NewNotify.NotifyStateClass = NewNotifyStateObject;
			bValidNotify = NewNotifyStateObject->CanBePlaced(Sequence);
			// Clamp duration into the sequence
			if (UAnimMontage* Montage = Cast<UAnimMontage>(Sequence))
			{
				NewNotify.SetDuration(FMath::Clamp(NewNotify.Duration, 1 / 30.0f, Montage->CalculateSequenceLength() - NewNotify.GetTime()));

			}
			else
			{
				NewNotify.SetDuration(FMath::Clamp(NewNotify.Duration, 1 / 30.0f, Sequence->GetPlayLength() - NewNotify.GetTime()));
			}
			NewNotify.EndTriggerTimeOffset = GetTriggerTimeOffsetForType(Sequence->CalculateOffsetForNotify(NewNotify.GetTime() + NewNotify.GetDuration()));
			NewNotify.EndLink.Link(Sequence, NewNotify.EndLink.GetTime());
		}

		NewNotify.Guid = FGuid::NewGuid();

		if (!bValidNotify)
		{
			// Paste failed, remove the notify
			Sequence->Notifies.RemoveAt(NewIdx);

			FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("FailedToPaste", "The notify is not allowed to be in this asset."));
		}
	}
	else
	{
		// Paste failed, remove the notify
		Sequence->Notifies.RemoveAt(NewIdx);
	}

	OnDeselectAllNotifies.ExecuteIfBound();
	Sequence->PostEditChange();
	Sequence->MarkPackageDirty();
	OnUpdatePanel.ExecuteIfBound();
}

void SAnimNotifyTrack::PasteSingleSyncMarker(FString& MarkerString, float PasteTime)
{
	if(UAnimSequence* AnimSeq = Cast<UAnimSequence>(Sequence))
	{
		int32 NewIdx = AnimSeq->AuthoredSyncMarkers.Add(FAnimSyncMarker());
		FArrayProperty* ArrayProperty = NULL;
		uint8* PropertyData = AnimSeq->FindSyncMarkerPropertyData(NewIdx, ArrayProperty);

		if (PropertyData && ArrayProperty)
		{
			ArrayProperty->Inner->ImportText_Direct(*MarkerString, PropertyData, NULL, PPF_Copy);

			FAnimSyncMarker& SyncMarker = AnimSeq->AuthoredSyncMarkers[NewIdx];

			if (PasteTime != -1.0f)
			{
				SyncMarker.Time = PasteTime;
			}

			// Make sure the notify is within the track area
			SyncMarker.Time = FMath::Clamp(SyncMarker.Time, 0.0f, Sequence->GetPlayLength());
			SyncMarker.TrackIndex = TrackIndex;

			SyncMarker.Guid = FGuid::NewGuid();
		}
		else
		{
			// Paste failed, remove the notify
			AnimSeq->AuthoredSyncMarkers.RemoveAt(NewIdx);
		}

		UBlendSpace::UpdateBlendSpacesUsingAnimSequence(Sequence);

		OnDeselectAllNotifies.ExecuteIfBound();
		Sequence->PostEditChange();
		Sequence->MarkPackageDirty();
		OnUpdatePanel.ExecuteIfBound();
	}
}

void SAnimNotifyTrack::AppendSelectedNodeWidgetsToArray(TArray<TSharedPtr<SAnimNotifyNode>>& NodeArray) const
{
	for(TSharedPtr<SAnimNotifyNode> Node : NotifyNodes)
	{
		if(Node->bSelected)
		{
			NodeArray.Add(Node);
		}
	}
}

void SAnimNotifyTrack::RefreshMarqueeSelectedNodes(const FSlateRect& Rect, FNotifyMarqueeOperation& Marquee)
{
	if(Marquee.Operation != FNotifyMarqueeOperation::Replace)
	{
		// Maintain the original selection from before the operation
		for(int32 Idx = 0 ; Idx < NotifyNodes.Num() ; ++Idx)
		{
			TSharedPtr<SAnimNotifyNode> Notify = NotifyNodes[Idx];
			bool bWasSelected = Marquee.OriginalSelection.Contains(Notify);
			if(bWasSelected)
			{
				SelectTrackObjectNode(Idx, true, false);
			}
			else if(SelectedNodeIndices.Contains(Idx))
			{
				DeselectTrackObjectNode(Idx, false);
			}
		}
	}

	for(int32 Index = 0 ; Index < NotifyNodes.Num() ; ++Index)
	{
		TSharedPtr<SAnimNotifyNode> Node = NotifyNodes[Index];
		FSlateRect NodeRect = FSlateRect(Node->GetWidgetPosition(), Node->GetWidgetPosition() + Node->GetSize());

		if(FSlateRect::DoRectanglesIntersect(Rect, NodeRect))
		{
			// Either select or deselect the intersecting node, depending on the type of selection operation
			if(Marquee.Operation == FNotifyMarqueeOperation::Remove)
			{
				if(SelectedNodeIndices.Contains(Index))
				{
					DeselectTrackObjectNode(Index, false);
				}
			}
			else
			{
				SelectTrackObjectNode(Index, true, false);
			}
		}
	}
}

FString SAnimNotifyTrack::MakeBlueprintNotifyName(const FString& InNotifyClassName)
{
	FString DefaultNotifyName = InNotifyClassName;
	DefaultNotifyName = DefaultNotifyName.Replace(TEXT("AnimNotify_"), TEXT(""), ESearchCase::CaseSensitive);
	DefaultNotifyName = DefaultNotifyName.Replace(TEXT("AnimNotifyState_"), TEXT(""), ESearchCase::CaseSensitive);

	return DefaultNotifyName;
}

void SAnimNotifyTrack::ClearNodeTooltips()
{
	FText EmptyTooltip;

	for (TSharedPtr<SAnimNotifyNode> Node : NotifyNodes)
	{
		Node->SetToolTipText(EmptyTooltip);
	}
}

const EVisibility SAnimNotifyTrack::GetTimingNodeVisibility(TSharedPtr<SAnimNotifyNode> NotifyNode)
{
	if(OnGetTimingNodeVisibility.IsBound())
	{
		if(FAnimNotifyEvent* Event = NotifyNode->NodeObjectInterface->GetNotifyEvent())
		{
			return Event->IsBranchingPoint() ? OnGetTimingNodeVisibility.Execute(ENotifyTimingElementType::BranchPointNotify) : OnGetTimingNodeVisibility.Execute(ENotifyTimingElementType::QueuedNotify);
		}
	}

	// No visibility defined, not visible
	return EVisibility::Hidden;
}

void SAnimNotifyTrack::UpdateCachedGeometry(const FGeometry& InGeometry)
{
	CachedGeometry = InGeometry;

	for(TSharedPtr<SAnimNotifyNode> Node : NotifyNodes)
	{
		Node->CachedTrackGeometry = InGeometry;
	}
}

//////////////////////////////////////////////////////////////////////////
// SSequenceEdTrack

void SNotifyEdTrack::Construct(const FArguments& InArgs)
{
	Sequence = InArgs._Sequence;
	TrackIndex = InArgs._TrackIndex;
	FAnimNotifyTrack& Track = Sequence->AnimNotifyTracks[InArgs._TrackIndex];
	// @Todo anim: we need to fix this to allow track color to be customizable. 
	// for now name, and track color are given
	Track.TrackColor = ((TrackIndex & 1) != 0) ? FLinearColor(0.9f, 0.9f, 0.9f, 0.9f) : FLinearColor(0.5f, 0.5f, 0.5f);

	TSharedRef<SAnimNotifyPanel> PanelRef = InArgs._AnimNotifyPanel.ToSharedRef();
	AnimPanelPtr = InArgs._AnimNotifyPanel;

	//////////////////////////////
	this->ChildSlot
	[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.FillWidth(1)
			[
				// Notification editor panel
				SAssignNew(NotifyTrack, SAnimNotifyTrack)
				.Sequence(Sequence)
				.TrackIndex(TrackIndex)
				.AnimNotifies(Track.Notifies)
				.AnimSyncMarkers(Track.SyncMarkers)
				.ViewInputMin(InArgs._ViewInputMin)
				.ViewInputMax(InArgs._ViewInputMax)
				.OnSelectionChanged(InArgs._OnSelectionChanged)
				.OnUpdatePanel(InArgs._OnUpdatePanel)
				.OnGetNotifyBlueprintData(InArgs._OnGetNotifyBlueprintData)
				.OnGetNotifyStateBlueprintData(InArgs._OnGetNotifyStateBlueprintData)
				.OnGetNotifyNativeClasses(InArgs._OnGetNotifyNativeClasses)
				.OnGetNotifyStateNativeClasses(InArgs._OnGetNotifyStateNativeClasses)
				.OnGetScrubValue(InArgs._OnGetScrubValue)
				.OnGetDraggedNodePos(InArgs._OnGetDraggedNodePos)
				.OnNodeDragStarted(InArgs._OnNodeDragStarted)
				.OnNotifyStateHandleBeingDragged(InArgs._OnNotifyStateHandleBeingDragged)
				.OnSnapPosition(InArgs._OnSnapPosition)
				.TrackColor(Track.TrackColor)
				.OnRequestTrackPan(FPanTrackRequest::CreateSP(PanelRef, &SAnimNotifyPanel::PanInputViewRange))
				.OnRequestOffsetRefresh(InArgs._OnRequestRefreshOffsets)
				.OnDeleteNotify(InArgs._OnDeleteNotify)
				.OnGetIsAnimNotifySelectionValidForReplacement(PanelRef, &SAnimNotifyPanel::IsNotifySelectionValidForReplacement)
				.OnReplaceSelectedWithNotify(PanelRef, &SAnimNotifyPanel::OnReplaceSelectedWithNotify)
				.OnReplaceSelectedWithBlueprintNotify(PanelRef, &SAnimNotifyPanel::OnReplaceSelectedWithNotifyBlueprint)
				.OnReplaceSelectedWithSyncMarker(PanelRef, &SAnimNotifyPanel::OnReplaceSelectedWithSyncMarker)
				.OnDeselectAllNotifies(InArgs._OnDeselectAllNotifies)
				.OnCopyNodes(InArgs._OnCopyNodes)
				.OnPasteNodes(InArgs._OnPasteNodes)
				.OnSetInputViewRange(InArgs._OnSetInputViewRange)
				.OnGetTimingNodeVisibility(InArgs._OnGetTimingNodeVisibility)
				.OnInvokeTab(InArgs._OnInvokeTab)
				.CommandList(PanelRef->GetCommandList())
			]
	];
}

bool SNotifyEdTrack::CanDeleteTrack()
{
	return AnimPanelPtr.Pin()->CanDeleteTrack(TrackIndex);
}

//////////////////////////////////////////////////////////////////////////
// FAnimNotifyPanelCommands

void FAnimNotifyPanelCommands::RegisterCommands()
{
	UI_COMMAND(DeleteNotify, "Delete", "Deletes the selected notifies.", EUserInterfaceActionType::Button, FInputChord(EKeys::Platform_Delete));
	UI_COMMAND(CopyNotifies, "Copy", "Copy animation notify events.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::C));
	UI_COMMAND(CutNotifies, "Cut", "Cut animation notify events.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::X));
	UI_COMMAND(PasteNotifies, "Paste", "Paste animation notify event here.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::V));
}

//////////////////////////////////////////////////////////////////////////
// SAnimNotifyPanel

void SAnimNotifyPanel::Construct(const FArguments& InArgs, const TSharedRef<FAnimModel>& InModel)
{
	SAnimTrackPanel::Construct( SAnimTrackPanel::FArguments()
		.WidgetWidth(InArgs._WidgetWidth)
		.ViewInputMin(InArgs._ViewInputMin)
		.ViewInputMax(InArgs._ViewInputMax)
		.InputMin(InArgs._InputMin)
		.InputMax(InArgs._InputMax)
		.OnSetInputViewRange(InArgs._OnSetInputViewRange));

	WeakModel = InModel;
	Sequence = InArgs._Sequence;
	OnInvokeTab = InArgs._OnInvokeTab;
	OnNotifiesChanged = InArgs._OnNotifiesChanged;
	OnSnapPosition = InArgs._OnSnapPosition;
	OnNotifyStateHandleBeingDragged = InArgs._OnNotifyStateHandleBeingDragged;
	OnNotifyNodesBeingDragged = InArgs._OnNotifyNodesBeingDragged;
	bIsSelecting = false;
	bIsUpdating = false;
	bUpdateRequested = false;
	bRefreshRequested = false;

	InModel->OnHandleObjectsSelected().AddSP(this, &SAnimNotifyPanel::HandleObjectsSelected);

	FAnimNotifyPanelCommands::Register();
	BindCommands();

	Sequence->RegisterOnNotifyChanged(UAnimSequenceBase::FOnNotifyChanged::CreateSP(this, &SAnimNotifyPanel::RequestRefresh ));

	InModel->GetEditableSkeleton()->RegisterOnNotifiesChanged(FSimpleDelegate::CreateSP(this, &SAnimNotifyPanel::RequestRefresh));
	InModel->OnTracksChanged().Add(FSimpleDelegate::CreateSP(this, &SAnimNotifyPanel::RequestRefresh));

	if(GEditor)
	{
		GEditor->RegisterForUndo(this);
	}

	CurrentPosition = InArgs._CurrentPosition;
	OnSelectionChanged = InArgs._OnSelectionChanged;
	WidgetWidth = InArgs._WidgetWidth;
	OnGetScrubValue = InArgs._OnGetScrubValue;
	OnRequestRefreshOffsets = InArgs._OnRequestRefreshOffsets;
	OnGetTimingNodeVisibility = InArgs._OnGetTimingNodeVisibility;

	this->ChildSlot
	[
		SAssignNew(PanelArea, SBorder)
		.Visibility(EVisibility::SelfHitTestInvisible)
		.AddMetaData<FTagMetaData>(TEXT("AnimNotify.Notify"))
		.BorderImage(FAppStyle::GetBrush("NoBorder"))
		.Padding(0.0f)
		.ColorAndOpacity(FLinearColor::White)
	];

	// Base notify classes used to search asset data for children.
	NotifyClassNames.Add(TEXT("Class'/Script/Engine.AnimNotify'"));
	NotifyStateClassNames.Add(TEXT("Class'/Script/Engine.AnimNotifyState'"));

	PopulateNotifyBlueprintClasses(NotifyClassNames);
	PopulateNotifyBlueprintClasses(NotifyStateClassNames);

	RequestUpdate();
}

SAnimNotifyPanel::~SAnimNotifyPanel()
{
	Sequence->UnregisterOnNotifyChanged(this);

	if(GEditor)
	{
		GEditor->UnregisterForUndo(this);
	}
}

FName SAnimNotifyPanel::GetNewTrackName() const
{
	TArray<FName> TrackNames;
	TrackNames.Reserve(50);

	for (const FAnimNotifyTrack& Track : Sequence->AnimNotifyTracks)
	{
		TrackNames.Add(Track.TrackName);
	}

	FName NameToTest;
	int32 TrackIndex = 1;
	
	do 
	{
		NameToTest = *FString::FromInt(TrackIndex++);
	} while (TrackNames.Contains(NameToTest));

	return NameToTest;
}

FReply SAnimNotifyPanel::InsertTrack(int32 TrackIndexToInsert)
{
	// before insert, make sure everything behind is fixed
	for (int32 I=TrackIndexToInsert; I<Sequence->AnimNotifyTracks.Num(); ++I)
	{
		FAnimNotifyTrack& Track = Sequence->AnimNotifyTracks[I];

		const int32 NewTrackIndex = I + 1;

		for (FAnimNotifyEvent* Notify : Track.Notifies)
		{
			// fix notifies indices
			Notify->TrackIndex = NewTrackIndex;
		}

		for (FAnimSyncMarker* SyncMarker : Track.SyncMarkers)
		{
			// fix notifies indices
			SyncMarker->TrackIndex = NewTrackIndex;
		}
	}

	FAnimNotifyTrack NewItem;
	NewItem.TrackName = GetNewTrackName();
	NewItem.TrackColor = FLinearColor::White;

	Sequence->AnimNotifyTracks.Insert(NewItem, TrackIndexToInsert);
	Sequence->PostEditChange();
	Sequence->MarkPackageDirty();

	RequestUpdate();

	return FReply::Handled();
}

FReply SAnimNotifyPanel::AddTrack()
{
	FAnimNotifyTrack NewItem;
	NewItem.TrackName = GetNewTrackName();
	NewItem.TrackColor = FLinearColor::White;

	Sequence->AnimNotifyTracks.Add(NewItem);
	Sequence->MarkPackageDirty();

	RequestUpdate();

	return FReply::Handled();
}

FReply SAnimNotifyPanel::DeleteTrack(int32 TrackIndexToDelete)
{
	if (Sequence->AnimNotifyTracks.IsValidIndex(TrackIndexToDelete))
	{
		if (Sequence->AnimNotifyTracks[TrackIndexToDelete].Notifies.Num() == 0)
		{
			// before insert, make sure everything behind is fixed
			for (int32 I=TrackIndexToDelete+1; I<Sequence->AnimNotifyTracks.Num(); ++I)
			{
				FAnimNotifyTrack& Track = Sequence->AnimNotifyTracks[I];
				const int32 NewTrackIndex = I - 1;

				for (FAnimNotifyEvent* Notify : Track.Notifies)
				{
					// fix notifies indices
					Notify->TrackIndex = NewTrackIndex;
				}

				for (FAnimSyncMarker* SyncMarker : Track.SyncMarkers)
				{
					// fix notifies indices
					SyncMarker->TrackIndex = NewTrackIndex;
				}
			}

			Sequence->AnimNotifyTracks.RemoveAt(TrackIndexToDelete);
			Sequence->PostEditChange();
			Sequence->MarkPackageDirty();
			RequestUpdate();
		}
	}
	return FReply::Handled();
}

bool SAnimNotifyPanel::CanDeleteTrack(int32 TrackIndexToDelete)
{
	if (Sequence->AnimNotifyTracks.Num() > 1 && Sequence->AnimNotifyTracks.IsValidIndex(TrackIndexToDelete))
	{
		return Sequence->AnimNotifyTracks[TrackIndexToDelete].Notifies.Num() == 0;
	}

	return false;
}

void SAnimNotifyPanel::OnCommitTrackName(const FText& InText, ETextCommit::Type CommitInfo, int32 TrackIndexToName)
{
	if (Sequence->AnimNotifyTracks.IsValidIndex(TrackIndexToName))
	{
		FScopedTransaction Transaction(FText::Format(LOCTEXT("RenameNotifyTrack", "Rename Notify Track to '{0}'"), InText));
		Sequence->Modify();

		FText TrimText = FText::TrimPrecedingAndTrailing(InText);
		Sequence->AnimNotifyTracks[TrackIndexToName].TrackName = FName(*TrimText.ToString());
	}
}

void SAnimNotifyPanel::Update()
{
	if(!bIsUpdating)
	{
		TGuardValue<bool> ScopeGuard(bIsUpdating, true);

		if(Sequence != NULL)
		{
			Sequence->RefreshCacheData();
		}

		RefreshNotifyTracks();

		OnNotifiesChanged.ExecuteIfBound();
	}
}

void SAnimNotifyPanel::RequestUpdate()
{
	bUpdateRequested = true;
}

void SAnimNotifyPanel::RequestRefresh()
{
	bRefreshRequested = true;
}

// Helper to save/restore selection state when widgets are recreated
struct FScopedSavedNotifySelection
{
	FScopedSavedNotifySelection(SAnimNotifyPanel& InPanel)
		: Panel(InPanel)
	{
		for (TSharedPtr<SAnimNotifyTrack> Track : InPanel.NotifyAnimTracks)
		{
			for(int32 NodeIndex = 0; NodeIndex < Track->GetNumNotifyNodes(); ++NodeIndex)
			{
				if(Track->IsNodeSelected(NodeIndex))
				{
					SelectedNodeGuids.Add(Track->GetNodeObjectInterface(NodeIndex)->GetGuid());	
				}
			}
		}
	}

	~FScopedSavedNotifySelection()
	{
		// Re-apply selection state
		for (TSharedPtr<SAnimNotifyTrack> Track : Panel.NotifyAnimTracks)
		{
			Track->SelectNodesByGuid(SelectedNodeGuids, false);
		}
	}

	SAnimNotifyPanel& Panel;
	TSet<FGuid> SelectedNodeGuids;
};

void SAnimNotifyPanel::RefreshNotifyTracks()
{
	check (Sequence);

	{
		FScopedSavedNotifySelection ScopedSelection(*this);

		TSharedPtr<SVerticalBox> NotifySlots;
		PanelArea->SetContent(
			SAssignNew( NotifySlots, SVerticalBox )
			);

		// Clear node tool tips to stop slate referencing them and possibly
		// causing a crash if the notify has gone away
		for (TSharedPtr<SAnimNotifyTrack> Track : NotifyAnimTracks)
		{
			Track->ClearNodeTooltips();
		}

		NotifyAnimTracks.Empty();
		NotifyEditorTracks.Empty();

		for(int32 TrackIndex = 0; TrackIndex < Sequence->AnimNotifyTracks.Num(); TrackIndex++)
		{
			FAnimNotifyTrack& Track = Sequence->AnimNotifyTracks[TrackIndex];
			TSharedPtr<SNotifyEdTrack> EdTrack;

			NotifySlots->AddSlot()
			.AutoHeight()
			.VAlign(VAlign_Center)
			[
				SAssignNew(EdTrack, SNotifyEdTrack)
				.TrackIndex(TrackIndex)
				.Sequence(Sequence)
				.AnimNotifyPanel(SharedThis(this))
				.WidgetWidth(WidgetWidth)
				.ViewInputMin(ViewInputMin)
				.ViewInputMax(ViewInputMax)
				.OnGetScrubValue(OnGetScrubValue)
				.OnGetDraggedNodePos(this, &SAnimNotifyPanel::CalculateDraggedNodePos)
				.OnUpdatePanel(this, &SAnimNotifyPanel::RequestUpdate)
				.OnGetNotifyBlueprintData(this, &SAnimNotifyPanel::OnGetNotifyBlueprintData, &NotifyClassNames)
				.OnGetNotifyStateBlueprintData(this, &SAnimNotifyPanel::OnGetNotifyBlueprintData, &NotifyStateClassNames)
				.OnGetNotifyNativeClasses(this, &SAnimNotifyPanel::OnGetNativeNotifyData, UAnimNotify::StaticClass(), &NotifyClassNames)
				.OnGetNotifyStateNativeClasses(this, &SAnimNotifyPanel::OnGetNativeNotifyData, UAnimNotifyState::StaticClass(), &NotifyStateClassNames)
				.OnSelectionChanged(this, &SAnimNotifyPanel::OnTrackSelectionChanged)
				.OnNodeDragStarted(this, &SAnimNotifyPanel::OnNotifyNodeDragStarted)
				.OnNotifyStateHandleBeingDragged(OnNotifyStateHandleBeingDragged)
				.OnSnapPosition(OnSnapPosition)
				.OnRequestRefreshOffsets(OnRequestRefreshOffsets)
				.OnDeleteNotify(this, &SAnimNotifyPanel::DeleteSelectedNodeObjects)
				.OnDeselectAllNotifies(this, &SAnimNotifyPanel::DeselectAllNotifies)
				.OnCopyNodes(this, &SAnimNotifyPanel::CopySelectedNodesToClipboard)
				.OnPasteNodes(this, &SAnimNotifyPanel::OnPasteNodes)
				.OnSetInputViewRange(this, &SAnimNotifyPanel::InputViewRangeChanged)
				.OnGetTimingNodeVisibility(OnGetTimingNodeVisibility)
				.OnInvokeTab(OnInvokeTab)
			];

			NotifyAnimTracks.Add(EdTrack->NotifyTrack);
			NotifyEditorTracks.Add(EdTrack);
		}
	}

	// Signal selection change to refresh details panel
	OnTrackSelectionChanged();
}

float SAnimNotifyPanel::CalculateDraggedNodePos() const
{
	return CurrentDragXPosition;
}

FReply SAnimNotifyPanel::OnNotifyNodeDragStarted(TArray<TSharedPtr<SAnimNotifyNode>> NotifyNodes, TSharedRef<SWidget> Decorator, const FVector2D& ScreenCursorPos, const FVector2D& ScreenNodePosition, const bool bDragOnMarker)
{
	TSharedRef<SOverlay> NodeDragDecoratorOverlay = SNew(SOverlay);
	TSharedRef<SBorder> NodeDragDecorator = SNew(SBorder)
	.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
	[
		NodeDragDecoratorOverlay
	];

	TArray<TSharedPtr<SAnimNotifyNode>> Nodes;

	for(TSharedPtr<SAnimNotifyTrack> Track : NotifyAnimTracks)
	{
		Track->DisconnectSelectedNodesForDrag(Nodes);
	}

	const FGeometry& PanelGeometry = GetCachedGeometry();

	// Absolute screen-space top-left of the selection — used downstream for cursor-relative drag tracking.
	FVector2D OverlayOrigin = Nodes[0]->GetScreenPosition();

	// Track-local horizontal width of the selection — used downstream for track-local clamping math.
	const float FirstNodeLocalX = PanelGeometry.AbsoluteToLocal(OverlayOrigin).X;
	float LocalMinX = FirstNodeLocalX;
	float LocalMaxX = FirstNodeLocalX + Nodes[0]->GetDurationSize();

	for (int32 Idx = 1; Idx < Nodes.Num(); ++Idx)
	{
		const TSharedPtr<SAnimNotifyNode>& Node = Nodes[Idx];
		const FVector2D NodeAbsPos = Node->GetScreenPosition();
		const float NodeLocalX = PanelGeometry.AbsoluteToLocal(NodeAbsPos).X;
		const float NodeDuration = Node->GetDurationSize();

		OverlayOrigin = FVector2D::Min(OverlayOrigin, NodeAbsPos);
		LocalMinX = FMath::Min(LocalMinX, NodeLocalX);
		LocalMaxX = FMath::Max(LocalMaxX, NodeLocalX + NodeDuration);
	}

	const FSlateRenderTransform InvTrackTransform = GetCachedGeometry().GetAccumulatedRenderTransform().Inverse();
	
	for(const TSharedPtr<SAnimNotifyNode>& Node : Nodes)
	{
		const FVector2D OffsetFromFirst(InvTrackTransform.TransformVector(Node->GetScreenPosition() - OverlayOrigin));
		
		NodeDragDecoratorOverlay->AddSlot()
			.Padding(FMargin(static_cast<float>(OffsetFromFirst.X), static_cast<float>(OffsetFromFirst.Y), 0.0f, 0.0f))
			[
				Node->AsShared()
			];
	}

	FPanTrackRequest PanRequestDelegate = FPanTrackRequest::CreateSP(this, &SAnimNotifyPanel::PanInputViewRange);
	FOnUpdatePanel UpdateDelegate = FOnUpdatePanel::CreateSP(this, &SAnimNotifyPanel::RequestUpdate);
	return FReply::Handled().BeginDragDrop(FNotifyDragDropOp::New(Nodes, NodeDragDecorator, NotifyAnimTracks, Sequence, ScreenCursorPos, OverlayOrigin, LocalMaxX - LocalMinX, CurrentDragXPosition, PanRequestDelegate, OnSnapPosition, UpdateDelegate, OnNotifyNodesBeingDragged));
}

float SAnimNotifyPanel::GetSequenceLength() const
{
	return Sequence->GetPlayLength();
}

void SAnimNotifyPanel::CutSelectedNodesToClipboard()
{
	CopySelectedNodesToClipboard();
	DeleteSelectedNodeObjects();
}

void SAnimNotifyPanel::PostUndo( bool bSuccess )
{
	if(Sequence != NULL)
	{
		Sequence->RefreshCacheData();
	}
	RefreshNotifyTracks();
}

void SAnimNotifyPanel::PostRedo( bool bSuccess )
{
	if(Sequence != NULL)
	{
		Sequence->RefreshCacheData();
	}
	RefreshNotifyTracks();
}

void SAnimNotifyPanel::OnDeletePressed()
{
	// If there's no focus on the panel it's likely the user is not editing notifies
	// so don't delete anything when the key is pressed.
	if(HasKeyboardFocus() || HasFocusedDescendants()) 
	{
		DeleteSelectedNodeObjects();
	}
}

void SAnimNotifyPanel::DeleteSelectedNodeObjects()
{
	TArray<INotifyNodeObjectInterface*> SelectedNodes;
	for (TSharedPtr<SAnimNotifyTrack> Track : NotifyAnimTracks)
	{
		Track->AppendSelectionToArray(SelectedNodes);
	}

	const bool bContainsSyncMarkers = SelectedNodes.ContainsByPredicate([](const INotifyNodeObjectInterface* Interface) { return Interface->GetType() == ENotifyNodeObjectTypes::NOTIFY; });

	if (SelectedNodes.Num() > 0)
	{
		FScopedTransaction Transaction(LOCTEXT("DeleteMarkers", "Delete Animation Markers"));
		Sequence->Modify(true);

		// As we address node object's source data by pointer, we need to mark for delete then
		// delete invalid entries to avoid concurrent modification of containers
		for (INotifyNodeObjectInterface* NodeObject : SelectedNodes)
		{
			NodeObject->MarkForDelete(Sequence);
		}

		FNotifyNodeInterface::RemoveInvalidNotifies(Sequence);
		FSyncMarkerNodeInterface::RemoveInvalidSyncMarkers(Sequence);

		if (bContainsSyncMarkers)
		{
			UBlendSpace::UpdateBlendSpacesUsingAnimSequence(Sequence);
		}
	}

	// clear selection and update the panel
	TArray<UObject*> Objects;
	OnSelectionChanged.ExecuteIfBound(Objects);

	RequestUpdate();
}

void SAnimNotifyPanel::SetSequence(class UAnimSequenceBase*	InSequence)
{
	if (InSequence != Sequence)
	{
		Sequence = InSequence;
		RequestUpdate();
	}
}

void SAnimNotifyPanel::OnTrackSelectionChanged()
{
	if(!bIsSelecting)
	{
		TGuardValue<bool> GuardValue(bIsSelecting, true);

		// Need to collect selection info from all tracks
		TArray<UObject*> NotifyObjects;

		for(int32 TrackIdx = 0 ; TrackIdx < NotifyAnimTracks.Num() ; ++TrackIdx)
		{
			TSharedPtr<SAnimNotifyTrack> Track = NotifyAnimTracks[TrackIdx];
			const TArray<int32>& TrackIndices = Track->GetSelectedNotifyIndices();
			for(int32 Idx : TrackIndices)
			{
				INotifyNodeObjectInterface* NodeObjectInterface = Track->GetNodeObjectInterface(Idx);
				if (FAnimNotifyEvent* NotifyEvent = NodeObjectInterface->GetNotifyEvent())
				{
					FString ObjName = MakeUniqueObjectName(GetTransientPackage(), UEditorNotifyObject::StaticClass()).ToString();
					UEditorNotifyObject* NewNotifyObject = NewObject<UEditorNotifyObject>(GetTransientPackage(), FName(*ObjName), RF_Public | RF_Standalone | RF_Transient);
					NewNotifyObject->InitFromAnim(Sequence, FOnAnimObjectChange::CreateSP(this, &SAnimNotifyPanel::OnNotifyObjectChanged));
					NewNotifyObject->InitialiseNotify(*Sequence->AnimNotifyTracks[TrackIdx].Notifies[Idx]);
					NotifyObjects.AddUnique(NewNotifyObject);
				}
			}
		}

		OnSelectionChanged.ExecuteIfBound(NotifyObjects);
	}
}

void SAnimNotifyPanel::DeselectAllNotifies()
{
	if(!bIsSelecting)
	{
		TGuardValue<bool> GuardValue(bIsSelecting, true);

		for(TSharedPtr<SAnimNotifyTrack> Track : NotifyAnimTracks)
		{
			Track->DeselectAllNotifyNodes(false);
		}

		TArray<UObject*> NotifyObjects;
		OnSelectionChanged.ExecuteIfBound(NotifyObjects);
	}
}

void SAnimNotifyPanel::CopySelectedNodesToClipboard() const
{
	// Grab the selected events
	TArray<INotifyNodeObjectInterface*> SelectedNodes;
	for(TSharedPtr<SAnimNotifyTrack> Track : NotifyAnimTracks)
	{
		Track->AppendSelectionToArray(SelectedNodes);
	}

	const FString HeaderString(TEXT("COPY_ANIMNOTIFYEVENT"));
	
	if (SelectedNodes.Num() > 0)
	{
		FString StrValue(HeaderString);

		// Sort by track
		SelectedNodes.Sort([](const INotifyNodeObjectInterface& A, const INotifyNodeObjectInterface& B)
		{
			return (A.GetTrackIndex() < B.GetTrackIndex()) || (A.GetTrackIndex() == B.GetTrackIndex() && A.GetTime() < B.GetTime());
		});

		// Need to find how many tracks this selection spans and the minimum time to use as the beginning of the selection
		int32 MinTrack = MAX_int32;
		int32 MaxTrack = MIN_int32;
		float MinTime = MAX_flt;
		for (const INotifyNodeObjectInterface* NodeObject : SelectedNodes)
		{
			MinTrack = FMath::Min(MinTrack, NodeObject->GetTrackIndex());
			MaxTrack = FMath::Max(MaxTrack, NodeObject->GetTrackIndex());
			MinTime = FMath::Min(MinTime, NodeObject->GetTime());
		}

		int32 TrackSpan = MaxTrack - MinTrack + 1;

		StrValue += FString::Printf(TEXT("OriginalTime=%f,"), MinTime);
		StrValue += FString::Printf(TEXT("OriginalLength=%f,"), Sequence->GetPlayLength());
		StrValue += FString::Printf(TEXT("TrackSpan=%d"), TrackSpan);

		for(const INotifyNodeObjectInterface* NodeObject : SelectedNodes)
		{
			// Locate the notify in the sequence, we need the sequence index; but also need to
			// keep the order we're currently in.

			StrValue += "\n";
			StrValue += FString::Printf(TEXT("AbsTime=%f,NodeObjectType=%i,"), NodeObject->GetTime(), (int32)NodeObject->GetType());

			NodeObject->ExportForCopy(Sequence, StrValue);
		}
		FPlatformApplicationMisc::ClipboardCopy(*StrValue);
	}
}

bool SAnimNotifyPanel::IsNotifySelectionValidForReplacement()
{
	// Grab the selected events
	TArray<INotifyNodeObjectInterface*> SelectedNodes;
	for (TSharedPtr<SAnimNotifyTrack> Track : NotifyAnimTracks)
	{
		Track->AppendSelectionToArray(SelectedNodes);
	}

	bool bSelectionContainsAnimNotify = false;
	bool bSelectionContainsAnimNotifyState = false;
	for (INotifyNodeObjectInterface* NodeObject : SelectedNodes)
	{
		FAnimNotifyEvent* NotifyEvent = NodeObject->GetNotifyEvent();
		if (NotifyEvent)
		{
			if (NotifyEvent->Notify)
			{
				bSelectionContainsAnimNotify = true;
			}
			else if (NotifyEvent->NotifyStateClass)
			{
				bSelectionContainsAnimNotifyState = true;
			}
			// Custom AnimNotifies have no class, but they are like AnimNotify class notifies in that they have no duration
			else
			{
				bSelectionContainsAnimNotify = true;
			}
		}
	}

	// Only allow replacement for selections that contain _only_ AnimNotifies, or _only_ AnimNotifyStates, but not both
	// (Want to disallow replacement of AnimNotify with AnimNotifyState, and vice-versa)
	bool bIsValidSelection = bSelectionContainsAnimNotify != bSelectionContainsAnimNotifyState;

	return bIsValidSelection;
}


void SAnimNotifyPanel::OnReplaceSelectedWithNotify(FString NewNotifyName, UClass* NewNotifyClass)
{
	TArray<INotifyNodeObjectInterface*> SelectedNodes;
	for (TSharedPtr<SAnimNotifyTrack> Track : NotifyAnimTracks)
	{
		Track->AppendSelectionToArray(SelectedNodes);
	}

	// Sort these since order is important for deletion
	SelectedNodes.Sort();

	const FScopedTransaction Transaction(LOCTEXT("ReplaceAnimNotify", "Replace Anim Notify"));
	Sequence->Modify(true);

	// Reverse through the array, replacing nodes.
	for(int32 index=SelectedNodes.Num()-1; index>=0; index--)
	{
		INotifyNodeObjectInterface* NodeObject = SelectedNodes[index];
		FAnimNotifyEvent* OldEvent = NodeObject->GetNotifyEvent();
		if (OldEvent)
		{
			float BeginTime = OldEvent->GetTime();
			float Length = OldEvent->GetDuration();
			int32 TargetTrackIndex = OldEvent->TrackIndex;
			float TriggerTimeOffset = OldEvent->TriggerTimeOffset;
			float EndTriggerTimeOffset = OldEvent->EndTriggerTimeOffset;
			int32 SlotIndex = OldEvent->GetSlotIndex();
			int32 EndSlotIndex = OldEvent->EndLink.GetSlotIndex();
			int32 SegmentIndex = OldEvent->GetSegmentIndex();
			int32 EndSegmentIndex = OldEvent->GetSegmentIndex();
			EAnimLinkMethod::Type LinkMethod = OldEvent->GetLinkMethod();
			EAnimLinkMethod::Type EndLinkMethod = OldEvent->EndLink.GetLinkMethod();

			FColor OldColor = OldEvent->NotifyColor;
			UAnimNotify* OldEventPayload = OldEvent->Notify;
			UAnimNotifyState* OldEventStatePayload = OldEvent->NotifyStateClass;

			// Delete old one before creating new one to avoid potential array re-allocation when array temporarily increases by 1 in size
			NodeObject->Delete(Sequence);
			FAnimNotifyEvent& NewEvent = NotifyAnimTracks[TargetTrackIndex]->CreateNewNotify(NewNotifyName, NewNotifyClass, BeginTime);

			NewEvent.TriggerTimeOffset = TriggerTimeOffset;
			NewEvent.ChangeSlotIndex(SlotIndex);
			NewEvent.SetSegmentIndex(SegmentIndex);
			NewEvent.ChangeLinkMethod(LinkMethod);
			NewEvent.NotifyColor = OldColor;

			// Copy what we can across from the payload
			if ((OldEventPayload != nullptr) && (NewEvent.Notify != nullptr))
			{
				UEngine::FCopyPropertiesForUnrelatedObjectsParams CopyParams;
				CopyParams.bNotifyObjectReplacement = true;
				UEngine::CopyPropertiesForUnrelatedObjects(OldEventPayload, NewEvent.Notify, CopyParams);
			}

			// For Anim Notify States, handle the end time and link
			if (NewEvent.NotifyStateClass != nullptr)
			{
				if (OldEventStatePayload != nullptr)
				{
					UEngine::FCopyPropertiesForUnrelatedObjectsParams CopyParams;
					CopyParams.bNotifyObjectReplacement = true;
					UEngine::CopyPropertiesForUnrelatedObjects(OldEventStatePayload, NewEvent.NotifyStateClass, CopyParams);
				}

				NewEvent.SetDuration(Length);
				NewEvent.EndTriggerTimeOffset = EndTriggerTimeOffset;
				NewEvent.EndLink.ChangeSlotIndex(EndSlotIndex);
				NewEvent.EndLink.SetSegmentIndex(EndSegmentIndex);
				NewEvent.EndLink.ChangeLinkMethod(EndLinkMethod);
			}
						
			NewEvent.Update();
		}
	}

	// clear selection  
	TArray<UObject*> Objects;
	OnSelectionChanged.ExecuteIfBound(Objects);
	// TODO: set selection to new notifies?
	// update the panel

	Sequence->PostEditChange();
	Sequence->MarkPackageDirty();

	RequestUpdate();
}

void SAnimNotifyPanel::OnReplaceSelectedWithNotifyBlueprint(FString NewBlueprintNotifyName, FString NewBlueprintNotifyClass)
{
	TSubclassOf<UObject> BlueprintClass = SAnimNotifyTrack::GetBlueprintClassFromPath(NewBlueprintNotifyClass);
	OnReplaceSelectedWithNotify(NewBlueprintNotifyName, BlueprintClass);
}

void SAnimNotifyPanel::OnReplaceSelectedWithSyncMarker(FString NewSyncMarkerName)
{
	if (UAnimSequence* Seq = Cast<UAnimSequence>(Sequence))
	{
		TArray<INotifyNodeObjectInterface*> SelectedNodes;
		for (TSharedPtr<SAnimNotifyTrack> Track : NotifyAnimTracks)
		{
			Track->AppendSelectionToArray(SelectedNodes);
		}

		// Sort these since order is important for deletion
		SelectedNodes.Sort();

		const FScopedTransaction Transaction(LOCTEXT("ReplaceSyncMarker", "Replace Sync Marker"));
		Seq->Modify(true);

		for (INotifyNodeObjectInterface* NodeObject : SelectedNodes)
		{
			if (NodeObject->GetType() == ENotifyNodeObjectTypes::SYNC_MARKER)
			{
				float Time = NodeObject->GetTime();
				int32 TrackIndex = NodeObject->GetTrackIndex();

				NodeObject->Delete(Seq);

				FAnimSyncMarker& SyncMarker = Seq->AuthoredSyncMarkers.AddDefaulted_GetRef();
				SyncMarker.MarkerName = FName(*NewSyncMarkerName);
				SyncMarker.TrackIndex = TrackIndex;
				SyncMarker.Time = Time;
				SyncMarker.Guid = FGuid::NewGuid();
			}
		}

		// clear selection  
		TArray<UObject*> Objects;
		OnSelectionChanged.ExecuteIfBound(Objects);

		Seq->PostEditChange();
		Seq->MarkPackageDirty();

		RequestUpdate();
	}
}

void SAnimNotifyPanel::OnPasteNodes(SAnimNotifyTrack* RequestTrack, float ClickTime, ENotifyPasteMode::Type PasteMode, ENotifyPasteMultipleMode::Type MultiplePasteType)
{
	if(RequestTrack == nullptr)
	{
		for(TSharedPtr<SAnimNotifyTrack> Track : NotifyAnimTracks)
		{
			if(Track->HasKeyboardFocus())
			{
				RequestTrack = Track.Get();
				if(ClickTime == -1.0f)
				{
					ClickTime = RequestTrack->GetLastClickedTime();
				}
				break;
			}
		}
	}

	int32 PasteIdx = RequestTrack != nullptr ? RequestTrack->GetTrackIndex() : 0;
	int32 NumTracks = NotifyAnimTracks.Num();
	FString PropString;
	const TCHAR* Buffer;
	float OrigBeginTime;
	float OrigLength;
	int32 TrackSpan;
	int32 FirstTrack = -1;
	float ScaleMultiplier = 1.0f;

	if(ReadNotifyPasteHeader(PropString, Buffer, OrigBeginTime, OrigLength, TrackSpan))
	{
		DeselectAllNotifies();

		FScopedTransaction Transaction(LOCTEXT("PasteNotifyEvent", "Paste Anim Notifies"));
		Sequence->Modify();

		if(ClickTime == -1.0f)
		{
			if(PasteMode == ENotifyPasteMode::OriginalTime)
			{
				// We want to place the notifies exactly where they were
				ClickTime = OrigBeginTime;
			}
			else
			{
				ClickTime = WeakModel.Pin()->GetScrubTime();
			}
		}

		// Expand the number of tracks if we don't have enough.
		check(TrackSpan > 0);
		if(PasteIdx + TrackSpan > NumTracks)
		{
			int32 TracksToAdd = (PasteIdx + TrackSpan) - NumTracks;
			while(TracksToAdd)
			{
				AddTrack();
				--TracksToAdd;
			}
			RefreshNotifyTracks(); 
			NumTracks = NotifyAnimTracks.Num();
		}

		// Scaling for relative paste
		if(MultiplePasteType == ENotifyPasteMultipleMode::Relative)
		{
			ScaleMultiplier = Sequence->GetPlayLength() / OrigLength;
		}

		// Process each line of the paste buffer and spawn notifies
		FString CurrentLine;
		while(FParse::Line(&Buffer, CurrentLine))
		{
			int32 OriginalTrack;
			float OrigTime;
			int32 NodeObjectType;
			float PasteTime = -1.0f;
			if (FParse::Value(*CurrentLine, TEXT("TrackIndex="), OriginalTrack) && FParse::Value(*CurrentLine, TEXT("AbsTime="), OrigTime) && FParse::Value(*CurrentLine, TEXT("NodeObjectType="), NodeObjectType))
			{
				const int32 FirstComma = CurrentLine.Find(TEXT(","), ESearchCase::CaseSensitive, ESearchDir::FromStart);
				const int32 SecondComma = CurrentLine.Find(TEXT(","), ESearchCase::CaseSensitive, ESearchDir::FromStart, FirstComma + 1);
				FString NotifyExportString = CurrentLine.RightChop(SecondComma+1);

				// Store the first track so we know where to place notifies
				if(FirstTrack < 0)
				{
					FirstTrack = OriginalTrack;
				}
				int32 TrackOffset = OriginalTrack - FirstTrack;

				float TimeOffset = OrigTime - OrigBeginTime;
				float TimeToPaste = ClickTime + TimeOffset * ScaleMultiplier;

				if (PasteIdx + TrackOffset < NotifyAnimTracks.Num())
				{
					TSharedPtr<SAnimNotifyTrack> TrackToUse = NotifyAnimTracks[PasteIdx + TrackOffset];
					if (NodeObjectType == ENotifyNodeObjectTypes::NOTIFY)
					{
						TrackToUse->PasteSingleNotify(NotifyExportString, TimeToPaste);
					}
					else if (NodeObjectType == ENotifyNodeObjectTypes::SYNC_MARKER)
					{
						TrackToUse->PasteSingleSyncMarker(NotifyExportString, TimeToPaste);
					}
					else
					{
						check(false); //Unknown value in paste
					}
				}
			}
		}
	}
}

void SAnimNotifyPanel::BindCommands()
{
	// This should not be called twice on the same instance
	check(!CommandList.IsValid());
	CommandList = MakeShareable(new FUICommandList);

	const FAnimNotifyPanelCommands& Commands = FAnimNotifyPanelCommands::Get();
	
	CommandList->MapAction(
		Commands.DeleteNotify,
		FExecuteAction::CreateSP(this, &SAnimNotifyPanel::OnDeletePressed));

	CommandList->MapAction(
		Commands.CutNotifies,
		FExecuteAction::CreateSP(this, &SAnimNotifyPanel::CutSelectedNodesToClipboard));
	
	CommandList->MapAction(
		Commands.CopyNotifies,
		FExecuteAction::CreateSP(this, &SAnimNotifyPanel::CopySelectedNodesToClipboard));

	CommandList->MapAction(
		Commands.PasteNotifies,
		FExecuteAction::CreateSP(this, &SAnimNotifyPanel::OnPasteNodes, (SAnimNotifyTrack*)nullptr, -1.0f, ENotifyPasteMode::MousePosition, ENotifyPasteMultipleMode::Absolute));
}

FReply SAnimNotifyPanel::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if(CommandList.IsValid() && CommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

FReply SAnimNotifyPanel::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (SAnimTrackPanel::OnMouseButtonDown(MyGeometry, MouseEvent).IsEventHandled())
	{
		return FReply::Handled();
	}

	bool bCreateMarquee = MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton) && (MouseEvent.IsAltDown());

	if(bCreateMarquee)
	{
		TArray<TSharedPtr<SAnimNotifyNode>> SelectedNodes;
		for(TSharedPtr<SAnimNotifyTrack> Track : NotifyAnimTracks)
		{
			Track->AppendSelectedNodeWidgetsToArray(SelectedNodes);
		}

		Marquee.Start(MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition()), Marquee.OperationTypeFromMouseEvent(MouseEvent), SelectedNodes);
		if(Marquee.Operation == FNotifyMarqueeOperation::Replace)
		{
			// Remove and Add operations preserve selections, replace starts afresh
			DeselectAllNotifies();
		}

		return FReply::Handled().DetectDrag(SharedThis(this), EKeys::LeftMouseButton);
	}

	return FReply::Unhandled();
}

FReply SAnimNotifyPanel::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if(Marquee.bActive)
	{
		OnTrackSelectionChanged();
		Marquee = FNotifyMarqueeOperation();
		return FReply::Handled().ReleaseMouseCapture();
	}

	return SAnimTrackPanel::OnMouseButtonUp(MyGeometry, MouseEvent);
}

FReply SAnimNotifyPanel::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	FReply BaseReply = SAnimTrackPanel::OnMouseMove(MyGeometry, MouseEvent);
	if(!BaseReply.IsEventHandled())
	{
		bool bLeftButton = MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton);

		if(bLeftButton && Marquee.bActive)
		{
			Marquee.Rect.UpdateEndPoint(MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition()));
			RefreshMarqueeSelectedNodes(MyGeometry);
			return FReply::Handled();
		}
	}

	return BaseReply;
}

int32 SAnimNotifyPanel::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	LayerId = SAnimTrackPanel::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);

	FVector2D Origin = AllottedGeometry.AbsoluteToLocal(Marquee.Rect.GetUpperLeft());
	FVector2D Extents = AllottedGeometry.AbsoluteToLocal(Marquee.Rect.GetSize());

	if(Marquee.IsValid())
	{
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId++,
			AllottedGeometry.ToPaintGeometry(Marquee.Rect.GetSize(), FSlateLayoutTransform(Marquee.Rect.GetUpperLeft())),
			FAppStyle::GetBrush(TEXT("MarqueeSelection"))
			);
	}

	return LayerId;
}

void SAnimNotifyPanel::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	if(bUpdateRequested)
	{
		Update();
		bUpdateRequested = false;
		bRefreshRequested = false;
	}
	if(bRefreshRequested)
	{
		RefreshNotifyTracks();
		bRefreshRequested = false;
	}
}

void SAnimNotifyPanel::RefreshMarqueeSelectedNodes(const FGeometry& PanelGeo)
{
	if(Marquee.IsValid())
	{
		const FSlateRect MarqueeRect = Marquee.Rect.ToSlateRect();
		const FVector2D MarqueeTopLeftAbsolute = PanelGeo.LocalToAbsolute(MarqueeRect.GetTopLeft());

		for(TSharedPtr<SAnimNotifyTrack> Track : NotifyAnimTracks)
		{
			if(Marquee.Operation == FNotifyMarqueeOperation::Replace || Marquee.OriginalSelection.Num() == 0)
			{
				Track->DeselectAllNotifyNodes(false);
			}

			const FGeometry& TrackGeo = Track->GetCachedGeometry();

			// Transform the Marquee Rect to Track Space
			const FSlateRect MarqueeTrackSpace = FSlateRect::FromPointAndExtent(TrackGeo.AbsoluteToLocal(MarqueeTopLeftAbsolute), MarqueeRect.GetSize());
			Track->RefreshMarqueeSelectedNodes(MarqueeTrackSpace, Marquee);
		}
	}
}

FReply SAnimNotifyPanel::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	Marquee.bActive = true;
	return FReply::Handled().CaptureMouse(SharedThis(this));
}

void SAnimNotifyPanel::OnFocusLost(const FFocusEvent& InFocusEvent)
{
	if(Marquee.bActive)
	{
		OnTrackSelectionChanged();
	}
	Marquee = FNotifyMarqueeOperation();
}

void SAnimNotifyPanel::PopulateNotifyBlueprintClasses(TArray<FString>& InOutAllowedClasses)
{
	TArray<FAssetData> TempArray;
	OnGetNotifyBlueprintData(TempArray, &InOutAllowedClasses);
}

void SAnimNotifyPanel::OnGetNotifyBlueprintData(TArray<FAssetData>& OutNotifyData, TArray<FString>* InOutAllowedClassNames)
{
	// If we have nothing to seach with, early out
	if(InOutAllowedClassNames == NULL || InOutAllowedClassNames->Num() == 0)
	{
		return;
	}

	TArray<FAssetData> AssetDataList;
	TArray<FString> FoundClasses;

	// Load the asset registry module
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

	// Collect a full list of assets with the specified class
	AssetRegistryModule.Get().GetAssetsByClass(UBlueprint::StaticClass()->GetClassPathName(), AssetDataList);


	int32 BeginClassCount = InOutAllowedClassNames->Num();
	int32 CurrentClassCount = -1;

	while(BeginClassCount != CurrentClassCount)
	{
		BeginClassCount = InOutAllowedClassNames->Num();

		for(int32 AssetIndex = 0; AssetIndex < AssetDataList.Num(); ++AssetIndex)
		{
			FAssetData& AssetData = AssetDataList[AssetIndex];
			FString TagValue = AssetData.GetTagValueRef<FString>(FBlueprintTags::ParentClassPath);

			if(InOutAllowedClassNames->Contains(TagValue))
			{
				FString GenClass = AssetData.GetTagValueRef<FString>(FBlueprintTags::GeneratedClassPath);
				const uint32 ClassFlags = AssetData.GetTagValueRef<uint32>(FBlueprintTags::ClassFlags);
				if (ClassFlags & CLASS_Abstract)
				{
					continue;
				}

				if(!OutNotifyData.Contains(AssetData))
				{
					// Output the assetdata and record it as found in this request
					OutNotifyData.Add(AssetData);
					FoundClasses.Add(GenClass);
				}

				if(!InOutAllowedClassNames->Contains(GenClass))
				{
					// Expand the class list to account for a new possible parent class found
					InOutAllowedClassNames->Add(GenClass);
				}
			}
		}

		CurrentClassCount = InOutAllowedClassNames->Num();
	}

	// Count native classes, so we don't remove them from the list
	int32 NumNativeClasses = 0;
	for(FString& AllowedClass : *InOutAllowedClassNames)
	{
		if(!AllowedClass.EndsWith(FString(TEXT("_C'"))))
		{
			++NumNativeClasses;
		}
	}

	if(FoundClasses.Num() < InOutAllowedClassNames->Num() - NumNativeClasses)
	{
		// Less classes found, some may have been deleted or reparented
		for(int32 ClassIndex = InOutAllowedClassNames->Num() - 1 ; ClassIndex >= 0 ; --ClassIndex)
		{
			FString& ClassName = (*InOutAllowedClassNames)[ClassIndex];
			if(ClassName.EndsWith(FString(TEXT("_C'"))) && !FoundClasses.Contains(ClassName))
			{
				InOutAllowedClassNames->RemoveAt(ClassIndex);
			}
		}
	}
}

void SAnimNotifyPanel::OnGetNativeNotifyData(TArray<UClass*>& OutClasses, UClass* NotifyOutermost, TArray<FString>* OutAllowedBlueprintClassNames)
{
	for(TObjectIterator<UClass> It ; It ; ++It)
	{
		UClass* Class = *It;

		if(Class->IsChildOf(NotifyOutermost) && Class->HasAllClassFlags(CLASS_Native) && !Class->IsInBlueprint())
		{
			OutClasses.Add(Class);
			// Form class name to search later
			FString ClassName = FObjectPropertyBase::GetExportPath(Class);
			OutAllowedBlueprintClassNames->AddUnique(ClassName);
		}
	}
}

void SAnimNotifyPanel::OnNotifyObjectChanged(UObject* EditorBaseObj, bool bRebuild)
{
	if(bRebuild)
	{
		if(UEditorNotifyObject* NotifyObject = Cast<UEditorNotifyObject>(EditorBaseObj))
		{
			FScopedSavedNotifySelection ScopedSelection(*this);

			for(FAnimNotifyEvent& Notify : Sequence->Notifies)
			{
				if(Notify.Guid == NotifyObject->Event.Guid)
				{
					if(NotifyAnimTracks.IsValidIndex(Notify.TrackIndex))
					{
						NotifyAnimTracks[Notify.TrackIndex]->Update();
					}
				}
			}
		}
	}
}

void SAnimNotifyPanel::OnNotifyTrackScrolled(float InScrollOffsetFraction)
{
	float Ratio = (ViewInputMax.Get() - ViewInputMin.Get()) / Sequence->GetPlayLength();
	float MaxOffset = (Ratio < 1.0f) ? 1.0f - Ratio : 0.0f;
	InScrollOffsetFraction = FMath::Clamp(InScrollOffsetFraction, 0.0f, MaxOffset);

	// Calculate new view ranges
	float NewMin = InScrollOffsetFraction * Sequence->GetPlayLength();
	float NewMax = (InScrollOffsetFraction + Ratio) * Sequence->GetPlayLength();
	
	InputViewRangeChanged(NewMin, NewMax);
}

void SAnimNotifyPanel::InputViewRangeChanged(float ViewMin, float ViewMax)
{
	float Ratio = (ViewMax - ViewMin) / Sequence->GetPlayLength();
	float OffsetFraction = ViewMin / Sequence->GetPlayLength();
	if(NotifyTrackScrollBar.IsValid())
	{
		NotifyTrackScrollBar->SetState(OffsetFraction, Ratio);
	}

	SAnimTrackPanel::InputViewRangeChanged(ViewMin, ViewMax);
}

void SAnimNotifyPanel::HandleObjectsSelected(const TArray<UObject*>& InObjects)
{
	if(!bIsSelecting)
	{
		DeselectAllNotifies();
	}
}

#undef LOCTEXT_NAMESPACE
