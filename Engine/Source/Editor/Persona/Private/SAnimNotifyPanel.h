// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/UObjectGlobals.h"
#include "Misc/Attribute.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Styling/AppStyle.h"
#include "Framework/MarqueeRect.h"
#include "SAnimTrackPanel.h"
#include "SAnimEditorBase.h"
#include "AssetRegistry/AssetData.h"
#include "Framework/Commands/Commands.h"
#include "SAnimTimingPanel.h"
#include "EditorUndoClient.h"
#include "AnimTimeline/AnimModel.h"
#include "Containers/ArrayView.h"
#include "SAnimNotifyShared.h"



class FSlateWindowElementList;
class SAnimNotifyNode;
class SAnimNotifyTrack;
class SBorder;
class SScrollBar;
struct Rect;

DECLARE_DELEGATE_OneParam( FOnSelectionChanged, const TArray<UObject*>& )
DECLARE_DELEGATE_RetVal( float, FOnGetScrubValue )
DECLARE_DELEGATE( FRefreshOffsetsRequest )
DECLARE_DELEGATE( FDeleteNotify )
DECLARE_DELEGATE_RetVal( bool, FOnGetIsAnimNotifySelectionValidForReplacement )
DECLARE_DELEGATE_TwoParams( FReplaceWithNotify, FString, UClass* )
DECLARE_DELEGATE_TwoParams( FReplaceWithBlueprintNotify, FString, FString )
DECLARE_DELEGATE_OneParam( FReplaceWithSyncMarker, FString )
DECLARE_DELEGATE( FDeselectAllNotifies )
DECLARE_DELEGATE_OneParam( FOnGetBlueprintNotifyData, TArray<FAssetData>& )
DECLARE_DELEGATE_OneParam( FOnGetNativeNotifyClasses, TArray<UClass*>&)


class SAnimNotifyNode;
class SAnimNotifyTrack;
class SNotifyEdTrack;
class FNotifyDragDropOp;

namespace ENotifyPasteMode
{
	enum Type
	{
		MousePosition,
		OriginalTime
	};
}

namespace ENotifyPasteMultipleMode
{
	enum Type
	{
		Relative,
		Absolute
	};
}

struct FNotifyMarqueeOperation
{
	FNotifyMarqueeOperation()
		: Operation(Add)
		, bActive(false)
	{
	}

	enum Type
	{
		/** Holding down Ctrl removes nodes */
		Remove,
		/** Holding down Shift adds to the selection */
		Add,
		/** When nothing is pressed, marquee replaces selection */
		Replace
	} Operation;

	bool IsValid() const
	{
		return Rect.IsValid() && bActive;
	}

	void Start(const FVector2D& InStartLocation, FNotifyMarqueeOperation::Type InOperationType, TArray<TSharedPtr<SAnimNotifyNode>>& InOriginalSelection)
	{
		Rect = FMarqueeRect(InStartLocation);
		Operation = InOperationType;
		OriginalSelection = InOriginalSelection;
	}

	void End()
	{
		Rect = FMarqueeRect();
	}


	/** Given a mouse event, figure out what the marquee selection should do based on the state of Shift and Ctrl keys */
	static FNotifyMarqueeOperation::Type OperationTypeFromMouseEvent(const FPointerEvent& MouseEvent)
	{
		if(MouseEvent.IsControlDown())
		{
			return FNotifyMarqueeOperation::Remove;
		}
		else if(MouseEvent.IsShiftDown())
		{
			return FNotifyMarqueeOperation::Add;
		}
		else
		{
			return FNotifyMarqueeOperation::Replace;
		}
	}

public:
	/** The marquee rectangle being dragged by the user */
	FMarqueeRect Rect;

	/** Whether the marquee has been activated, usually by a drag */
	bool bActive;

	/** The original selection state before the marquee selection */
	TArray<TSharedPtr<SAnimNotifyNode>> OriginalSelection;
};




//////////////////////////////////////////////////////////////////////////
// SAnimNotifyPanel

class FAnimNotifyPanelCommands : public TCommands<FAnimNotifyPanelCommands>
{
public:
	FAnimNotifyPanelCommands()
		: TCommands<FAnimNotifyPanelCommands>("AnimNotifyPanel", NSLOCTEXT("Contexts", "AnimNotifyPanel", "Anim Notify Panel"), NAME_None, FAppStyle::GetAppStyleSetName())
	{

	}

	TSharedPtr<FUICommandInfo> DeleteNotify;

	TSharedPtr<FUICommandInfo> CopyNotifies;
	
	TSharedPtr<FUICommandInfo> CutNotifies;

	TSharedPtr<FUICommandInfo> PasteNotifies;

	virtual void RegisterCommands() override;
};

// @todo anim : register when it's opened for the animsequence
// broadcast when animsequence changed, so that we refresh for multiple window
class SAnimNotifyPanel: public SAnimTrackPanel, public FEditorUndoClient
{
public:
	SLATE_BEGIN_ARGS( SAnimNotifyPanel )
		: _Sequence()
		, _CurrentPosition()
		, _ViewInputMin()
		, _ViewInputMax()
		, _InputMin()
		, _InputMax()
		, _OnSetInputViewRange()
		, _OnSelectionChanged()
		, _OnGetScrubValue()
		, _OnRequestRefreshOffsets()
	{}

	SLATE_ARGUMENT( class UAnimSequenceBase*, Sequence)
	SLATE_ARGUMENT( float, WidgetWidth )
	SLATE_ATTRIBUTE( float, CurrentPosition )
	SLATE_ATTRIBUTE( float, ViewInputMin )
	SLATE_ATTRIBUTE( float, ViewInputMax )
	SLATE_ATTRIBUTE( float, InputMin )
	SLATE_ATTRIBUTE( float, InputMax )
	SLATE_EVENT( FOnSetInputViewRange, OnSetInputViewRange )
	SLATE_EVENT( FOnSelectionChanged, OnSelectionChanged )
	SLATE_EVENT( FOnGetScrubValue, OnGetScrubValue )
	SLATE_EVENT( FRefreshOffsetsRequest, OnRequestRefreshOffsets )
	SLATE_EVENT( FOnGetTimingNodeVisibility, OnGetTimingNodeVisibility )
	SLATE_EVENT( FOnInvokeTab, OnInvokeTab )
	SLATE_EVENT( FSimpleDelegate, OnNotifiesChanged )
	SLATE_EVENT( FOnSnapPosition, OnSnapPosition )
	SLATE_EVENT( FOnNotifyStateHandleBeingDragged, OnNotifyStateHandleBeingDragged)
	SLATE_EVENT( FOnNotifyNodesBeingDragged, OnNotifyNodesBeingDragged)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<FAnimModel>& InModel);
	virtual ~SAnimNotifyPanel();

	void SetSequence(class UAnimSequenceBase *	InSequence);

	// Generate a new track name (smallest integer number that isn't currently used)
	FName GetNewTrackName() const;

	FReply AddTrack();
	FReply InsertTrack(int32 TrackIndexToInsert);
	FReply DeleteTrack(int32 TrackIndexToDelete);
	bool CanDeleteTrack(int32 TrackIndexToDelete);
	
	// Handler function for renaming a notify track
	void OnCommitTrackName(const FText& InText, ETextCommit::Type CommitInfo, int32 TrackIndexToName);

	// Request a deferred Update call
	void RequestUpdate();
	
	void Update();

	/** Returns the position of the notify node currently being dragged. Returns -1 if no node is being dragged */
	float CalculateDraggedNodePos() const;

	/**Handler for when a notify node drag has been initiated */
	FReply OnNotifyNodeDragStarted(TArray<TSharedPtr<SAnimNotifyNode>> NotifyNodes, TSharedRef<SWidget> Decorator, const FVector2D& ScreenCursorPos, const FVector2D& ScreenNodePosition, const bool bDragOnMarker);

	virtual float GetSequenceLength() const override;

	void CutSelectedNodesToClipboard();
	
	void CopySelectedNodesToClipboard() const;
	void OnPasteNodes(SAnimNotifyTrack* RequestTrack, float ClickTime, ENotifyPasteMode::Type PasteMode, ENotifyPasteMultipleMode::Type MultiplePasteType);

	/** SWidget Interface */
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual void OnFocusLost(const FFocusEvent& InFocusEvent) override;
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;
	/** End SWidget Interface */

	void RefreshMarqueeSelectedNodes(const FGeometry& PanelGeo);

	void OnNotifyObjectChanged(UObject* EditorBaseObj, bool bRebuild);

	/** Check to make sure the current AnimNotify selection is a valid selection for replacing (i.e., AnimNotifies and AnimNotifyStates aren't mixed together in the selection) */
	bool IsNotifySelectionValidForReplacement();

	/** Handler for replacing with notify */
	void OnReplaceSelectedWithNotify(FString NewNotifyName, UClass* NewNotifyClass);

	/** Handler for replacing with notify blueprint */
	void OnReplaceSelectedWithNotifyBlueprint(FString NewBlueprintNotifyName, FString NewBlueprintNotifyClass);

	/** Handler for replacing with sync marker */
	void OnReplaceSelectedWithSyncMarker(FString NewBlueprintNotifyName);

	void HandleObjectsSelected(const TArray<UObject*>& InObjects);

	TSharedRef<FUICommandList> GetCommandList() const { return CommandList.ToSharedRef(); }

private:
	friend struct FScopedSavedNotifySelection;

	TWeakPtr<FAnimModel> WeakModel;
	TSharedPtr<SBorder> PanelArea;
	TSharedPtr<SScrollBar> NotifyTrackScrollBar;
	class UAnimSequenceBase* Sequence;
	float WidgetWidth;
	TAttribute<float> CurrentPosition;
	FOnSelectionChanged OnSelectionChanged;
	FOnGetScrubValue OnGetScrubValue;
	FOnGetTimingNodeVisibility OnGetTimingNodeVisibility;

	/** Manager for mouse controlled marquee selection */
	FNotifyMarqueeOperation Marquee;

	/** Delegate to request a refresh of the offsets calculated for notifies */
	FRefreshOffsetsRequest OnRequestRefreshOffsets;

	/** Store the position of a currently dragged node for display across tracks */
	float CurrentDragXPosition;

	/** Cached list of anim tracks for notify node drag drop */
	TArray<TSharedPtr<SAnimNotifyTrack>> NotifyAnimTracks;

	/** Cached list of Notify editor tracks */
	TArray<TSharedPtr<SNotifyEdTrack>> NotifyEditorTracks;

	// Request a deferred RefreshNotifyTracks call
	void RequestRefresh();

	// this just refresh notify tracks - UI purpose only
	// do not call this from here. This gets called by asset. 
	void RefreshNotifyTracks();

	/** FEditorUndoClient interface */
	virtual void PostUndo( bool bSuccess ) override;
	virtual void PostRedo( bool bSuccess ) override;

	/** Handler for delete command */
	void OnDeletePressed();

	/** Deletes all currently selected notifies in the panel */
	void DeleteSelectedNodeObjects();

	/** We support keyboard focus to detect when we should process key commands like delete */
	virtual bool SupportsKeyboardFocus() const override
	{
		return true;
	}

	// Called when a track changes it's selection; iterates all tracks collecting selected items
	void OnTrackSelectionChanged();

	// Called to deselect all notifies across all tracks
	void DeselectAllNotifies();

	// Binds the UI commands for this widget to delegates
	void BindCommands();

	/** Populates the given class array with all classes deriving from those originally present
	 * @param InAssetsToSearch Assets to search to detect child classes
	 * @param InOutAllowedClassNames Classes to allow, this will be expanded to cover all derived classes of those originally present
	 */
	void PopulateNotifyBlueprintClasses(TArray<FString>& InOutAllowedClasses);

	/** Find blueprints matching allowed classes and all derived blueprints 
	 * @param OutNotifyData Asset data matching allowed classes and their children
	 * @param InOutAllowedClassNames Classes to allow, this will be expanded to cover all derived classes of those originally present
	 */
	void OnGetNotifyBlueprintData(TArray<FAssetData>& OutNotifyData, TArray<FString>* InOutAllowedClassNames);

	/** Find classes that inherit from NotifyOutermost and add correctly formatted class name to OutAllowedBlueprintClassNames
	 *  to allow us to find blueprints inherited from those types without loading the blueprints.
	 *	@param OutClasses Array of classes that inherit from NotifyOutermost
	 *	@param NotifyOutermost Outermost notify class to detect children of
	 *	@param OutAllowedBlueprintClassNames list of class names to add the native class names to
	 */
	void OnGetNativeNotifyData(TArray<UClass*>& OutClasses, UClass* NotifyOutermost, TArray<FString>* OutAllowedBlueprintClassNames);

	void OnNotifyTrackScrolled(float InScrollOffsetFraction);

	virtual void InputViewRangeChanged(float ViewMin, float ViewMax) override;

	/** Delegate used to snap when dragging */
	FOnSnapPosition OnSnapPosition;

	/** UI commands for this widget */
	TSharedPtr<FUICommandList> CommandList;
	
	/** Classes that are known to be derived from blueprint notifies */
	TArray<FString> NotifyClassNames;

	/** Classes that are known to be derived from blueprint state notifies */
	TArray<FString> NotifyStateClassNames;

	/** Handle to the registered OnPropertyChangedHandle delegate */
	FDelegateHandle OnPropertyChangedHandleDelegateHandle;

	/** Delegate used to invoke a tab */
	FOnInvokeTab OnInvokeTab;

	/** Delegate used to inform others that notifies have changed (for timing) */
	FSimpleDelegate OnNotifiesChanged;

	/** Delegate used to inform others that a notify state handle is being dragged */
	FOnNotifyStateHandleBeingDragged OnNotifyStateHandleBeingDragged;
	
	/** Delegate used to inform others that one or more notifies nodes are being dragged */
	FOnNotifyNodesBeingDragged OnNotifyNodesBeingDragged;

	/** Recursion guard for selection */
	bool bIsSelecting;

	/** Recursion guard for updating */
	bool bIsUpdating;

	/** Flags to handle deferred updates */
	bool bUpdateRequested;
	bool bRefreshRequested;
};

