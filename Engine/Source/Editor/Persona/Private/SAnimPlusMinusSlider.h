// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Layout/Margin.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Styling/AppStyle.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SSlider.h"

// Utility widget to set-up a common style for -/+ sliders in Persona
class SAnimPlusMinusSlider
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SAnimPlusMinusSlider)
		: _ExtraWidget()
		{}
		
		/** The label to use on the widget */
		SLATE_ARGUMENT (FText, Label)

		/** Called when the '-' button is clicked */
		SLATE_EVENT( FOnClicked, OnMinusClicked )
		SLATE_ARGUMENT( FText, MinusTooltip)

		/** The slider value to display */
		SLATE_ATTRIBUTE( float, SliderValue )
		/** Called when the value is changed by slider or typing */
		SLATE_EVENT( FOnFloatValueChanged, OnSliderValueChanged )
		/** Tooltip to use for slider control */
		SLATE_ARGUMENT( FText, SliderTooltip )

		/** Called when the '+' button is clicked */
		SLATE_EVENT( FOnClicked, OnPlusClicked )

		/** Tooltip to show on the plus button */
		SLATE_ARGUMENT( FText, PlusTooltip)

		/** Informational widget to display to the right of slider */
		SLATE_ARGUMENT( TSharedPtr<SWidget>, ExtraWidget )
	SLATE_END_ARGS()

	/** Constructs this widget from its declaration */
	void Construct(const FArguments& InArgs )
	{
		TSharedPtr<SHorizontalBox> SliderSlot;

		this->ChildSlot
		[
			SNew( SVerticalBox )
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding( FMargin(0.0f, 2.0f, 70.0f, 2.0f) )
			.HAlign( HAlign_Left )
			[
				SNew( STextBlock )
				.Text( InArgs._Label )
				.Font( FAppStyle::GetFontStyle( TEXT( "MenuItem.Font" ) ) )
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding( FMargin(0.0f, 4.0f) )
			[	
				SAssignNew(SliderSlot, SHorizontalBox )
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding( FMargin(0.0f, 0.0f, 1.0f, 0.0f) )
				[
					// '-' Button
					SNew(SButton)
					.Text(NSLOCTEXT("AnimationPlusMinusSlider", "Subtract", "-"))
					.TextStyle( FAppStyle::Get(), "ContentBrowser.NoneButtonText")
					.ButtonStyle( FAppStyle::Get(), "NoBorder" )
					.OnClicked( InArgs._OnMinusClicked )
					.ToolTipText( InArgs._MinusTooltip )
				]

				+SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.Padding( FMargin(0.0f, 2.0f) )
				[
					//Slider control
					SNew(SSlider)
					.Value( InArgs._SliderValue )
					.OnValueChanged( InArgs._OnSliderValueChanged )
					.ToolTipText( InArgs._SliderTooltip )
				]

				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding( FMargin(1.0f, 0.0f, 0.0f, 0.0f) )
				[
					// '+' Button
					SNew(SButton)
					.Text(NSLOCTEXT("AnimationPlusMinusSlider", "Add", "+"))
					.TextStyle( FAppStyle::Get(), "ContentBrowser.NoneButtonText" )
					.ButtonStyle( FAppStyle::Get(), "NoBorder" )
					.OnClicked( InArgs._OnPlusClicked )
					.ToolTipText( InArgs._PlusTooltip )
				]
			]
		];

		// Add in extra widget if provided
		if (InArgs._ExtraWidget.IsValid())
		{
			SliderSlot->AddSlot()
			.AutoWidth()
			.Padding( 5.0f, 6.0f, 0.0f, 2.0f)
			[
				InArgs._ExtraWidget->AsShared()
			];
		}
	}

};
