// Copyright Epic Games, Inc. All Rights Reserved.


#include "SAnimSegmentsPanel.h"
#include "Animation/Skeleton.h"
#include "Misc/MessageDialog.h"
#include "Modules/ModuleManager.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "AssetRegistry/AssetData.h"
#include "Animation/AnimSequenceBase.h"
#include "Animation/AnimSequence.h"

#include "Animation/AnimCompositeBase.h"

#include "ScopedTransaction.h"
#include "DragAndDrop/AssetDragDropOp.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "Framework/Application/SlateApplication.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Editor.h"
#include "Widgets/Layout/SBox.h"

#define LOCTEXT_NAMESPACE "AnimSegmentPanel"

//////////////////////////////////////////////////////////////////////////
// SAnimSegmentPanel

void SAnimSegmentsPanel::Construct(const FArguments& InArgs)
{
	bDragging = false;
	const int32 NumTracks = 2;

	AnimTrack = InArgs._AnimTrack;
	SlotName = InArgs._SlotName;
	ViewInputMin = InArgs._ViewInputMin;
	ViewInputMax = InArgs._ViewInputMax;

	OnAnimSegmentNodeClickedDelegate		= InArgs._OnAnimSegmentNodeClicked;
	OnAnimSegmentNodeDoubleClickedDelegate	= InArgs._OnAnimSegmentNodeDoubleClicked;
	OnPreAnimUpdateDelegate					= InArgs._OnPreAnimUpdate;
	OnPostAnimUpdateDelegate				= InArgs._OnPostAnimUpdate;
	OnAnimSegmentRemovedDelegate			= InArgs._OnAnimSegmentRemoved;
	OnAnimReplaceMapping					= InArgs._OnAnimReplaceMapping;
	OnDiffFromParentAsset					= InArgs._OnDiffFromParentAsset;
	OnGetNodeColor							= InArgs._OnGetNodeColor;
	OnIsAnimAssetValid						= InArgs._OnIsAnimAssetValid;

	bChildAnimMontage = InArgs._bChildAnimMontage;

	// Register and bind ui commands
	FAnimSegmentsPanelCommands::Register();
	BindCommands();

	// Empty out current widget array
	TrackWidgets.Empty();

	// Animation Segment tracks
	TArray<TSharedPtr<STrackNode>> AnimNodes;

	FLinearColor SelectedColor = FAppStyle::GetSlateColor("SelectionColor").GetSpecifiedColor();

	TSharedPtr<SVerticalBox> AnimSegmentTracks;

	ChildSlot
	[
		SAssignNew(AnimSegmentTracks, SVerticalBox)
	];

	for (int32 TrackIdx=0; TrackIdx < NumTracks; TrackIdx++)
	{
		TSharedPtr<STrack> AnimSegmentTrack;

		if (bChildAnimMontage)
		{
			AnimSegmentTracks->AddSlot()
				.AutoHeight()
				.VAlign(VAlign_Center)
				.Padding(0.0f)
				[
					SAssignNew(AnimSegmentTrack, STrack)
					.ViewInputMin(ViewInputMin)
					.ViewInputMax(ViewInputMax)
					.TrackMaxValue(InArgs._TrackMaxValue)
					.TrackNumDiscreteValues(InArgs._TrackNumDiscreteValues)
					.OnTrackRightClickContextMenu(InArgs._OnTrackRightClickContextMenu)
					.OnTrackDragDrop(this, &SAnimSegmentsPanel::OnTrackDragDrop)
					.OnAssetDragDrop(this, &SAnimSegmentsPanel::OnAssetDragDrop)
				];
		}
		else
		{
			AnimSegmentTracks->AddSlot()
				.AutoHeight()
				.VAlign(VAlign_Center)
				.Padding(0.0f)
				[
					SAssignNew(AnimSegmentTrack, STrack)
					.ViewInputMin(ViewInputMin)
					.ViewInputMax(ViewInputMax)
					.TrackMaxValue(InArgs._TrackMaxValue)
					//section stuff
					.OnBarDrag(InArgs._OnBarDrag)
					.OnBarDrop(InArgs._OnBarDrop)
					.OnBarClicked(InArgs._OnBarClicked)
					.DraggableBars(InArgs._DraggableBars)
					.DraggableBarSnapPositions(InArgs._DraggableBarSnapPositions)
					.TrackNumDiscreteValues(InArgs._TrackNumDiscreteValues)
					.OnTrackRightClickContextMenu(InArgs._OnTrackRightClickContextMenu)
					.OnTrackDragDrop(this, &SAnimSegmentsPanel::OnTrackDragDrop)
					.OnAssetDragDrop(this, &SAnimSegmentsPanel::OnAssetDragDrop)
				];
		}

		TrackWidgets.Add(AnimSegmentTrack);
	}

	// Generate Nodes and map them to tracks
	for ( int32 SegmentIdx=0; SegmentIdx < AnimTrack->AnimSegments.Num(); SegmentIdx++ )
	{
		if (bChildAnimMontage)
		{
			TrackWidgets[SegmentIdx % TrackWidgets.Num()]->AddTrackNode(
				SNew(STrackNode)
				.ViewInputMax(this->ViewInputMax)
				.ViewInputMin(this->ViewInputMin)
				.NodeColor(this, &SAnimSegmentsPanel::GetNodeColor, SegmentIdx)
				.SelectedNodeColor(SelectedColor)
				.DataLength(this, &SAnimSegmentsPanel::GetSegmentLength, SegmentIdx)
				.DataStartPos(this, &SAnimSegmentsPanel::GetSegmentStartPos, SegmentIdx)
				.NodeName(this, &SAnimSegmentsPanel::GetAnimSegmentName, SegmentIdx)
				.ToolTipText(this, &SAnimSegmentsPanel::GetAnimSegmentDetailedInfo, SegmentIdx)
				.OnTrackNodeDropped(this, &SAnimSegmentsPanel::OnSegmentDropped, SegmentIdx)
				.OnNodeRightClickContextMenu(this, &SAnimSegmentsPanel::SummonSegmentNodeContextMenu, SegmentIdx)
				.NodeSelectionSet(InArgs._NodeSelectionSet)
			);
		}
		else
		{
			TrackWidgets[SegmentIdx % TrackWidgets.Num()]->AddTrackNode(
				SNew(STrackNode)
				.ViewInputMax(this->ViewInputMax)
				.ViewInputMin(this->ViewInputMin)
				.NodeColor(this, &SAnimSegmentsPanel::GetNodeColor, SegmentIdx)
				.SelectedNodeColor(SelectedColor)
				.DataLength(this, &SAnimSegmentsPanel::GetSegmentLength, SegmentIdx)
				.DataStartPos(this, &SAnimSegmentsPanel::GetSegmentStartPos, SegmentIdx)
				.NodeName(this, &SAnimSegmentsPanel::GetAnimSegmentName, SegmentIdx)
				.ToolTipText(this, &SAnimSegmentsPanel::GetAnimSegmentDetailedInfo, SegmentIdx)
				.OnTrackNodeDragged(this, &SAnimSegmentsPanel::SetSegmentStartPos, SegmentIdx)
				.OnTrackNodeDropped(this, &SAnimSegmentsPanel::OnSegmentDropped, SegmentIdx)
				.OnNodeRightClickContextMenu(this, &SAnimSegmentsPanel::SummonSegmentNodeContextMenu, SegmentIdx)
				.OnTrackNodeClicked(this, &SAnimSegmentsPanel::OnAnimSegmentNodeClicked, SegmentIdx)
				.OnTrackNodeDoubleClicked(this, &SAnimSegmentsPanel::OnAnimSegmentNodeDoubleClicked, SegmentIdx)
				.NodeSelectionSet(InArgs._NodeSelectionSet)
			);
		}
	}
}

bool SAnimSegmentsPanel::ValidIndex(int32 AnimSegmentIndex) const
{
	return (AnimTrack && AnimTrack->AnimSegments.IsValidIndex(AnimSegmentIndex));
}

FLinearColor	SAnimSegmentsPanel::GetNodeColor(int32 AnimSegmentIndex) const
{
	static const FLinearColor DisabledColor(64, 64, 64);
	static const FLinearColor ChildMontageColor(128, 255, 0);

	if (ValidIndex(AnimSegmentIndex) && AnimTrack->AnimSegments[AnimSegmentIndex].IsValid())
	{
		bool bUseModifiedChildColor = bChildAnimMontage && OnDiffFromParentAsset.IsBound()
			&& OnDiffFromParentAsset.Execute(SlotName, AnimSegmentIndex, AnimTrack->AnimSegments[AnimSegmentIndex]);

		if (bUseModifiedChildColor)
		{
			return ChildMontageColor;
		}
		else if(OnGetNodeColor.IsBound())
		{
			return OnGetNodeColor.Execute(AnimTrack->AnimSegments[AnimSegmentIndex]);
		}
	}

	return DisabledColor;
}

float SAnimSegmentsPanel::GetSegmentLength(int32 AnimSegmentIndex) const
{
	if (ValidIndex(AnimSegmentIndex))
	{
		return AnimTrack->AnimSegments[AnimSegmentIndex].GetLength();
	}
	return 0.f;
}

float SAnimSegmentsPanel::GetSegmentStartPos(int32 AnimSegmentIndex) const
{
	if (ValidIndex(AnimSegmentIndex))
	{
		return AnimTrack->AnimSegments[AnimSegmentIndex].StartPos;
	}
	return 0.f;
}

FString	SAnimSegmentsPanel::GetAnimSegmentName(int32 AnimSegmentIndex) const
{
	if (ValidIndex(AnimSegmentIndex))
	{
		FString TitleLabel;
		if(const UAnimSequenceBase* AnimReference = AnimTrack->AnimSegments[AnimSegmentIndex].GetAnimReference())
		{
			FString AssetName = AnimReference->GetName();
			if (AnimTrack->AnimSegments[AnimSegmentIndex].IsValid() == false)
			{
				TitleLabel = FString::Printf(TEXT("Error : %s"), *AssetName);
			}
			else if (bChildAnimMontage)
			{
				TitleLabel = FString::Printf(TEXT("Child : %s"), *AssetName);
			}
			else
			{
				TitleLabel = AssetName;
			}
		}
		else
		{
			TitleLabel = LOCTEXT("UnboundAnimSequence", "Unbound Animation Sequence").ToString();
		}

		return TitleLabel;
	}
	return FString();
}

FText SAnimSegmentsPanel::GetAnimSegmentDetailedInfo(int32 AnimSegmentIndex) const
{
	if (ValidIndex(AnimSegmentIndex))
	{
		FAnimSegment& AnimSegment = AnimTrack->AnimSegments[AnimSegmentIndex];
		if (const UAnimSequenceBase* AnimReference = AnimTrack->AnimSegments[AnimSegmentIndex].GetAnimReference())
		{
			static const FNumberFormattingOptions FormatOptions = FNumberFormattingOptions()
				.SetMinimumFractionalDigits(2)
				.SetMaximumFractionalDigits(2);

			if (AnimTrack->AnimSegments[AnimSegmentIndex].IsValid())
			{
				return FText::Format(LOCTEXT("AnimSegmentPanel_GetAnimSegmentDetailedInfoFmt", "{0} {1} {2}"), FText::FromString(AnimReference->GetName()), FText::AsNumber(AnimSegment.GetLength(), &FormatOptions), 
					AnimTrack->AnimSegments[AnimSegmentIndex].IsPlayLengthOutOfDate() ? LOCTEXT("AnimSegmentPanel_GetAnimSegmentDetailedInfoFmt_Warning_PlayTimeIncorrect", "(segment length does not match animation play length)") : FText::FromString(FString()));
			}
			else
			{
				return FText::Format(LOCTEXT("AnimSegmentPanel_GetAnimSegmentDetailedInfoFmt_Error_RecursiveReference", "{0} {1} - ERROR: Recursive Reference Found"), FText::FromString(AnimReference->GetName()), FText::AsNumber(AnimSegment.GetLength(), &FormatOptions));  
			}
			
		}
	}
	return FText::GetEmpty();
}

void SAnimSegmentsPanel::SetSegmentStartPos(float NewStartPos, int32 AnimSegmentIndex)
{
	if (ValidIndex(AnimSegmentIndex))
	{
		if(!bDragging)
		{
			const FScopedTransaction Transaction( LOCTEXT("AnimSegmentPanel_SetSegmentStart", "Edit Segment Start Time") );
			OnPreAnimUpdateDelegate.ExecuteIfBound();
			bDragging = true;
		}

		AnimTrack->AnimSegments[AnimSegmentIndex].StartPos = NewStartPos;
		AnimTrack->CollapseAnimSegments();
	}
}

void SAnimSegmentsPanel::OnSegmentDropped(int32 AnimSegmentIndex)
{
	if(bDragging)
	{
		bDragging = false;
		OnPostAnimUpdateDelegate.ExecuteIfBound();
	}
}

void SAnimSegmentsPanel::SummonSegmentNodeContextMenu(FMenuBuilder& MenuBuilder, int32 AnimSegmentIndex)
{
	FUIAction UIAction;

	if (bChildAnimMontage)
	{
		MenuBuilder.BeginSection("AnimSegmentsLabel", LOCTEXT("Anim Segment", "Anim Segment"));
		{
			// if different than parent
			UIAction.ExecuteAction.BindRaw(this, &SAnimSegmentsPanel::RevertToParent, AnimSegmentIndex);
			MenuBuilder.AddMenuEntry(LOCTEXT("RevertToParentSegment", "Revert To Parent"), LOCTEXT("RevertToParentSegment_ToolTip", "Revert to Parent Animation"), FSlateIcon(), UIAction);
			MenuBuilder.AddSubMenu(LOCTEXT("PickAnimationForTheSegment", "Replace animation with..."), LOCTEXT("PickAnimationForTheSegment_TooTip", "Replace the current animation with another animation."), FNewMenuDelegate::CreateSP(this, &SAnimSegmentsPanel::FillSubMenu, AnimSegmentIndex));

			MenuBuilder.AddMenuSeparator();
			// open asset option
			UIAction.ExecuteAction.BindRaw(this, &SAnimSegmentsPanel::OpenAsset, AnimSegmentIndex);
			MenuBuilder.AddMenuEntry(LOCTEXT("OpenAssetOfSegment", "Open Asset"), LOCTEXT("OpenAssetOfSegment_ToolTip", "Open Asset"), FSlateIcon(), UIAction);
		}

		MenuBuilder.EndSection();
	}
	else
	{
		MenuBuilder.BeginSection("AnimSegmentsLabel", LOCTEXT("Anim Segment", "Anim Segment"));
		{
			UIAction.ExecuteAction.BindRaw(this, &SAnimSegmentsPanel::RemoveAnimSegment, AnimSegmentIndex);
			MenuBuilder.AddMenuEntry(LOCTEXT("DeleteSegment", "Delete Segment"), LOCTEXT("DeleteSegmentHint", "Delete Segment"), FSlateIcon(), UIAction);
			// open asset option
			UIAction.ExecuteAction.BindRaw(this, &SAnimSegmentsPanel::OpenAsset, AnimSegmentIndex);
			MenuBuilder.AddMenuEntry(LOCTEXT("OpenAssetOfSegment", "Open Asset"), LOCTEXT("OpenAssetOfSegment_ToolTip", "Open Asset"), FSlateIcon(), UIAction);
			
			// Reset segment start position/length to (0.f/LengthOfAnimationSequence)
			UIAction.ExecuteAction.BindRaw(this, &SAnimSegmentsPanel::ResetSegmentLength, AnimSegmentIndex);
			MenuBuilder.AddMenuEntry(LOCTEXT("ResetSegmentLength", "Reset Segment Length"), LOCTEXT("ResetSegment_ToolTip", "Resets Segment its StartOffset and Length to match the referenced Animation"), FSlateIcon(), UIAction);			
		}
		MenuBuilder.EndSection();
	}
}

void SAnimSegmentsPanel::AddAnimSegment( UAnimSequenceBase* NewSequenceBase, float NewStartPos )
{
	const FScopedTransaction Transaction( LOCTEXT("AnimSegmentPanel_AddSegment", "Add Segment") );
	OnPreAnimUpdateDelegate.ExecuteIfBound();

	FAnimSegment NewSegment;
	NewSegment.SetAnimReference(NewSequenceBase, true);

	NewSegment.StartPos = NewStartPos;

	AnimTrack->AnimSegments.Add(NewSegment);
	OnPostAnimUpdateDelegate.ExecuteIfBound();
}

void SAnimSegmentsPanel::ReplaceAnimSegment(int32 AnimSegmentIndex, UAnimSequenceBase* NewSequenceBase)
{
	const FScopedTransaction Transaction(LOCTEXT("AnimSegmentPanel_ReplaceSegment", "Replace Segment"));
	if (AnimTrack->AnimSegments.IsValidIndex(AnimSegmentIndex))
	{
		UAnimSequenceBase* OldSequenceBase = AnimTrack->AnimSegments[AnimSegmentIndex].GetAnimReference();
		if (OldSequenceBase != NewSequenceBase)
		{
			OnPreAnimUpdateDelegate.ExecuteIfBound();
			OnAnimReplaceMapping.ExecuteIfBound(SlotName, AnimSegmentIndex, OldSequenceBase, NewSequenceBase);
			OnPostAnimUpdateDelegate.ExecuteIfBound();
		}
	}

	// it doesn't work well if I leave the window open. The delegate goes weired or it stop showing the popups. 
	FSlateApplication::Get().DismissAllMenus();
}

void SAnimSegmentsPanel::ReplaceAnimSegment(const FAssetData& NewSequenceData, int32 AnimSegmentIndex)
{
	UAnimSequenceBase* NewSequenceBase = Cast<UAnimSequenceBase> (NewSequenceData.GetAsset());
	if (NewSequenceBase)
	{
		ReplaceAnimSegment(AnimSegmentIndex, NewSequenceBase);
	}
}

void SAnimSegmentsPanel::ReplaceAnimSegment(UAnimSequenceBase* NewSequenceBase, float NewStartPos)
{
	int32 SegmentIdx = AnimTrack->GetSegmentIndexAtTime(NewStartPos);
	ReplaceAnimSegment(SegmentIdx, NewSequenceBase);
}

bool SAnimSegmentsPanel::IsValidToAdd(UAnimSequenceBase* NewSequenceBase, FText* OutReason /*= nullptr*/) const
{
	if (AnimTrack == nullptr || NewSequenceBase == nullptr)
	{
		return false;
	}

	if (!AnimTrack->IsValidToAdd(NewSequenceBase, OutReason) || (OnIsAnimAssetValid.IsBound() && !OnIsAnimAssetValid.Execute(NewSequenceBase, OutReason)))
    {
        return false;
    } 

	return true;
}

void SAnimSegmentsPanel::RemoveAnimSegment(int32 AnimSegmentIndex)
{
	if(ValidIndex(AnimSegmentIndex))
	{
		const FScopedTransaction Transaction( LOCTEXT("RemoveAnimSegmentTransaction", "Remove Segment") );
		OnPreAnimUpdateDelegate.ExecuteIfBound();

		AnimTrack->AnimSegments.RemoveAt(AnimSegmentIndex);

		OnAnimSegmentRemovedDelegate.ExecuteIfBound(AnimSegmentIndex);
		OnPostAnimUpdateDelegate.ExecuteIfBound();
	}
}

void SAnimSegmentsPanel::RevertToParent(int32 AnimSegmentIndex)
{
	ReplaceAnimSegment(AnimSegmentIndex, nullptr);
}

void SAnimSegmentsPanel::OpenAsset(int32 AnimSegmentIndex)
{
	if (ValidIndex(AnimSegmentIndex))
	{
		if (UAnimSequenceBase* Asset = AnimTrack->AnimSegments[AnimSegmentIndex].GetAnimReference())
		{
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(Asset);
		}
	}
}

void SAnimSegmentsPanel::ResetSegmentLength(int32 AnimSegmentIndex)
{
	if (ValidIndex(AnimSegmentIndex))
	{
		if (UAnimSequenceBase* Asset = AnimTrack->AnimSegments[AnimSegmentIndex].GetAnimReference())
		{
			const FScopedTransaction Transaction( LOCTEXT("ResetAnimSegmentTransaction", "Reset Segment Length") );
			OnPreAnimUpdateDelegate.ExecuteIfBound();
			{
				AnimTrack->AnimSegments[AnimSegmentIndex].AnimStartTime = 0.f;
				AnimTrack->AnimSegments[AnimSegmentIndex].AnimEndTime = Asset->GetPlayLength();
			}
			OnPostAnimUpdateDelegate.ExecuteIfBound();
		}
	}
}

void SAnimSegmentsPanel::FillSubMenu(FMenuBuilder& MenuBuilder, int32 AnimSegmentIndex)
{
	if (ValidIndex(AnimSegmentIndex))
	{
		UAnimSequenceBase* OldSequenceBase = AnimTrack->AnimSegments[AnimSegmentIndex].GetAnimReference();

		if (ensureAlways(OldSequenceBase))
		{
			FAssetPickerConfig AssetPickerConfig;

			/** The asset picker will only show skeletons */
			AssetPickerConfig.Filter.ClassPaths.Add(OldSequenceBase->GetClass()->GetClassPathName());
			AssetPickerConfig.Filter.bRecursiveClasses = false;
			AssetPickerConfig.bAllowNullSelection = false;

			// only do this for anim sequence because we don't know additive or not otherwise from asset registry
			TEnumAsByte<EAdditiveAnimationType> AdditiveType;
			bool bFilterAdditive = OldSequenceBase->GetClass() == UAnimSequence::StaticClass();
			if(bFilterAdditive)
			{
				AdditiveType = CastChecked<UAnimSequence>(OldSequenceBase)->AdditiveAnimType;
			}

			AssetPickerConfig.OnShouldFilterAsset = FOnShouldFilterAsset::CreateRaw(this, &SAnimSegmentsPanel::ShouldFilter, OldSequenceBase->GetSkeleton(), bFilterAdditive, AdditiveType);

			/** The delegate that fires when an asset was selected */
			AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateRaw(this, &SAnimSegmentsPanel::ReplaceAnimSegment, AnimSegmentIndex);

			/** The default view mode should be a list view */
			AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;

			FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

			MenuBuilder.AddWidget(
				SNew(SBox)
				.MaxDesiredHeight(600.0f)
				[
					ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
				], 
				FText(),
				true
			);
		}
	}
}

bool SAnimSegmentsPanel::ShouldFilter(const FAssetData& DataToDisplay, USkeleton* InSkeleton, bool bInFilterAdditive, TEnumAsByte<EAdditiveAnimationType> InAdditiveType)
{
	bool bFilter = false;
	if(bInFilterAdditive)
	{
		UEnum* AdditiveTypeEnum = StaticEnum<EAdditiveAnimationType>();
		const FString EnumString = DataToDisplay.GetTagValueRef<FString>(GET_MEMBER_NAME_CHECKED(UAnimSequence, AdditiveAnimType));
		EAdditiveAnimationType AdditiveType = (!EnumString.IsEmpty() ? (EAdditiveAnimationType)AdditiveTypeEnum->GetValueByName(*EnumString) : AAT_None);
		bFilter |= (AdditiveType != InAdditiveType);
	}

	bFilter |= InSkeleton->ShouldFilterAsset(DataToDisplay);

	return bFilter;
}

void SAnimSegmentsPanel::OnTrackDragDrop( TSharedPtr<FDragDropOperation> DragDropOp, float DataPos )
{
	if (DragDropOp.IsValid() && DragDropOp->IsOfType<FAssetDragDropOp>())
	{
		TSharedPtr<FAssetDragDropOp> AssetOp = StaticCastSharedPtr<FAssetDragDropOp>(DragDropOp);
		if (AssetOp->HasAssets())
		{
			bool bFailedToAdd = false; 
		
			if(bChildAnimMontage)
			{
				UAnimSequenceBase* DroppedSequence = FAssetData::GetFirstAsset<UAnimSequenceBase>(AssetOp->GetAssets());
				if (IsValidToAdd(DroppedSequence))
				{
					ReplaceAnimSegment(DroppedSequence, DataPos);
				}
				else
				{
					bFailedToAdd = true;
				}
			}
			else
			{
				const FScopedTransaction Transaction( LOCTEXT("AnimSegmentPanel_AddSegments", "Add Segments") );
			
				for(const FAssetData& DroppedAssetData : AssetOp->GetAssets())
				{
					UAnimSequenceBase* DroppedSequence = Cast<UAnimSequenceBase>(DroppedAssetData.GetAsset());
					if (IsValidToAdd(DroppedSequence))
					{
						AddAnimSegment(DroppedSequence, DataPos);
						DataPos += DroppedSequence->GetPlayLength();
					}
					else
					{
						bFailedToAdd = true;
					}
				}
			}
		}
	}
}

bool SAnimSegmentsPanel::OnAssetDragDrop(TSharedPtr<FAssetDragDropOp> AssetDragDropOp)
{
	if (AssetDragDropOp.IsValid())
	{
		if (AssetDragDropOp->HasAssets())
		{			
			for(const FAssetData& DroppedAssetData : AssetDragDropOp->GetAssets())
			{
				FText FailureReason;
				UAnimSequenceBase* DroppedSequence = Cast<UAnimSequenceBase>(DroppedAssetData.GetAsset());

				bool bInvalidAsset = false;
				if (!IsValidToAdd(DroppedSequence, &FailureReason))
				{
					AssetDragDropOp->SetToolTip(FailureReason, FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error")));
					return false;
				}
			}
			
			AssetDragDropOp->SetToolTip(FText(), FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK")));
			return true;
		}
	}

	return false;
}

void SAnimSegmentsPanel::OnAnimSegmentNodeClicked(int32 SegmentIdx)
{
	OnAnimSegmentNodeClickedDelegate.ExecuteIfBound(SegmentIdx);
}

void SAnimSegmentsPanel::OnAnimSegmentNodeDoubleClicked(int32 SegmentIdx)
{
	if (OnAnimSegmentNodeDoubleClickedDelegate.IsBound())
	{
		OnAnimSegmentNodeDoubleClickedDelegate.ExecuteIfBound(SegmentIdx);
	}
	else
	{
		OpenAsset(SegmentIdx);
	}
}

void SAnimSegmentsPanel::RemoveSelectedAnimSegments()
{
	TArray<int32> SelectedNodeIndices;
	for(int i = 0 ; i < TrackWidgets.Num() ; ++i)
	{
		TSharedPtr<STrack> Track = TrackWidgets[i];
		Track->GetSelectedNodeIndices(SelectedNodeIndices);

		// Reverse order to preserve indices
		for(int32 j = SelectedNodeIndices.Num() - 1 ; j >= 0 ; --j)
		{
			// Segments are placed on one of two tracks with the first segment always residing
			// in track 0 - need to modify the index from track and index to data index.
			int32 ModifiedIndex = i + 2 * SelectedNodeIndices[j];
			RemoveAnimSegment(ModifiedIndex);
		}
	}
}

void SAnimSegmentsPanel::BindCommands()
{
	check(!UICommandList.IsValid());

	UICommandList = MakeShareable(new FUICommandList);
	const FAnimSegmentsPanelCommands& Commands = FAnimSegmentsPanelCommands::Get();

	// do not allow to delete if child anim montage
	if (!bChildAnimMontage)
	{
		UICommandList->MapAction(
			Commands.DeleteSegment,
			FExecuteAction::CreateSP(this, &SAnimSegmentsPanel::RemoveSelectedAnimSegments));
	}
}

FReply SAnimSegmentsPanel::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if(UICommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

void FAnimSegmentsPanelCommands::RegisterCommands()
{
	// this is here for key handling
	UI_COMMAND(DeleteSegment, "Delete", "Deletes the selected segment", EUserInterfaceActionType::Button, FInputChord(EKeys::Platform_Delete));
}

#undef LOCTEXT_NAMESPACE
