// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Widgets/SCompoundWidget.h"
#include "Framework/Commands/Commands.h"
#include "Animation/AnimTypes.h"
#include "Styling/AppStyle.h"
#include "STrack.h"
#include "SAnimTrackPanel.h"

struct FAssetData;
class FMenuBuilder;
class UAnimSequenceBase;
struct FAnimSegment;
class USkeleton;

DECLARE_DELEGATE( FOnPreAnimUpdate )
DECLARE_DELEGATE( FOnPostAnimUpdate )
DECLARE_DELEGATE_OneParam( FOnAnimSegmentNodeClicked, int32 )
DECLARE_DELEGATE_OneParam( FOnAnimSegmentNodeDoubleClicked, int32)
DECLARE_DELEGATE_OneParam( FOnAnimSegmentRemoved, int32 )
DECLARE_DELEGATE_FourParams( FOnAnimReplaceMapping, FName, int32, UAnimSequenceBase*, UAnimSequenceBase*)
DECLARE_DELEGATE_RetVal_ThreeParams(bool, FOnDiffFromParentAsset, FName, int32, const FAnimSegment& )
DECLARE_DELEGATE_RetVal_OneParam(FLinearColor, FOnGetNodeColor, const FAnimSegment&)

//////////////////////////////////////////////////////////////////////////
// SAnimSegmentsPanel

// UI Commands
class FAnimSegmentsPanelCommands : public TCommands < FAnimSegmentsPanelCommands >
{
public:
	FAnimSegmentsPanelCommands()
		: TCommands<FAnimSegmentsPanelCommands>("AnimMontagePanel", NSLOCTEXT("Contexts", "AnimMontagePanel", "Anim Montage Panel"), NAME_None, FAppStyle::GetAppStyleSetName())
	{

	}

	TSharedPtr<FUICommandInfo> DeleteSegment;

	virtual void RegisterCommands() override;
};

// TODO: comment about segment panel
class SAnimSegmentsPanel: public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SAnimSegmentsPanel )
		: _AnimTrack(NULL)
		, _ViewInputMin()
		, _ViewInputMax()
		, _bChildAnimMontage(false)
	{}

	SLATE_ARGUMENT(struct FAnimTrack*, AnimTrack)
	SLATE_ARGUMENT(FName, SlotName)
	SLATE_ARGUMENT(STrackNodeSelectionSet *, NodeSelectionSet)
	SLATE_ARGUMENT(TSharedPtr<FTrackColorTracker>, ColorTracker)

	SLATE_ATTRIBUTE(float, ViewInputMin)
	SLATE_ATTRIBUTE(float, ViewInputMax)

	SLATE_EVENT(FOnGetNodeColor, OnGetNodeColor)
	SLATE_ATTRIBUTE(TArray<float>, DraggableBars)
	SLATE_ATTRIBUTE(TArray<float>, DraggableBarSnapPositions)
	SLATE_ATTRIBUTE(float, ScrubPosition)
	SLATE_ATTRIBUTE(float, TrackMaxValue)
	SLATE_ATTRIBUTE(int32, TrackNumDiscreteValues)

	SLATE_EVENT(FOnAnimSegmentNodeClicked, OnAnimSegmentNodeClicked)
	SLATE_EVENT(FOnAnimSegmentNodeDoubleClicked, OnAnimSegmentNodeDoubleClicked)
	SLATE_EVENT(FOnPreAnimUpdate, OnPreAnimUpdate)
	SLATE_EVENT(FOnPostAnimUpdate, OnPostAnimUpdate)
	SLATE_EVENT(FOnAnimSegmentRemoved, OnAnimSegmentRemoved)
	SLATE_EVENT(FOnAnimReplaceMapping, OnAnimReplaceMapping)
	SLATE_EVENT(FOnDiffFromParentAsset, OnDiffFromParentAsset)
	SLATE_EVENT(FIsAnimAssetValid, OnIsAnimAssetValid)

	SLATE_EVENT( FOnBarDrag,				OnBarDrag)
	SLATE_EVENT( FOnBarDrop,				OnBarDrop)
	SLATE_EVENT( FOnBarClicked,				OnBarClicked)

	SLATE_EVENT( FOnTrackRightClickContextMenu,	OnTrackRightClickContextMenu)

	SLATE_ARGUMENT(bool, bChildAnimMontage)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	// We support keyboard focus to receive UI key events
	bool SupportsKeyboardFocus() const override {return true;}

	// Key handler
	FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

private:

	FOnPreAnimUpdate				OnPreAnimUpdateDelegate;
	FOnPostAnimUpdate				OnPostAnimUpdateDelegate;
	FOnAnimSegmentNodeClicked		OnAnimSegmentNodeClickedDelegate;
	FOnAnimSegmentNodeDoubleClicked OnAnimSegmentNodeDoubleClickedDelegate;
	FOnAnimSegmentRemoved			OnAnimSegmentRemovedDelegate;
	FOnAnimReplaceMapping			OnAnimReplaceMapping;
	FOnDiffFromParentAsset			OnDiffFromParentAsset;
	FOnGetNodeColor					OnGetNodeColor;

	enum ETrackViewStyle
	{
		TRACK_Single,
		TRACK_Double,
		TRACK_ExpandAll
	};

	bool				ValidIndex(int32 AnimSegmentIndex) const;
	FLinearColor		GetNodeColor(int32 AnimSegmentIndex) const;
	float				GetSegmentLength(int32 AnimSegmentIndex) const;
	float				GetSegmentStartPos(int32 AnimSegmentIndex) const;
	FString				GetAnimSegmentName(int32 AnimSegmentIndex) const;
	FText				GetAnimSegmentDetailedInfo(int32 AnimSegmentIndex) const;

	void				SetSegmentStartPos(float NewStartPos, int32 AnimSegmentIndex);
	void				OnSegmentDropped(int32 AnimSegmentIndex);

	void				ShowSegmentInDetailsView(int32 AnimSlotIndex, int32 AnimSegmentIndex);

	void				SummonSegmentNodeContextMenu( FMenuBuilder& MenuBuilder, int32 AnimSegmentIndex );

	void				AddAnimSegment(UAnimSequenceBase *NewSequenceBase, float NewStartPos );
	bool				IsValidToAdd(UAnimSequenceBase* NewSequenceBase, FText* OutReason = nullptr) const;
	void				OnTrackDragDrop( TSharedPtr<FDragDropOperation> DragDropOp, float DataPos );
	bool				OnAssetDragDrop(TSharedPtr<FAssetDragDropOp> AssetDragDropOp);
	void				OnAnimSegmentNodeClicked(int32 SegmentIdx);
	void				OnAnimSegmentNodeDoubleClicked(int32 SegmentIdx);

	// child anim montage
	void				ReplaceAnimSegment(UAnimSequenceBase* NewSequenceBase, float NewStartPos);
	void				ReplaceAnimSegment(int32 AnimSegmentIndex, UAnimSequenceBase* NewSequenceBase);
	void				ReplaceAnimSegment(const FAssetData& NewSequenceData, int32 AnimSegmentIndex);
	bool				ShouldFilter(const FAssetData& DataToDisplay, USkeleton* InSkeleton, bool bInFilterAdditive, TEnumAsByte<EAdditiveAnimationType> InAdditiveType);

	// Remove all selected anim segments in all segment tracks
	void RemoveSelectedAnimSegments();

	// Remove the specified anim segment
	void RemoveAnimSegment(int32 AnimSegmentIndex);
	void RevertToParent(int32 AnimSegmentIndex);
	void OpenAsset(int32 AnimSegmentIndex);
	void ResetSegmentLength(int32 AnimSegmentIndex);
	void FillSubMenu(FMenuBuilder& MenuBuilder, int32 AnimSegmentIndex);

	/** Bind UI commands for this widget */
	void				BindCommands();

	ETrackViewStyle		TrackStyle;

	TAttribute<float>	ViewInputMin;
	TAttribute<float>	ViewInputMax;

	/** Anim Track */
	struct FAnimTrack*	AnimTrack;
	/** Slot name, used when bChildAnimMontage == true */
	FName SlotName;

	bool bDragging;

	/** List of UI commands for this widget */
	TSharedPtr<FUICommandList> UICommandList;

	/** List of widgets representing tracks */
	TArray<TSharedPtr<STrack>> TrackWidgets;

	TAttribute<FLinearColor> DefaultNodeColor;

	FIsAnimAssetValid OnIsAnimAssetValid;
 	/* 
	 * Child Anim Montage: Child Anim Montage only can replace name of animations, and no other meaningful edits 
	 * as it will derive every data from Parent. There might be some other data that will allow to be replaced, but for now, it is
	 * not. 
	 */
	bool bChildAnimMontage;
};
