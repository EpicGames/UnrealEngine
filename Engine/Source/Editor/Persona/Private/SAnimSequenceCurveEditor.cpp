// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAnimSequenceCurveEditor.h"
#include "AnimatedRange.h"
#include "CurveEditor.h"
#include "RichCurveEditorModel.h"
#include "Animation/AnimSequenceBase.h"
#include "SCurveEditorPanel.h"
#include "Tree/SCurveEditorTreeTextFilter.h"
#include "Tree/SCurveEditorTreeFilterStatusBar.h"
#include "Tree/SCurveEditorTree.h"
#include "Tree/SCurveEditorTreeSelect.h"
#include "Tree/SCurveEditorTreePin.h"
#include "Widgets/Layout/SScrollBorder.h"
#include "Tree/ICurveEditorTreeItem.h"
#include "AnimTimeline/SAnimTimelineTransportControls.h"
#include "Tree/CurveEditorTreeFilter.h"
#include "Editor.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Styling/AppStyle.h"
#include "Animation/AnimData/IAnimationDataModel.h"
#include "ToolMenus.h"

#define LOCTEXT_NAMESPACE "SAnimSequenceCurveEditor"

FRichCurveEditorModelNamed::FRichCurveEditorModelNamed(const FName& InName, ERawCurveTrackTypes InType, int32 InCurveIndex, UAnimSequenceBase* InAnimSequence, FCurveEditorTreeItemID InTreeId /*= FCurveEditorTreeItemID()*/)
: FRichCurveEditorModel(InAnimSequence)
, CurveName(InName)
, AnimSequence(InAnimSequence)
, CurveIndex(InCurveIndex)
, Type(InType)
, TreeId(InTreeId)
, CurveId(FAnimationCurveIdentifier(InName, Type))
, bCurveRemoved(false)
{
	CurveModifiedDelegate.AddRaw(this, &FRichCurveEditorModelNamed::CurveHasChanged);

	InAnimSequence->GetDataModel()->GetModifiedEvent().AddRaw(this, &FRichCurveEditorModelNamed::OnModelHasChanged);

	if (Type == ERawCurveTrackTypes::RCT_Transform)
	{
		UAnimationCurveIdentifierExtensions::GetTransformChildCurveIdentifier(CurveId, (ETransformCurveChannel)(CurveIndex / 3), (EVectorCurveChannel)(CurveIndex % 3));

		const UEnum* TransformChannelEnum = StaticEnum<ETransformCurveChannel>();
		const UEnum* VectorChannelEnum = StaticEnum<EVectorCurveChannel>();
		check(TransformChannelEnum && VectorChannelEnum);

		const FString ChannelString = TransformChannelEnum->GetAuthoredNameStringByValue((int64)CurveId.Channel);
		const FString AxisString = VectorChannelEnum->GetAuthoredNameStringByValue((int64)CurveId.Axis);
		const FString ChannelAxisName = ChannelString + TEXT(".") + AxisString;
		const FString LongName = FString::Printf(TEXT("%s.%s"), *CurveName.ToString(), *ChannelAxisName);

		SetShortDisplayName(FText::FromString(AxisString));
		SetLongDisplayName(FText::FromString(LongName));
		SetIntentionName(ChannelAxisName);
		SetLongIntentionName(LongName);
		SetChannelName(FName(*ChannelAxisName));
	}
	else
	{
		const FText DisplayText = FText::FromName(CurveName);
		SetShortDisplayName(DisplayText);
		SetLongDisplayName(DisplayText);
		SetIntentionName(CurveName.ToString());
		SetLongIntentionName(CurveName.ToString());
		SetChannelName(CurveName);
	}

	UpdateCachedCurve();
}


FRichCurveEditorModelNamed::~FRichCurveEditorModelNamed()
{
	AnimSequence->GetDataModel()->GetModifiedEvent().RemoveAll(this);
}

bool FRichCurveEditorModelNamed::IsValid() const
{
	return AnimSequence->GetDataModel()->FindCurve(FAnimationCurveIdentifier(CurveName, Type)) != nullptr;
}

FRichCurve& FRichCurveEditorModelNamed::GetRichCurve()
{
	check(AnimSequence.Get() != nullptr);
	return CachedCurve;
}

const FRichCurve& FRichCurveEditorModelNamed::GetReadOnlyRichCurve() const
{
	return const_cast<FRichCurveEditorModelNamed*>(this)->GetRichCurve();
}

void FRichCurveEditorModelNamed::SetKeyPositions(TArrayView<const FKeyHandle> InKeys, TArrayView<const FKeyPosition> InKeyPositions, EPropertyChangeType::Type ChangeType)
{
	const bool bInteractiveChange = ChangeType == EPropertyChangeType::Interactive;

	// Open bracket in case this is an interactive change
	if (bInteractiveChange && !InteractiveBracket.IsValid())
	{
		IAnimationDataController& Controller = AnimSequence->GetController();
		InteractiveBracket = MakeUnique<IAnimationDataController::FScopedBracket>(Controller, LOCTEXT("SetKeyPositions", "Set Key Positions"));
	}
	
	FRichCurveEditorModel::SetKeyPositions(InKeys, InKeyPositions, ChangeType);

	// Close bracket, if open, in case this is was a non-interactive change
	if (!bInteractiveChange && InteractiveBracket.IsValid())
	{
		InteractiveBracket.Reset();
	}
}

void FRichCurveEditorModelNamed::SetKeyAttributes(TArrayView<const FKeyHandle> InKeys, TArrayView<const FKeyAttributes> InAttributes, EPropertyChangeType::Type ChangeType)
{
	const bool bInteractiveChange = ChangeType == EPropertyChangeType::Interactive;

	// Open bracket in case this is an interactive change
	if (bInteractiveChange && !InteractiveBracket.IsValid())
	{
		IAnimationDataController& Controller = AnimSequence->GetController();
		InteractiveBracket = MakeUnique<IAnimationDataController::FScopedBracket>(Controller, LOCTEXT("SetKeyAttributes", "Set Key Attributes"));
	}
		
	FRichCurveEditorModel::SetKeyAttributes(InKeys, InAttributes, ChangeType);
	// Close bracket, if open, in case this is was a non-interactive change
	if (!bInteractiveChange && InteractiveBracket.IsValid())
	{
		InteractiveBracket.Reset();
	}
}

void FRichCurveEditorModelNamed::SetCurveAttributes(const FCurveAttributes& InCurveAttributes)
{	
	IAnimationDataController& Controller = AnimSequence->GetController();
	Controller.SetCurveAttributes(CurveId, InCurveAttributes);
}

void FRichCurveEditorModelNamed::CurveHasChanged()
{	
	IAnimationDataController& Controller = AnimSequence->GetController();

	switch (Type)
	{
		case ERawCurveTrackTypes::RCT_Vector:
		{
			ensure(false);
			break;
		}
		case ERawCurveTrackTypes::RCT_Transform:
		case ERawCurveTrackTypes::RCT_Float:
		{
			Controller.SetCurveKeys(CurveId, CachedCurve.GetConstRefOfKeys());
			break;
		}
	}
}

void FRichCurveEditorModelNamed::OnModelHasChanged(const EAnimDataModelNotifyType& NotifyType, IAnimationDataModel* Model, const FAnimDataModelNotifPayload& Payload)
{
	NotifyCollector.Handle(NotifyType);

	switch (NotifyType)
	{
		case EAnimDataModelNotifyType::CurveAdded:
		case EAnimDataModelNotifyType::CurveChanged:
		case EAnimDataModelNotifyType::CurveFlagsChanged:
		case EAnimDataModelNotifyType::CurveScaled:
		{
			const FCurvePayload& TypedPayload = Payload.GetPayload<FCurvePayload>();
			if (TypedPayload.Identifier.CurveName == CurveName)
			{
				if (NotifyCollector.IsNotWithinBracket())
				{
					UpdateCachedCurve();
				}
				else
				{
					// Curve was re-added after removal in same bracket
					if (bCurveRemoved && NotifyType == EAnimDataModelNotifyType::CurveAdded)
					{
						bCurveRemoved = false;
					}
				}
			}

			break;
		}

		case EAnimDataModelNotifyType::CurveRemoved:
		{
			// Curve was removed
			const FCurveRemovedPayload& TypedPayload = Payload.GetPayload<FCurveRemovedPayload>();
			if (TypedPayload.Identifier.CurveName == CurveName)
			{
				bCurveRemoved = true;
			}
			break;
		}

		case EAnimDataModelNotifyType::CurveRenamed:
		{
			const FCurveRenamedPayload& TypedPayload = Payload.GetPayload<FCurveRenamedPayload>();
			if (TypedPayload.Identifier == CurveId)
			{
				CurveName = TypedPayload.NewIdentifier.CurveName;
				CurveId = TypedPayload.NewIdentifier;

				if (NotifyCollector.IsNotWithinBracket())
				{
					UpdateCachedCurve();
				}
			}

			break;
		}

		case EAnimDataModelNotifyType::BracketClosed:
		{
			if (NotifyCollector.IsNotWithinBracket())
			{
				if (!bCurveRemoved && NotifyCollector.Contains({ EAnimDataModelNotifyType::CurveAdded, EAnimDataModelNotifyType::CurveChanged, EAnimDataModelNotifyType::CurveFlagsChanged, EAnimDataModelNotifyType::CurveScaled, EAnimDataModelNotifyType::CurveRenamed }))
				{
					UpdateCachedCurve();
				}
			}
			break;
		}
	}
}

void FRichCurveEditorModelNamed::UpdateCachedCurve()
{
	const FAnimCurveBase* CurveBase = AnimSequence->GetDataModel()->FindCurve(CurveId);
	
	check(CurveBase);	// If this fails lifetime contracts have been violated - this curve should always be present if this model exists

	const FRichCurve* CurveToCopyFrom = [this, CurveBase]() -> const FRichCurve*
	{
		switch (Type)
		{
			case ERawCurveTrackTypes::RCT_Vector:
			{
				ensure(false);
				const FVectorCurve& VectorCurve = *(static_cast<const FVectorCurve*>(CurveBase));
				check(CurveIndex < 3);
				return &VectorCurve.FloatCurves[CurveIndex];
			}
			case ERawCurveTrackTypes::RCT_Transform:
			{
				const FTransformCurve& TransformCurve = *(static_cast<const FTransformCurve*>(CurveBase));
				check(CurveIndex < 9);
				const int32 SubCurveIndex = CurveIndex % 3;
				switch (CurveIndex)
				{
				default:
					check(false);
					// fall through
				case 0:
				case 1:
				case 2:
					return &TransformCurve.TranslationCurve.FloatCurves[SubCurveIndex];
				case 3:
				case 4:
				case 5:
					return &TransformCurve.RotationCurve.FloatCurves[SubCurveIndex];
				case 6:
				case 7:
				case 8:
					return &TransformCurve.ScaleCurve.FloatCurves[SubCurveIndex];
				}

			}
			case ERawCurveTrackTypes::RCT_Float:
			default:
			{
				const FFloatCurve& FloatCurve = *(static_cast<const FFloatCurve*>(CurveBase));
				check(CurveIndex == 0);
				return &FloatCurve.FloatCurve;
			}
		}
	}();
	
	if (ensure(CurveToCopyFrom))
	{
		CachedCurve = *CurveToCopyFrom;
	}
}

class FAnimSequenceCurveEditorItem : public ICurveEditorTreeItem
{
public:
	FAnimSequenceCurveEditorItem(const FName& InName, ERawCurveTrackTypes InType, int32 InCurveIndex, UAnimSequenceBase* InAnimSequence, const FText& InCurveDisplayName, const FLinearColor& InCurveColor, FSimpleDelegate InOnCurveModified, FCurveEditorTreeItemID InTreeId)
		: Name(InName)
		, Type(InType)
		, CurveIndex(InCurveIndex)
		, AnimSequence(InAnimSequence)
		, CurveDisplayName(InCurveDisplayName)
		, CurveColor(InCurveColor)
		, OnCurveModified(InOnCurveModified)
		, TreeId(InTreeId)
	{
	}

	virtual TSharedPtr<SWidget> GenerateCurveEditorTreeWidget(const FName& InColumnName, TWeakPtr<FCurveEditor> InCurveEditor, FCurveEditorTreeItemID InTreeItemID, const TSharedRef<ITableRow>& InTableRow) override
	{
		if (InColumnName == ColumnNames.Label)
		{
			return SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.Padding(FMargin(4.f))
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Right)
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text(CurveDisplayName)
					.ColorAndOpacity(FSlateColor(CurveColor))
				];
		}
		else if (InColumnName == ColumnNames.SelectHeader)
		{
			return SNew(SCurveEditorTreeSelect, InCurveEditor, InTreeItemID, InTableRow);
		}
		else if (InColumnName == ColumnNames.PinHeader)
		{
			return SNew(SCurveEditorTreePin, InCurveEditor, InTreeItemID, InTableRow);
		}

		return nullptr;
	}

	virtual void CreateCurveModels(TArray<TUniquePtr<FCurveModel>>& OutCurveModels) override
	{
		TUniquePtr<FRichCurveEditorModelNamed> NewCurveModel = MakeUnique<FRichCurveEditorModelNamed>(Name, Type, CurveIndex, AnimSequence.Get(), TreeId);
		NewCurveModel->SetShortDisplayName(CurveDisplayName);
		NewCurveModel->SetLongDisplayName(CurveDisplayName);
		NewCurveModel->SetColor(CurveColor);
		NewCurveModel->OnCurveModified().Add(OnCurveModified);

		OutCurveModels.Add(MoveTemp(NewCurveModel));
	}

	virtual bool PassesFilter(const FCurveEditorTreeFilter* InFilter) const override
	{
		if (InFilter->GetType() == ECurveEditorTreeFilterType::Text)
		{
			const FCurveEditorTreeTextFilter* Filter = static_cast<const FCurveEditorTreeTextFilter*>(InFilter);
			for (const FCurveEditorTreeTextFilterTerm& Term : Filter->GetTerms())
			{
				for(const FCurveEditorTreeTextFilterToken& Token : Term.ChildToParentTokens)
				{
					if(Token.Match(*CurveDisplayName.ToString()))
					{
						return true;
					}
				}
			}

			return false;
		}

		return false;
	}

	FName Name;
	ERawCurveTrackTypes Type;
	int32 CurveIndex;
	TWeakObjectPtr<UAnimSequenceBase> AnimSequence;
	FText CurveDisplayName;
	FLinearColor CurveColor;
	FSimpleDelegate OnCurveModified;
	FCurveEditorTreeItemID TreeId;
};

class FAnimSequenceCurveEditorBounds : public ICurveEditorBounds
{
public:
	FAnimSequenceCurveEditorBounds(TSharedPtr<ITimeSliderController> InExternalTimeSliderController)
		: ExternalTimeSliderController(InExternalTimeSliderController)
	{}

	virtual void GetInputBounds(double& OutMin, double& OutMax) const override
	{
		FAnimatedRange ViewRange = ExternalTimeSliderController.Pin()->GetViewRange();
		OutMin = ViewRange.GetLowerBoundValue();
		OutMax = ViewRange.GetUpperBoundValue();
	}

	virtual void SetInputBounds(double InMin, double InMax) override
	{
		ExternalTimeSliderController.Pin()->SetViewRange(InMin, InMax, EViewRangeInterpolation::Immediate);
	}

	TWeakPtr<ITimeSliderController> ExternalTimeSliderController;
};

void SAnimSequenceCurveEditor::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	IAnimSequenceCurveEditor::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
	
	for (const FAnimationCurveIdentifier& CurveToRemove : CurvesToRemove)
	{
		const int32 ChannelIndices = CurveToRemove.CurveType == ERawCurveTrackTypes::RCT_Transform ? 9 : 1;
		for (int32 ChannelIndex = 0; ChannelIndex < ChannelIndices; ++ChannelIndex)
		{
			RemoveCurve(CurveToRemove.CurveName, CurveToRemove.CurveType, ChannelIndex);
		}
	}
	CurvesToRemove.Reset();
}

SAnimSequenceCurveEditor::~SAnimSequenceCurveEditor()
{
	if(AnimSequence)
	{
		AnimSequence->GetDataModel()->GetModifiedEvent().RemoveAll(this);
	}
}

void SAnimSequenceCurveEditor::Construct(const FArguments& InArgs, const TSharedRef<IPersonaPreviewScene>& InPreviewScene, UAnimSequenceBase* InAnimSequence)
{
	CurveEditor = MakeShared<FCurveEditor>();
	CurveEditor->GridLineLabelFormatXAttribute = LOCTEXT("GridXLabelFormat", "{0}s");
	CurveEditor->SetBounds(MakeUnique<FAnimSequenceCurveEditorBounds>(InArgs._ExternalTimeSliderController));

	FCurveEditorInitParams CurveEditorInitParams;
	CurveEditor->InitCurveEditor(CurveEditorInitParams);
	CurveEditor->InputSnapRateAttribute = InAnimSequence->GetSamplingFrameRate();

	AnimSequence = InAnimSequence;

	AnimSequence->GetDataModel()->GetModifiedEvent().AddRaw(this, &SAnimSequenceCurveEditor::OnModelHasChanged);

	CurveEditorTree = SNew(SCurveEditorTree, CurveEditor)
		.OnContextMenuOpening(this, &SAnimSequenceCurveEditor::OnContextMenuOpening);

	TSharedRef<SCurveEditorPanel> CurveEditorPanel = SNew(SCurveEditorPanel, CurveEditor.ToSharedRef())
		.GridLineTint(FLinearColor(0.f, 0.f, 0.f, 0.3f))
		.ExternalTimeSliderController(InArgs._ExternalTimeSliderController)
		.TabManager(InArgs._TabManager)
		.TreeContent()
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SAssignNew(CurveEditorSearchBox, SCurveEditorTreeTextFilter, CurveEditor)
			]
			+SVerticalBox::Slot()
			[
				SNew(SScrollBorder, CurveEditorTree.ToSharedRef())
				[
					CurveEditorTree.ToSharedRef()
				]
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SCurveEditorTreeFilterStatusBar, CurveEditor)
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Center)
			[
				SNew(SAnimTimelineTransportControls, InPreviewScene, InAnimSequence)
			]
		];

	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.0f, 0.0f, 0.0f, 3.0f)
		[
			MakeToolbar(CurveEditorPanel)
		]
		+SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			CurveEditorPanel
		]
	];
}

void SAnimSequenceCurveEditor::OnModelHasChanged(const EAnimDataModelNotifyType& NotifyType, IAnimationDataModel* Model, const FAnimDataModelNotifPayload& Payload)
{
	switch(NotifyType)
	{
	case EAnimDataModelNotifyType::CurveRemoved:
	case EAnimDataModelNotifyType::CurveRenamed:
		{
			const FCurvePayload& TypedPayload = Payload.GetPayload<FCurvePayload>();
			CurvesToRemove.Add(TypedPayload.Identifier);
			break;
		}			
	case EAnimDataModelNotifyType::CurveFlagsChanged:
		{
			const FCurveFlagsChangedPayload& TypedPayload = Payload.GetPayload<FCurveFlagsChangedPayload>();
			if(Model->FindCurve(TypedPayload.Identifier)->GetCurveTypeFlag(AACF_Metadata))
			{
				CurvesToRemove.Add(TypedPayload.Identifier);
			}
			break;
		}
	}
	
}

TSharedRef<SWidget> SAnimSequenceCurveEditor::MakeToolbar(TSharedRef<SCurveEditorPanel> InEditorPanel)
{
	FSlimHorizontalToolBarBuilder ToolBarBuilder(InEditorPanel->GetCommands(), FMultiBoxCustomization::None, InEditorPanel->GetToolbarExtender(), true);
	ToolBarBuilder.BeginSection("Asset");
	ToolBarBuilder.EndSection();
	// We just use all of the extenders as our toolbar, we don't have a need to create a separate toolbar.
	return ToolBarBuilder.MakeWidget();
}

TSharedPtr<SWidget> SAnimSequenceCurveEditor::OnContextMenuOpening()
{
	const TArray<FCurveEditorTreeItemID>& Selection = CurveEditorTree->GetSelectedItems();
	if (Selection.Num())
	{
		UToolMenus* ToolMenus = UToolMenus::Get();
		static const FName MenuName = "SAnimSequenceCurveEditor.CurveEditorTreeContextMenu";
		if (!ToolMenus->IsMenuRegistered(MenuName))
		{
			ToolMenus->RegisterMenu(MenuName);
		}

		FToolMenuContext Context;
		UToolMenu* Menu = ToolMenus->GenerateMenu(MenuName, Context);

		FToolMenuSection& Section = Menu->AddSection("Selection", LOCTEXT("SelectionLablel", "Selection"));
		Section.AddMenuEntry("RemoveSelectedCurves", LOCTEXT("RemoveCurveLabel", "Stop editing selected curve(s)"),
			LOCTEXT("RemoveCurveTooltip", "Removes the currently selected curve(s) from editing"),
			FSlateIcon(),
			FToolUIActionChoice(FExecuteAction::CreateLambda([this]()
				{
					// Remove all selected tree items, and associated curves
					TArray<FCurveModelID> ModelIDs;				
					TArray<FCurveEditorTreeItemID> Selection = CurveEditorTree->GetSelectedItems();
					for (const FCurveEditorTreeItemID& SelectedItem : Selection)
					{
						ModelIDs.Append(CurveEditor->GetTreeItem(SelectedItem).GetCurves());
						CurveEditor->RemoveTreeItem(SelectedItem);
					}
					CurveEditorTree->ClearSelection();

					for (const FCurveModelID& ID : ModelIDs)
					{
						CurveEditor->RemoveCurve(ID);
					}
				}))				
			);

		return ToolMenus->GenerateWidget(Menu);
	}

	return SNullWidget::NullWidget;
}

void SAnimSequenceCurveEditor::ResetCurves()
{
	CurveEditor->RemoveAllTreeItems();
	CurveEditor->RemoveAllCurves();
}

void SAnimSequenceCurveEditor::AddCurve(const FText& InCurveDisplayName, const FLinearColor& InCurveColor, const FName& InName, ERawCurveTrackTypes InType, int32 InCurveIndex, FSimpleDelegate InOnCurveModified)
{
	// Ensure that curve is not already being edited
	for (const TPair<FCurveEditorTreeItemID, FCurveEditorTreeItem>& ItemPair : CurveEditor->GetTree()->GetAllItems())
	{
		TSharedPtr<FAnimSequenceCurveEditorItem> Item = StaticCastSharedPtr<FAnimSequenceCurveEditorItem>(ItemPair.Value.GetItem());
		if (Item->Name == InName && Item->Type == InType && Item->CurveIndex == InCurveIndex)
		{
			return;
		}
	}
	
	FCurveEditorTreeItem* TreeItem = CurveEditor->AddTreeItem(FCurveEditorTreeItemID());
	TreeItem->SetStrongItem(MakeShared<FAnimSequenceCurveEditorItem>(InName, InType, InCurveIndex, AnimSequence, InCurveDisplayName, InCurveColor, InOnCurveModified, TreeItem->GetID()));

	// Update selection
	const TMap<FCurveEditorTreeItemID, ECurveEditorTreeSelectionState>& Selection = CurveEditor->GetTreeSelection();
	TArray<FCurveEditorTreeItemID> NewSelection;
	NewSelection.Add(TreeItem->GetID());
	for(const auto& SelectionPair : Selection)
	{
		if(SelectionPair.Value != ECurveEditorTreeSelectionState::None)
		{
			NewSelection.Add(SelectionPair.Key);
		}
	}
	CurveEditor->SetTreeSelection(MoveTemp(NewSelection));
}

void SAnimSequenceCurveEditor::RemoveCurve(const FName& InName, ERawCurveTrackTypes InType, int32 InCurveIndex)
{
	for(const FCurveEditorTreeItemID& TreeItemID : CurveEditor->GetRootTreeItems())
	{
		FCurveEditorTreeItem& TreeItem = CurveEditor->GetTreeItem(TreeItemID);
		TSharedPtr<FAnimSequenceCurveEditorItem> CurveItem = StaticCastSharedPtr<FAnimSequenceCurveEditorItem>(TreeItem.GetItem());
		if(CurveItem->Name == InName && CurveItem->Type == InType && CurveItem->CurveIndex == InCurveIndex)
		{
			CurveEditor->RemoveTreeItem(TreeItemID);
			break;
		}
	}
	
	for(const auto& CurvePair : CurveEditor->GetCurves())
	{
		FRichCurveEditorModelNamed* Model = static_cast<FRichCurveEditorModelNamed*>(CurvePair.Value.Get());
		if(Model->CurveName == InName && Model->Type == InType && Model->CurveIndex == InCurveIndex)
		{
			// Cache ID to prevent use after release
			CurveEditor->RemoveCurve(CurvePair.Key);
			break;
		}
	}
}

void SAnimSequenceCurveEditor::ZoomToFit()
{
	CurveEditor->ZoomToFit(EAxisList::Y);
}

#undef LOCTEXT_NAMESPACE
