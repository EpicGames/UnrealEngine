// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IAnimSequenceCurveEditor.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "EditorUndoClient.h"
#include "RichCurveEditorModel.h"
#include "Animation/SmartName.h"
#include "Animation/AnimCurveTypes.h"
#include "CurveEditorTypes.h"
#include "Animation/AnimSequenceBase.h"

class FCurveEditor;
class ITimeSliderController;
class SCurveEditorTree;
class IPersonaPreviewScene;
class SCurveEditorPanel;
class FTabManager;

// Model that references a named curve, rather than a raw pointer, so we avoid issues with 
// reallocation of the curves arrays under the UI
class FRichCurveEditorModelNamed : public FRichCurveEditorModel
{
public:
	FRichCurveEditorModelNamed(const FName& InName, ERawCurveTrackTypes InType, int32 InCurveIndex, UAnimSequenceBase* InAnimSequence, FCurveEditorTreeItemID InTreeId = FCurveEditorTreeItemID());
	
	virtual ~FRichCurveEditorModelNamed();

	virtual bool IsValid() const override;
	virtual FRichCurve& GetRichCurve() override;
	virtual const FRichCurve& GetReadOnlyRichCurve() const override;

	virtual void SetKeyPositions(TArrayView<const FKeyHandle> InKeys, TArrayView<const FKeyPosition> InKeyPositions, EPropertyChangeType::Type ChangeType) override;
	virtual void SetKeyAttributes(TArrayView<const FKeyHandle> InKeys, TArrayView<const FKeyAttributes> InAttributes, EPropertyChangeType::Type ChangeType = EPropertyChangeType::Unspecified) override;
	virtual void SetCurveAttributes(const FCurveAttributes& InCurveAttributes) override;

	void CurveHasChanged();
	void OnModelHasChanged(const EAnimDataModelNotifyType& NotifyType, IAnimationDataModel* Model, const FAnimDataModelNotifPayload& Payload);
	void UpdateCachedCurve();

	FName CurveName;
	TWeakObjectPtr<UAnimSequenceBase> AnimSequence;
	int32 CurveIndex;
	ERawCurveTrackTypes Type;
	FCurveEditorTreeItemID TreeId;
	
	FAnimationCurveIdentifier CurveId;
	UE::Anim::FAnimDataModelNotifyCollector NotifyCollector;
	FRichCurve CachedCurve;
	bool bCurveRemoved;

	TUniquePtr<IAnimationDataController::FScopedBracket> InteractiveBracket;
};

class SAnimSequenceCurveEditor : public IAnimSequenceCurveEditor
{
	SLATE_BEGIN_ARGS(SAnimSequenceCurveEditor) {}

	SLATE_ARGUMENT(TSharedPtr<ITimeSliderController>, ExternalTimeSliderController)

	SLATE_ARGUMENT(TSharedPtr<FTabManager>, TabManager)

	SLATE_END_ARGS()

	~SAnimSequenceCurveEditor();

	void Construct(const FArguments& InArgs, const TSharedRef<IPersonaPreviewScene>& InPreviewScene, UAnimSequenceBase* InAnimSequence);

	/** IAnimSequenceCurveEditor interface */
	virtual void ResetCurves() override;
	virtual void AddCurve(const FText& InCurveDisplayName, const FLinearColor& InCurveColor, const FName& InName, ERawCurveTrackTypes InType, int32 InCurveIndex, FSimpleDelegate InOnCurveModified) override;
	virtual void RemoveCurve(const FName& InName, ERawCurveTrackTypes InType, int32 InCurveIndex) override;
	virtual void ZoomToFit() override;
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	
	void OnModelHasChanged(const EAnimDataModelNotifyType& NotifyType, IAnimationDataModel* Model, const FAnimDataModelNotifPayload& Payload);
private:
	// Build the toolbar for this curve editor
	TSharedRef<SWidget> MakeToolbar(TSharedRef<SCurveEditorPanel> InEditorPanel);
	TSharedPtr<SWidget> OnContextMenuOpening();

private:
	/** The actual curve editor */
	TSharedPtr<FCurveEditor> CurveEditor;

	/** The search widget for filtering curves in the Curve Editor tree. */
	TSharedPtr<SWidget> CurveEditorSearchBox;

	/** The anim sequence we are editing */
	UAnimSequenceBase* AnimSequence;

	/** The tree widget in the curve editor */
	TSharedPtr<SCurveEditorTree> CurveEditorTree;

	/** Curves to be removed from editing on the next ::Tick */	
	TArray<FAnimationCurveIdentifier> CurvesToRemove; 
};
