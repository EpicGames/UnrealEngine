// Copyright Epic Games, Inc. All Rights Reserved.


#include "SAnimStreamableEditor.h"
#include "IDocumentation.h"
#include "AnimTimeline/AnimModel_AnimSequenceBase.h"
#include "AnimTimeline/SAnimTimeline.h"

//////////////////////////////////////////////////////////////////////////
// SAnimStreamableEditor

TSharedRef<SWidget> SAnimStreamableEditor::CreateDocumentAnchor()
{
	return IDocumentation::Get()->CreateAnchor(TEXT("AnimatingObjects"));
}

void SAnimStreamableEditor::Construct(const FArguments& InArgs, const TSharedRef<class IPersonaPreviewScene>& InPreviewScene, const TSharedRef<class IEditableSkeleton>& InEditableSkeleton, const TSharedRef<FUICommandList>& InCommandList)
{
	StreamableAnim = InArgs._StreamableAnim;
	check(StreamableAnim);

	AnimModel = MakeShared<FAnimModel_AnimSequenceBase>(InPreviewScene, InEditableSkeleton, InCommandList, StreamableAnim);
	AnimModel->OnSelectObjects = FOnObjectsSelected::CreateSP(this, &SAnimEditorBase::OnSelectionChanged);
	AnimModel->OnInvokeTab = InArgs._OnInvokeTab;

	AnimModel->OnEditCurves = FOnEditCurves::CreateLambda([this, InOnEditCurves = InArgs._OnEditCurves](UAnimSequenceBase* InAnimSequence, const TArray<IAnimationEditor::FCurveEditInfo>& InCurveInfo, const TSharedPtr<ITimeSliderController>& InExternalTimeSliderController)
	{
		InOnEditCurves.ExecuteIfBound(InAnimSequence, InCurveInfo, TimelineWidget->GetTimeSliderController());
	});

	AnimModel->Initialize();

	SAnimEditorBase::Construct(SAnimEditorBase::FArguments()
		.OnObjectsSelected(InArgs._OnObjectsSelected)
		.AnimModel(AnimModel),
		InPreviewScene);

	if (GEditor)
	{
		GEditor->RegisterForUndo(this);
	}
}

SAnimStreamableEditor::~SAnimStreamableEditor()
{
	if (GEditor)
	{
		GEditor->UnregisterForUndo(this);
	}
}

void SAnimStreamableEditor::PostUndo(bool bSuccess)
{
	PostUndoRedo();
}

void SAnimStreamableEditor::PostRedo(bool bSuccess)
{
	PostUndoRedo();
}

void SAnimStreamableEditor::PostUndoRedo()
{
	GetPreviewScene()->SetPreviewAnimationAsset(StreamableAnim);

	AnimModel->RefreshTracks();
}
