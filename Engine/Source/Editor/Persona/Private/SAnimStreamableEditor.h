// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "SAnimEditorBase.h"
#include "Animation/AnimStreamable.h"

class SAnimNotifyPanel;
class FAnimModel_AnimSequenceBase;

//////////////////////////////////////////////////////////////////////////
// SAnimStreamableEditor

/** Overall streamable animation editing widget. This mostly contains functions for editing the UAnimStreamable. */
class SAnimStreamableEditor : public SAnimEditorBase, public FEditorUndoClient
{
public:
	SLATE_BEGIN_ARGS( SAnimStreamableEditor )
		: _StreamableAnim(nullptr)
	{}

	SLATE_ARGUMENT( UAnimStreamable*, StreamableAnim)
	SLATE_EVENT(FOnObjectsSelected, OnObjectsSelected)
	SLATE_EVENT(FOnInvokeTab, OnInvokeTab)
	SLATE_EVENT(FOnEditCurves, OnEditCurves)

	SLATE_END_ARGS()

private:
	TSharedPtr<FAnimModel_AnimSequenceBase> AnimModel;

public:
	~SAnimStreamableEditor();

	void Construct(const FArguments& InArgs, const TSharedRef<class IPersonaPreviewScene>& InPreviewScene, const TSharedRef<class IEditableSkeleton>& InEditableSkeleton, const TSharedRef<FUICommandList>& InCommandList);

	/** Return the streamable animation being edited */
	UAnimStreamable* GetStreamableAnim() const { return StreamableAnim; }
	virtual UAnimationAsset* GetEditorObject() const override { return GetStreamableAnim(); }

private:
	/** Pointer to the streamable animation being edited */
	UAnimStreamable* StreamableAnim;

public:
	//~ Begin SAnimEditorBase Interface
	virtual TSharedRef<SWidget> CreateDocumentAnchor() override;
	//~ End SAnimEditorBase Interface

	/** FEditorUndoClient interface */
	virtual void PostUndo( bool bSuccess ) override;
	virtual void PostRedo( bool bSuccess ) override;

	/** Post undo **/
	void PostUndoRedo();
};
