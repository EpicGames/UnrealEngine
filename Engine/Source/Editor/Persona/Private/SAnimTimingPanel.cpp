// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAnimTimingPanel.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/MenuStack.h"
#include "Fonts/FontMeasure.h"
#include "Styling/CoreStyle.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SToolTip.h"
#include "Animation/AnimMontage.h"
#include "Preferences/PersonaOptions.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "STimingTrack.h"
#include "ISkeletonEditorModule.h"
#include "Modules/ModuleManager.h"
#include "IEditableSkeleton.h"
#include "Editor.h"
#include "AnimTimeline/AnimModel.h"
#include "AnimTimeline/AnimModel_AnimMontage.h"
#include "Preferences/PersonaOptions.h"
#include "FrameNumberDisplayFormat.h"

#define LOCTEXT_NAMESPACE "AnimTimingPanel"

namespace AnimTimingConstants
{
	static const float DefaultNodeSize = 18.0f;
	static const int32 FontSize = 10;
}

class SAnimTimingNodeTooltip : public SToolTip
{
public:
	void Construct(const FArguments& InArgs, const TSharedRef<FTimingRelevantElementBase>& InElement)
	{
		Element = InElement;
		DescriptionBox = SNew(SVerticalBox);

		FArguments Args = InArgs;
		Args._TextMargin = FMargin(1.0f);
		Args._BorderImage = FAppStyle::GetBrush("ContentBrowser.TileViewToolTip.ToolTipBorder");
		Args._Content.Widget =
			SNew(SBorder)
			.Padding(3)
			.BorderImage(FAppStyle::GetBrush("ContentBrowser.TileViewTooltip.NonContentBorder"))
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0, 0, 0, 3)
				[
					SNew(SBorder)
					.Padding(6)
					.BorderImage(FAppStyle::GetBrush("ContentBrowser.TileViewTooltip.ContentBorder"))
					[
						SNew(SBox)
						.HAlign(HAlign_Left)
						[
							SNew(STextBlock)
							.Text(FText::FromName(Element->GetTypeName()))
							.Font(FAppStyle::GetFontStyle("ContentBrowser.TileViewTooltip.NameFont"))
						]
					]
				]

				+ SVerticalBox::Slot()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.FillWidth(1.0f)
					[
						SNew(SBorder)
						.Padding(3)
						.BorderImage(FAppStyle::GetBrush("ContentBrowser.TileViewTooltip.ContentBorder"))
						[
							DescriptionBox.ToSharedRef()
						]
					]
				]
			];

		SToolTip::Construct(Args);
	}

	virtual void OnOpening() override
	{
		DescriptionBox->ClearChildren();

		TMap<FString, FText> DescriptionItems;
		Element->GetDescriptionItems(DescriptionItems);
		for(TPair<FString, FText> ItemPair : DescriptionItems)
		{
			DescriptionBox->AddSlot()
				.AutoHeight()
				.Padding(0, 0, 3, 0)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(STextBlock)
						.Text(FText::Format(LOCTEXT("Item", "{0}: "), FText::FromString(ItemPair.Key)))
						.ColorAndOpacity(FSlateColor::UseSubduedForeground())
					]

					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(STextBlock)
						.Text(ItemPair.Value)
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
				];
		}
	}

	TSharedPtr<SVerticalBox> DescriptionBox;
	TSharedPtr<FTimingRelevantElementBase> Element;
};

void SAnimTimingNode::Construct(const FArguments& InArgs)
{
	Element = InArgs._InElement;

	const FSlateBrush* StyleInfo = FAppStyle::GetBrush(TEXT("SpecialEditableTextImageNormal"));
	static FSlateFontInfo LabelFont = FCoreStyle::GetDefaultFontStyle("Regular", AnimTimingConstants::FontSize);

	UPersonaOptions* EditorOptions = UPersonaOptions::StaticClass()->GetDefaultObject<UPersonaOptions>();
	check(EditorOptions);

	// Pick the colour of the node from the type of the element
	FLinearColor NodeColour = FLinearColor::White;
	switch(Element->GetType())
	{
		case ENotifyTimingElementType::QueuedNotify:
		case ENotifyTimingElementType::NotifyStateBegin:
		case ENotifyTimingElementType::NotifyStateEnd:
		{
			NodeColour = EditorOptions->NotifyTimingNodeColor;
			break;
		}
		case ENotifyTimingElementType::BranchPointNotify:
		{
			NodeColour = EditorOptions->BranchingPointTimingNodeColor;
			break;
		}
		case ENotifyTimingElementType::Section:
		{
			NodeColour = EditorOptions->SectionTimingNodeColor;
			break;
		}
		default:
			break;
	}

	this->ChildSlot
	[
		SNew(SBorder)
		.BorderImage(StyleInfo)
		.BorderBackgroundColor(NodeColour)
		[
			SNew(STextBlock)
			.Justification(ETextJustify::Center)
			.Text(FText::AsNumber(Element->TriggerIdx))
			.Font(LabelFont)
			.ColorAndOpacity(FSlateColor(FLinearColor::Black))
		]
	];

	if(InArgs._bUseTooltip)
	{
		SetToolTip(SNew(SAnimTimingNodeTooltip, Element.ToSharedRef()));
	}
}

FVector2D SAnimTimingNode::ComputeDesiredSize(float) const
{
	// Desired height is always the same (a little less than the track height) but the width depends on the text we display
	const TSharedRef< FSlateFontMeasure > FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
	static FSlateFontInfo LabelFont = FCoreStyle::GetDefaultFontStyle("Regular", AnimTimingConstants::FontSize);
	const float TextWidth = static_cast<float>(FontMeasureService->Measure(FString::FromInt(Element->TriggerIdx), LabelFont).X);
	return FVector2D(FMath::Max(AnimTimingConstants::DefaultNodeSize, TextWidth), AnimTimingConstants::DefaultNodeSize);
}

void SAnimTimingTrackNode::Construct(const FArguments& InArgs)
{
	const TAttribute<float> TimeAttr = TAttribute<float>::Create(TAttribute<float>::FGetter::CreateSP(InArgs._Element.ToSharedRef(), &FTimingRelevantElementBase::GetElementTime));

	STrackNode::Construct(STrackNode::FArguments()
		.ViewInputMin(InArgs._ViewInputMin)
		.ViewInputMax(InArgs._ViewInputMax)
		.DataStartPos(TimeAttr)
		.NodeName(InArgs._NodeName)
		.CenterOnPosition(true)
		.AllowDrag(false)
		.OverrideContent()
			[
				SNew(SBox)
				.HAlign(HAlign_Center)
				[
					SNew(SAnimTimingNode)
					.InElement(InArgs._Element)
					.bUseTooltip(InArgs._bUseTooltip)
				]
			]
		);
}

void SAnimTimingPanel::Construct(const FArguments& InArgs, const TSharedRef<FAnimModel_AnimMontage>& InModel)
{
	WeakModel = InModel;

	SAnimTrackPanel::Construct(SAnimTrackPanel::FArguments()
		.WidgetWidth(InArgs._WidgetWidth)
		.ViewInputMin(InArgs._ViewInputMin)
		.ViewInputMax(InArgs._ViewInputMax)
		.InputMin(InArgs._InputMin)
		.InputMax(InArgs._InputMax)
		.OnSetInputViewRange(InArgs._OnSetInputViewRange));

	AnimSequence = InArgs._InSequence;

	check(AnimSequence);

	this->ChildSlot
	[
		SAssignNew(PanelArea, SBorder)
		.BorderImage(FAppStyle::GetBrush("NoBorder"))
		.Padding(0.0f)
		.ColorAndOpacity(FLinearColor::White)
	];

	Update();

	// Register to some delegates to update the interface
	ISkeletonEditorModule& SkeletonEditorModule = FModuleManager::LoadModuleChecked<ISkeletonEditorModule>("SkeletonEditor");
	TSharedPtr<IEditableSkeleton> EditableSkeleton = SkeletonEditorModule.CreateEditableSkeleton(AnimSequence->GetSkeleton());
	EditableSkeleton->RegisterOnNotifiesChanged(FSimpleDelegate::CreateSP(this, &SAnimTimingPanel::RefreshTrackNodes));

	InModel->OnTracksChanged().Add(FSimpleDelegate::CreateSP(this, &SAnimTimingPanel::RefreshTrackNodes));
}

void SAnimTimingPanel::Update()
{
	check(PanelArea.IsValid());

	TSharedPtr<SVerticalBox> TimingSlots;
	
	PanelArea->SetContent(SAssignNew(TimingSlots, SVerticalBox));

	TimingSlots->AddSlot()
		.AutoHeight()
		.VAlign(VAlign_Center)
		[
			SAssignNew(Track, STimingTrack)
			.ViewInputMin(ViewInputMin)
			.ViewInputMax(ViewInputMax)
			.TrackMinValue(InputMin)
			.TrackMaxValue(InputMax)
			.TrackNumDiscreteValues(AnimSequence->GetNumberOfSampledKeys())
		];

	RefreshTrackNodes();
}

void SAnimTimingPanel::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SAnimTrackPanel::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
}

void SAnimTimingPanel::RefreshTrackNodes()
{
	Elements.Empty();
	GetTimingRelevantElements(AnimSequence, Elements);

	Track->ClearTrack();

	int32 NumElements = Elements.Num();
	for(int32 ElementIdx = 0 ; ElementIdx < NumElements ; ++ElementIdx)
	{
		TSharedPtr<FTimingRelevantElementBase> Element = Elements[ElementIdx];
		if(WeakModel.Pin()->IsTimingElementDisplayEnabled(Element->GetType()))
		{
			Track->AddTrackNode(
				SNew(SAnimTimingTrackNode)
				.ViewInputMin(ViewInputMin)
				.ViewInputMax(ViewInputMax)
				.DataStartPos(Element.ToSharedRef(), &FTimingRelevantElementBase::GetElementTime)
				.NodeName(FString::FromInt(ElementIdx + 1))
				.NodeColor(FLinearColor::Yellow)
				.Visibility(EVisibility::HitTestInvisible)
				.Element(Element)
				);
		}
	}
}

bool SAnimTimingPanel::IsElementDisplayEnabled(ENotifyTimingElementType::Type ElementType) const
{
	return WeakModel.Pin()->IsTimingElementDisplayEnabled(ElementType);
}

void SAnimTimingPanel::OnElementDisplayEnabledChanged(ENotifyTimingElementType::Type ElementType)
{
	WeakModel.Pin()->ToggleTimingElementDisplayEnabled(ElementType);
	Update();
}

ECheckBoxState SAnimTimingPanel::IsElementDisplayChecked(ENotifyTimingElementType::Type ElementType) const
{
	return WeakModel.Pin()->IsTimingElementDisplayEnabled(ElementType) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

EVisibility SAnimTimingPanel::IsElementDisplayVisible(ENotifyTimingElementType::Type ElementType) const
{
	return WeakModel.Pin()->IsTimingElementDisplayEnabled(ElementType) ? EVisibility::Visible : EVisibility::Hidden;
}

void SAnimTimingPanel::GetTimingRelevantElements(UAnimSequenceBase* Sequence, TArray<TSharedPtr<FTimingRelevantElementBase>>& Elements)
{
	if(Sequence)
	{
		// Grab notifies
		int32 NumNotifies = Sequence->Notifies.Num();
		for(int32 NotifyIdx = 0 ; NotifyIdx < NumNotifies ; ++NotifyIdx)
		{
			FAnimNotifyEvent& Notify = Sequence->Notifies[NotifyIdx];

			FTimingRelevantElement_Notify* Element = new FTimingRelevantElement_Notify;
			Element->Sequence = Sequence;
			Element->NotifyIndex = NotifyIdx;
			Elements.Add(TSharedPtr<FTimingRelevantElementBase>(Element));

			if(Notify.NotifyStateClass)
			{
				// Add the end marker
				FTimingRelevantElement_NotifyStateEnd* EndElement = new FTimingRelevantElement_NotifyStateEnd;
				EndElement->Sequence = Sequence;
				EndElement->NotifyIndex = NotifyIdx;
				Elements.Add(TSharedPtr<FTimingRelevantElementBase>(EndElement));
			}
		}

		// Check for a montage and extract Montage elements
		if(UAnimMontage* Montage = Cast<UAnimMontage>(Sequence))
		{
			// Add sections
			int32 NumSections = Montage->CompositeSections.Num();

			for(int32 SectionIdx = 0 ; SectionIdx < NumSections ; ++SectionIdx)
			{
				FCompositeSection& Section = Montage->CompositeSections[SectionIdx];

				FTimingRelevantElement_Section* Element = new FTimingRelevantElement_Section;
				Element->Montage = Montage;
				Element->SectionIdx = SectionIdx;
				Elements.Add(TSharedPtr<FTimingRelevantElementBase>(Element));
			}
		}

		// Sort everything and give them trigger orders
		Elements.Sort([](const TSharedPtr<FTimingRelevantElementBase>& A, const TSharedPtr<FTimingRelevantElementBase>& B)
		{
			return A->Compare(*B);
		});

		int32 NumElements = Elements.Num();
		for(int32 Idx = 0 ; Idx < NumElements ; ++Idx)
		{
			Elements[Idx]->TriggerIdx = Idx;
		}
	}
}

FName FTimingRelevantElement_Section::GetTypeName()
{
	return FName("Montage Section");
}

float FTimingRelevantElement_Section::GetElementTime() const
{
	check(Montage)
	if(Montage->CompositeSections.IsValidIndex(SectionIdx))
	{
		return Montage->CompositeSections[SectionIdx].GetTime();
	}
	return -1.0f;
}

ENotifyTimingElementType::Type FTimingRelevantElement_Section::GetType()
{
	return ENotifyTimingElementType::Section;
}

void FTimingRelevantElement_Section::GetDescriptionItems(TMap<FString, FText>& Items)
{
	check(Montage);
	if(Montage->CompositeSections.IsValidIndex(SectionIdx))
	{
		FCompositeSection& Section = Montage->CompositeSections[SectionIdx];

		Items.Add(LOCTEXT("SectionName", "Name").ToString(), FText::FromName(Section.SectionName));
		if(GetDefault<UPersonaOptions>()->TimelineDisplayFormat == EFrameNumberDisplayFormats::Frames)
		{
			Items.Add(LOCTEXT("SectionTriggerFrame", "Trigger Frame").ToString(), FText::Format(LOCTEXT("SectionTriggerFrameValue", "{0}"), FText::AsNumber(Montage->GetFrameAtTime(Section.GetTime()))));
		}
		else
		{
			FNumberFormattingOptions NumberOptions;
			NumberOptions.MinimumFractionalDigits = 3;
			Items.Add(LOCTEXT("SectionTriggerTime", "Trigger Time").ToString(), FText::Format(LOCTEXT("SectionTriggerTimeValue", "{0}s"), FText::AsNumber(Section.GetTime(), &NumberOptions)));
		}	
	}
}

FName FTimingRelevantElement_Notify::GetTypeName()
{
	FName TypeName;
	switch(GetType())
	{
		case ENotifyTimingElementType::NotifyStateBegin:
			TypeName = FName("Notify State (Begin)");
			break;
		case ENotifyTimingElementType::BranchPointNotify:
			TypeName = FName("Branching Point");
			break;
		default:
			TypeName = FName("Notify");
	}

	return TypeName;
}

float FTimingRelevantElement_Notify::GetElementTime() const
{
	check(Sequence);
	if(Sequence->Notifies.IsValidIndex(NotifyIndex))
	{
		return Sequence->Notifies[NotifyIndex].GetTriggerTime();
	}
	return -1.0f;
}

ENotifyTimingElementType::Type FTimingRelevantElement_Notify::GetType()
{
	check(Sequence);
	if(Sequence->Notifies.IsValidIndex(NotifyIndex))
	{
		FAnimNotifyEvent& Event = Sequence->Notifies[NotifyIndex];

		if(Event.IsBranchingPoint())
		{
			return ENotifyTimingElementType::BranchPointNotify;
		}
		else
		{
			if(Event.NotifyStateClass)
			{
				return ENotifyTimingElementType::NotifyStateBegin;
			}
			else
			{
				return ENotifyTimingElementType::QueuedNotify;
			}
		}
	}

	return ENotifyTimingElementType::Max;
}

void FTimingRelevantElement_Notify::GetDescriptionItems(TMap<FString, FText>& Items)
{
	check(Sequence);
	if(Sequence->Notifies.IsValidIndex(NotifyIndex))
	{
		FAnimNotifyEvent& Event = Sequence->Notifies[NotifyIndex];

		FNumberFormattingOptions NumberOptions;
		NumberOptions.MinimumFractionalDigits = 3;

		Items.Add(LOCTEXT("NotifyName", "Name").ToString(), FText::FromName(Event.NotifyName));

		if(GetDefault<UPersonaOptions>()->TimelineDisplayFormat == EFrameNumberDisplayFormats::Frames)
		{
			Items.Add(LOCTEXT("NotifyTriggerFrame", "Trigger Frame").ToString(), FText::Format(LOCTEXT("NotifyTriggerFrame_Val", "{0}"), FText::AsNumber(Sequence->GetFrameAtTime(Event.GetTime()))));
		}
		else
		{
			Items.Add(LOCTEXT("NotifyTriggerTime", "Trigger Time").ToString(), FText::Format(LOCTEXT("NotifyTriggerTime_Val", "{0}s"), FText::AsNumber(Event.GetTime(), &NumberOptions)));
		}	

		// +1 as we start at 1 when showing tracks to the user
		Items.Add(LOCTEXT("TrackIdx", "Track").ToString(), FText::AsNumber(Event.TrackIndex + 1));

		if(Event.NotifyStateClass)
		{
			Items.Add(LOCTEXT("NotifyDuration", "Duration").ToString(), FText::Format(LOCTEXT("NotifyDuration_Val", "{0}s"), FText::AsNumber(Event.GetDuration(), &NumberOptions)));
		}
	}
}

int32 FTimingRelevantElement_Notify::GetElementSortPriority() const
{
	check(Sequence);
	if(Sequence->Notifies.IsValidIndex(NotifyIndex))
	{
		FAnimNotifyEvent& Event = Sequence->Notifies[NotifyIndex];
		return Event.TrackIndex;
	}
	return 0;
}

FName FTimingRelevantElement_NotifyStateEnd::GetTypeName()
{
	return FName("Notify State (End)");
}

float FTimingRelevantElement_NotifyStateEnd::GetElementTime() const
{
	check(Sequence);
	if(Sequence->Notifies.IsValidIndex(NotifyIndex))
	{
		FAnimNotifyEvent& Event = Sequence->Notifies[NotifyIndex];
		if(Event.NotifyStateClass)
		{
			return Event.GetEndTriggerTime();
		}
	}
	return -1.0f;
}

ENotifyTimingElementType::Type FTimingRelevantElement_NotifyStateEnd::GetType()
{
	return ENotifyTimingElementType::NotifyStateEnd;
}

#undef LOCTEXT_NAMESPACE
