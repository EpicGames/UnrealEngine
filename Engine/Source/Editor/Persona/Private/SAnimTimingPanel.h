// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Widgets/SCompoundWidget.h"
#include "SAnimTrackPanel.h"
#include "STrack.h"
#include "SAnimTimingPanelShared.h"

class SBorder;
class UAnimMontage;
class UAnimSequenceBase;
class FAnimModel;
class FAnimModel_AnimMontage;

//////////////////////////////////////////////////////////////////////////
// FTimingRelevantElement - data object holding timing data

// Small helper to store information about timing relevant elements (notifies, branch points, sections etc.)
struct FTimingRelevantElement_Notify : public FTimingRelevantElementBase
{
	UAnimSequenceBase* Sequence;	// The sequence the notify exists within
	int32 NotifyIndex;				// The index of the notify in the sequence

	virtual FName GetTypeName() override;
	virtual float GetElementTime() const override;
	virtual int32 GetElementSortPriority() const override;
	virtual ENotifyTimingElementType::Type GetType() override;
	virtual void GetDescriptionItems(TMap<FString, FText>& Items) override;
};

// Small helper to store information about timing relevant elements (notifies, branch points, sections etc.)
struct FTimingRelevantElement_NotifyStateEnd : public FTimingRelevantElement_Notify
{
	virtual FName GetTypeName() override;
	virtual float GetElementTime() const override;
	virtual ENotifyTimingElementType::Type GetType() override;
};

struct FTimingRelevantElement_Section : public FTimingRelevantElementBase
{
	UAnimMontage* Montage;	// The montage the section exists within
	int32 SectionIdx;		// The index of the section in the montage

	virtual FName GetTypeName() override;
	virtual float GetElementTime() const override;
	virtual ENotifyTimingElementType::Type GetType() override;
	virtual void GetDescriptionItems(TMap<FString, FText>& Items) override;
};

// Delegate to get the visibility of a type of timing node on an external panel (not the timing track)
DECLARE_DELEGATE_RetVal_OneParam(EVisibility, FOnGetTimingNodeVisibility, ENotifyTimingElementType::Type)



//////////////////////////////////////////////////////////////////////////
// Track node containing an identifier for trigger order of a timing element
//////////////////////////////////////////////////////////////////////////
class SAnimTimingTrackNode : public STrackNode
{
public:
	SLATE_BEGIN_ARGS(SAnimTimingTrackNode)
		: _bUseTooltip(true)
	{}

	SLATE_ATTRIBUTE(float, ViewInputMin)
	SLATE_ATTRIBUTE(float, ViewInputMax)
	SLATE_ATTRIBUTE(float, DataStartPos)
	SLATE_ATTRIBUTE(FString, NodeName)
	SLATE_ATTRIBUTE(FLinearColor, NodeColor)
	SLATE_ARGUMENT(TSharedPtr<FTimingRelevantElementBase>, Element)
	SLATE_ARGUMENT(bool, bUseTooltip)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

};

//////////////////////////////////////////////////////////////////////////
// Panel containing a track for timing nodes, to display the order
// that events in a montage occur (Notifies, sections, branching pts etc.)
//////////////////////////////////////////////////////////////////////////
class SAnimTimingPanel : public SAnimTrackPanel
{
public:

	SLATE_BEGIN_ARGS(SAnimTimingPanel)
		: _InSequence(nullptr)
		, _WidgetWidth(0)
		, _CurrentPosition(0)
		, _ViewInputMin(0)
		, _ViewInputMax(0)
		, _InputMin(0)
		, _InputMax(0)
		, _OnSetInputViewRange()
	{}

	SLATE_ARGUMENT(UAnimSequenceBase*, InSequence)
	SLATE_ARGUMENT(float, WidgetWidth)
	SLATE_ATTRIBUTE(float, CurrentPosition)
	SLATE_ATTRIBUTE(float, ViewInputMin)
	SLATE_ATTRIBUTE(float, ViewInputMax)
	SLATE_ATTRIBUTE(float, InputMin)
	SLATE_ATTRIBUTE(float, InputMax)
	SLATE_EVENT(FOnSetInputViewRange, OnSetInputViewRange)

	SLATE_END_ARGS()

	// Construct the panel
	// @param InArgs - Slate arguments
	void Construct(const FArguments& InArgs, const TSharedRef<FAnimModel_AnimMontage>& InModel);

	// Updates panel widgets
	void Update();


	// Access to display enabled flags in multiple forms
	bool IsElementDisplayEnabled(ENotifyTimingElementType::Type ElementType) const;
	ECheckBoxState IsElementDisplayChecked(ENotifyTimingElementType::Type ElementType) const;
	EVisibility IsElementDisplayVisible(ENotifyTimingElementType::Type ElementType) const;

	// Callback from slate when element display flag changes
	// @param ElementType - Flag to set
	void OnElementDisplayEnabledChanged(ENotifyTimingElementType::Type ElementType);

	// Inspects the provided sequence, collect and sorts the requested elements
	// @param Sequence - Sequence to inspect
	// @param Elements - Out array of elements
	static void GetTimingRelevantElements(UAnimSequenceBase* Sequence, TArray<TSharedPtr<FTimingRelevantElementBase>>& Elements);

protected:

	// Tick the panel state
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	// Clears the timing track and rebuilds the nodes
	void RefreshTrackNodes();

	// The model for our data
	TWeakPtr<FAnimModel_AnimMontage> WeakModel;
	// Observed timing elements
	TArray<TSharedPtr<FTimingRelevantElementBase>> Elements;
	// Anim sequence that contains the timing elements we are observing
	UAnimSequenceBase* AnimSequence;
	// Main panel widget
	TSharedPtr<SBorder> PanelArea;
	// The track to place timing nodes on
	TSharedPtr<STrack> Track;
};
