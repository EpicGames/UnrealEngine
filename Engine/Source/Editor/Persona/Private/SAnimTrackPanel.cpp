// Copyright Epic Games, Inc. All Rights Reserved.


#include "SAnimTrackPanel.h"

#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "SlotBase.h"
#include "Types/SlateEnums.h"
#include "Types/SlateStructs.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"


#define LOCTEXT_NAMESPACE "AnimTrackPanel"

//////////////////////////////////////////////////////////////////////////
// S2ColumnWidget

void S2ColumnWidget::Construct(const FArguments& InArgs)
{
	this->ChildSlot
		[
			SNew( SBorder )
			.Padding( FMargin(2.f, 2.f) )
			[
				SNew(SHorizontalBox)

				+SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.FillWidth(1)
				[
					SAssignNew(LeftColumn, SVerticalBox)
				]

				+SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Right)
					[
						SNew( SBox )
						.WidthOverride(InArgs._WidgetWidth)
						.HAlign(HAlign_Center)
						[
							SAssignNew(RightColumn,SVerticalBox)
						]
					]
			]
		];
}

//////////////////////////////////////////////////////////////////////////
// SAnimTrackPanel

void SAnimTrackPanel::Construct(const FArguments& InArgs)
{
	ViewInputMin = InArgs._ViewInputMin;
	ViewInputMax = InArgs._ViewInputMax;
	InputMin = InArgs._InputMin;
	InputMax = InArgs._InputMax;
	OnSetInputViewRange = InArgs._OnSetInputViewRange;

	WidgetWidth = InArgs._WidgetWidth;
}

TSharedRef<class S2ColumnWidget> SAnimTrackPanel::Create2ColumnWidget( TSharedRef<SVerticalBox> Parent )
{
	TSharedPtr<S2ColumnWidget> NewTrack;
	Parent->AddSlot()
		.AutoHeight()
		.VAlign(VAlign_Center)
		[
			SAssignNew (NewTrack, S2ColumnWidget)
			.WidgetWidth(WidgetWidth)
		];

	return NewTrack.ToSharedRef();
}

void SAnimTrackPanel::PanInputViewRange(int32 ScreenDelta, FVector2f ScreenViewSize)
{
	FTrackScaleInfo ScaleInfo(ViewInputMin.Get(),  ViewInputMax.Get(), 0.f, 0.f, ScreenViewSize);

	float InputDeltaX = ScaleInfo.PixelsPerInput > 0.0f ? ScreenDelta/ScaleInfo.PixelsPerInput : 0.0f;

	float NewViewInputMin = ViewInputMin.Get() + InputDeltaX;
	float NewViewInputMax = ViewInputMax.Get() + InputDeltaX;

	InputViewRangeChanged(NewViewInputMin, NewViewInputMax);
}

void SAnimTrackPanel::PanInputViewRange(int32 ScreenDelta, FVector2D ScreenViewSize)
{
	PanInputViewRange(ScreenDelta, UE::Slate::CastToVector2f(ScreenViewSize));
}

void SAnimTrackPanel::InputViewRangeChanged(float ViewMin, float ViewMax)
{
	OnSetInputViewRange.ExecuteIfBound(ViewMin, ViewMax);
}

#undef LOCTEXT_NAMESPACE
