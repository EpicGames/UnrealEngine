// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "Containers/Array.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Math/Color.h"
#include "Math/Vector2D.h"
#include "Misc/Attribute.h"
#include "SCurveEditor.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class SVerticalBox;

//////////////////////////////////////////////////////////////////////////
// FTrackColorTracker

class FTrackColorTracker
{
public:
	FTrackColorTracker()
		: CurrentColor(0)
	{
	}

	/** Add a colour to our list of colour options */
	void AddColor(FLinearColor NewColor)
	{
		Colors.Add(NewColor);
	}

	/** return the next track colour to use from our list, looping if necessary */
	FLinearColor GetNextColor()
	{
		FLinearColor& Col = Colors[CurrentColor++];
		CurrentColor %= Colors.Num();
		return Col;
	}

private:

	TArray<FLinearColor>	Colors;
	int32					CurrentColor;
};

//////////////////////////////////////////////////////////////////////////
// S2ColumnWidget
//
// Widget for drawing any number of widgets in 2 columns, useful for track layout
class S2ColumnWidget : public SCompoundWidget
{
public:
	enum EDefaultColumnWidth
	{
		DEFAULT_RIGHT_COLUMN_WIDTH = 176
	};
	TSharedPtr<SVerticalBox> LeftColumn;
	TSharedPtr<SVerticalBox> RightColumn;

	SLATE_BEGIN_ARGS( S2ColumnWidget )
	{}
	SLATE_ARGUMENT( float, WidgetWidth )
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
};

//////////////////////////////////////////////////////////////////////////
// SAnimTrackPanel

class SAnimTrackPanel: public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SAnimTrackPanel )
		: _ViewInputMin()
		, _ViewInputMax()
		, _InputMin()
		, _InputMax()
		, _OnSetInputViewRange()
	{}

	SLATE_ARGUMENT( float, WidgetWidth )
	SLATE_ATTRIBUTE( float, ViewInputMin )
	SLATE_ATTRIBUTE( float, ViewInputMax )
	SLATE_ATTRIBUTE( float, InputMin )
	SLATE_ATTRIBUTE( float, InputMax )
	SLATE_EVENT( FOnSetInputViewRange, OnSetInputViewRange )
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	UE_DEPRECATED(5.6, "Use the helper function accepting FVector2f instead of FVector2D")
	void PanInputViewRange(int32 ScreenDelta, FVector2D ScreenViewSize);
	void PanInputViewRange(int32 ScreenDelta, FVector2f ScreenViewSize);

	virtual float GetSequenceLength() const {return 0.0f;}

protected:

	// Calls out to notify of a range change, can be overridden by derived classes to respond
	// but they must call this version too after processing range changes
	virtual void InputViewRangeChanged(float ViewMin, float ViewMax);

	/** Create a 2 column widget */
	TSharedRef<class S2ColumnWidget> Create2ColumnWidget( TSharedRef<SVerticalBox> Parent );	

	TAttribute<float> ViewInputMin;
	TAttribute<float> ViewInputMax;
	TAttribute<float> InputMin;
	TAttribute<float> InputMax;
	FOnSetInputViewRange OnSetInputViewRange;

	/** Controls the width of the tracks column */
	float WidgetWidth;
};
