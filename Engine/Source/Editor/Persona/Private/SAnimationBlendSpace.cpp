// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAnimationBlendSpace.h"

#include "Widgets/Notifications/SNotificationList.h"
#include "Editor.h"

#include "SlateOptMacros.h"
#include "Fonts/FontMeasure.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SButton.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "DragAndDrop/AssetDragDropOp.h"
#include "ScopedTransaction.h"
#include "AnimPreviewInstance.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Animation/AimOffsetBlendSpace.h"
#include "Animation/AimOffsetBlendSpace1D.h"
#include "SAnimationBlendSpaceGridWidget.h"
#include "Animation/AnimSequenceHelpers.h"
#include "Animation/AnimSequence.h"
#include "Engine/SkeletalMeshSocket.h"
#include "PersonaBlendSpaceAnalysis.h"

#define LOCTEXT_NAMESPACE "BlendSpaceEditor"

SBlendSpaceEditor::~SBlendSpaceEditor()
{
	FCoreUObjectDelegates::OnObjectPropertyChanged.Remove(OnPropertyChangedHandleDelegateHandle);
}

void SBlendSpaceEditor::Construct(const FArguments& InArgs)
{
	BlendSpace = InArgs._BlendSpace;

	OnBlendSpaceSampleAdded = InArgs._OnBlendSpaceSampleAdded;
	OnBlendSpaceSampleDuplicated = InArgs._OnBlendSpaceSampleDuplicated;
	OnBlendSpaceSampleRemoved = InArgs._OnBlendSpaceSampleRemoved;
	OnBlendSpaceSampleReplaced = InArgs._OnBlendSpaceSampleReplaced;
	OnSetPreviewPosition = InArgs._OnSetPreviewPosition;

	bShouldSetPreviewPosition = false;

	SAnimEditorBase::Construct(SAnimEditorBase::FArguments()
		.DisplayAnimTimeline(false)
		.DisplayAnimScrubBar(InArgs._DisplayScrubBar)
		.DisplayAnimScrubBarEditing(false),
		PreviewScenePtr.Pin());

	NonScrollEditorPanels->AddSlot()
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			SNew(SOverlay)
			+ SOverlay::Slot()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.FillWidth(1)
				.Padding(4.0f)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.FillWidth(1)
					.Padding(2) 
					[
						SNew(SVerticalBox)
						// Grid area
						+SVerticalBox::Slot()
						.FillHeight(1)
						[
							SAssignNew(BlendSpaceGridWidget, SBlendSpaceGridWidget)
							.Cursor(EMouseCursor::Crosshairs)
							.BlendSpaceBase(BlendSpace)
							.NotifyHook(this)
							.Position(InArgs._PreviewPosition)
							.FilteredPosition(InArgs._PreviewFilteredPosition)
							.OnSampleMoved(this, &SBlendSpaceEditor::OnSampleMoved)
							.OnSampleRemoved(this, &SBlendSpaceEditor::OnSampleRemoved)
							.OnSampleAdded(this, &SBlendSpaceEditor::OnSampleAdded)
							.OnSampleDuplicated(this, &SBlendSpaceEditor::OnSampleDuplicated)
							.OnSampleReplaced(this, &SBlendSpaceEditor::OnSampleReplaced)
							.OnNavigateUp(InArgs._OnBlendSpaceNavigateUp)
							.OnNavigateDown(InArgs._OnBlendSpaceNavigateDown)
							.OnCanvasDoubleClicked(InArgs._OnBlendSpaceCanvasDoubleClicked)
							.OnSampleDoubleClicked(InArgs._OnBlendSpaceSampleDoubleClicked)
							.OnExtendSampleTooltip(InArgs._OnExtendSampleTooltip)
							.OnGetBlendSpaceSampleName(InArgs._OnGetBlendSpaceSampleName)
							.StatusBarName(InArgs._StatusBarName)
						]
					]
				]
			]
		]
	];

	OnPropertyChangedHandle = FCoreUObjectDelegates::FOnObjectPropertyChanged::FDelegate::CreateRaw(this, &SBlendSpaceEditor::OnPropertyChanged);
	OnPropertyChangedHandleDelegateHandle = FCoreUObjectDelegates::OnObjectPropertyChanged.Add(OnPropertyChangedHandle);

	// Force a resampling of the data on construction - it should be fast, and ensures that the
	// runtime data are in sync with what is in the editor, so there's no surprises if the user
	// changes something. 
	ResampleData();
}

void SBlendSpaceEditor::Construct(const FArguments& InArgs, const TSharedRef<class IPersonaPreviewScene>& InPreviewScene)
{
	PreviewScenePtr = InPreviewScene;

	Construct(InArgs);
}

void SBlendSpaceEditor::OnSampleMoved(const int32 SampleIndex, const FVector& NewValue, bool bIsInteractive)
{
	bool bMoveSuccessful = true;

	// If this is an interactive operation and a transaction has not been opened yet, or if it's a non-interactive operation setup a transaction
	if (!bIsInteractive || !SampleMoveTransaction.IsValid())
	{
		SampleMoveTransaction = MakeUnique<FScopedTransaction>(LOCTEXT("MoveSample", "Moving Blendspace Sample"));		
		BlendSpace->Modify();
	}
	
	if (BlendSpace->IsValidBlendSampleIndex(SampleIndex) && BlendSpace->GetBlendSample(SampleIndex).SampleValue != NewValue && !BlendSpace->IsTooCloseToExistingSamplePoint(NewValue, SampleIndex))
	{
		bMoveSuccessful = BlendSpace->EditSampleValue(SampleIndex, NewValue);
		if (bMoveSuccessful)
		{
			BlendSpace->ValidateSampleData();
			BlendSpace->PostEditChange();
			ResampleData();
		}
	}

	// If this was non-interactive operation, either a single change or final change at the end of an interactive operation close out the transaction	
	if (!bIsInteractive)
	{
		ensure(SampleMoveTransaction.IsValid());
		SampleMoveTransaction.Reset();
	}
}

void SBlendSpaceEditor::OnSampleRemoved(const int32 SampleIndex)
{
	FScopedTransaction ScopedTransaction(LOCTEXT("RemoveSample", "Removing Blendspace Sample"));
	BlendSpace->Modify();

	const bool bRemoveSuccessful = BlendSpace->DeleteSample(SampleIndex);
	if (bRemoveSuccessful)
	{
		ResampleData();
		BlendSpace->ValidateSampleData();

		BlendSpaceGridWidget->InvalidateCachedData();
		BlendSpaceGridWidget->InvalidateState();

		OnBlendSpaceSampleRemoved.ExecuteIfBound(SampleIndex);
		BlendSpace->PostEditChange();
	}
}

//======================================================================================================================
int32 SBlendSpaceEditor::OnSampleAdded(UAnimSequence* Animation, const FVector& Value, bool bRunAnalysis)
{
	FScopedTransaction ScopedTransaction(LOCTEXT("AddSample", "Adding Blendspace Sample"));
	BlendSpace->Modify();

	FVector AdjustedValue = Value;
	bool bAnalyzed[3] = { false, false, false };

	if(BlendSpace->IsAsset() && bRunAnalysis)
	{
		AdjustedValue = BlendSpaceAnalysis::CalculateSampleValue(*BlendSpace, *Animation, 1.0, Value, bAnalyzed);
	}
		
	int32 NewSampleIndex = -1;

	if(BlendSpace->IsAsset())
	{
		NewSampleIndex = BlendSpace->AddSample(Animation, AdjustedValue);
	}
	else
	{
		NewSampleIndex = BlendSpace->AddSample(AdjustedValue);
	}

	if (NewSampleIndex >= 0)
	{
		ResampleData();
		BlendSpace->ValidateSampleData();

		BlendSpaceGridWidget->InvalidateCachedData();
		BlendSpaceGridWidget->InvalidateState();

		if (OnBlendSpaceSampleAdded.IsBound())
		{
			OnBlendSpaceSampleAdded.Execute(Animation, AdjustedValue, bRunAnalysis);
		}
		BlendSpace->PostEditChange();
	}

	return NewSampleIndex;
}

void SBlendSpaceEditor::OnSampleDuplicated(const int32 SampleIndex, const FVector& NewValue, bool bRunAnalysis)
{
	FScopedTransaction ScopedTransaction(LOCTEXT("DuplicateSample", "Duplicating Blendspace Sample"));
	BlendSpace->Modify();

	const FBlendSample& OrigSample = BlendSpace->GetBlendSample(SampleIndex);
	OnSampleAdded(OrigSample.Animation, NewValue, bRunAnalysis);
}

void SBlendSpaceEditor::OnSampleReplaced(int32 InSampleIndex, UAnimSequence* Animation)
{
	FScopedTransaction ScopedTransaction(LOCTEXT("ReplaceSample", "Replacing Blendspace Sample"));
	BlendSpace->Modify();

	bool bUpdateSuccessful = false;
	if(BlendSpace->IsAsset())
	{
		bUpdateSuccessful = BlendSpace->ReplaceSampleAnimation(InSampleIndex, Animation);
	}
	else
	{
		bUpdateSuccessful = true;
	}

	if (bUpdateSuccessful)
	{
		ResampleData();
		BlendSpace->ValidateSampleData();
		OnBlendSpaceSampleReplaced.ExecuteIfBound(InSampleIndex, Animation);
	}
}

void SBlendSpaceEditor::PostUndoRedo()
{
	// Validate and resample blend space data
	BlendSpace->ValidateSampleData();
	ResampleData();

	// Invalidate widget data
	BlendSpaceGridWidget->InvalidateCachedData();

	// Invalidate sample indices used for UI info
	BlendSpaceGridWidget->InvalidateState();

	// Set flag which will update the preview value in the next tick (this due the recreation of data after Undo)
	bShouldSetPreviewPosition = true;
}

TSharedPtr<class IPersonaPreviewScene> SBlendSpaceEditor::GetPreviewScene() const
{
	return PreviewScenePtr.Pin();
}

void SBlendSpaceEditor::UpdatePreviewParameter() const
{
	if(GetPreviewScene().IsValid())
	{
		class UDebugSkelMeshComponent* Component = GetPreviewScene()->GetPreviewMeshComponent();

		if (Component != nullptr && Component->IsPreviewOn())
		{
			if (Component->PreviewInstance->GetCurrentAsset() == BlendSpace)
			{
				const FVector PreviewPosition = BlendSpaceGridWidget->GetPreviewPosition();
				Component->PreviewInstance->SetBlendSpacePosition(PreviewPosition);
				GetPreviewScene()->InvalidateViews();
			}
		}
	}
	else if(OnSetPreviewPosition.IsBound())
	{
		const FVector PreviewPosition = BlendSpaceGridWidget->GetPreviewPosition();
		OnSetPreviewPosition.Execute(PreviewPosition);
	}
}

void SBlendSpaceEditor::UpdateFromBlendSpaceState() const
{
	if (GetPreviewScene().IsValid())
	{
		class UDebugSkelMeshComponent* Component = GetPreviewScene()->GetPreviewMeshComponent();

		if (Component != nullptr && Component->IsPreviewOn())
		{
			if (Component->PreviewInstance->GetCurrentAsset() == BlendSpace)
			{
				FVector FilteredPosition;
				FVector Position;
				Component->PreviewInstance->GetBlendSpaceState(Position, FilteredPosition);
				BlendSpaceGridWidget->SetPreviewingState(Position, FilteredPosition);
			}
		}
	}
}

void SBlendSpaceEditor::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	// Update the preview as long as its enabled
	if (BlendSpaceGridWidget->IsPreviewing() || bShouldSetPreviewPosition)
	{
		UpdatePreviewParameter();
		bShouldSetPreviewPosition = false;
	}

	UpdateFromBlendSpaceState();
}

void SBlendSpaceEditor::OnPropertyChanged(UObject* ObjectBeingModified, FPropertyChangedEvent& PropertyChangedEvent)
{
	if (ObjectBeingModified == BlendSpace)
	{
		BlendSpace->ValidateSampleData();
		ResampleData();
		BlendSpaceGridWidget->InvalidateCachedData();
	}
}

void SBlendSpaceEditor::NotifyPreChange(FProperty* PropertyAboutToChange)
{
	if (BlendSpace)
	{
		BlendSpace->Modify();
	}	
}

void SBlendSpaceEditor::NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged)
{
	if (BlendSpace)
	{
		BlendSpace->ValidateSampleData();
		ResampleData();
		BlendSpace->MarkPackageDirty();
	}
}

void SBlendSpaceEditor::ResampleData()
{
	if (BlendSpace)
	{
		BlendSpace->ResampleData();
	}
}

#undef LOCTEXT_NAMESPACE
