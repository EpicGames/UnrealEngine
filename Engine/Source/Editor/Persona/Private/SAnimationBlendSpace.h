// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "UObject/UObjectGlobals.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "IPersonaPreviewScene.h"
#include "Rendering/DrawElements.h"
#include "SAnimEditorBase.h"
#include "Animation/BlendSpace.h"
#include "Widgets/Input/SCheckBox.h"
#include "Misc/NotifyHook.h"
#include "EditorUndoClient.h"

class FScopedTransaction;
class SBlendSpaceGridWidget;

class SBlendSpaceEditor : public SAnimEditorBase, public FNotifyHook, public FSelfRegisteringEditorUndoClient
{
public:
	SLATE_BEGIN_ARGS(SBlendSpaceEditor)
		: _BlendSpace(nullptr)
		, _DisplayScrubBar(true)
		, _StatusBarName(TEXT("AssetEditor.AnimationEditor.MainMenu"))
		{}
	SLATE_ARGUMENT(UBlendSpace*, BlendSpace)
	SLATE_ARGUMENT(bool, DisplayScrubBar)
	SLATE_EVENT(FOnBlendSpaceNavigateUp, OnBlendSpaceNavigateUp)
	SLATE_EVENT(FOnBlendSpaceNavigateDown, OnBlendSpaceNavigateDown)
	SLATE_EVENT(FOnBlendSpaceCanvasDoubleClicked, OnBlendSpaceCanvasDoubleClicked)
	SLATE_EVENT(FOnBlendSpaceSampleDoubleClicked, OnBlendSpaceSampleDoubleClicked)
	SLATE_EVENT(FOnBlendSpaceSampleAdded, OnBlendSpaceSampleAdded)
	SLATE_EVENT(FOnBlendSpaceSampleDuplicated, OnBlendSpaceSampleDuplicated)
	SLATE_EVENT(FOnBlendSpaceSampleRemoved, OnBlendSpaceSampleRemoved)
	SLATE_EVENT(FOnBlendSpaceSampleReplaced, OnBlendSpaceSampleReplaced)
	SLATE_EVENT(FOnGetBlendSpaceSampleName, OnGetBlendSpaceSampleName) 
	SLATE_EVENT(FOnExtendBlendSpaceSampleTooltip, OnExtendSampleTooltip)
	SLATE_EVENT(FOnSetBlendSpacePreviewPosition, OnSetPreviewPosition)
	SLATE_ATTRIBUTE(FVector, PreviewPosition)
	SLATE_ATTRIBUTE(FVector, PreviewFilteredPosition)
	SLATE_ARGUMENT(FName, StatusBarName)
	SLATE_END_ARGS()

	~SBlendSpaceEditor();

	void Construct(const FArguments& InArgs);

	void Construct(const FArguments& InArgs, const TSharedRef<class IPersonaPreviewScene>& InPreviewScene);
	
	// Begin SWidget overrides
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	// End SWidget overrides

	// Begin FNotifyHook overrides
	virtual void NotifyPreChange(FProperty* PropertyAboutToChange) override;
	virtual void NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged) override;
	// End FNotifyHook overrides

protected:
	void OnSampleMoved(const int32 SampleIndex, const FVector& NewValue, bool bIsInteractive);
	void OnSampleRemoved(const int32 SampleIndex);
	int32 OnSampleAdded(UAnimSequence* Animation, const FVector& Value, bool bRunAnalysis);
	void OnSampleDuplicated(const int32 SampleIndex, const FVector& NewValue, bool bRunAnalysis);
	void OnSampleReplaced(const int32 SampleIndex, UAnimSequence* Animation);

	// Begin SAnimEditorBase overrides
	virtual UAnimationAsset* GetEditorObject() const override { return BlendSpace; }
	// End SAnimEditorBase overrides

	// FSelfRegisteringEditorUndoClient interface
	virtual void PostUndo(bool bSuccess) override { PostUndoRedo(); }
	virtual void PostRedo(bool bSuccess) override { PostUndoRedo(); }

	/** Delegate which is called when the Editor has performed an undo operation*/
	void PostUndoRedo();

	/** Updates Persona's preview window */
	void UpdatePreviewParameter() const;
	
	/** Updates Persona's preview window with additional data that should always be shown. */
	void UpdateFromBlendSpaceState() const;

	/** Retrieves the preview scene shown by Persona */
	TSharedPtr<class IPersonaPreviewScene> GetPreviewScene() const;

	/** Global callback to anticipate on changes to the blend space */
	void OnPropertyChanged(UObject* ObjectBeingModified, FPropertyChangedEvent& PropertyChangedEvent);

	void ResampleData();

protected:
	/** The blend space being edited */
	UBlendSpace* BlendSpace;

	/** The preview scene we are viewing */
	TWeakPtr<class IPersonaPreviewScene> PreviewScenePtr;
	
	/** Pointer to the grid widget which displays the blendspace visualization */
	TSharedPtr<class SBlendSpaceGridWidget> BlendSpaceGridWidget;	

	// Property changed delegate
	FCoreUObjectDelegates::FOnObjectPropertyChanged::FDelegate OnPropertyChangedHandle;	

	/** Handle to the registered OnPropertyChangedHandle delegate */
	FDelegateHandle OnPropertyChangedHandleDelegateHandle;

	/** Delegate called when a sample is added */
	FOnBlendSpaceSampleAdded OnBlendSpaceSampleAdded;

	/** Delegate called when a sample is duplicated */
	FOnBlendSpaceSampleDuplicated OnBlendSpaceSampleDuplicated;

	/** Delegate called when a sample is removed */
	FOnBlendSpaceSampleRemoved OnBlendSpaceSampleRemoved;

	/** Delegate called when a sample is replaced */
	FOnBlendSpaceSampleReplaced OnBlendSpaceSampleReplaced;

	/** Delegate called to externally control the preview position */
	FOnSetBlendSpacePreviewPosition OnSetPreviewPosition;

	/** Flag to check whether or not the preview value should be (re-)set on the next tick */
	bool bShouldSetPreviewPosition = true;

	/** Cross-framed boundary scoped transaction, used for sticking to single transaction for interactive operations (dragging sample or spinbox) */
	TUniquePtr<FScopedTransaction> SampleMoveTransaction;
};
