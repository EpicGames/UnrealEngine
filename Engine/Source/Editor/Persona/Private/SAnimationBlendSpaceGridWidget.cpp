// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAnimationBlendSpaceGridWidget.h"

#include "Animation/AnimSequence.h"
#include "Animation/BlendSpace.h"
#include "Animation/BlendSpace1D.h"

#include "Widgets/Input/SButton.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Rendering/DrawElements.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/SToolTip.h"

#include "IDetailsView.h"
#include "UObject/StructOnScope.h"
#include "Styling/AppStyle.h"
#include "PropertyEditorModule.h"
#include "IStructureDetailsView.h"

#include "Customization/BlendSampleDetails.h"
#include "AssetRegistry/AssetData.h"
#include "DragAndDrop/AssetDragDropOp.h"
#include "Settings/EditorStyleSettings.h"

#include "Widgets/Input/SButton.h"
#include "Fonts/FontMeasure.h"
#include "Modules/ModuleManager.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Styling/StyleColors.h"

#include "JsonObjectConverter.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Misc/StringOutputDevice.h"
#include "AnimGraphNode_BlendSpaceGraphBase.h"
#include "BlendSpaceGraph.h"
#include "AnimationBlendSpaceSampleGraph.h"
#include "EdGraphUtilities.h"
#include "AnimGraphNode_BlendSpaceSampleResult.h"
#include "ScopedTransaction.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/Commands/UICommandList.h"

//UE_DISABLE_OPTIMIZATION

#define LOCTEXT_NAMESPACE "SAnimationBlendSpaceGridWidget"

// Draws additional data on the triangulation to help debugging
//#define DEBUG_BLENDSPACE_TRIANGULATION

// Threshold for it being considered a problem that when a lookup is made at the same location as a
// sample, the returned weight is less than this.
static const float SampleLookupWeightThreshold = 0.2f;

// Flag any triangle that has an interior angle smaller than this (degrees), calculated from the
// normalized positions
static const float CriticalTriangulationAngle = 1.0f;

// Flag any triangle that has a smaller area than this (normalized units)
static const float CriticalTriangulationArea = 5e-4f;

// Identifies the clipboard contents as being a blend sample
static const FString BlendSampleClipboardHeaderAsset = TEXT("COPY_BLENDSAMPLE_ASSET");
static const FString BlendSampleClipboardHeaderGraph = TEXT("COPY_BLENDSAMPLE_GRAPH");

//======================================================================================================================
// Paint a filled triangle
static void PaintTriangle(
	const FVector2D&         P0,
	const FVector2D&         P1,
	const FVector2D&         P2,
	const FGeometry&         AllottedGeometry,
	FLinearColor             Color,
	const FSlateBrush*       Brush,
	FSlateWindowElementList& OutDrawElements,
	int32                    DrawLayerId)
{
	const FVector2D* Points[3] = { &P0, &P1, &P2 };

	TArray<FSlateVertex> Vertices;
	Vertices.Reserve(3);

	for (int32 PointIndex = 0; PointIndex != 3; ++PointIndex)
	{
		Vertices.AddZeroed();
		FSlateVertex& NewVert = Vertices.Last();
		NewVert.Position = FVector2f(AllottedGeometry.LocalToAbsolute(*Points[PointIndex]));	// LWC_TODO: Precision loss
		NewVert.Color = Color.ToFColor(false);
	}

	// Fill by making triangles
	TArray<SlateIndex> VertexIndices = { 0, 1, 2 };
	FSlateDrawElement::MakeCustomVerts(
		OutDrawElements, DrawLayerId, Brush->GetRenderingResource(), Vertices, VertexIndices, nullptr, 0, 0);
}

//======================================================================================================================
// Paints a filled polygon with outline, defined by a set of points which don't need to be sorted.
// This will handle concave polygons, but only if the centroid lies inside the polygon.
static void PaintPolygon(
	TArray<FVector2D>&       Points,
	const FGeometry&         AllottedGeometry,
	FLinearColor             FillColor,
	FLinearColor             OutlineColor,
	const FSlateBrush*       Brush,
	FSlateWindowElementList& OutDrawElements,
	int32                    DrawLayerId)
{
	TArray<FSlateVertex> Vertices;
	Vertices.Reserve(Points.Num() + 1);

	// Add a mid-position vertex so that we handle polygons that aren't completely convex
	Vertices.AddZeroed();

	FSlateVertex& MidVertex = Vertices.Last();
	for (int32 PointIndex = 0; PointIndex != Points.Num(); ++PointIndex)
	{
		Vertices.AddZeroed();
		FSlateVertex& NewVert = Vertices.Last();
		NewVert.Position = FVector2f(AllottedGeometry.LocalToAbsolute(Points[PointIndex]));	// LWC_TODO: Precision loss
		NewVert.Color = FillColor.ToFColor(false);
		MidVertex.Position += NewVert.Position;
	}
	MidVertex.Position /= static_cast<float>(Points.Num());
	MidVertex.Color = FillColor.ToFColor(false);

	// Make sure the points all wind correctly relative to the mid point
	struct FComparePoints
	{
		FComparePoints(const FSlateVertex& Mid) : MidPoint(Mid) {}
		bool operator()(const FSlateVertex& A, const FSlateVertex& B) const
		{
			const FVector2D DeltaA = FVector2D(A.Position) - FVector2D(MidPoint.Position);
			const FVector2D DeltaB = FVector2D(B.Position) - FVector2D(MidPoint.Position);
			const double AngleA = FMath::Atan2(DeltaA.Y, DeltaA.X);
			const double AngleB = FMath::Atan2(DeltaB.Y, DeltaB.X);
			return AngleA < AngleB;
		}
		FSlateVertex MidPoint;
	};
	Algo::Sort(MakeArrayView(Vertices.GetData() + 1, Vertices.Num() - 1), FComparePoints(MidVertex));

	if (FillColor.A > 0)
	{
		// Fill by making triangles
		TArray<SlateIndex> VertexIndices;
		for (int VertexIndex = 1; VertexIndex < Vertices.Num(); ++VertexIndex)
		{
			VertexIndices.Add(0);
			VertexIndices.Add(VertexIndex);
			VertexIndices.Add(VertexIndex + 1 >= Vertices.Num() ? 1 : VertexIndex + 1);
		}

		FSlateDrawElement::MakeCustomVerts(
			OutDrawElements, DrawLayerId, Brush->GetRenderingResource(), Vertices, VertexIndices, nullptr, 0, 0);
	}

	if (OutlineColor.A > 0)
	{
		TArray<FVector2D> LinePoints;
		LinePoints.Reserve(Points.Num() + 1);
		for (int VertexIndex = 1; VertexIndex <= Vertices.Num(); ++VertexIndex)
		{
			LinePoints.Add(FVector2D(VertexIndex < Vertices.Num() ? Vertices[VertexIndex].Position : Vertices[1].Position));
		}
		FSlateDrawElement::MakeLines(
			OutDrawElements, DrawLayerId + 1, FPaintGeometry(), LinePoints, ESlateDrawEffect::None, OutlineColor, true, 1.0f);
	}
}

//======================================================================================================================
static void PaintCircle(
	const FVector2D&         Centre,
	const float              Radius,
	const int32              NumVerts,
	const FGeometry&         AllottedGeometry,
	FLinearColor             FillColor,
	FLinearColor             OutlineColor,
	const FSlateBrush*       Brush,
	FSlateWindowElementList& OutDrawElements,
	int32                    DrawLayerId)
{
	// NumVerts needs to be a multiple of 4
	int32 NumVertsPerSector = ((NumVerts + 3) / 4);
	TArray<FVector2D> Points;
	Points.SetNum(NumVertsPerSector * 4);
	for (int32 Index = 0 ; Index != NumVertsPerSector ; ++Index)
	{
		float Angle = Index * (HALF_PI / NumVertsPerSector);
		float S, C;
		FMath::SinCos(&S, &C, Angle);
		Points[Index + NumVertsPerSector * 0] = Centre + FVector2D(C * Radius, S * Radius);
		Points[Index + NumVertsPerSector * 1] = Centre + FVector2D(-S * Radius, C * Radius);
		Points[Index + NumVertsPerSector * 2] = Centre + FVector2D(-C * Radius, -S * Radius);
		Points[Index + NumVertsPerSector * 3] = Centre + FVector2D(S * Radius, -C * Radius);
	}
	PaintPolygon(Points, AllottedGeometry, FillColor, OutlineColor, Brush, OutDrawElements, DrawLayerId);
}

void SBlendSpaceGridWidget::Construct(const FArguments& InArgs)
{
	BlendSpaceBase = InArgs._BlendSpaceBase;
	PreviousBlendSpaceBase = BlendSpaceBase.Get();
	TargetPosition = InArgs._Position;
	FilteredPosition = InArgs._FilteredPosition;
	NotifyHook = InArgs._NotifyHook;
	OnSampleAdded = InArgs._OnSampleAdded;
	OnSampleDuplicated = InArgs._OnSampleDuplicated;
	OnSampleMoved = InArgs._OnSampleMoved;
	OnSampleRemoved = InArgs._OnSampleRemoved;
	OnSampleReplaced = InArgs._OnSampleReplaced;
	OnNavigateUp = InArgs._OnNavigateUp;
	OnNavigateDown = InArgs._OnNavigateDown;
	OnCanvasDoubleClicked = InArgs._OnCanvasDoubleClicked;
	OnSampleDoubleClicked = InArgs._OnSampleDoubleClicked;
	OnGetBlendSpaceSampleName = InArgs._OnGetBlendSpaceSampleName;
	OnExtendSampleTooltip = InArgs._OnExtendSampleTooltip;
	bReadOnly = InArgs._ReadOnly;
	bShowAxisLabels = InArgs._ShowAxisLabels;
	bShowSettingsButtons = InArgs._ShowSettingsButtons;
	StatusBarName = InArgs._StatusBarName;

	GridType = (BlendSpaceBase.Get() != nullptr && BlendSpaceBase.Get()->IsA<UBlendSpace1D>()) ? EGridType::SingleAxis : EGridType::TwoAxis;
	BlendParametersToDraw = (GridType == EGridType::SingleAxis) ? 1 : 2;
	
	HighlightedSampleIndex = SelectedSampleIndex = DraggedSampleIndex = ToolTipSampleIndex = INDEX_NONE;
	DragState = EDragState::None;
	// Initialize flags 
	bHighlightPreviewPin = false;
	// Initialize preview value to center or the grid
	PreviewPosition.X = BlendSpaceBase.Get() != nullptr ? (BlendSpaceBase.Get()->GetBlendParameter(0).GetRange() * .5f) + BlendSpaceBase.Get()->GetBlendParameter(0).Min : 0.0f;
	PreviewPosition.Y = BlendSpaceBase.Get() != nullptr ? (GridType == EGridType::TwoAxis ? (BlendSpaceBase.Get()->GetBlendParameter(1).GetRange() * .5f) + BlendSpaceBase.Get()->GetBlendParameter(1).Min : 0.0f) : 0.0f;
	PreviewPosition.Z = 0.0f;

	PreviewFilteredPosition = PreviewPosition;

	bShowTriangulation = true;
	bMouseIsOverGeometry = false;
	bRefreshCachedData = true;
	bStretchToFit = true;
	bShowAnimationNames = false;

	// Register and bind all our menu commands
	FGenericCommands::Register();
	BindCommands();

	InvalidSamplePositionDragDropText = FText::FromString(TEXT("Invalid Sample Position"));

	// Retrieve UI color values
	KeyColor = FAppStyle::GetSlateColor("BlendSpaceKey.Regular");
	HighlightKeyColor = FAppStyle::GetSlateColor("BlendSpaceKey.Highlight");
	SelectKeyColor = FAppStyle::GetSlateColor("BlendSpaceKey.Pressed");
	PreDragKeyColor = FAppStyle::GetSlateColor("BlendSpaceKey.Pressed");
	DragKeyColor = FAppStyle::GetSlateColor("BlendSpaceKey.Drag");
	InvalidColor = FAppStyle::GetSlateColor("BlendSpaceKey.Invalid");
	DropKeyColor = FAppStyle::GetSlateColor("BlendSpaceKey.Drop");
	PreviewKeyColor = FAppStyle::GetSlateColor("BlendSpaceKey.Preview");
	GridLinesColor = GetDefault<UEditorStyleSettings>()->RegularColor;
	GridOutlineColor = GetDefault<UEditorStyleSettings>()->RuleColor;
	TriangulationColor = FSlateColor(EStyleColor::Foreground);
	TriangulationCurrentColor = FSlateColor(EStyleColor::Highlight);

	// Retrieve background and sample key brushes 
	BackgroundImage = FAppStyle::GetBrush(TEXT("Graph.Panel.SolidBackground"));
	KeyBrush = FAppStyle::GetBrush("CurveEd.CurveKey");
	PreviewBrush = FAppStyle::GetBrush("BlendSpaceEditor.PreviewIcon");
	ArrowBrushes[(uint8)EArrowDirection::Up] = FAppStyle::GetBrush("BlendSpaceEditor.ArrowUp");
	ArrowBrushes[(uint8)EArrowDirection::Down] = FAppStyle::GetBrush("BlendSpaceEditor.ArrowDown");
	ArrowBrushes[(uint8)EArrowDirection::Right] = FAppStyle::GetBrush("BlendSpaceEditor.ArrowRight");
	ArrowBrushes[(uint8)EArrowDirection::Left] = FAppStyle::GetBrush("BlendSpaceEditor.ArrowLeft");
	LabelBrush = FAppStyle::GetBrush(TEXT("BlendSpaceEditor.LabelBackground"));
	
	// Retrieve font data 
	FontInfo = FAppStyle::GetFontStyle("CurveEd.InfoFont");

	// Initialize UI layout values
	KeySize = FVector2D(11.0f, 11.0f);
	PreviewSize = FVector2D(21.0f, 21.0f);
	DragThreshold = 9.0f;
	ClickAndHighlightThreshold = 12.0f;
	TextMargin = 8.0f;
	GridMargin = bShowAxisLabels ?  FMargin(MaxVerticalAxisTextWidth + (TextMargin * 2.0f), TextMargin, (HorizontalAxisMaxTextWidth *.5f) + TextMargin, MaxHorizontalAxisTextHeight + (TextMargin * 2.0f)) : 
									FMargin(TextMargin, TextMargin, TextMargin, TextMargin);

	const bool bShowInputBoxLabel = true;
	// Widget construction
	this->ChildSlot
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()				
				[
					SNew(SBorder)
					.VAlign(VAlign_Top)
					.HAlign(HAlign_Left)
					.BorderImage(FAppStyle::GetBrush("NoBorder"))
					.DesiredSizeScale(FVector2D(1.0f, 1.0f))
					.Padding_Lambda([&]() { return FMargin(GridMargin.Left + 6.f, 0, 0, 0) + GridRatioMargin; })
					[
						SNew(SVerticalBox)
						+ SVerticalBox::Slot()
						.AutoHeight()
						[
							SNew(SHorizontalBox)
							+ SHorizontalBox::Slot() // Button to show triangulation
							.AutoWidth()
							[
								SNew(SBorder)
								.BorderImage(FAppStyle::GetBrush("NoBorder"))
								.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &SBlendSpaceGridWidget::GetTriangulationButtonVisibility)))		
								.VAlign(VAlign_Center)
								[
									SNew(SButton)
									.ToolTipText(LOCTEXT("ShowTriangulation", "Show Triangulation"))
									.OnClicked(this, &SBlendSpaceGridWidget::ToggleTriangulationVisibility)
									.ButtonColorAndOpacity_Lambda([this]() -> FLinearColor { return bShowTriangulation ? FAppStyle::GetSlateColor("SelectionColor").GetSpecifiedColor() : FLinearColor::White; })
									.ContentPadding(1.f)
									[
										SNew(SImage)
										.Image(FAppStyle::GetBrush("BlendSpaceEditor.ToggleTriangulation"))
										.ColorAndOpacity(FSlateColor::UseForeground())
									]
								]
							]
	
							+ SHorizontalBox::Slot() // Button to toggle labels
							.AutoWidth()
							[
								SNew(SBorder)
								.BorderImage(FAppStyle::GetBrush("NoBorder"))
								.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &SBlendSpaceGridWidget::GetAnimationNamesButtonVisibility)))
								.VAlign(VAlign_Center)
								[
									SNew(SButton)
									.ToolTipText(LOCTEXT("ShowAnimationNames", "Show Sample Names"))
									.OnClicked(this, &SBlendSpaceGridWidget::ToggleShowAnimationNames)
									.ButtonColorAndOpacity_Lambda([this]() -> FLinearColor { return bShowAnimationNames ? FAppStyle::GetSlateColor("SelectionColor").GetSpecifiedColor() : FLinearColor::White; })
									.ContentPadding(1.f)
									[
										SNew(SImage)
										.Image(FAppStyle::GetBrush("BlendSpaceEditor.ToggleLabels"))
										.ColorAndOpacity(FSlateColor::UseForeground())
									]
								]
							]

							+ SHorizontalBox::Slot() // Button to fit or stretch the graph
							.AutoWidth()
							[
								SNew(SBorder)
								.BorderImage(FAppStyle::GetBrush("NoBorder"))
								.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &SBlendSpaceGridWidget::GetFittingButtonVisibility)))
								.VAlign(VAlign_Center)
								[
									SNew(SButton)
									.ToolTipText(this, &SBlendSpaceGridWidget::GetFittingTypeButtonToolTipText)
									.OnClicked(this, &SBlendSpaceGridWidget::ToggleFittingType)
									.ContentPadding(1.f)
									.ButtonColorAndOpacity_Lambda([this]() -> FLinearColor { return bStretchToFit ? FAppStyle::GetSlateColor("SelectionColor").GetSpecifiedColor() : FLinearColor::White; })
									[
										SNew(SImage)
										.Image(FAppStyle::GetBrush("BlendSpaceEditor.ZoomToFit"))
										.ColorAndOpacity(FSlateColor::UseForeground())
									]
								]
							]

							+ SHorizontalBox::Slot() // Sample X value input
							.AutoWidth()
							[
								SNew(SBorder)
								.BorderImage(FAppStyle::GetBrush("NoBorder"))
								.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &SBlendSpaceGridWidget::GetInputBoxVisibility, 0)))
								.VAlign(VAlign_Center)
								[
									CreateGridEntryBox(0, bShowInputBoxLabel).ToSharedRef()
								]
							]
	
							+ SHorizontalBox::Slot() // Sample Y value input
							.AutoWidth()
							[
								SNew(SBorder)
								.BorderImage(FAppStyle::GetBrush("NoBorder"))
								.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &SBlendSpaceGridWidget::GetInputBoxVisibility, 1)))
								.VAlign(VAlign_Center)
								[
									CreateGridEntryBox(1, bShowInputBoxLabel).ToSharedRef()
								]
							]
						]
						
						+ SVerticalBox::Slot() // Tip for dragging in, when there are no samples
						.AutoHeight()
						.Padding(FMargin(2.0f, 3.0f, 0.0f, 0.0f ))
						[
							SNew(STextBlock)
							.Text(LOCTEXT("BlendSpaceSamplesToolTip", "Drag and Drop Animations from the Asset Browser to place Sample Points"))
							.Font(FAppStyle::GetFontStyle(TEXT("AnimViewport.MessageFont")))
							.ColorAndOpacity(FLinearColor(1.0f, 1.0f, 1.0f, 0.7f))
							.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &SBlendSpaceGridWidget::GetSampleToolTipVisibility)))
						]

						+ SVerticalBox::Slot() // Tip for adjusting the preview point
						.AutoHeight()
						.Padding(FMargin(2.0f, 3.0f, 0.0f, 0.0f))
						[
							SNew(STextBlock)
							.Text(LOCTEXT("BlendspacePreviewToolTip", "Hold Control to set the Preview Point (Green)" ))
							.Font(FAppStyle::GetFontStyle(TEXT("AnimViewport.MessageFont")))
							.ColorAndOpacity(FLinearColor(1.0f, 1.0f, 1.0f, 0.7f))
							.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &SBlendSpaceGridWidget::GetPreviewToolTipVisibility)))
						]
					]
				]
			]
		]
	];

	SAssignNew(ToolTip, SToolTip)
	.BorderImage(FCoreStyle::Get().GetBrush("ToolTip.Background"))
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(STextBlock)
			.Text(this, &SBlendSpaceGridWidget::GetToolTipAnimationName)
			.Font(FCoreStyle::Get().GetFontStyle("ToolTip.LargerFont"))
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(STextBlock)
			.Text(this, &SBlendSpaceGridWidget::GetToolTipSampleValue)
			.Font(FCoreStyle::Get().GetFontStyle("ToolTip.LargerFont"))
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(STextBlock)
			.Text(this, &SBlendSpaceGridWidget::GetToolTipSampleValidity)
			.Font(FCoreStyle::Get().GetFontStyle("ToolTip.LargerFont"))
			.Visibility_Lambda([this]() { return GetToolTipSampleValidity().IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible; })
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SAssignNew(ToolTipExtensionContainer, SBox)
		]
	];

	if(TargetPosition.IsSet())
	{
		StartPreviewing();
	}
}

SBlendSpaceGridWidget::~SBlendSpaceGridWidget()
{
	EnableStatusBarMessage(false);
}

TSharedPtr<SWidget> SBlendSpaceGridWidget::CreateGridEntryBox(const int32 BoxIndex, const bool bShowLabel)
{
	return SNew(SNumericEntryBox<float>)
		.Font(FAppStyle::GetFontStyle("CurveEd.InfoFont"))
		.Value(this, &SBlendSpaceGridWidget::GetInputBoxValue, BoxIndex)
		.UndeterminedString(LOCTEXT("MultipleValues", "Multiple Values"))
		.OnValueCommitted(this, &SBlendSpaceGridWidget::OnInputBoxValueCommited, BoxIndex)
		.OnValueChanged(this, &SBlendSpaceGridWidget::OnInputBoxValueChanged, BoxIndex, true)
		.OnBeginSliderMovement(this, &SBlendSpaceGridWidget::OnInputSliderBegin, BoxIndex)
		.OnEndSliderMovement(this, &SBlendSpaceGridWidget::OnInputSliderEnd, BoxIndex)
		.LabelVAlign(VAlign_Center)
		.AllowSpin(true)
		.MinValue(this, &SBlendSpaceGridWidget::GetInputBoxMinValue, BoxIndex)
		.MaxValue(this, &SBlendSpaceGridWidget::GetInputBoxMaxValue, BoxIndex)
		.MinSliderValue(this, &SBlendSpaceGridWidget::GetInputBoxMinValue, BoxIndex)
		.MaxSliderValue(this, &SBlendSpaceGridWidget::GetInputBoxMaxValue, BoxIndex)
		.MinDesiredValueWidth(60.0f)
		.Label()
		[
			SNew(STextBlock)
			.Visibility(bShowLabel ? EVisibility::Visible : EVisibility::Collapsed)
			.Text_Lambda([this, BoxIndex]() { return (BoxIndex == 0) ? ParameterXName : ParameterYName; })
		];
}

int32 SBlendSpaceGridWidget::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled && IsEnabled());
	
	PaintBackgroundAndGrid(AllottedGeometry, MyCullingRect, OutDrawElements, LayerId);

#if 0
	// Showing the sample-weights on the grid points is not useful to end users, but can be helpful when debugging
	// the grid-based interpolation.
	PaintGridSampleWeights(AllottedGeometry, MyCullingRect, OutDrawElements, LayerId);
#endif

	PaintTriangulation(AllottedGeometry, MyCullingRect, OutDrawElements, LayerId);
	PaintSampleKeys(AllottedGeometry, MyCullingRect, OutDrawElements, LayerId);

	if(bShowAxisLabels)
	{
		PaintAxisText(AllottedGeometry, MyCullingRect, OutDrawElements, LayerId);
	}

	if (bShowAnimationNames)
	{
		PaintAnimationNames(AllottedGeometry, MyCullingRect, OutDrawElements, LayerId);
	}

	return LayerId;
}

void SBlendSpaceGridWidget::PaintBackgroundAndGrid(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32& DrawLayerId) const
{
	if(const UBlendSpace* BlendSpace = BlendSpaceBase.Get())
	{
		// Create the grid
		const FVector2D GridSize = CachedGridRectangle.GetSize();
		const FVector2D GridOffset = CachedGridRectangle.GetTopLeft();

		// Fill the background of the grid
		FSlateDrawElement::MakeBox( OutDrawElements, DrawLayerId + 1, AllottedGeometry.ToPaintGeometry(GridSize, FSlateLayoutTransform(GridOffset)), BackgroundImage );
		
		TArray<FVector2D> LinePoints;

		// Draw grid lines
		LinePoints.SetNumZeroed(2);
		const FVector2D StartVectors[2] = { FVector2D(1.0f, 0.0f), FVector2D(0.0f, 1.0f) };
		const FVector2D OffsetVectors[2] = { FVector2D(0.0f, GridSize.Y), FVector2D(GridSize.X, 0.0f) };
		for (uint32 ParameterIndex = 0; ParameterIndex < BlendParametersToDraw; ++ParameterIndex)
		{
			const FBlendParameter& BlendParameter = BlendSpace->GetBlendParameter(ParameterIndex);
			const float Steps = static_cast<float>(GridSize[ParameterIndex] / ( BlendParameter.GridNum));

			for (int32 Index = 1; Index < BlendParameter.GridNum; ++Index)
			{			
				// Calculate line points
				LinePoints[0] = ((Index * Steps) * StartVectors[ParameterIndex]) + GridOffset;
				LinePoints[1] = LinePoints[0] + OffsetVectors[ParameterIndex];

				FSlateDrawElement::MakeLines( OutDrawElements, DrawLayerId + 2, AllottedGeometry.ToPaintGeometry(), LinePoints, ESlateDrawEffect::None, GridLinesColor, true);
			}
		}

		// Draw outer grid lines separately (this will avoid missing lines with 1D blend spaces)
		LinePoints.SetNumZeroed(5);

		// Top line
		LinePoints[0] = GridOffset;

		LinePoints[1] = GridOffset;
		LinePoints[1].X += GridSize.X;

		LinePoints[2] = GridOffset;
		LinePoints[2].X += GridSize.X;
		LinePoints[2].Y += GridSize.Y;

		LinePoints[3] = GridOffset;
		LinePoints[3].Y += GridSize.Y;

		LinePoints[4] = GridOffset;

		FSlateDrawElement::MakeLines( OutDrawElements, DrawLayerId + 3, AllottedGeometry.ToPaintGeometry(),	LinePoints, ESlateDrawEffect::None, GridOutlineColor, true, 2.0f);

	}
	
	DrawLayerId += 3;
}

//======================================================================================================================
float SBlendSpaceGridWidget::GetSampleLookupWeight(int32 SampleIndex) const
{
	const UBlendSpace* BlendSpace = BlendSpaceBase.Get();
	if (BlendSpace && BlendSpace->bInterpolateUsingGrid)
	{
		const TArray<FBlendSample>& Samples = BlendSpace->GetBlendSamples();
		const FBlendSample& Sample = Samples[SampleIndex];
			
		TArray<FBlendSampleData> SampleDataList;
		int32 TempTriangulationIndex = -1;
		BlendSpace->GetSamplesFromBlendInput(Sample.SampleValue, SampleDataList, TempTriangulationIndex, true);
		float LookedUpSampleWeight = 0.0f;
		for (const FBlendSampleData& LookedUpSample : SampleDataList)
		{
			if (LookedUpSample.SampleDataIndex == SampleIndex)
			{
				LookedUpSampleWeight += LookedUpSample.GetClampedWeight();
			}
		}
		return FMath::Clamp(LookedUpSampleWeight, 0.0f, 1.0f);
	}
	return 1.0f; // Return 1 to avoid anything treating this as a problem
}


//======================================================================================================================
void SBlendSpaceGridWidget::PaintSampleKeys(
	const FGeometry&         AllottedGeometry, 
	const FSlateRect&        MyCullingRect, 
	FSlateWindowElementList& OutDrawElements, 
	int32&                   DrawLayerId) const
{
	const int32 FilteredPositionLayer = DrawLayerId + 1;
	const int32 PreviewPositionLayer = DrawLayerId + 2;
	const int32 SampleLayer = DrawLayerId + 3;
	
	if(const UBlendSpace* BlendSpace = BlendSpaceBase.Get())
	{
		// Draw keys
		const TArray<FBlendSample>& Samples = BlendSpace->GetBlendSamples();
		for (int32 SampleIndex = 0; SampleIndex < Samples.Num(); ++SampleIndex)
		{
			const FBlendSample& Sample = Samples[SampleIndex];
		
			FLinearColor DrawColor = KeyColor.GetSpecifiedColor();
			if (DraggedSampleIndex == SampleIndex)
			{
				DrawColor = (DragState == EDragState::PreDrag) ? PreDragKeyColor.GetSpecifiedColor() : DragKeyColor.GetSpecifiedColor();
			}
			else if (SelectedSampleIndex == SampleIndex)
			{
				DrawColor = SelectKeyColor.GetSpecifiedColor();
			}
			else if (HighlightedSampleIndex == SampleIndex)
			{
				DrawColor = HighlightKeyColor.GetSpecifiedColor();
			}
			else if(!Sample.bIsValid)
			{
				DrawColor = InvalidColor.GetSpecifiedColor();
			}

			const FVector2D GridPosition = SampleValueToScreenPosition(Sample.SampleValue) - (KeySize * 0.5f);
			FSlateDrawElement::MakeBox(
				OutDrawElements, SampleLayer, AllottedGeometry.ToPaintGeometry(KeySize, FSlateLayoutTransform(GridPosition)), 
				KeyBrush, ESlateDrawEffect::None, DrawColor );

			const float SampleLookupWeight = GetSampleLookupWeight(SampleIndex);
			if (SampleLookupWeight <= SampleLookupWeightThreshold)
			{
				const FVector2D CirclePosition = SampleValueToScreenPosition(Sample.SampleValue);
				FLinearColor IsolatedColor = FLinearColor::Red;
				IsolatedColor.A = SampleLookupWeightThreshold > 0.0f ? 
					(SampleLookupWeightThreshold - SampleLookupWeight) / SampleLookupWeightThreshold :
					1.0f;
				IsolatedColor.A *= 0.4f;
				PaintCircle(CirclePosition, 8.0f, 12, AllottedGeometry, IsolatedColor, IsolatedColor,
							LabelBrush, OutDrawElements, SampleLayer - 1);
			}
		}

		// Always draw the filtered position which comes back from whatever is running
		{
			const FVector2D GridPosition = SampleValueToScreenPosition(PreviewFilteredPosition) - (PreviewSize * .5f);
			FSlateDrawElement::MakeBox(
				OutDrawElements, FilteredPositionLayer, AllottedGeometry.ToPaintGeometry(PreviewSize, FSlateLayoutTransform(GridPosition)), 
				PreviewBrush, ESlateDrawEffect::None, PreviewKeyColor.GetSpecifiedColor() * 0.7f);
		}

		// Always draw the preview position
		{
			const FVector2D GridPosition = SampleValueToScreenPosition(PreviewPosition) - (PreviewSize * .5f);
			FSlateDrawElement::MakeBox(
				OutDrawElements, PreviewPositionLayer, AllottedGeometry.ToPaintGeometry(PreviewSize, FSlateLayoutTransform(GridPosition)), 
				PreviewBrush, ESlateDrawEffect::None, PreviewKeyColor.GetSpecifiedColor());
		}

		if (DragState == EDragState::DragDrop || DragState == EDragState::InvalidDragDrop)
		{
			const FVector2D GridPoint = SnapScreenPositionToGrid(
				LocalMousePosition, FSlateApplication::Get().GetModifierKeys().IsShiftDown()) - (KeySize * .5f);
			FSlateDrawElement::MakeBox(
				OutDrawElements, SampleLayer, AllottedGeometry.ToPaintGeometry(KeySize, FSlateLayoutTransform(GridPoint)), KeyBrush, ESlateDrawEffect::None,
				(DragState == EDragState::DragDrop) ? DropKeyColor.GetSpecifiedColor() : InvalidColor.GetSpecifiedColor() );
		}

		// Also show the weights that are getting picked up as bars, using two overlaid boxes
		if (bSamplePreviewing && FSlateApplication::Get().GetModifierKeys().IsAltDown())
		{
			for (const FBlendSampleData& PreviewedSample : PreviewedSamples)
			{
				float Weight = PreviewedSample.TotalWeight;
				int32 SampleIndex = PreviewedSample.SampleDataIndex;
				FVector2D Point = SampleValueToScreenPosition(Samples[SampleIndex].SampleValue);

				float MaxWeightWidth = 48;
				float WeightHeight = 6;
				float Border = 1.0f;

				Point.Y -= KeySize.Y / 2 + WeightHeight * 1.25;
				Point.X -= MaxWeightWidth * 0.5f;

				FGeometry FillGeometry = AllottedGeometry.MakeChild(
					FVector2D(MaxWeightWidth, WeightHeight),
					FSlateLayoutTransform(FVector2D(Point.X, Point.Y))
				);
				FGeometry BorderGeometry = AllottedGeometry.MakeChild(
					FVector2D(Weight * (MaxWeightWidth - 2 * Border), WeightHeight - 2 * Border),
					FSlateLayoutTransform(FVector2D(Point.X + Border, Point.Y + Border))
				);

				FSlateDrawElement::MakeBox(
					OutDrawElements, DrawLayerId + 1, FillGeometry.ToPaintGeometry(),
					LabelBrush, ESlateDrawEffect::None, FLinearColor::Black);

				FSlateDrawElement::MakeBox(
					OutDrawElements, DrawLayerId + 2, BorderGeometry.ToPaintGeometry(),
					LabelBrush, ESlateDrawEffect::None, FLinearColor::Gray);
			}
		}
	}

	DrawLayerId += 3;
}

//======================================================================================================================
void SBlendSpaceGridWidget::PaintAxisText(
	const FGeometry&         AllottedGeometry, 
	const FSlateRect&        MyCullingRect, 
	FSlateWindowElementList& OutDrawElements, 
	int32&                   DrawLayerId) const
{
	const TSharedRef< FSlateFontMeasure > FontMeasure = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
	const FVector2D GridCenter = CachedGridRectangle.GetCenter();

	// X axis
	FString Text = ParameterXName.ToString();
	FVector2D TextSize = FontMeasure->Measure(Text, FontInfo);

	// arrow left
	
	FVector2D ArrowSize = ArrowBrushes[(uint8)EArrowDirection::Left]->GetImageSize();
	FVector2D TextPosition = FVector2D(GridCenter.X - (TextSize.X * .5f), CachedGridRectangle.Bottom + TextMargin + (ArrowSize.Y * .25f));
	FVector2D ArrowPosition = FVector2D(TextPosition.X - ArrowSize.X - 10.f/* give padding*/, TextPosition.Y);
	FSlateDrawElement::MakeBox(
		OutDrawElements, DrawLayerId + 1, AllottedGeometry.ToPaintGeometry(ArrowSize, FSlateLayoutTransform(ArrowPosition)), 
		ArrowBrushes[(uint8)EArrowDirection::Left], ESlateDrawEffect::None, FLinearColor::White);

	// Label
	FSlateDrawElement::MakeText(
		OutDrawElements, DrawLayerId + 1, AllottedGeometry.MakeChild(FVector2D(1.0f, 1.0f), FSlateLayoutTransform(TextPosition)).ToPaintGeometry(), 
		Text, FontInfo, ESlateDrawEffect::None, FLinearColor::White);

	// arrow right
	ArrowSize = ArrowBrushes[(uint8)EArrowDirection::Right]->GetImageSize();
	ArrowPosition = FVector2D(TextPosition.X + TextSize.X + 10.f/* give padding*/, TextPosition.Y);
	FSlateDrawElement::MakeBox(
		OutDrawElements, DrawLayerId + 1, AllottedGeometry.ToPaintGeometry(ArrowSize, FSlateLayoutTransform(ArrowPosition)), 
		ArrowBrushes[(uint8)EArrowDirection::Right], ESlateDrawEffect::None, FLinearColor::White);

	Text = FString::SanitizeFloat(SampleValueMin.X);
	TextSize = FontMeasure->Measure(Text, FontInfo);

	// Minimum value
	FVector2D MinTextPosition(CachedGridRectangle.Left - (TextSize.X * .5f), CachedGridRectangle.Bottom + TextMargin + (TextSize.Y * .25f));
	FSlateDrawElement::MakeText(
		OutDrawElements, DrawLayerId + 1, AllottedGeometry.MakeChild(FVector2D(1.0f, 1.0f), FSlateLayoutTransform(MinTextPosition)).ToPaintGeometry(),
		Text, FontInfo, ESlateDrawEffect::None, FLinearColor::White);

	Text = FString::SanitizeFloat(SampleValueMax.X);
	TextSize = FontMeasure->Measure(Text, FontInfo);

	// Maximum value
	FVector2D MaxTextPosition(CachedGridRectangle.Right - (TextSize.X * .5f), CachedGridRectangle.Bottom + TextMargin + (TextSize.Y * .25f));
	FSlateDrawElement::MakeText(
		OutDrawElements, DrawLayerId + 1, AllottedGeometry.MakeChild(FVector2D(1.0f, 1.0f), FSlateLayoutTransform(MaxTextPosition)).ToPaintGeometry(),
		Text, FontInfo, ESlateDrawEffect::None, FLinearColor::White);

	// Only draw Y axis labels if this is a 2D grid
	if (GridType == EGridType::TwoAxis)
	{
		// Y axis
		Text = ParameterYName.ToString();
		TextSize = FontMeasure->Measure(Text, FontInfo);

		// arrow up
		ArrowSize = ArrowBrushes[(uint8)EArrowDirection::Up]->GetImageSize();
		TextPosition = FVector2D(((GridMargin.Left - TextSize.X) * 0.5f - (ArrowSize.X * .25f)) + GridRatioMargin.Left, GridCenter.Y - (TextSize.Y * .5f));
		ArrowPosition = FVector2D(TextPosition.X + TextSize.X * 0.5f - ArrowSize.X * 0.5f, TextPosition.Y - ArrowSize.Y - 10.f/* give padding*/);
		FSlateDrawElement::MakeBox(
			OutDrawElements, DrawLayerId + 1, AllottedGeometry.ToPaintGeometry(ArrowSize, FSlateLayoutTransform(ArrowPosition)), 
			ArrowBrushes[(uint8)EArrowDirection::Up], ESlateDrawEffect::None, FLinearColor::White);

		// Label
		FSlateDrawElement::MakeText(
			OutDrawElements, DrawLayerId + 1, AllottedGeometry.MakeChild(FVector2D(1.0f, 1.0f), FSlateLayoutTransform(TextPosition)).ToPaintGeometry(),
			Text,	FontInfo, ESlateDrawEffect::None, FLinearColor::White);

		// arrow down
		ArrowSize = ArrowBrushes[(uint8)EArrowDirection::Down]->GetImageSize();
		ArrowPosition = FVector2D(TextPosition.X + TextSize.X * 0.5f - ArrowSize.X * 0.5f, TextPosition.Y + TextSize.Y + 10.f/* give padding*/);
		FSlateDrawElement::MakeBox(
			OutDrawElements, DrawLayerId + 1, AllottedGeometry.ToPaintGeometry(ArrowSize, FSlateLayoutTransform(ArrowPosition)), 
			ArrowBrushes[(uint8)EArrowDirection::Down], ESlateDrawEffect::None, FLinearColor::White);

		Text = FString::SanitizeFloat(SampleValueMin.Y);
		TextSize = FontMeasure->Measure(Text, FontInfo);

		// Minimum value
		FVector2D MinValuePosition(((GridMargin.Left - TextSize.X) * 0.5f - (TextSize.X * .25f)) + GridRatioMargin.Left, CachedGridRectangle.Bottom - (TextSize.Y * .5f));
		FSlateDrawElement::MakeText(OutDrawElements, DrawLayerId + 1, AllottedGeometry.MakeChild(
				FVector2D(1.0f, 1.0f),
				FSlateLayoutTransform(MinValuePosition)
			).ToPaintGeometry(), 
			Text, FontInfo, ESlateDrawEffect::None, FLinearColor::White);

		Text = FString::SanitizeFloat(SampleValueMax.Y);
		TextSize = FontMeasure->Measure(Text, FontInfo);

		// Maximum value
		FSlateDrawElement::MakeText(OutDrawElements, DrawLayerId + 1, AllottedGeometry.MakeChild(
			FVector2D(1.0f, 1.0f), FSlateLayoutTransform( FVector2D( ((GridMargin.Left - TextSize.X) * 0.5f - (TextSize.X * .25f) ) + GridRatioMargin.Left,  CachedGridRectangle.Top - (TextSize.Y * .5f)) )).ToPaintGeometry(),
			Text, FontInfo, ESlateDrawEffect::None, FLinearColor::White);
	}

	DrawLayerId += 1;
}

//======================================================================================================================
void SBlendSpaceGridWidget::PaintGridSampleWeights(
	const FGeometry& AllottedGeometry,
	const FSlateRect& MyCullingRect,
	FSlateWindowElementList& OutDrawElements,
	int32& DrawLayerId) const
{
	const UBlendSpace* BlendSpace = BlendSpaceBase.Get();
	if (!BlendSpace)
	{
		return;
	}
	if (!BlendSpace->bInterpolateUsingGrid)
	{
		return;
	}

	const TSharedRef< FSlateFontMeasure > FontMeasure = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();

	const TArray<FEditorElement>& GridSamples = BlendSpace->GetGridSamples();
	int32 NumGridSamples = GridSamples.Num();
	const TArray<FBlendSample>& Samples = BlendSpace->GetBlendSamples();
	for (int32 GridSampleIndex = 0 ; GridSampleIndex != NumGridSamples ; ++GridSampleIndex)
	{
		const FEditorElement& EditorElement = GridSamples[GridSampleIndex];

		int32 TextOffset = 0;
		for (int32 ElementIndex = 0 ; ElementIndex != 3 ; ++ElementIndex)
		{
			float SampleWeight = EditorElement.Weights[ElementIndex];
			int32 SampleIndex = EditorElement.Indices[ElementIndex];
			if (SampleWeight <= 0 || SampleIndex < 0)
			{
				continue;
			}

			const FBlendSample& Sample = Samples[SampleIndex];

			const FText Name = FText::Format(LOCTEXT("SampleNameFormatWeight", "{0} ({1}) {2}"), 
				GetSampleName(Sample, SampleIndex), FText::AsNumber(SampleIndex), SampleWeight);
			const FVector2D TextSize = FontMeasure->Measure(Name, FontInfo);
			const FVector2D Padding = FVector2D(12.0f, 4.0f);

			FVector GridSamplePosition = BlendSpace->GetGridPosition(GridSampleIndex);

			// Show the sample name/index/weight, going progressively up (because sample labels are
			// below the grid points)
			FVector2D GridPosition = SampleValueToScreenPosition(GridSamplePosition);
			GridPosition += FVector2D(-TextSize.X / 2, -2 * KeySize.Y);
			GridPosition.Y -= TextSize.Y * TextOffset++;
			GridPosition.X -= Padding.X / 2;
			GridPosition.Y += Padding.Y / 2;

			FSlateDrawElement::MakeText(
				OutDrawElements, DrawLayerId + 2, AllottedGeometry.MakeChild(FVector2f(1.0f, 1.0f),
					FSlateLayoutTransform(GridPosition + Padding / 2)).ToPaintGeometry(), Name, FontInfo, ESlateDrawEffect::None, FLinearColor::White);
		}
	}
}


//======================================================================================================================
void SBlendSpaceGridWidget::PaintTriangulation(
	const FGeometry&         AllottedGeometry, 
	const FSlateRect&        MyCullingRect, 
	FSlateWindowElementList& OutDrawElements, 
	int32&                   DrawLayerId) const
{
	const UBlendSpace* BlendSpace = BlendSpaceBase.Get();
	if (!BlendSpace)
	{
		return;
	}
	if ((bReadOnly && BlendSpace->bInterpolateUsingGrid) || (!bReadOnly && !bShowTriangulation))
	{
		return;
	}

	TArray<FVector2D> PolygonPoints; 
	const TArray<FBlendSample>& Samples = BlendSpace->GetBlendSamples();
	if (!BlendSpace->bInterpolateUsingGrid)
	{
		// Use runtime triangulation
		const FBlendSpaceData& BlendSpaceData = BlendSpace->GetBlendSpaceData();
		for (const FBlendSpaceSegment& Segment : BlendSpaceData.Segments)
		{
			int32 SampleIndex = Segment.SampleIndices[0];
			int32 SampleIndex1 = Segment.SampleIndices[1];
			TArray<FVector2D> Points;
			Points.Add(SampleValueToScreenPosition(Samples[SampleIndex].SampleValue));
			Points.Add(SampleValueToScreenPosition(Samples[SampleIndex1].SampleValue));
			FSlateDrawElement::MakeLines(
				OutDrawElements, DrawLayerId + 1, AllottedGeometry.ToPaintGeometry(), Points,
				ESlateDrawEffect::None, TriangulationColor.GetSpecifiedColor(), true, 0.5f);
		}

		const float CriticalDot = FMath::Cos(FMath::DegreesToRadians(CriticalTriangulationAngle));
		for (const FBlendSpaceTriangle& Triangle : BlendSpaceData.Triangles)
		{
			FLinearColor TriangleFillColor = TriangulationCurrentColor.GetSpecifiedColor();
			FLinearColor TriangleLineColor = TriangulationColor.GetSpecifiedColor();
			TriangleFillColor.A = 0.03f; // Alpha for tinting the triangulation background
			int32 TriangleLayer = DrawLayerId + 1;

			FVector2D ScreenPositions[3] = {
				SampleValueToScreenPosition(Samples[Triangle.SampleIndices[0]].SampleValue),
				SampleValueToScreenPosition(Samples[Triangle.SampleIndices[1]].SampleValue),
				SampleValueToScreenPosition(Samples[Triangle.SampleIndices[2]].SampleValue),
			};

			// Show invalid triangles even if there's only one triangle, because that probably
			// happened when somebody failed to place the sample points in a proper line.
			{
				FVector2D NormalizedPositions[3] = {
					SampleValueToNormalizedPosition(Samples[Triangle.SampleIndices[0]].SampleValue),
					SampleValueToNormalizedPosition(Samples[Triangle.SampleIndices[1]].SampleValue),
					SampleValueToNormalizedPosition(Samples[Triangle.SampleIndices[2]].SampleValue),
				};

				for (int32 Index = 0; Index != 3; ++Index)
				{
					FVector2D A = NormalizedPositions[(Index + 2) % 3] - NormalizedPositions[(Index + 1) % 3];
					FVector2D B = NormalizedPositions[Index] - NormalizedPositions[(Index + 1) % 3];
					double Dot = A.GetSafeNormal() | B.GetSafeNormal();
					double Area = 0.5f * FMath::Abs(B ^ A);
					if (Dot > CriticalDot || Area < CriticalTriangulationArea)
					{
						TriangleFillColor = FLinearColor::Red;
						TriangleFillColor.A = 0.5f;
						TriangleLineColor = FLinearColor::Red;
						TriangleLayer = DrawLayerId + 10; // just bump it up so it definitely shows
					}
				}
			}

			FVector2D MidPoint(0, 0);
			PolygonPoints.Empty(3);
			for (int32 Index0 = 0; Index0 != FBlendSpaceTriangle::NUM_VERTICES; ++Index0)
			{
				int32 Index1 = (Index0 + 1) % FBlendSpaceTriangle::NUM_VERTICES;
				int32 Index2 = (Index0 + 2) % FBlendSpaceTriangle::NUM_VERTICES;
				int32 SampleIndex0 = Triangle.SampleIndices[Index0];
				int32 SampleIndex1 = Triangle.SampleIndices[Index1];
				int32 SampleIndex2 = Triangle.SampleIndices[Index2];

				TArray<FVector2D> Points = { ScreenPositions[Index0], ScreenPositions[Index1] };
				MidPoint += SampleValueToScreenPosition(Samples[SampleIndex0].SampleValue) / 3.0f;

				FSlateDrawElement::MakeLines(
					OutDrawElements, TriangleLayer, AllottedGeometry.ToPaintGeometry(), Points,
					ESlateDrawEffect::None, TriangleLineColor, true, 0.5f);
				PolygonPoints.Push(ScreenPositions[Index0]);
			}

			PaintTriangle(PolygonPoints[0], PolygonPoints[1], PolygonPoints[2], AllottedGeometry, 
							TriangleFillColor, LabelBrush, OutDrawElements, DrawLayerId);

#ifdef DEBUG_BLENDSPACE_TRIANGULATION
			// Draw the adjacent triangle indices around the perimeter
			if (!bSamplePreviewing)
			{
				for (int32 Index0 = 0; Index0 != FBlendSpaceTriangle::NUM_VERTICES; ++Index0)
				{
					if (Triangle.EdgeInfo[Index0].NeighbourTriangleIndex < 0)
					{
						int32 Index1 = (Index0 + 1) % FBlendSpaceTriangle::NUM_VERTICES;
						FVector2D MidEdge = ( ScreenPositions[Index0] + ScreenPositions[Index1]) * 0.5;
						float PullInAmount = 0.2;
						FSlateDrawElement::MakeText(
							OutDrawElements, DrawLayerId + 1, AllottedGeometry.MakeChild(
								FMath::Lerp(ScreenPositions[Index0], MidEdge, PullInAmount), FVector2D(1.0f, 1.0f)).ToPaintGeometry(),
							FText::AsNumber(Triangle.EdgeInfo[Index0].AdjacentPerimeterTriangleIndices[0]),
							FontInfo, ESlateDrawEffect::None, FLinearColor::Red);

						FSlateDrawElement::MakeText(
							OutDrawElements, DrawLayerId + 1, AllottedGeometry.MakeChild(
								FMath::Lerp(ScreenPositions[Index1], MidEdge, PullInAmount), FVector2D(1.0f, 1.0f)).ToPaintGeometry(),
							FText::AsNumber(Triangle.EdgeInfo[Index0].AdjacentPerimeterTriangleIndices[1]),
							FontInfo, ESlateDrawEffect::None, FLinearColor::Red);
					}
				}
			}
#endif

#ifdef DEBUG_BLENDSPACE_TRIANGULATION
			// Draw the triangle indices for debugging
			FText Text = FText::AsNumber(&Triangle - &BlendSpaceData.Triangles[0]);
			FSlateDrawElement::MakeText(
				OutDrawElements, DrawLayerId + 1, AllottedGeometry.MakeChild(
					FVector2D(MidPoint.X, MidPoint.Y), FVector2D(1.0f, 1.0f)).ToPaintGeometry(),
				Text, FontInfo, ESlateDrawEffect::None, FLinearColor::Gray);
#endif
		}
	}

	// Draw the current triangle (or polygon)
	if (bSamplePreviewing && FSlateApplication::Get().GetModifierKeys().IsAltDown())
	{
		PolygonPoints.Empty(3);
		for (const FBlendSampleData& PreviewedSample : PreviewedSamples)
		{
			float Weight = PreviewedSample.TotalWeight;
			if (Weight)
			{
				int32 SampleIndex = PreviewedSample.SampleDataIndex;
				FVector2D Point = SampleValueToScreenPosition(Samples[SampleIndex].SampleValue);
				PolygonPoints.Push(Point);
			}
		}
		if (PolygonPoints.Num())
		{
			FLinearColor FillColor = TriangulationCurrentColor.GetSpecifiedColor();
			FillColor.A = 0.2f; // Alpha for the current triangulation triangle
			FLinearColor OutlineColor = FillColor;
			OutlineColor.A = 0.5f;
			PaintPolygon(PolygonPoints, AllottedGeometry, FillColor, OutlineColor, LabelBrush, OutDrawElements, DrawLayerId);
		}
	}

	DrawLayerId += 1;
}

FText SBlendSpaceGridWidget::GetSampleName(const FBlendSample& InBlendSample, int32 InSampleIndex) const
{
	if(OnGetBlendSpaceSampleName.IsBound())
	{
		return FText::FromName(OnGetBlendSpaceSampleName.Execute(InSampleIndex));
	}
	else
	{
		if(InBlendSample.Animation != nullptr)
		{
			return FText::FromString(InBlendSample.Animation->GetName());
		}
	}
		
	return LOCTEXT("NoAnimationSetTooltipText", "No Animation Set");
}

void SBlendSpaceGridWidget::PaintAnimationNames(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32& DrawLayerId) const
{
	if(const UBlendSpace* BlendSpace = BlendSpaceBase.Get())
	{
		const TSharedRef< FSlateFontMeasure > FontMeasure = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
		const TArray<FBlendSample>& Samples = BlendSpace->GetBlendSamples();
		for (int32 SampleIndex = 0; SampleIndex < Samples.Num(); ++SampleIndex)
		{
			const FBlendSample& Sample = Samples[SampleIndex];

			const FText Name = !Sample.bUseSingleFrameForBlending ? 
				FText::Format(LOCTEXT("SampleNameFormat", "{0} ({1})"), GetSampleName(Sample, SampleIndex), FText::AsNumber(SampleIndex)) : 
				FText::Format(LOCTEXT("FrameIndexSampleNameFormat", "{0} ({1}) (Frame {2})"), GetSampleName(Sample, SampleIndex), FText::AsNumber(SampleIndex), FText::AsNumber(Sample.FrameIndexToSample));

			const FVector2D TextSize = FontMeasure->Measure(Name, FontInfo);
			const FVector2D Padding = FVector2D(12.0f, 4.0f);

			FVector2D GridPosition = SampleValueToScreenPosition(Sample.SampleValue);
			GridPosition += FVector2D(-TextSize.X / 2, KeySize.Y / 2);
			GridPosition.X -= Padding.X / 2;
			GridPosition.Y += Padding.Y / 2;

			FSlateDrawElement::MakeBox(
				OutDrawElements, DrawLayerId + 1, AllottedGeometry.MakeChild(TextSize + Padding, FSlateLayoutTransform(GridPosition)).ToPaintGeometry(),
				LabelBrush, ESlateDrawEffect::None, FLinearColor::Black);
			FSlateDrawElement::MakeText(
				OutDrawElements, DrawLayerId + 2, AllottedGeometry.MakeChild(FVector2D(1.0f, 1.0f), FSlateLayoutTransform(GridPosition + Padding/2)).ToPaintGeometry(),
				Name, FontInfo, ESlateDrawEffect::None, FLinearColor::White);
		}
	}

	DrawLayerId += 2;
}

FReply SBlendSpaceGridWidget::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	if(!bReadOnly && BlendSpaceBase.IsSet())
	{
		// Check if we are in dropping state and if so snap to the grid and try to add the sample
		if (DragState == EDragState::DragDrop || DragState == EDragState::InvalidDragDrop || DragState == EDragState::DragDropOverride)
		{
			if (DragState == EDragState::DragDrop)
			{
				TSharedPtr<FAssetDragDropOp> DragDropOperation = DragDropEvent.GetOperationAs<FAssetDragDropOp>();
				if (DragDropOperation.IsValid())
				{
					const FVector SampleValue = ScreenPositionToSampleValueWithSnapping(
						LocalMousePosition, FSlateApplication::Get().GetModifierKeys().IsShiftDown());

					const TArray<FAssetData>& Assets = DragDropOperation->GetAssets();
					for (int Index = 0 ; Index != Assets.Num() ; ++Index)
					{
						const FAssetData& AssetData = Assets[Index];
						UObject* Asset = AssetData.GetAsset();
						UAnimSequence* Animation = (UAnimSequence*) Asset;
						if (OnSampleAdded.IsBound())
						{
							OnSampleAdded.Execute(Animation, SampleValue, true);
						}
					}
				}	
			}
			else if (DragState == EDragState::DragDropOverride)
			{
				TSharedPtr<FAssetDragDropOp> DragDropOperation = DragDropEvent.GetOperationAs<FAssetDragDropOp>();
				if (DragDropOperation.IsValid())
				{
					UAnimSequence* Animation = FAssetData::GetFirstAsset<UAnimSequence>(DragDropOperation->GetAssets());
					int32 DroppedSampleIndex = GetClosestSamplePointIndexToMouse();
					OnSampleReplaced.ExecuteIfBound(DroppedSampleIndex, Animation);
				}
			}

			DragState = EDragState::None;
		}

		DragDropAnimationSequence = nullptr;
		DragDropAnimationName = FText::GetEmpty();
		HoveredAnimationName = FText::GetEmpty();
	}

	return FReply::Unhandled();
}

void SBlendSpaceGridWidget::OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	if(!bReadOnly && BlendSpaceBase.IsSet())
	{
		if (DragDropEvent.GetOperationAs<FAssetDragDropOp>().IsValid())
		{
			DragState = IsValidDragDropOperation(DragDropEvent, InvalidDragDropText) ? EDragState::DragDrop : EDragState::InvalidDragDrop;
		}
	}
}

FReply SBlendSpaceGridWidget::OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	if(!bReadOnly && BlendSpaceBase.IsSet())
	{
		if (DragState == EDragState::DragDrop || DragState == EDragState::InvalidDragDrop || DragState == EDragState::DragDropOverride)
		{		
			LocalMousePosition = MyGeometry.AbsoluteToLocal(DragDropEvent.GetScreenSpacePosition());				
	
			// Always update the tool tip, in case it became invalid
			TSharedPtr<FAssetDragDropOp> DragDropOperation = DragDropEvent.GetOperationAs<FAssetDragDropOp>();
			if (DragDropOperation.IsValid())
			{
				DragDropOperation->SetToolTip(GetToolTipSampleValue(), DragDropOperation->GetIcon());
			}		

			return FReply::Handled();
		}
	}
	return FReply::Unhandled();
}

void SBlendSpaceGridWidget::OnDragLeave(const FDragDropEvent& DragDropEvent)
{
	if(!bReadOnly && BlendSpaceBase.IsSet())
	{
		if (DragState == EDragState::DragDrop || DragState == EDragState::InvalidDragDrop || DragState == EDragState::DragDropOverride)
		{
			DragState = EDragState::None;
			DragDropAnimationSequence = nullptr;
			DragDropAnimationName = FText::GetEmpty();
			HoveredAnimationName = FText::GetEmpty();
		}
	}
}

FReply SBlendSpaceGridWidget::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if(!bReadOnly && BlendSpaceBase.IsSet())
	{
		if (this->HasMouseCapture())
		{
			if (DragState == EDragState::None || DragState == EDragState::PreDrag)
			{
				ProcessClick(MyGeometry, MouseEvent);
			}
			else if (DragState == EDragState::DragSample)
			{
				// Process drag ending			
				ResetToolTip();
				OnSampleMoved.ExecuteIfBound(DraggedSampleIndex, LastDragPosition, false);
			}

			// Reset drag state and index
			DragState = EDragState::None;
			DraggedSampleIndex = INDEX_NONE;

			return FReply::Handled().ReleaseMouseCapture();
		}
		else
		{
			return ProcessClick(MyGeometry, MouseEvent);
		}
	}

	return FReply::Unhandled();
}

FReply SBlendSpaceGridWidget::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if(!bReadOnly && BlendSpaceBase.IsSet())
	{
		if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
		{
			// If we are over a sample, make it our currently (dragged) sample
			if (HighlightedSampleIndex != INDEX_NONE)
			{
				DraggedSampleIndex = SelectedSampleIndex = HighlightedSampleIndex;
				HighlightedSampleIndex = INDEX_NONE;
				ResetToolTip();
				DragState = EDragState::PreDrag;
				MouseDownPosition = LocalMousePosition;

				// Start mouse capture
				return FReply::Handled().CaptureMouse(SharedThis(this));
			}		
		}

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FReply SBlendSpaceGridWidget::OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	if(!bReadOnly && BlendSpaceBase.IsSet())
	{
		if (InMouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
		{
			if (SelectedSampleIndex != INDEX_NONE)
			{
				OnSampleDoubleClicked.ExecuteIfBound(SelectedSampleIndex);
			}
			else
			{
				OnCanvasDoubleClicked.ExecuteIfBound();
			}
			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

FReply SBlendSpaceGridWidget::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if(!bReadOnly && BlendSpaceBase.IsSet())
	{
		EnableStatusBarMessage(true);
	}

	// Cache the mouse position in local and screen space
	LocalMousePosition = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
	LastMousePosition = MouseEvent.GetScreenSpacePosition();

	if(!bReadOnly)
	{
		if (this->HasMouseCapture())
		{
			if (DragState == EDragState::None)
			{
				if (HighlightedSampleIndex != INDEX_NONE)
				{
					DragState = EDragState::DragSample;
					DraggedSampleIndex = HighlightedSampleIndex;
					HighlightedSampleIndex = INDEX_NONE;
					return FReply::Handled();
				}
			}
			else if (DragState == EDragState::PreDrag)
			{
				// Actually start dragging
				if ((LocalMousePosition - MouseDownPosition).SizeSquared() > DragThreshold)
				{
					DragState = EDragState::DragSample;
					HighlightedSampleIndex = INDEX_NONE;
					ShowToolTip();
					return FReply::Handled();
				}
			}
		}
		else if (IsHovered() && bMouseIsOverGeometry)
		{
			if (MouseEvent.IsControlDown())
			{
				StartPreviewing();
				DragState = EDragState::Preview;
				// Make tool tip visible (this will display the current preview sample value)
				ShowToolTip();			

				// Set flag for showing advanced preview info in tooltip
				bAdvancedPreview = MouseEvent.IsAltDown();
				return FReply::Handled();
			}
			else if(TargetPosition.IsSet())
			{
				StartPreviewing();
				DragState = EDragState::None;
				ShowToolTip();

				// Set flag for showing advanced preview info in tooltip
				bAdvancedPreview = MouseEvent.IsAltDown();
				return FReply::Handled();
			}
			else if (bSamplePreviewing)
			{
				StopPreviewing();
				DragState = EDragState::None;
				ResetToolTip();
				return FReply::Handled();
			}
		}
	}

	return FReply::Unhandled();
}

FReply SBlendSpaceGridWidget::ProcessClick(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if(!bReadOnly && BlendSpaceBase.IsSet())
	{
		if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
		{
			SelectedSampleIndex = INDEX_NONE;

			if (HighlightedSampleIndex == INDEX_NONE)
			{
				// If there isn't any sample currently being highlighted, retrieve all of them and see if we are over one
				SelectedSampleIndex = GetClosestSamplePointIndexToMouse();
			}
			else
			{
				// If we are over a sample, make it the selected sample index
				SelectedSampleIndex = HighlightedSampleIndex;
				HighlightedSampleIndex = INDEX_NONE;			
			}
		}
		else if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
		{
			auto PushMenu = [this, &MouseEvent](TSharedPtr<SWidget> InMenuContent)
			{
				if (InMenuContent.IsValid())
				{
					const FWidgetPath WidgetPath = MouseEvent.GetEventPath() != nullptr ? *MouseEvent.GetEventPath() : FWidgetPath();
					const FVector2D MousePosition = MouseEvent.GetScreenSpacePosition();
					// This is of a fixed size atm since MenuContent->GetDesiredSize() will not take the detail 
					// customization into account and return an incorrect (small) size
					const FVector2D ExpectedSize(300, 300);
					const FVector2D MenuPosition = FSlateApplication::Get().CalculatePopupWindowPosition(
						FSlateRect(static_cast<float>(MousePosition.X), static_cast<float>(MousePosition.Y), static_cast<float>(MousePosition.X), static_cast<float>(MousePosition.Y)), ExpectedSize, false);

					FSlateApplication::Get().PushMenu(
						AsShared(),
						WidgetPath,
						InMenuContent.ToSharedRef(),
						MenuPosition,
						FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu)
						);
				}
			};

			// If we are over a sample open a context menu for editing its data
			if (HighlightedSampleIndex != INDEX_NONE)
			{	
				SelectedSampleIndex = HighlightedSampleIndex;

				// Create context menu
				TSharedPtr<SWidget> MenuContent = CreateBlendSampleContextMenu();
		
				// Reset highlight sample index
				HighlightedSampleIndex = INDEX_NONE;

				PushMenu(MenuContent);

				return FReply::Handled().SetUserFocus(MenuContent.ToSharedRef(), EFocusCause::SetDirectly).ReleaseMouseCapture();
			}
			else
			{
				TSharedPtr<SWidget> MenuContent = CreateNewBlendSampleContextMenu(MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition()));
			
				PushMenu(MenuContent);
			
				return FReply::Handled().SetUserFocus(MenuContent.ToSharedRef(), EFocusCause::SetDirectly).ReleaseMouseCapture();
			}
		}
	}

	return FReply::Unhandled();
}

FReply SBlendSpaceGridWidget::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if(!bReadOnly && BlendSpaceBase.IsSet())
	{
		if (UICommandList->ProcessCommandBindings(InKeyEvent))
		{
			return FReply::Handled();
		}

		// Start previewing when either one of the shift keys is pressed
		if (IsHovered() && bMouseIsOverGeometry)
		{
			if ((InKeyEvent.GetKey() == EKeys::LeftControl) || (InKeyEvent.GetKey() == EKeys::RightControl))
			{
				StartPreviewing();
				DragState = EDragState::Preview;
				// Make tool tip visible (this will display the current preview sample value)
				ShowToolTip();
				return FReply::Handled();
			}
		
			// Set flag for showing advanced preview info in tooltip
			if ((InKeyEvent.GetKey() == EKeys::LeftAlt) || (InKeyEvent.GetKey() == EKeys::RightAlt))
			{
				bAdvancedPreview = true;
				return FReply::Handled();
			}
			
			if (InKeyEvent.GetKey() == EKeys::PageUp)
			{
				OnNavigateUp.ExecuteIfBound();
				return FReply::Handled();
			}
			else if (InKeyEvent.GetKey() == EKeys::PageDown)
			{
				OnNavigateDown.ExecuteIfBound();
				return FReply::Handled();
			}
		}
	}

	return FReply::Unhandled();
}

FReply SBlendSpaceGridWidget::OnKeyUp(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{	
	if(!bReadOnly && BlendSpaceBase.IsSet())
	{
		// Stop previewing when shift keys are released 
		if ((InKeyEvent.GetKey() == EKeys::LeftControl) || (InKeyEvent.GetKey() == EKeys::RightControl))
		{
			StopPreviewing();
			DragState = EDragState::None;
			ResetToolTip();
			return FReply::Handled();
		}

		if((InKeyEvent.GetKey() == EKeys::LeftAlt) || (InKeyEvent.GetKey() == EKeys::RightAlt))
		{
			bAdvancedPreview = false;
			return FReply::Handled();
		}

		// Pressing esc will remove the current key selection
		if( InKeyEvent.GetKey() == EKeys::Escape)
		{
			SelectedSampleIndex = INDEX_NONE;
		}
	}

	return FReply::Unhandled();
}

void SBlendSpaceGridWidget::MakeViewContextMenuEntries(FMenuBuilder& InMenuBuilder)
{
	InMenuBuilder.BeginSection("ViewOptions", LOCTEXT("ViewOptionsMenuHeader", "View Options"));
	{
		if (GetTriangulationButtonVisibility() == EVisibility::Visible)
		{
			TAttribute<FText> ShowTriangulation = TAttribute<FText>::Create(
				[this]()
				{
					return (BlendSpaceBase.Get() && BlendSpaceBase.Get()->bInterpolateUsingGrid) 
						? LOCTEXT("ShowGridToSampleConnections", "Show Grid/Sample Connections")
						: LOCTEXT("ShowTriangulation", "Show Triangulation");
				});
			TAttribute<FText> ShowTriangulationToolTip = TAttribute<FText>::Create(
				[this]()
				{
					return (BlendSpaceBase.Get() && BlendSpaceBase.Get()->bInterpolateUsingGrid) 
						? LOCTEXT("ShowGridToSampleConnectionsToolTip", "Show which samples each grid point is associated with")
						: LOCTEXT("ShowTriangulationToolTip", "Show the Delaunay triangulation for all blend space samples");
				});

			InMenuBuilder.AddMenuEntry(
				ShowTriangulation,
				ShowTriangulationToolTip,
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "BlendSpaceEditor.ToggleTriangulation"),
				FUIAction(
					FExecuteAction::CreateLambda([this](){ bShowTriangulation = !bShowTriangulation; }),
					FCanExecuteAction(),
					FGetActionCheckState::CreateLambda([this](){ return bShowTriangulation ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
				),
				NAME_None,
				EUserInterfaceActionType::ToggleButton
			);
		}

		InMenuBuilder.AddMenuEntry(
			LOCTEXT("ShowAnimationNames", "Show Sample Names"),
			LOCTEXT("ShowAnimationNamesToolTip", "Show the names of each of the samples"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "BlendSpaceEditor.ToggleLabels"),
			FUIAction(
				FExecuteAction::CreateLambda([this](){ bShowAnimationNames = !bShowAnimationNames; }),
				FCanExecuteAction(),
				FGetActionCheckState::CreateLambda([this](){ return bShowAnimationNames ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);

		InMenuBuilder.AddMenuEntry(
			LOCTEXT("StretchFittingText", "Stretch Grid to Fit"),
			LOCTEXT("StretchFittingTextToolTip", "Whether to stretch the grid to fit or to fit the grid to the largest axis"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "BlendSpaceEditor.ZoomToFit"),
			FUIAction(
				FExecuteAction::CreateLambda([this](){ ToggleFittingType(); }),
				FCanExecuteAction(),
				FGetActionCheckState::CreateLambda([this](){ return bStretchToFit ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
	}
	InMenuBuilder.EndSection();
}

TSharedPtr<SWidget> SBlendSpaceGridWidget::CreateBlendSampleContextMenu()
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, UICommandList);

	TSharedPtr<IStructureDetailsView> StructureDetailsView;
	// Initialize details view
	FDetailsViewArgs DetailsViewArgs;
	{
		DetailsViewArgs.bAllowSearch = false;
		DetailsViewArgs.bHideSelectionTip = true;
		DetailsViewArgs.bLockable = false;
		DetailsViewArgs.bSearchInitialKeyFocus = true;
		DetailsViewArgs.bUpdatesFromSelection = false;
		DetailsViewArgs.NotifyHook = NotifyHook;
		DetailsViewArgs.bShowOptions = true;
		DetailsViewArgs.bShowModifiedPropertiesOption = false;		
	}
	
	FStructureDetailsViewArgs StructureViewArgs;
	{
		StructureViewArgs.bShowObjects = true;
		StructureViewArgs.bShowAssets = true;
		StructureViewArgs.bShowClasses = true;
		StructureViewArgs.bShowInterfaces = true;
	}
	
	StructureDetailsView = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor")
		.CreateStructureDetailView(DetailsViewArgs, StructureViewArgs, nullptr, LOCTEXT("SampleData", "Blend Sample"));
	{
		if(const UBlendSpace* BlendSpace = BlendSpaceBase.Get())
		{
			const FBlendSample& Sample = BlendSpace->GetBlendSample(HighlightedSampleIndex);		
			StructureDetailsView->GetDetailsView()->SetGenericLayoutDetailsDelegate(
				FOnGetDetailCustomizationInstance::CreateStatic(
					&FBlendSampleDetails::MakeInstance, BlendSpace, this, HighlightedSampleIndex));

			FStructOnScope* Struct = new FStructOnScope(FBlendSample::StaticStruct(), (uint8*)&Sample);
			Struct->SetPackage(BlendSpace->GetOutermost());
			StructureDetailsView->SetStructureData(MakeShareable(Struct));
		}
	}
	
	MenuBuilder.BeginSection("Sample", LOCTEXT("SampleMenuHeader", "Sample"));
	{
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Cut);
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Copy);
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Delete);
		MenuBuilder.AddWidget(StructureDetailsView->GetWidget().ToSharedRef(), FText::GetEmpty(), true);
	}
	MenuBuilder.EndSection();

	MakeViewContextMenuEntries(MenuBuilder);

	return MenuBuilder.MakeWidget();
}

TSharedPtr<SWidget> SBlendSpaceGridWidget::CreateNewBlendSampleContextMenu(const FVector2D& InMousePosition)
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, UICommandList);

	FVector NewSampleValue;
	if(FSlateApplication::Get().GetModifierKeys().IsShiftDown())
	{
		NewSampleValue = ScreenPositionToSampleValue(SnapScreenPositionToGrid(InMousePosition, FSlateApplication::Get().GetModifierKeys().IsShiftDown()), false);
	}
	else
	{
		const FVector2D GridPosition(FMath::Clamp(InMousePosition.X, CachedGridRectangle.Left, CachedGridRectangle.Right),
			FMath::Clamp(InMousePosition.Y, CachedGridRectangle.Top, CachedGridRectangle.Bottom));
		NewSampleValue = ScreenPositionToSampleValue(GridPosition, false);
	}

	if(const UBlendSpace* BlendSpace = BlendSpaceBase.Get())
	{
		MenuBuilder.BeginSection("Sample", LOCTEXT("SampleMenuHeader", "Sample"));
		{
			MenuBuilder.AddMenuEntry(FGenericCommands::Get().Paste);
			MenuBuilder.AddMenuEntry(FGenericCommands::Get().Delete);

			if(!BlendSpace->IsAsset())
			{
				// Blend space graph - add a new graph sample
				MenuBuilder.AddMenuEntry(
					LOCTEXT("AddNewSample", "Add New Sample"),
					LOCTEXT("AddNewSampleTooltip", "Add a new sample to the blendspace at this location"),
						FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Plus"),
						FUIAction(
							FExecuteAction::CreateLambda(
								[this, NewSampleValue]()
								{
									if (OnSampleAdded.IsBound())
									{
										OnSampleAdded.Execute(nullptr, NewSampleValue, false);
									}
								})
						)
				);
			}
		}
		MenuBuilder.EndSection();
	}

	MakeViewContextMenuEntries(MenuBuilder);

	return MenuBuilder.MakeWidget();
}

FReply SBlendSpaceGridWidget::ToggleTriangulationVisibility()
{
	bShowTriangulation = !bShowTriangulation;
	return FReply::Handled();
}

void SBlendSpaceGridWidget::CalculateGridPoints()
{
	CachedGridPoints.Empty(SampleGridDivisions.X * SampleGridDivisions.Y);
	CachedSamplePoints.Empty(SampleGridDivisions.X * SampleGridDivisions.Y);
	if (SampleGridDivisions.X <= 0 || (GridType == EGridType::TwoAxis && SampleGridDivisions.Y <= 0))
	{
		return;
	}
	for (int32 GridY = 0; GridY < ((GridType == EGridType::TwoAxis) ? SampleGridDivisions.Y + 1 : 1); ++GridY)
	{
		for (int32 GridX = 0; GridX < SampleGridDivisions.X + 1; ++GridX)
		{
			// Calculate grid point in 0-1 form
			FVector2D GridPoint(
				GridX * (1.0f / SampleGridDivisions.X), 
				(GridType == EGridType::TwoAxis) ? GridY * (1.0f / SampleGridDivisions.Y) : 0.5f);

			// Multiply with size and offset according to the grid layout
			GridPoint *= CachedGridRectangle.GetSize();
			GridPoint += CachedGridRectangle.GetTopLeft();
			CachedGridPoints.Add(GridPoint);

			CachedSamplePoints.Add(FVector(
				SampleValueMin.X + (GridX * (SampleValueRange.X / SampleGridDivisions.X)),
				(GridType == EGridType::TwoAxis) ? SampleValueMax.Y - (GridY * (SampleValueRange.Y / SampleGridDivisions.Y)) : 0.0f, 
				0.0f));
		}
	}
}

const FVector2D SBlendSpaceGridWidget::SnapScreenPositionToGrid(const FVector2D& InPosition, bool bForceSnap) const
{
	const int32 GridPointIndex = FindClosestGridPointIndexFromScreenPosition(InPosition);
	const FVector2D& GridPoint = CachedGridPoints[GridPointIndex];

	const bool bSnapX = bForceSnap || bSampleSnapToGrid[0];
	const bool bSnapY = bForceSnap || bSampleSnapToGrid[1];

	return FVector2D
	{
		bSnapX ? GridPoint.X : InPosition.X,
		bSnapY ? GridPoint.Y : InPosition.Y
	};
}

const FVector SBlendSpaceGridWidget::ScreenPositionToSampleValueWithSnapping(const FVector2D& InPosition, bool bForceSnap) const
{
	FVector SampleValue = ScreenPositionToSampleValue(InPosition, true);

	const int32 GridPointIndex = FindClosestGridPointIndexFromScreenPosition(InPosition);
	const FVector GridPos = CachedSamplePoints[GridPointIndex];

	const bool bSnapX = bForceSnap || bSampleSnapToGrid[0];
	const bool bSnapY = bForceSnap || bSampleSnapToGrid[1];
	return FVector
	{
		bSnapX ? GridPos.X : SampleValue.X,
		bSnapY ? GridPos.Y : SampleValue.Y,
		0.f
	};
}

int32 SBlendSpaceGridWidget::FindClosestGridPointIndexFromScreenPosition(const FVector2D& InPosition) const
{
	// Clamp the screen position to the grid
	const FVector2D GridPosition(
		FMath::Clamp(InPosition.X, CachedGridRectangle.Left, CachedGridRectangle.Right),
		FMath::Clamp(InPosition.Y, CachedGridRectangle.Top, CachedGridRectangle.Bottom));
	// Find the closest grid point
	double Distance = TNumericLimits<double>::Max();
	int32 GridPointIndex = INDEX_NONE;
	for (int32 Index = 0; Index < CachedGridPoints.Num(); ++Index)
	{
		const FVector2D& GridPoint = CachedGridPoints[Index];
		const double DistanceToGrid = FVector2D::DistSquared(GridPosition, GridPoint);
		if (DistanceToGrid < Distance)
		{
			Distance = DistanceToGrid;
			GridPointIndex = Index;
		}
	}

	checkf(GridPointIndex != INDEX_NONE, TEXT("Unable to find gridpoint"));

	return GridPointIndex;
}

const FVector2D SBlendSpaceGridWidget::SampleValueToNormalizedPosition(const FVector& SampleValue) const
{
	// Convert the sample value to 0 to 1 form
	FVector2D NormalizedPosition(
		((SampleValue.X - SampleValueMin.X) / SampleValueRange.X),
		GridType == EGridType::TwoAxis ? ((SampleValueMax.Y - SampleValue.Y) / SampleValueRange.Y) : 0.5f);
	return NormalizedPosition;	
}

const FVector2D SBlendSpaceGridWidget::SampleValueToScreenPosition(const FVector& SampleValue) const
{
	const FVector2D NormalizedPosition = SampleValueToNormalizedPosition(SampleValue);
	const FVector2D GridSize = CachedGridRectangle.GetSize();
	const FVector2D GridCorner = CachedGridRectangle.GetCenter() - 0.5f * GridSize;
	const FVector2D ScreenPosition = GridCorner + NormalizedPosition * CachedGridRectangle.GetSize();
	return ScreenPosition;	
}

const FVector SBlendSpaceGridWidget::ScreenPositionToSampleValue(const FVector2D& ScreenPosition, bool bClamp) const
{
	FVector2D LocalGridPosition = ScreenPosition;
	// Move to center of grid and convert to 0 - 1 form
	LocalGridPosition -= CachedGridRectangle.GetCenter();
	LocalGridPosition /= (CachedGridRectangle.GetSize() * 0.5f);
	LocalGridPosition += FVector2D::UnitVector;
	LocalGridPosition *= 0.5f;

	// Calculate the sample value by mapping it to the blend parameter range
	FVector SampleValue
	(		
		(LocalGridPosition.X * SampleValueRange.X) + SampleValueMin.X,
		(GridType == EGridType::TwoAxis) ? SampleValueMax.Y - (LocalGridPosition.Y * SampleValueRange.Y) : 0.0f,
		0.f
	);
	if (bClamp)
	{
		SampleValue.X = FMath::Clamp(SampleValue.X, SampleValueMin.X, SampleValueMax.X);
		SampleValue.Y = FMath::Clamp(SampleValue.Y, SampleValueMin.Y, SampleValueMax.Y);
	}
	return SampleValue;
}

const FSlateRect SBlendSpaceGridWidget::GetGridRectangleFromGeometry(const FGeometry& MyGeometry)
{
	const float TopOffset = bReadOnly ? 0.0f : 20.0f; // Ideally we'd get the size of the buttons (showing the label/triangulation etc)
	FSlateRect WindowRect = FSlateRect(0, TopOffset, static_cast<float>(MyGeometry.GetLocalSize().X), static_cast<float>(MyGeometry.GetLocalSize().Y));
	if (!bStretchToFit)
	{
		UpdateGridRatioMargin(WindowRect.GetSize());
	}

	return WindowRect.InsetBy(GridMargin + GridRatioMargin);
}

bool SBlendSpaceGridWidget::IsSampleValueWithinMouseRange(const FVector& SampleValue, float& OutDistance) const
{
	const FVector2D GridPosition = SampleValueToScreenPosition(SampleValue);
	OutDistance = static_cast<float>(FVector2D::Distance(LocalMousePosition, GridPosition));
	return (OutDistance < ClickAndHighlightThreshold);
}

int32 SBlendSpaceGridWidget::GetClosestSamplePointIndexToMouse() const
{
	float BestDistance = FLT_MAX;
	int32 BestIndex = INDEX_NONE;

	if(const UBlendSpace* BlendSpace = BlendSpaceBase.Get())
	{
		const TArray<FBlendSample>& Samples = BlendSpace->GetBlendSamples();
		for (int32 SampleIndex = 0; SampleIndex < Samples.Num(); ++SampleIndex)
		{
			const FBlendSample& Sample = Samples[SampleIndex];
			float Distance;
			if (IsSampleValueWithinMouseRange(Sample.SampleValue, Distance))
			{
				if(Distance < BestDistance)
				{
					BestDistance = Distance;
					BestIndex = SampleIndex;
				}
			}
		}
	}

	return BestIndex;
}

void SBlendSpaceGridWidget::StartPreviewing()
{
	bSamplePreviewing = true;
	LastPreviewingMousePosition = LocalMousePosition;
}

void SBlendSpaceGridWidget::StopPreviewing()
{
	bSamplePreviewing = false;
}

FText SBlendSpaceGridWidget::GetToolTipSampleValidity() const
{
	const UBlendSpace* BlendSpace = BlendSpaceBase.Get();
	FText ToolTipText = FText::GetEmpty();
	if (!bReadOnly && BlendSpace && BlendSpace->bInterpolateUsingGrid)
	{
		int32 SampleIndex = INDEX_NONE;
		if (DragState == EDragState::None)
		{
			SampleIndex = HighlightedSampleIndex;
		}
		else if (DragState == EDragState::DragSample)
		{
			SampleIndex = DraggedSampleIndex;
		}
		else
		{
			SampleIndex = INDEX_NONE;
		}

		if (SampleIndex != INDEX_NONE && BlendSpace->IsValidBlendSampleIndex(SampleIndex))
		{
			float SampleLookupWeight = GetSampleLookupWeight(SampleIndex);
			if (SampleLookupWeight >= 1.0f)
			{
				return ToolTipText;
			}
			else if (SampleLookupWeight <= 0.0f)
			{
				ToolTipText = FText::Format(
					LOCTEXT("SampleValidityZero", "Self weight is zero"), 
					SampleLookupWeight);
			}
			else if (SampleLookupWeight <= SampleLookupWeightThreshold)
			{
				ToolTipText = FText::Format(
					LOCTEXT("SampleValidityLow", "Self weight is low: {0}"), 
					SampleLookupWeight);
			}
			else if (SampleLookupWeight < 1.0f)
			{
				ToolTipText = FText::Format(
					LOCTEXT("SampleValidity", "Self weight: {0}"), 
					SampleLookupWeight);
			}
		}
	}
	return ToolTipText;
}

FText SBlendSpaceGridWidget::GetToolTipAnimationName() const
{
	FText ToolTipText = FText::GetEmpty();
	if(const UBlendSpace* BlendSpace = BlendSpaceBase.Get())
	{
		const FText PreviewValue = LOCTEXT("PreviewValueTooltip", "Preview Value");

		if(bReadOnly)
		{
			ToolTipText = PreviewValue;
		}
		else
		{
			switch (DragState)
			{
				// If we are not dragging, but over a valid blend sample return its animation asset name
				case EDragState::None:
				{		
					if (bHighlightPreviewPin)
					{
						ToolTipText = PreviewValue;
					}
					else if (HighlightedSampleIndex != INDEX_NONE && BlendSpace->IsValidBlendSampleIndex(HighlightedSampleIndex))
					{
						const FBlendSample& BlendSample = BlendSpace->GetBlendSample(HighlightedSampleIndex);
						ToolTipText = !BlendSample.bUseSingleFrameForBlending ? 
							FText::Format(LOCTEXT("SampleNameFormat", "{0} ({1})"), GetSampleName(BlendSample, HighlightedSampleIndex), FText::AsNumber(HighlightedSampleIndex)) : 
							FText::Format(LOCTEXT("FrameIndexSampleNameFormat", "{0} ({1}) (Frame {2})"), GetSampleName(BlendSample, HighlightedSampleIndex), FText::AsNumber(HighlightedSampleIndex), FText::AsNumber(BlendSample.FrameIndexToSample));
					}
					else if(TargetPosition.IsSet())
					{
						ToolTipText = PreviewValue;
					}
					break;
				}

				case EDragState::PreDrag:
				{
					break;
				}

				// If we are dragging a sample return the dragged sample's animation asset name
				case EDragState::DragSample:
				{
					if (BlendSpace->IsValidBlendSampleIndex(DraggedSampleIndex))
					{
						const FBlendSample& BlendSample = BlendSpace->GetBlendSample(DraggedSampleIndex);
						ToolTipText = !BlendSample.bUseSingleFrameForBlending ? 
							FText::Format(LOCTEXT("SampleNameFormat", "{0} ({1})"), GetSampleName(BlendSample, DraggedSampleIndex), FText::AsNumber(DraggedSampleIndex)) : 
							FText::Format(LOCTEXT("FrameIndexSampleNameFormat", "{0} ({1}) (Frame {2})"), GetSampleName(BlendSample, DraggedSampleIndex), FText::AsNumber(DraggedSampleIndex), FText::AsNumber(BlendSample.FrameIndexToSample));
					}
			
					break;
				}

				// If we are performing a drag/drop operation return the cached operation animation name
				case EDragState::DragDrop:
				{
					ToolTipText = DragDropAnimationName;
					break;
				}

				case EDragState::DragDropOverride:
				{
					ToolTipText = DragDropAnimationName;
					break;
				}

				case EDragState::InvalidDragDrop:
				{
					break;
				}
		
				// If we are previewing return a descriptive label
				case EDragState::Preview:
				{
					ToolTipText = PreviewValue;
					break;
				}
				default:
					check(false);
			}
		}
	}

	return ToolTipText;
}

FText SBlendSpaceGridWidget::GetToolTipSampleValue() const
{
	FText ToolTipText = FText::GetEmpty();

	if(const UBlendSpace* BlendSpace = BlendSpaceBase.Get())
	{
		static const FTextFormat OneAxisFormat = LOCTEXT("OneAxisFormat", "{0}: {1}");
		static const FTextFormat TwoAxisFormat = LOCTEXT("TwoAxisFormat", "{0}: {1}  {2}: {3}");
		const FTextFormat& ValueFormattingText = (GridType == EGridType::TwoAxis) ? TwoAxisFormat : OneAxisFormat;

		auto AddAdvancedPreview = [this, &ToolTipText, BlendSpace]()
		{
			FTextBuilder TextBuilder;
			TextBuilder.AppendLine(ToolTipText);

			if (bAdvancedPreview)
			{				
				for (const FBlendSampleData& SampleData : PreviewedSamples)
				{
					if(BlendSpace->IsValidBlendSampleIndex(SampleData.SampleDataIndex))
					{
						const FBlendSample& BlendSample = BlendSpace->GetBlendSample(SampleData.SampleDataIndex);
						const FText SampleNameWithWeight = !BlendSample.bUseSingleFrameForBlending ? 
							FText::Format(LOCTEXT("SampleFormat", "{0} ({1}): {2}"), GetSampleName(BlendSample, SampleData.SampleDataIndex), FText::AsNumber(SampleData.SampleDataIndex),FText::AsNumber(SampleData.TotalWeight)) : 
							FText::Format(LOCTEXT("FrameIndexSampleFormat", "{0} ({1}) (Frame {2}): {3}"), GetSampleName(BlendSample, SampleData.SampleDataIndex), FText::AsNumber(SampleData.SampleDataIndex), FText::AsNumber(BlendSample.FrameIndexToSample), FText::AsNumber(SampleData.TotalWeight));
						TextBuilder.AppendLine(SampleNameWithWeight);
					}
				}
			}

			ToolTipText = TextBuilder.ToText();
		};

		if(bReadOnly)
		{
			ToolTipText = FText::Format(ValueFormattingText, ParameterXName, FText::FromString(FString::SanitizeFloat(PreviewPosition.X)), ParameterYName, FText::FromString(FString::SanitizeFloat(PreviewPosition.Y)));

			AddAdvancedPreview();
		}
		else
		{
			switch (DragState)
			{
				// If we are over a sample return its sample value if valid and otherwise show an error message as to why the sample is invalid
				case EDragState::None:
				{		
					if (bHighlightPreviewPin)
					{
						ToolTipText = FText::Format(ValueFormattingText, ParameterXName, FText::FromString(FString::SanitizeFloat(PreviewPosition.X)), ParameterYName, FText::FromString(FString::SanitizeFloat(PreviewPosition.Y)));

						AddAdvancedPreview();
					}
					else if (HighlightedSampleIndex != INDEX_NONE && BlendSpace->IsValidBlendSampleIndex(HighlightedSampleIndex))
					{
						const FBlendSample& BlendSample = BlendSpace->GetBlendSample(HighlightedSampleIndex);

						// Check if the sample is valid
						if (BlendSample.bIsValid)
						{
							ToolTipText = FText::Format(ValueFormattingText, ParameterXName, FText::FromString(FString::SanitizeFloat(BlendSample.SampleValue.X)), ParameterYName, FText::FromString(FString::SanitizeFloat(BlendSample.SampleValue.Y)));
						}
						else
						{
							ToolTipText = GetSampleErrorMessage(BlendSample);
						}
					}
					else if(TargetPosition.IsSet())
					{
						ToolTipText = FText::Format(ValueFormattingText, ParameterXName, FText::FromString(FString::SanitizeFloat(PreviewPosition.X)), ParameterYName, FText::FromString(FString::SanitizeFloat(PreviewPosition.Y)));

						AddAdvancedPreview();
					}
					break;
				}

				case EDragState::PreDrag:
				{
					break;
				}

				// If we are dragging a sample return the current sample value it is hovered at
				case EDragState::DragSample:
				{
					if (DraggedSampleIndex != INDEX_NONE)
					{
						const FBlendSample& BlendSample = BlendSpace->GetBlendSample(DraggedSampleIndex);
						ToolTipText = FText::Format(ValueFormattingText, ParameterXName, FText::FromString(FString::SanitizeFloat(BlendSample.SampleValue.X)), ParameterYName, FText::FromString(FString::SanitizeFloat(BlendSample.SampleValue.Y)));
					}
					break;
				}

				// If we are performing a drag and drop operation return the current sample value it is hovered at
				case EDragState::DragDrop:
				{
					const FVector SampleValue = ScreenPositionToSampleValueWithSnapping(LocalMousePosition, FSlateApplication::Get().GetModifierKeys().IsShiftDown());

					ToolTipText = FText::Format(ValueFormattingText, ParameterXName, FText::FromString(FString::SanitizeFloat(SampleValue.X)), ParameterYName, FText::FromString(FString::SanitizeFloat(SampleValue.Y)));

					break;
				}

				case EDragState::DragDropOverride:
				{
					if(HoveredAnimationName.IsEmpty())
					{
						static const FTextFormat OverrideAnimationFormat = LOCTEXT("InvalidSampleChangingFormat", "Changing sample to {0}");
						ToolTipText = FText::Format(OverrideAnimationFormat, DragDropAnimationName);
					}
					else
					{
						static const FTextFormat OverrideAnimationFormat = LOCTEXT("ValidSampleChangingFormat", "Changing sample from {0} to {1}");
						ToolTipText = FText::Format(OverrideAnimationFormat, HoveredAnimationName, DragDropAnimationName);
					}
					break;
				}
				// If the drag and drop operation is invalid return the cached error message as to why it is invalid
				case EDragState::InvalidDragDrop:
				{
					ToolTipText = InvalidDragDropText;
					break;
				}

				// If we are setting the preview value return the current preview sample value
				case EDragState::Preview:
				{
					ToolTipText = FText::Format(ValueFormattingText, ParameterXName, FText::FromString(FString::SanitizeFloat(PreviewPosition.X)), ParameterYName, FText::FromString(FString::SanitizeFloat(PreviewPosition.Y)));

					AddAdvancedPreview();
					break;
				}
				default:
					check(false);
			}
		}
	}

	return ToolTipText;
}

void SBlendSpaceGridWidget::EnableStatusBarMessage(bool bEnable)
{
	if(!bReadOnly)
	{
		if(bEnable)
		{
			if (!StatusBarMessageHandle.IsValid())
			{
				if(UStatusBarSubsystem* StatusBarSubsystem = GEditor->GetEditorSubsystem<UStatusBarSubsystem>())
				{
					StatusBarMessageHandle = StatusBarSubsystem->PushStatusBarMessage(StatusBarName, MakeAttributeLambda([]()
					{
						return LOCTEXT("StatusBarMssage", "Hold Ctrl to move preview value, and Alt to show weight details. Click and drag sample points to move them, with Shift to snap to the grid.");
					}));
				}
			}
		}
		else
		{
			if (StatusBarMessageHandle.IsValid())
			{
				if(UStatusBarSubsystem* StatusBarSubsystem = GEditor->GetEditorSubsystem<UStatusBarSubsystem>())
				{
					StatusBarSubsystem->PopStatusBarMessage(StatusBarName, StatusBarMessageHandle);
					StatusBarMessageHandle.Reset();
				}
			}
		}
	}
}

FText SBlendSpaceGridWidget::GetSampleErrorMessage(const FBlendSample &BlendSample) const
{
	const FVector2D GridPosition = SampleValueToScreenPosition(BlendSample.SampleValue);
	// Either an invalid animation asset set
	if (BlendSample.Animation == nullptr)
	{
		static const FText NoAnimationErrorText = LOCTEXT("NoAnimationErrorText", "Invalid Animation for Sample");
		return NoAnimationErrorText;
	}
	// Or not aligned on the grid (which means that it does not match one of the cached grid points, == for FVector2D fails to compare though :/)
	else if (!CachedGridPoints.FindByPredicate([&](const FVector2D& Other) { return FMath::IsNearlyEqual(GridPosition.X, Other.X) && FMath::IsNearlyEqual(GridPosition.Y, Other.Y);}))
	{
		static const FText SampleNotAtGridPoint = LOCTEXT("SampleNotAtGridPointErrorText", "Sample is not on a valid Grid Point");
		return SampleNotAtGridPoint;
	}
	else if (const UBlendSpace* BlendSpaceBaseObject = BlendSpaceBase.Get())
	{
		if (!BlendSpaceBaseObject->IsSingleFrameBlendingIndexInBounds(BlendSample))
		{
			static const FText SamplingIndexOutOfBounds = LOCTEXT("SingleFrameBlendingIndexOutOfBounds", "Single frame blending index is out of bounds.");
			return SamplingIndexOutOfBounds;
		}
	}

	static const FText UnknownError = LOCTEXT("UnknownErrorText", "Sample is invalid for an Unknown Reason");
	return UnknownError;
}

void SBlendSpaceGridWidget::ShowToolTip()
{
	if(HighlightedSampleIndex != INDEX_NONE && ToolTipSampleIndex != HighlightedSampleIndex)
	{
		ToolTipSampleIndex = HighlightedSampleIndex;
		if(OnExtendSampleTooltip.IsBound())
		{
			ToolTipExtensionContainer->SetContent(OnExtendSampleTooltip.Execute(HighlightedSampleIndex));
		}
	}
	
	SetToolTip(ToolTip);
}

void SBlendSpaceGridWidget::ResetToolTip()
{
	ToolTipSampleIndex = INDEX_NONE;
	ToolTipExtensionContainer->SetContent(SNullWidget::NullWidget);
	SetToolTip(nullptr);
}

EVisibility SBlendSpaceGridWidget::GetInputBoxVisibility(const int32 ParameterIndex) const
{
	bool bVisible = !bReadOnly;
	// Only show input boxes when a sample is selected (hide it when one is being dragged since we have the tooltip information as well)
	bVisible &= (SelectedSampleIndex != INDEX_NONE && DraggedSampleIndex == INDEX_NONE);
	if ( ParameterIndex == 1 )
	{ 
		bVisible &= (GridType == EGridType::TwoAxis);
	}

	return bVisible ? EVisibility::Visible : EVisibility::Collapsed;
}

TOptional<float> SBlendSpaceGridWidget::GetInputBoxValue(const int32 ParameterIndex) const
{
	checkf(ParameterIndex < 3, TEXT("Invalid parameter index, suppose to be within FVector array range"));
	float ReturnValue = 0.0f;
	if(const UBlendSpace* BlendSpace = BlendSpaceBase.Get())
	{
		if (SelectedSampleIndex != INDEX_NONE && SelectedSampleIndex < BlendSpace->GetNumberOfBlendSamples())
		{
			const FBlendSample& BlendSample = BlendSpace->GetBlendSample(SelectedSampleIndex);
			ReturnValue = static_cast<float>(BlendSample.SampleValue[ParameterIndex]);
		}
	}
	return ReturnValue;
}

TOptional<float> SBlendSpaceGridWidget::GetInputBoxMinValue(const int32 ParameterIndex) const
{
	checkf(ParameterIndex < 3, TEXT("Invalid parameter index, suppose to be within FVector array range"));
	return static_cast<float>(SampleValueMin[ParameterIndex]);
}

TOptional<float> SBlendSpaceGridWidget::GetInputBoxMaxValue(const int32 ParameterIndex) const
{
	checkf(ParameterIndex < 3, TEXT("Invalid parameter index, suppose to be within FVector array range"));
	return static_cast<float>(SampleValueMax[ParameterIndex]);
}

float SBlendSpaceGridWidget::GetInputBoxDelta(const int32 ParameterIndex) const
{
	checkf(ParameterIndex < 3, TEXT("Invalid parameter index, suppose to be within FVector array range"));
	return static_cast<float>(SampleGridDelta[ParameterIndex]);
}

void SBlendSpaceGridWidget::OnInputBoxValueCommited(const float NewValue, ETextCommit::Type CommitType, const int32 ParameterIndex)
{
	OnInputBoxValueChanged(NewValue, ParameterIndex, false);
}

void SBlendSpaceGridWidget::OnInputBoxValueChanged(const float NewValue, const int32 ParameterIndex, bool bIsInteractive)
{
	// Ignore any SNumericEntryBox.OnValueChanged broadcasts if sliding has finished and OnInputBoxValueCommited will have been broadcasted already
	if (bIsInteractive && !bSliderMovement[ParameterIndex])
	{
		return;
	}
	
	checkf(ParameterIndex < 2, TEXT("Invalid parameter index, suppose to be within FVector array range"));

	if (SelectedSampleIndex != INDEX_NONE && BlendSpaceBase.Get() != nullptr)
	{
		// Retrieve current sample value
		const FBlendSample& Sample = BlendSpaceBase.Get()->GetBlendSample(SelectedSampleIndex);
		FVector SampleValue = Sample.SampleValue;

		// Calculate snapped value
		if (bSampleSnapToGrid[ParameterIndex])
		{
			const double MinOffset = NewValue - SampleValueMin[ParameterIndex];
			double GridSteps = MinOffset / SampleGridDelta[ParameterIndex];
			int64 FlooredSteps = FMath::FloorToInt(GridSteps);
			GridSteps -= FlooredSteps;
			FlooredSteps = (GridSteps > .5) ? FlooredSteps + 1 : FlooredSteps;

			// Temporary snap this value to closest point on grid (since the spin box delta does not provide the desired functionality)
			SampleValue[ParameterIndex] = SampleValueMin[ParameterIndex] + (FlooredSteps * SampleGridDelta[ParameterIndex]);
		}
		else
		{
			SampleValue[ParameterIndex] = NewValue;
		}

		OnSampleMoved.ExecuteIfBound(SelectedSampleIndex, SampleValue, bIsInteractive);
	}
}

void SBlendSpaceGridWidget::OnInputSliderBegin(const int32 ParameterIndex)
{
	ensure(bSliderMovement[ParameterIndex] == false);
	bSliderMovement[ParameterIndex] = true;
}

void SBlendSpaceGridWidget::OnInputSliderEnd(const float NewValue, const int32 ParameterIndex)
{
	ensure(bSliderMovement[ParameterIndex] == true);
	bSliderMovement[ParameterIndex] = false;
}

EVisibility SBlendSpaceGridWidget::GetSampleToolTipVisibility() const
{
	// Show tool tip when the grid is empty
	return (!bReadOnly && BlendSpaceBase.Get() != nullptr && BlendSpaceBase.Get()->GetNumberOfBlendSamples() == 0) ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SBlendSpaceGridWidget::GetPreviewToolTipVisibility() const
{
	// Only show preview tooltip until the user discovers the functionality
	return bReadOnly ? EVisibility::Collapsed : EVisibility::Visible;
}

EVisibility SBlendSpaceGridWidget::GetTriangulationButtonVisibility() const
{
	if (bShowSettingsButtons && GridType == EGridType::TwoAxis)
	{
		if (const UBlendSpace* BlendSpace = BlendSpaceBase.Get())
		{
			if (!BlendSpace->bInterpolateUsingGrid)
			{
				return EVisibility::Visible;
			}
		}
	}
	return  EVisibility::Collapsed;
}

EVisibility SBlendSpaceGridWidget::GetAnimationNamesButtonVisibility() const
{
	return bShowSettingsButtons ? EVisibility::Visible : EVisibility::Collapsed;
}

FReply SBlendSpaceGridWidget::ToggleFittingType()
{
	bStretchToFit = !bStretchToFit;

	// If toggle to stretching, reset the margin immediately
	if (bStretchToFit)
	{
		GridRatioMargin.Top = GridRatioMargin.Bottom = GridRatioMargin.Left = GridRatioMargin.Right = 0.0f;
	}

	return FReply::Handled();
}

FReply SBlendSpaceGridWidget::ToggleShowAnimationNames()
{
	bShowAnimationNames = !bShowAnimationNames;
	return FReply::Handled();
}

void SBlendSpaceGridWidget::UpdateGridRatioMargin(const FVector2D& GeometrySize)
{
	if (GridType == EGridType::TwoAxis)
	{
		// Reset values first
		GridRatioMargin.Top = GridRatioMargin.Bottom = GridRatioMargin.Left = GridRatioMargin.Right = 0.0f;
		if (GeometrySize.Y > GeometrySize.X)
		{
			const double Difference = GeometrySize.Y - GeometrySize.X;
			GridRatioMargin.Top = GridRatioMargin.Bottom = static_cast<float>(Difference) * 0.5f;
		}
		else if (GeometrySize.X > GeometrySize.Y)
		{
			const double Difference = GeometrySize.X - GeometrySize.Y;
			GridRatioMargin.Left = GridRatioMargin.Right = static_cast<float>(Difference) * 0.5f;
		}
	}
}

FText SBlendSpaceGridWidget::GetFittingTypeButtonToolTipText() const
{
	static const FText StretchText = LOCTEXT("StretchFittingText", "Stretch Grid to Fit");
	static const FText GridRatioText = LOCTEXT("GridRatioFittingText", "Fit Grid to Largest Axis");
	return (bStretchToFit) ? GridRatioText : StretchText;
}

EVisibility SBlendSpaceGridWidget::GetFittingButtonVisibility() const
{
	return (bShowSettingsButtons && (GridType == EGridType::TwoAxis)) ? EVisibility::Visible : EVisibility::Collapsed;
}

void SBlendSpaceGridWidget::UpdateCachedBlendParameterData()
{
	if(const UBlendSpace* BlendSpace = BlendSpaceBase.Get())
	{
		const FBlendParameter& BlendParameterX = BlendSpace->GetBlendParameter(0);
		const FBlendParameter& BlendParameterY = BlendSpace->GetBlendParameter(1);
		SampleValueRange.X = BlendParameterX.Max - BlendParameterX.Min;
		SampleValueRange.Y = BlendParameterY.Max - BlendParameterY.Min;
	
		SampleValueMin.X = BlendParameterX.Min;
		SampleValueMin.Y = BlendParameterY.Min;

		SampleValueMax.X = BlendParameterX.Max;
		SampleValueMax.Y = BlendParameterY.Max;

		SampleGridDelta = SampleValueRange;
		SampleGridDelta.X /= (BlendParameterX.GridNum);
		SampleGridDelta.Y /= (BlendParameterY.GridNum);

		bSampleSnapToGrid[0] = BlendParameterX.bSnapToGrid;
		bSampleSnapToGrid[1] = BlendParameterY.bSnapToGrid;

		SampleGridDivisions.X = BlendParameterX.GridNum;
		SampleGridDivisions.Y = BlendParameterY.GridNum;

		ParameterXName = FText::FromString(BlendParameterX.DisplayName);
		ParameterYName = FText::FromString(BlendParameterY.DisplayName);
	
		const TSharedRef< FSlateFontMeasure > FontMeasure = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
		MaxVerticalAxisTextWidth = HorizontalAxisMaxTextWidth = MaxHorizontalAxisTextHeight = 0.0f;
		FVector2D TextSize = FontMeasure->Measure(ParameterYName, FontInfo);	
		MaxVerticalAxisTextWidth = FMath::Max(MaxVerticalAxisTextWidth, static_cast<float>(TextSize.X));

		TextSize = FontMeasure->Measure(FString::SanitizeFloat(SampleValueMin.Y), FontInfo);
		MaxVerticalAxisTextWidth = FMath::Max(MaxVerticalAxisTextWidth, static_cast<float>(TextSize.X));

		TextSize = FontMeasure->Measure(FString::SanitizeFloat(SampleValueMax.Y), FontInfo);
		MaxVerticalAxisTextWidth = FMath::Max(MaxVerticalAxisTextWidth, static_cast<float>(TextSize.X));
	
		TextSize = FontMeasure->Measure(ParameterXName, FontInfo);
		MaxHorizontalAxisTextHeight = FMath::Max(MaxHorizontalAxisTextHeight, static_cast<float>(TextSize.Y));

		TextSize = FontMeasure->Measure(FString::SanitizeFloat(SampleValueMin.X), FontInfo);
		MaxHorizontalAxisTextHeight = FMath::Max(MaxHorizontalAxisTextHeight, static_cast<float>(TextSize.Y));

		TextSize = FontMeasure->Measure(FString::SanitizeFloat(SampleValueMax.X), FontInfo);
		MaxHorizontalAxisTextHeight = FMath::Max(MaxHorizontalAxisTextHeight, static_cast<float>(TextSize.Y));
		HorizontalAxisMaxTextWidth = static_cast<float>(TextSize.X);
	}
}

void SBlendSpaceGridWidget::OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{	
	SCompoundWidget::OnMouseEnter(MyGeometry, MouseEvent);
	bMouseIsOverGeometry = true;
	EnableStatusBarMessage(true);
}

void SBlendSpaceGridWidget::OnMouseLeave(const FPointerEvent& MouseEvent)
{
	SCompoundWidget::OnMouseLeave(MouseEvent);
	bMouseIsOverGeometry = false;
	EnableStatusBarMessage(false);
}

void SBlendSpaceGridWidget::OnFocusLost(const FFocusEvent& InFocusEvent)
{
	SCompoundWidget::OnFocusLost(InFocusEvent);

	if (DragState == EDragState::DragSample)
	{
		OnSampleMoved.ExecuteIfBound(DraggedSampleIndex, LastDragPosition, false);
	}	
	HighlightedSampleIndex = DraggedSampleIndex = INDEX_NONE;
	DragState = EDragState::None;
	bSamplePreviewing = false;
	ResetToolTip();
	EnableStatusBarMessage(false);
}

bool SBlendSpaceGridWidget::SupportsKeyboardFocus() const
{
	return true;
}

void SBlendSpaceGridWidget::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	const int32 PreviousSampleIndex = HighlightedSampleIndex;
	HighlightedSampleIndex = INDEX_NONE;
	const bool bPreviousHighlightPreviewPin = bHighlightPreviewPin;

	if(const UBlendSpace* BlendSpace = BlendSpaceBase.Get())
	{
		if(PreviousBlendSpaceBase.Get() != BlendSpace)
		{
			PreviousBlendSpaceBase = BlendSpace;
			InvalidateCachedData();
		}

		GridType = BlendSpace->IsA<UBlendSpace1D>() ? EGridType::SingleAxis : EGridType::TwoAxis;
		BlendParametersToDraw = (GridType == EGridType::SingleAxis) ? 1 : 2;

		if(!bReadOnly)
		{
			if (DragState == EDragState::None)
			{
				// Check if we are highlighting preview pin
				float Distance;
				bHighlightPreviewPin = IsSampleValueWithinMouseRange(PreviewPosition, Distance);
				if (bHighlightPreviewPin)
				{
					if (bHighlightPreviewPin != bPreviousHighlightPreviewPin)
					{
						ShowToolTip();
					}
				}
				else if (bPreviousHighlightPreviewPin != bHighlightPreviewPin)
				{
					ResetToolTip();
				}
		
				// Determine highlighted sample
				HighlightedSampleIndex = GetClosestSamplePointIndexToMouse();

				if (!bHighlightPreviewPin)
				{
					// If we started selecting or selected a different sample make sure we show/hide the tooltip
					if (PreviousSampleIndex != HighlightedSampleIndex)
					{
						if (HighlightedSampleIndex != INDEX_NONE)
						{
							ShowToolTip();
						}
						else
						{
							ResetToolTip();
						}
					}
				}
		}
			else if (DragState == EDragState::DragSample)
			{
				// If we are dragging a sample, find out whether or not it has actually moved to a
				// different grid position since the last tick and update the blend space accordingly
				FVector SampleValue = ScreenPositionToSampleValueWithSnapping(LocalMousePosition, FSlateApplication::Get().GetModifierKeys().IsShiftDown());
				// Only allow dragging on each axis if not locked
				const FBlendSample& BlendSample = BlendSpace->GetBlendSample(DraggedSampleIndex);
				if (SampleValue != LastDragPosition)
				{
					LastDragPosition = SampleValue;
					OnSampleMoved.ExecuteIfBound(DraggedSampleIndex, SampleValue, true);
				}
			}
			else if (DragState == EDragState::DragDrop || DragState == EDragState::InvalidDragDrop || DragState == EDragState::DragDropOverride)
			{
				// Validate that the sample is not overlapping with a current sample when doing a
				// drag/drop operation and that we are dropping a valid animation for the blend
				// space (type)
				const FVector DropSampleValue = ScreenPositionToSampleValueWithSnapping(LocalMousePosition, FSlateApplication::Get().GetModifierKeys().IsShiftDown());
				const bool bValidPosition = BlendSpace->IsSampleWithinBounds(DropSampleValue);
				const bool bExistingSample = BlendSpace->IsTooCloseToExistingSamplePoint(DropSampleValue, INDEX_NONE);
				const bool bValidSequence = ValidateAnimationSequence(DragDropAnimationSequence, InvalidDragDropText);
		
				if (!bValidSequence)
				{
					DragState = EDragState::InvalidDragDrop;
				}
				else if (!bValidPosition)
				{			
					InvalidDragDropText = InvalidSamplePositionDragDropText;
					DragState = EDragState::InvalidDragDrop;
				}
				else if (bExistingSample)
				{	
					const TArray<FBlendSample>& Samples = BlendSpace->GetBlendSamples();
					for (int32 SampleIndex = 0; SampleIndex < Samples.Num(); ++SampleIndex)
					{
						const FBlendSample& Sample = Samples[SampleIndex];
						if (Sample.SampleValue == DropSampleValue)
						{
							HoveredAnimationName = Sample.Animation ? FText::FromString(Sample.Animation->GetName()) : FText::GetEmpty();
							break;
						}
					}

					DragState = EDragState::DragDropOverride;			
				}
				else if (bValidPosition && bValidSequence && !bExistingSample)
				{
					DragState = EDragState::DragDrop;
				}
			}
		}

		// Check if we should update the preview sample value
		if (bSamplePreviewing)
		{
			// Clamping happens later
			LastPreviewingMousePosition.X = LocalMousePosition.X;
			LastPreviewingMousePosition.Y = LocalMousePosition.Y;
			FModifierKeysState ModifierKeyState = FSlateApplication::Get().GetModifierKeys();
			bool bIsManualPreviewing = !bReadOnly && IsHovered() && bMouseIsOverGeometry && ModifierKeyState.IsControlDown();
			if (TargetPosition.IsSet() && !bIsManualPreviewing)
			{
				PreviewPosition = TargetPosition.Get();
				if (bReadOnly)
				{
					// Happens when we are showing in the graph - don't want to render outside the valid region
					PreviewPosition = BlendSpace->GetClampedAndWrappedBlendInput(PreviewPosition);
				}
			}
			else
			{
				PreviewPosition = ScreenPositionToSampleValue(LastPreviewingMousePosition, true);
			}
			

			if (FilteredPosition.IsSet())
			{
				PreviewFilteredPosition = BlendSpace->GetClampedAndWrappedBlendInput(FilteredPosition.Get());
			}

			// Retrieve and cache weighted samples
			PreviewedSamples.Empty(4);
			BlendSpace->GetSamplesFromBlendInput(PreviewPosition, PreviewedSamples, CachedTriangulationIndex, false);
		}
	}

	// Refresh cache blendspace/grid data if needed
	if (bRefreshCachedData)
	{
		UpdateCachedBlendParameterData();
		GridMargin = bShowAxisLabels ?  FMargin(MaxVerticalAxisTextWidth + (TextMargin * 2.0f), TextMargin, (HorizontalAxisMaxTextWidth *.5f) + TextMargin, MaxHorizontalAxisTextHeight + (TextMargin * 2.0f)) : 
										FMargin(TextMargin, TextMargin, TextMargin, TextMargin);
		bRefreshCachedData = false;
	}
	
	// Always need to update the rectangle and grid points according to the geometry (this can differ per tick)
	CachedGridRectangle = GetGridRectangleFromGeometry(AllottedGeometry);
	CalculateGridPoints();
}

const FVector SBlendSpaceGridWidget::GetPreviewPosition() const
{	
	return PreviewPosition;
}

void SBlendSpaceGridWidget::SetPreviewingState(const FVector& InPosition, const FVector& InFilteredPosition)
{
	if (const UBlendSpace* BlendSpace = BlendSpaceBase.Get())
	{
		PreviewFilteredPosition = BlendSpace->GetClampedAndWrappedBlendInput(InFilteredPosition);
	}
	else
	{
		PreviewFilteredPosition = InFilteredPosition;
	}
	PreviewPosition = InPosition;
}

void SBlendSpaceGridWidget::InvalidateCachedData()
{
	bRefreshCachedData = true;	
}

void SBlendSpaceGridWidget::InvalidateState()
{
	if (HighlightedSampleIndex != INDEX_NONE)
	{
		ResetToolTip();
	}

	if (DragState != EDragState::None)
	{
		DragState = EDragState::None;
	}
	
	SelectedSampleIndex = (BlendSpaceBase.Get() != nullptr && BlendSpaceBase.Get()->IsValidBlendSampleIndex(SelectedSampleIndex)) ? SelectedSampleIndex : INDEX_NONE;
	HighlightedSampleIndex = DraggedSampleIndex = INDEX_NONE;
}

const bool SBlendSpaceGridWidget::IsValidDragDropOperation(const FDragDropEvent& DragDropEvent, FText& InvalidOperationText)
{
	bool bResult = false;

	TSharedPtr<FAssetDragDropOp> DragDropOperation = DragDropEvent.GetOperationAs<FAssetDragDropOp>();

	if (DragDropOperation.IsValid())
	{
		// Check whether or not this animation is compatible with the blend space
		DragDropAnimationSequence = FAssetData::GetFirstAsset<UAnimSequence>(DragDropOperation->GetAssets());
		if (DragDropAnimationSequence)
		{
			bResult = ValidateAnimationSequence(DragDropAnimationSequence, InvalidOperationText);
		}
		else
		{
			// If is isn't an animation set error message
			bResult = false;
			InvalidOperationText = FText::FromString("Invalid Asset Type");
		}		
	}

	if (!bResult)
	{
		DragDropOperation->SetToolTip(InvalidOperationText, DragDropOperation->GetIcon());
	}
	else
	{
		DragDropAnimationName = FText::FromString(DragDropAnimationSequence->GetName());
	}

	return bResult;
}

bool SBlendSpaceGridWidget::ValidateAnimationSequence(const UAnimSequence* AnimationSequence, FText& InvalidOperationText) const
{	
	if (AnimationSequence != nullptr)
	{
		if(const UBlendSpace* BlendSpace = BlendSpaceBase.Get())
		{
			if(BlendSpace->IsAsset())
			{
				// If there are any existing blend samples check whether or not the the animation should be additive and if so if the additive matches the existing samples
				if ( BlendSpace->GetNumberOfBlendSamples() > 0)
				{
					const bool bIsAdditive = BlendSpace->ShouldAnimationBeAdditive();
					if (AnimationSequence->IsValidAdditive() != bIsAdditive)
					{
						InvalidOperationText = FText::FromString(bIsAdditive ? "Animation should be additive" : "Animation should be non-additive");
						return false;
					}

					// If it is the supported additive type, but does not match existing samples
					if (!BlendSpace->DoesAnimationMatchExistingSamples(AnimationSequence))
					{
						InvalidOperationText = FText::FromString("Additive Animation Type does not match existing Samples");
						return false;
					}
				}

				// Check if the supplied animation is of a different additive animation type 
				if (!BlendSpace->IsAnimationCompatible(AnimationSequence))
				{
					InvalidOperationText = FText::FromString("Invalid Additive Animation Type");
					return false;
				}

				// Check if the supplied animation is compatible with the skeleton
				if (!BlendSpace->IsAnimationCompatibleWithSkeleton(AnimationSequence))
				{
					InvalidOperationText = FText::FromString("Animation is incompatible with the skeleton");
					return false;
				}
			}
		}
	}

	return AnimationSequence != nullptr;
}

const bool SBlendSpaceGridWidget::IsPreviewing() const 
{ 
	FModifierKeysState ModifierKeyState = FSlateApplication::Get().GetModifierKeys();
	bool bIsManualPreviewing = !bReadOnly && IsHovered() && bMouseIsOverGeometry && ModifierKeyState.IsControlDown();
	return (bSamplePreviewing && !TargetPosition.IsSet()) || (TargetPosition.IsSet() && bIsManualPreviewing);
}

void SBlendSpaceGridWidget::BindCommands()
{
	// This should not be called twice on the same instance
	check(!UICommandList.IsValid());
	UICommandList = MakeShared<FUICommandList>();

	UICommandList->MapAction(
		FGenericCommands::Get().Cut,
		FExecuteAction::CreateSP(this, &SBlendSpaceGridWidget::OnBlendSampleCut),
		FCanExecuteAction::CreateSP(this, &SBlendSpaceGridWidget::CanBlendSampleCutCopy)
	);
	
	UICommandList->MapAction(
		FGenericCommands::Get().Copy,
		FExecuteAction::CreateSP(this, &SBlendSpaceGridWidget::OnBlendSampleCopy),
		FCanExecuteAction::CreateSP(this, &SBlendSpaceGridWidget::CanBlendSampleCutCopy)
	);

	UICommandList->MapAction(
		FGenericCommands::Get().Paste,
		FExecuteAction::CreateSP(this, &SBlendSpaceGridWidget::OnBlendSamplePaste),
		FCanExecuteAction::CreateSP(this, &SBlendSpaceGridWidget::CanBlendSamplePaste)
	);

	UICommandList->MapAction(
		FGenericCommands::Get().Delete,
		FExecuteAction::CreateSP(this, &SBlendSpaceGridWidget::OnBlendSampleDelete),
		FCanExecuteAction::CreateSP(this, &SBlendSpaceGridWidget::CanBlendSampleDelete)
	);
}

bool SBlendSpaceGridWidget::CanBlendSampleCutCopy()
{
	return BlendSpaceBase.Get()->IsValidBlendSampleIndex(SelectedSampleIndex);
}

bool SBlendSpaceGridWidget::CanBlendSamplePaste()
{
	FString PastedText;
	FPlatformApplicationMisc::ClipboardPaste(PastedText);
	return PastedText.StartsWith(BlendSpaceBase.Get()->IsAsset() ? BlendSampleClipboardHeaderAsset : BlendSampleClipboardHeaderGraph);
}

bool SBlendSpaceGridWidget::CanBlendSampleDelete()
{
	return BlendSpaceBase.Get()->IsValidBlendSampleIndex(SelectedSampleIndex);
}

void SBlendSpaceGridWidget::OnBlendSampleDelete()
{
	if (SelectedSampleIndex != INDEX_NONE)
	{
		OnSampleRemoved.ExecuteIfBound(SelectedSampleIndex);

		if (SelectedSampleIndex == HighlightedSampleIndex)
		{
			HighlightedSampleIndex = INDEX_NONE;
			ResetToolTip();
		}

		SelectedSampleIndex = INDEX_NONE;
	}
}

void SBlendSpaceGridWidget::OnBlendSampleCut()
{
	if (BlendSpaceBase.Get()->IsValidBlendSampleIndex(SelectedSampleIndex))
	{
		OnBlendSampleCopy();
		OnSampleRemoved.ExecuteIfBound(SelectedSampleIndex);
	}
}

void SBlendSpaceGridWidget::OnBlendSampleCopy()
{
	typedef TJsonWriter<TCHAR, TPrettyJsonPrintPolicy<TCHAR>> FStringWriter;
	typedef TJsonWriterFactory<TCHAR, TPrettyJsonPrintPolicy<TCHAR>> FStringWriterFactory;
	TSharedRef<FJsonObject> RootJsonObject = MakeShareable(new FJsonObject());

	const UBlendSpace* BlendSpace = BlendSpaceBase.Get();
	if (!BlendSpace->IsValidBlendSampleIndex(SelectedSampleIndex))
	{
		return;
	}

	if (BlendSpace->IsAsset())
	{
		FBlendSample SampleToCopy = BlendSpace->GetBlendSample(SelectedSampleIndex);
		FJsonObjectConverter::UStructToJsonObject(FBlendSample::StaticStruct(), &SampleToCopy, RootJsonObject, 0 /* CheckFlags */, 0 /* SkipFlags */);
	}
	else
	{
		UBlendSpaceGraph* BlendSpaceGraph = CastChecked<UBlendSpaceGraph>(BlendSpace->GetOuter());
		UAnimGraphNode_BlendSpaceGraphBase* BlendSpaceNode = CastChecked<UAnimGraphNode_BlendSpaceGraphBase>(BlendSpaceGraph->GetOuter());
		UAnimationBlendSpaceSampleGraph* GraphSampleToCopy = CastChecked<UAnimationBlendSpaceSampleGraph>(BlendSpaceNode->GetGraphs()[SelectedSampleIndex]);

		TSet<UObject*> NodesSet;
		for (TObjectPtr<UEdGraphNode>& Node : GraphSampleToCopy->Nodes)
		{
			NodesSet.Add(Node);
		}

		FStringOutputDevice NodesString;
		FEdGraphUtilities::ExportNodesToText(NodesSet, NodesString);
		RootJsonObject->SetField(TEXT("Nodes"), MakeShared<FJsonValueString>(NodesString));

		// Output Animation Pose node is not pasted as it already exists in the pasted graph
		// Take note of the node that connects to it to reconstruct the link (if exists) on paste
		UEdGraphPin* ResultNodePosePin = GraphSampleToCopy->ResultNode->FindPinChecked(TEXT("Result"), EEdGraphPinDirection::EGPD_Input);
		if (ResultNodePosePin->LinkedTo.Num() == 1)
		{
			UEdGraphPin* ConnectedNodePosePin = ResultNodePosePin->LinkedTo[0];
			UEdGraphNode* OwningNode = ConnectedNodePosePin->GetOwningNode();
			RootJsonObject->SetField(TEXT("NodeConnectedToResult"), MakeShared<FJsonValueString>(OwningNode->NodeGuid.ToString()));
		}
	}

	FString SerializedStr;
	TSharedRef<FStringWriter> Writer = FStringWriterFactory::Create(&SerializedStr);
	FJsonSerializer::Serialize(RootJsonObject, Writer);
	SerializedStr = *FString::Printf(TEXT("%s%s"), BlendSpace->IsAsset() ? *BlendSampleClipboardHeaderAsset : *BlendSampleClipboardHeaderGraph, *SerializedStr);
	FPlatformApplicationMisc::ClipboardCopy(*SerializedStr);
}

void SBlendSpaceGridWidget::OnBlendSamplePaste()
{
	FString PastedText;
	FPlatformApplicationMisc::ClipboardPaste(PastedText);

	check(OnSampleAdded.IsBound());

	const FVector SampleValue = ScreenPositionToSampleValue(LocalMousePosition, true); // Paste the sample at the cursor's location
	int32 NewSampleIndex = INDEX_NONE;

	if (BlendSpaceBase.Get()->IsAsset())
	{
		check(PastedText.StartsWith(BlendSampleClipboardHeaderAsset));
		PastedText.RightChopInline(BlendSampleClipboardHeaderAsset.Len());

		TSharedPtr<FJsonObject> RootJsonObject;
		TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(PastedText);
		if (!FJsonSerializer::Deserialize(Reader, RootJsonObject))
		{
			return;
		}

		FBlendSample SampleToPaste;
		if (FJsonObjectConverter::JsonObjectToUStruct(RootJsonObject.ToSharedRef(), FBlendSample::StaticStruct(), &SampleToPaste, 0 /* CheckFlags */, 0 /* SkipFlags */))
		{
			const FScopedTransaction Transaction(LOCTEXT("PasteBlendSpaceSample", "Paste Blend Space Sample"));
			NewSampleIndex = OnSampleAdded.Execute(SampleToPaste.Animation, SampleValue, false);
		}
	}
	else
	{
		check(PastedText.StartsWith(BlendSampleClipboardHeaderGraph));
		PastedText.RightChopInline(BlendSampleClipboardHeaderGraph.Len());

		TSharedPtr<FJsonObject> RootJsonObject;
		TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(PastedText);
		if (!FJsonSerializer::Deserialize(Reader, RootJsonObject))
		{
			return;
		}

		check(RootJsonObject->HasField(TEXT("Nodes")));

		const FScopedTransaction Transaction(LOCTEXT("PasteBlendSpaceSample", "Paste Blend Space Sample"));

		UBlendSpaceGraph* BlendSpaceGraph = CastChecked<UBlendSpaceGraph>(BlendSpaceBase.Get()->GetOuter());
		UAnimGraphNode_BlendSpaceGraphBase* BlendSpaceNode = CastChecked<UAnimGraphNode_BlendSpaceGraphBase>(BlendSpaceGraph->GetOuter());
		UAnimationBlendSpaceSampleGraph* DestinationGraph = nullptr;
		NewSampleIndex = OnSampleAdded.Execute(nullptr, SampleValue, false);
		DestinationGraph = CastChecked<UAnimationBlendSpaceSampleGraph>(BlendSpaceNode->GetGraphs()[NewSampleIndex]);

		FString NodesSetString = RootJsonObject->GetStringField(TEXT("Nodes"));
		TSet<UEdGraphNode*> ImportedNodes;
		FEdGraphUtilities::ImportNodesFromText(DestinationGraph, NodesSetString, ImportedNodes);

		// Reconstruct link to output, if exists
		if (RootJsonObject->HasField(TEXT("NodeConnectedToResult")))
		{
			FString NodeConnectedToResult = RootJsonObject->GetStringField(TEXT("NodeConnectedToResult"));
			FGuid ResultNodeGuid(NodeConnectedToResult);
			
			UEdGraphPin* ResultNodePosePin = DestinationGraph->ResultNode->FindPinChecked(TEXT("Result"), EEdGraphPinDirection::EGPD_Input);
			check(ResultNodePosePin->LinkedTo.Num() == 0);
			for (UEdGraphNode* Node : ImportedNodes)
			{
				if (Node->NodeGuid == ResultNodeGuid)
				{
					UEdGraphPin* PosePin = Node->FindPinChecked(TEXT("Pose"), EEdGraphPinDirection::EGPD_Output);
					check(PosePin->LinkedTo.Num() == 0);

					PosePin->MakeLinkTo(ResultNodePosePin);
					break;
				}
			}
		}

		for (UEdGraphNode* ImportedNode : ImportedNodes)
		{
			ImportedNode->CreateNewGuid();
		}
	}

	SelectedSampleIndex = NewSampleIndex;
	HighlightedSampleIndex = INDEX_NONE;
}

#undef LOCTEXT_NAMESPACE // "SAnimationBlendSpaceGridWidget"
