// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/SlateColor.h"
#include "Layout/Visibility.h"
#include "Layout/SlateRect.h"
#include "Input/Reply.h"
#include "Layout/Margin.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Fonts/SlateFontInfo.h"
#include "Widgets/SToolTip.h"
#include "Misc/NotifyHook.h"
#include "Animation/AnimationAsset.h"
#include "PersonaDelegates.h"
#include "StatusBarSubsystem.h"

class FPaintArgs;
class FSlateWindowElementList;
class SBox;
class UAnimSequence;
class UBlendSpace;

DECLARE_DELEGATE_ThreeParams(FOnSampleMoved, const int32 /*SampleIndex*/, const FVector& /*SampleValue*/, bool /*bIsInteractive*/);
DECLARE_DELEGATE_TwoParams(FOnSampleAnimationChanged, UAnimSequence*, const FVector&);

class SBlendSpaceGridWidget : public SCompoundWidget, public FNotifyHook
{
public:
	friend class FBlendSampleDetails;
public:
	SLATE_BEGIN_ARGS(SBlendSpaceGridWidget) 
		: _BlendSpaceBase(nullptr)
		, _NotifyHook(nullptr)
		, _ReadOnly(false)
		, _ShowAxisLabels(true)
		, _ShowSettingsButtons(true)
		, _StatusBarName(TEXT("AssetEditor.AnimationEditor.MainMenu"))
	{}
		SLATE_ATTRIBUTE(const UBlendSpace*, BlendSpaceBase)
		SLATE_ATTRIBUTE(FVector, Position)
		SLATE_ATTRIBUTE(FVector, FilteredPosition)
		SLATE_ARGUMENT(FNotifyHook*, NotifyHook)
		SLATE_ARGUMENT(bool, ReadOnly)
		SLATE_ARGUMENT(bool, ShowAxisLabels)
		SLATE_ARGUMENT(bool, ShowSettingsButtons)
		SLATE_EVENT(FOnExtendBlendSpaceSampleTooltip, OnExtendSampleTooltip)
		SLATE_EVENT(FOnSampleMoved, OnSampleMoved)
		SLATE_EVENT(FOnBlendSpaceSampleRemoved, OnSampleRemoved)
		SLATE_EVENT(FOnBlendSpaceSampleAdded, OnSampleAdded)
		SLATE_EVENT(FOnBlendSpaceSampleDuplicated, OnSampleDuplicated)
		SLATE_EVENT(FOnBlendSpaceSampleReplaced, OnSampleReplaced)
		SLATE_EVENT(FOnBlendSpaceNavigateUp, OnNavigateUp)
		SLATE_EVENT(FOnBlendSpaceNavigateDown, OnNavigateDown)
		SLATE_EVENT(FOnBlendSpaceCanvasDoubleClicked, OnCanvasDoubleClicked)
		SLATE_EVENT(FOnBlendSpaceSampleDoubleClicked, OnSampleDoubleClicked)
		SLATE_EVENT(FOnGetBlendSpaceSampleName, OnGetBlendSpaceSampleName)
		SLATE_ARGUMENT(FName, StatusBarName)
	SLATE_END_ARGS()

protected:
	enum class EGridType
	{
		SingleAxis,
		TwoAxis
	};

	/** Represents the different states of a drag operation */
	enum class EDragState
	{
		/** The user has clicked a mouse button, but hasn't moved more then the drag threshold. */
		PreDrag,
		/** The user is dragging the selected sample. */
		DragSample,
		/** The user is setting the preview value */
		Preview,
		/** The user is dropping a new sample onto the grid */
		DragDrop,
		/** The user is dropping a new animation to an existing sample on the grid */
		DragDropOverride,
		/** The user is dropping an invalid animation sequence onto the grid */
		InvalidDragDrop,
		/** There is no active drag operation. */
		None
	};

public:

	~SBlendSpaceGridWidget();

	/**
	* Construct this widget
	*
	* @param	InArgs	The declaration data for this widget
	*/
	void Construct(const FArguments& InArgs);
	
	// SWidget Interface
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	virtual void OnDragLeave(const FDragDropEvent& DragDropEvent) override;
	virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;
	virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	virtual FReply OnKeyUp(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	virtual void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override;
	virtual void OnFocusLost(const FFocusEvent& InFocusEvent) override;
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	virtual bool SupportsKeyboardFocus() const override;
	// End of SWidget

	/** Returns the sample value for previewing the blend space */
	const FVector GetPreviewPosition() const;

	/** Records the state of the blend space being used for previewing so it can be displayed */
	void SetPreviewingState(const FVector& InPosition, const FVector& InFilteredPosition);

	/** Flag whether or not the user is actively previewing the blend space (moving the sample value) */
	const bool IsPreviewing() const;

	int32 GetSelectedSampleIndex() const { return SelectedSampleIndex; }

	void InvalidateCachedData();
	void InvalidateState();
protected:
	/** Drawing functionality for grid, legend, key and triangulation **/
	void PaintBackgroundAndGrid(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32& DrawLayerId) const;
	void PaintSampleKeys(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32& DrawLayerId) const;
	void PaintAxisText(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32& DrawLayerId) const;
	void PaintTriangulation(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32& DrawLayerId) const;
	void PaintAnimationNames(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32& DrawLayerId) const;
	void PaintGridSampleWeights(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32& DrawLayerId) const;

	/** Validation for drag and drop operation, will populate InvalidOperationText and return false in case it is invalid */
	const bool IsValidDragDropOperation(const FDragDropEvent& DragDropEvent, FText& InvalidOperationText);

	/** Validation functionality for the given animation sequence, will populate InvalidOperationText and return false in case it is invalid */
	bool ValidateAnimationSequence(const UAnimSequence* AnimationSequence, FText& InvalidOperationText) const;

	/** Preview toggling*/
	void StartPreviewing();
	void StopPreviewing();

	/** Common view options for the context menus */
	void MakeViewContextMenuEntries(FMenuBuilder& InMenuBuilder);

	/** Blend sample context menu creation */
	TSharedPtr<SWidget> CreateBlendSampleContextMenu();

	/** Create a new blend sample via the context menu */
	TSharedPtr<SWidget> CreateNewBlendSampleContextMenu(const FVector2D& InMousePosition);

	/** Construct the grid widget to change the grid position for the selected sample */
	TSharedPtr<SWidget> CreateGridEntryBox(const int32 BoxIndex, const bool bShowLabel);

	/** Handles a mouse click operation on mouse up */
	FReply ProcessClick(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);
	/** Toggle for drawing the triangulation (only available for 2D blendspace) */
	FReply ToggleTriangulationVisibility();
	/** Returns whether or not the triangulation button should be visible, see above*/
	EVisibility GetTriangulationButtonVisibility() const;
	/** Returns whether or not the animation names button should be visible */
	EVisibility GetAnimationNamesButtonVisibility() const;
	/** Toggle for how the grid area is determined (only available for 2D blendspace) */
	FReply ToggleFittingType();
	/** Returns whether or not the fitting button should be visible */
	EVisibility GetFittingButtonVisibility() const;
	/** Calculates a margin offset according to whether or not we should take into account the largest axis when creating the grid area*/
	void UpdateGridRatioMargin(const FVector2D& GeometrySize);
	FText GetFittingTypeButtonToolTipText() const;
	/** Toggles the animation labels being shown */
	FReply ToggleShowAnimationNames();

	/** Calculates the screen space grid points */
	void CalculateGridPoints();
	/** Snaps the given screen position to the closest grid point */
	const FVector2D SnapScreenPositionToGrid(const FVector2D& InPosition, bool ForceSnap) const;
	/** Snaps the given screen position to the sample value on the grid */
	const FVector ScreenPositionToSampleValueWithSnapping(const FVector2D& InPosition, bool bForceSnap) const;
	/** Returns the index of the closest grid point to the given mouse position */
	int32 FindClosestGridPointIndexFromScreenPosition(const FVector2D& InPosition) const;
	/** Converts the given sample value to a screen space position */
	const FVector2D SampleValueToScreenPosition(const FVector& SampleValue) const;
	/** Converts the given sample value to a normalized position (between 0 and 1) */
	const FVector2D SampleValueToNormalizedPosition(const FVector& SampleValue) const;
	/** Converts a screen space (grid) position to a sample value, with optional clamping */
	const FVector ScreenPositionToSampleValue(const FVector2D& GridPosition, bool bClamp) const;
	/** Returns the (calculated) grid rectangle given the supplied geometry, and calls UpdateGridRatioMargin */
	const FSlateRect GetGridRectangleFromGeometry(const FGeometry& MyGeometry);
	/** Checks whether or not the blendspace sample value is within the range of the mouse position */
	bool IsSampleValueWithinMouseRange(const FVector& SampleValue, float& OutDistance) const;
	/** Get the sample index that most closely corresponds to the mouse position */
	int32 GetClosestSamplePointIndexToMouse() const;

	/** Sets the tooltip instance on the underlying widget instance */
	void ShowToolTip();
	/** Resets the tooltip instance on the underlying widget instance to nullptr*/
	void ResetToolTip();
	/** Tool tip text content getters*/
	FText GetToolTipAnimationName() const;
	FText GetToolTipSampleValue() const;
	FText GetToolTipSampleValidity() const;
	FText GetSampleErrorMessage(const struct FBlendSample& BlendSample) const;

	/** Functionality for the grid input box widgets (also used by FBlendSampleDetails) */
	EVisibility GetInputBoxVisibility(const int32 ParameterIndex) const;
	TOptional<float> GetInputBoxValue(const int32 ParameterIndex) const;
	TOptional<float> GetInputBoxMinValue(const int32 ParameterIndex) const;
	TOptional<float> GetInputBoxMaxValue(const int32 ParameterIndex) const;
	float GetInputBoxDelta(const int32 ParameterIndex) const;
	void OnInputBoxValueCommited(const float NewValue, ETextCommit::Type CommitType, const int32 ParameterIndex);
	void OnInputBoxValueChanged(const float NewValue, const int32 ParameterIndex, bool bIsInteractive);
	void OnInputSliderBegin(const int32 ParameterIndex);
	void OnInputSliderEnd(const float NewValue, const int32 ParameterIndex);

	/** Binds commands to functions in this class */
	void BindCommands();

	/** Handlers for blend sample actions*/
	void OnBlendSampleCut();
	void OnBlendSampleCopy();
	void OnBlendSamplePaste();
	void OnBlendSampleDelete();

	/** Returns whether or not the action is possible */
	bool CanBlendSampleCutCopy();
	bool CanBlendSamplePaste();
	bool CanBlendSampleDelete();

	/** Returns whether or not the sample tool tips should be visible */
	EVisibility GetSampleToolTipVisibility() const;
	EVisibility GetPreviewToolTipVisibility() const;

	/** Updates the cached blend parameter data */
	void UpdateCachedBlendParameterData();

	/** Gets the name to display for a sample */
	FText GetSampleName(const FBlendSample& InBlendSample, int32 InSampleIndex) const;

	/** Enable/disable the status bar message */
	void EnableStatusBarMessage(bool bEnable);

	/**
	 * This returns the weight for this sample if a lookup was made on its position. Ideally this would be 1 
	 * (and will be for triangulation). It will not always be so when using the grid.
	 */
	float GetSampleLookupWeight(int32 SampleIndex) const;
private:
	/** Currently visualized blendspace (const to ensure changes to it are only made within SAnimationBlendSpace */
	TAttribute<const UBlendSpace*> BlendSpaceBase;
	/** Previous blendspace, cached here to allow us to update cached data when it changes. */
	TWeakObjectPtr<const UBlendSpace> PreviousBlendSpaceBase;
	/** Current BlendSpace params (driven from outside the widget) */
	TAttribute<FVector> TargetPosition;
	/** Current BlendSpace state - filtered version of Position (driven from outside the widget) */
	TAttribute<FVector> FilteredPosition;
	/** Notify hook (ptr to SAnimationBlendSpace instance), which is required for transacting FBlendSample object when edited using the context-menu/structure details panel */
	FNotifyHook* NotifyHook;
	/** Number of Blend Parameters to draw */
	uint32 BlendParametersToDraw;
	/** Grid type (either 1D or 2D) */
	EGridType GridType;

	/** Cached mouse interaction data */
	FVector2D LastMousePosition;
	FVector2D LocalMousePosition;
	FVector2D MouseDownPosition;
	bool bMouseIsOverGeometry;

	/** Selection and highlight sample index/state */
	int32 SelectedSampleIndex;
	int32 HighlightedSampleIndex;
	int32 ToolTipSampleIndex;
	bool bHighlightPreviewPin;

	/** Drag state and data (not drag/drop) */
	EDragState DragState;
	int32 DraggedSampleIndex;
	FVector LastDragPosition;

	/** Currently set preview blend sample value and state data */
	bool bSamplePreviewing;
	FVector2D LastPreviewingMousePosition;
	FVector PreviewPosition;
	bool bAdvancedPreview;
	TArray<FBlendSampleData> PreviewedSamples;
	/** Previous position in the triangulation/segmentation */
	int32 CachedTriangulationIndex = -1;

	/** Preview state that may have been returned */
	FVector PreviewFilteredPosition;

	/** Tooltip ptr which is shown when hovering/dropping/dragging a sample*/
	TSharedPtr<SToolTip> ToolTip;

	/** Container box used to extend a tooltip */
	TSharedPtr<SBox> ToolTipExtensionContainer;

	/** Drag and drop data */
	FText InvalidDragDropText;
	FText InvalidSamplePositionDragDropText;
	FText DragDropAnimationName;
	FText HoveredAnimationName;
	UAnimSequence* DragDropAnimationSequence;

	/** Cached values for the grid input boxes */
	float CachedInputBoxValues[2];
	bool bSliderMovement[2] = {false, false};

	// Cached grid data (derived from the blend space)
	FSlateRect CachedGridRectangle;
	FVector2D SampleValueMin;
	FVector2D SampleValueMax;
	FVector2D SampleValueRange;
	FVector2D SampleGridDelta;
	bool      bSampleSnapToGrid[2];
	FIntPoint SampleGridDivisions;
	FText ParameterXName;
	FText ParameterYName;
	TArray<FVector2D> CachedGridPoints;
	TArray<FVector> CachedSamplePoints;

	/** Commands that are bound to delegates */
	TSharedPtr<class FUICommandList> UICommandList;

	/** Whether or not the cached data should be refreshed on the next tick*/
	bool bRefreshCachedData;

	/** Cached draw-able axis information */
	FVector2D XAxisTextSize;
	float MaxVerticalAxisTextWidth;
	float MaxHorizontalAxisTextHeight;
	float HorizontalAxisMaxTextWidth;
	FVector2D YAxisTextSize;

	/** Delegates populated from SAnimationBlendSpace and used as callbacks */
	FOnBlendSpaceSampleAdded OnSampleAdded;
	FOnBlendSpaceSampleDuplicated OnSampleDuplicated;
	FOnSampleMoved OnSampleMoved;
	FOnBlendSpaceSampleRemoved OnSampleRemoved;
	FOnBlendSpaceSampleReplaced OnSampleReplaced;
	FOnBlendSpaceNavigateUp OnNavigateUp;
	FOnBlendSpaceNavigateDown OnNavigateDown;
	FOnBlendSpaceCanvasDoubleClicked OnCanvasDoubleClicked;
	FOnBlendSpaceSampleDoubleClicked OnSampleDoubleClicked;
	FOnGetBlendSpaceSampleName OnGetBlendSpaceSampleName;
	FOnExtendBlendSpaceSampleTooltip OnExtendSampleTooltip;

	/** Threshold values for hovering, click and dragging samples */
	float DragThreshold;
	float ClickAndHighlightThreshold;

	/** Sample drawing data */
	FVector2D KeySize;
	FVector2D PreviewSize;
	const FSlateBrush* KeyBrush;
	const FSlateBrush* PreviewBrush;
	const FSlateBrush* BackgroundImage;
	const FSlateBrush* LabelBrush;
	enum class EArrowDirection : uint8 
	{
		Left, 
		Right, 
		Up, 
		Down, 
		Max
	};
	const FSlateBrush* ArrowBrushes[(int32)EArrowDirection::Max];

	/** Individual sample state colours */
	FSlateColor HighlightKeyColor;
	FSlateColor KeyColor;
	FSlateColor SelectKeyColor;
	FSlateColor PreDragKeyColor;
	FSlateColor DragKeyColor;
	FSlateColor InvalidColor;
	FSlateColor DropKeyColor;
	FSlateColor PreviewKeyColor;

	/** Grid drawing and layout data */
	FMargin GridMargin;
	FLinearColor GridLinesColor;
	FLinearColor GridOutlineColor;
	FSlateColor TriangulationColor;
	FSlateColor TriangulationCurrentColor;
	FSlateFontInfo FontInfo;
	float TextMargin;
	bool bShowTriangulation;
	bool bShowAnimationNames;
	bool bStretchToFit;
	FMargin GridRatioMargin;
	bool bReadOnly;
	bool bShowAxisLabels;
	bool bShowSettingsButtons;

	FStatusBarMessageHandle StatusBarMessageHandle;

	FName StatusBarName;
};
