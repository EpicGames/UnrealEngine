// Copyright Epic Games, Inc. All Rights Reserved.


#include "SAnimationDlgs.h"
#include "Modules/ModuleManager.h"
#include "Misc/PackageName.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Editor.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Input/SButton.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"

#define LOCTEXT_NAMESPACE "SAnimationDlgs"

/////////////////////////////////////////////////////
// create path picker for importing assets
/////////////////////////////////////////////////////

FText SImportPathDialog::LastUsedAssetPath;

void SImportPathDialog::Construct(const FArguments& InArgs)
{
	AssetPath = FText::FromString(FPackageName::GetLongPackagePath(InArgs._DefaultAssetPath.ToString()));

	if(AssetPath.IsEmpty())
	{
		AssetPath = LastUsedAssetPath;
	}
	else
	{
		LastUsedAssetPath = AssetPath;
	}

	FPathPickerConfig PathPickerConfig;
	PathPickerConfig.DefaultPath = AssetPath.ToString();
	PathPickerConfig.OnPathSelected = FOnPathSelected::CreateSP(this, &SImportPathDialog::OnPathChange);
	PathPickerConfig.bAddDefaultPath = true;

	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");

	SWindow::Construct(SWindow::FArguments()
		.Title(LOCTEXT("SImportPathDialog_Title", "Select folder to import to"))
		.SupportsMinimize(false)
		.SupportsMaximize(false)
		//.SizingRule( ESizingRule::Autosized )
		.ClientSize(FVector2D(450, 450))
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot() // Add user input block
			.Padding(2)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				[
					SNew(SVerticalBox)

					+SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("SelectPath", "Select Path to create animation"))
						.Font(FCoreStyle::GetDefaultFontStyle("Regular", 14))
					]

					+SVerticalBox::Slot()
					.FillHeight(1)
					.Padding(3)
					[
						ContentBrowserModule.Get().CreatePathPicker(PathPickerConfig)
					]
				]
			]

			+SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			.Padding(5)
			[
				SNew(SUniformGridPanel)
				.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
				.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
				.MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))
				+SUniformGridPanel::Slot(0, 0)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
					.Text(LOCTEXT("OK", "OK"))
					.OnClicked(this, &SImportPathDialog::OnButtonClick, EAppReturnType::Ok)
					.IsEnabled(this, &SImportPathDialog::IsOkButtonEnabled)
				]
				+SUniformGridPanel::Slot(1, 0)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
					.Text(LOCTEXT("Cancel", "Cancel"))
					.OnClicked(this, &SImportPathDialog::OnButtonClick, EAppReturnType::Cancel)
				]
			]
		]);
}

void SImportPathDialog::OnPathChange(const FString& NewPath)
{
	AssetPath = FText::FromString(NewPath);
	LastUsedAssetPath = AssetPath;
}

FReply SImportPathDialog::OnButtonClick(EAppReturnType::Type ButtonID)
{
	UserResponse = ButtonID;
	RequestDestroyWindow();

	return FReply::Handled();
}

bool SImportPathDialog::IsOkButtonEnabled() const
{
	return !AssetPath.IsEmptyOrWhitespace();
}

EAppReturnType::Type SImportPathDialog::ShowModal()
{
	GEditor->EditorAddModalWindow(SharedThis(this));
	return UserResponse;
}

FString SImportPathDialog::GetAssetPath()
{
	return AssetPath.ToString();
}

#undef LOCTEXT_NAMESPACE

