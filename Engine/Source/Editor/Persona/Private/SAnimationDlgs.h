// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWindow.h"

class SImportPathDialog: public SWindow
{
public:
	SLATE_BEGIN_ARGS(SImportPathDialog)
	{
	}

	SLATE_ARGUMENT(FText, DefaultAssetPath)
	SLATE_END_ARGS()

	SImportPathDialog()
		: UserResponse(EAppReturnType::Cancel)
	{
		}

	void Construct(const FArguments& InArgs);

public:
	/** Displays the dialog in a blocking fashion */
	EAppReturnType::Type ShowModal();

	/** Gets the resulting asset path */
	FString GetAssetPath();

protected:
	void OnPathChange(const FString& NewPath);
	FReply OnButtonClick(EAppReturnType::Type ButtonID);
	bool IsOkButtonEnabled() const;
	EAppReturnType::Type UserResponse;
	FText AssetPath;
	FText AssetName;
	FText Duration;

	static FText LastUsedAssetPath;
};


