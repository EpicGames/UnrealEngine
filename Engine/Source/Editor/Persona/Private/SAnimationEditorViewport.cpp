// Copyright Epic Games, Inc. All Rights Reserved.


#include "SAnimationEditorViewport.h"

#include "AdvancedPreviewSceneMenus.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SButton.h"
#include "Animation/AnimMontage.h"
#include "Preferences/PersonaOptions.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "SceneInterface.h"

#include "SAnimationScrubPanel.h"
#include "SAnimMontageScrubPanel.h"
#include "AnimViewportMenuCommands.h"
#include "AnimViewportShowCommands.h"
#include "AnimViewportLODCommands.h"
#include "AnimViewportPlaybackCommands.h"
#include "AnimPreviewInstance.h"
#include "Widgets/Input/STextComboBox.h"
#include "IEditableSkeleton.h"
#include "EditorAxisDisplayInfo.h"
#include "EditorViewportCommands.h"
#include "TabSpawners.h"
#include "ShowFlagMenuCommands.h"
#include "BufferVisualizationMenuCommands.h"
#include "NaniteVisualizationMenuCommands.h"
#include "UICommandList_Pinnable.h"
#include "IPersonaEditorModeManager.h"
#include "PreviewProfileController.h"
#include "Materials/Material.h"
#include "EditorFontGlyphs.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "ContextObjectStore.h"
#include "IPersonaEditMode.h"

#include "IPersonaToolkit.h"
#include "IPinnedCommandList.h"
#include "IPinnedCommandListModule.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "SNameComboBox.h"
#include "ToolMenus.h"
#include "Viewports.h"
#include "ViewportToolbar/AnimViewportContext.h"
#include "ViewportToolbar/AnimationEditorMenus.h"
#include "ViewportToolbar/UnrealEdViewportToolbar.h"
#include "Widgets/Text/SRichTextBlock.h"

#define LOCTEXT_NAMESPACE "PersonaViewportToolbar"

//////////////////////////////////////////////////////////////////////////
// SAnimationEditorViewport

void SAnimationEditorViewport::Construct(const FArguments& InArgs, const FAnimationEditorViewportRequiredArgs& InRequiredArgs)
{
	PreviewScenePtr = InRequiredArgs.PreviewScene;
	TabBodyPtr = InRequiredArgs.TabBody;
	AssetEditorToolkitPtr = InRequiredArgs.AssetEditorToolkit;
	Extenders = InArgs._Extenders;
	ContextName = InArgs._ContextName;
	bShowShowMenu = InArgs._ShowShowMenu;
	bShowLODMenu = InArgs._ShowLODMenu;
	bShowPlaySpeedMenu = InArgs._ShowPlaySpeedMenu;
	bShowStats = InArgs._ShowStats;
	bShowFloorOptions = InArgs._ShowFloorOptions;
	bShowTurnTable = InArgs._ShowTurnTable;
	bShowPhysicsMenu = InArgs._ShowPhysicsMenu;
	ViewportIndex = InRequiredArgs.ViewportIndex;

	SEditorViewport::Construct(
		SEditorViewport::FArguments()
			.IsEnabled(FSlateApplication::Get().GetNormalExecutionAttribute())
			.AddMetaData<FTagMetaData>(TEXT("Persona.Viewport"))
		);

	Client->VisibilityDelegate.BindSP(this, &SAnimationEditorViewport::IsVisible);

	// restore last used feature level
	auto ScenePtr = PreviewScenePtr.Pin();
	if (ScenePtr.IsValid())
	{
		UWorld* World = ScenePtr->GetWorld();
		if (World != nullptr && GWorld != nullptr && GWorld->Scene != nullptr)
		{
			World->ShaderPlatformChanged(GWorld->Scene->GetShaderPlatform());
		}
	}

	UEditorEngine* Editor = (UEditorEngine*)GEngine;
	PreviewFeatureLevelChangedHandle = Editor->OnPreviewShaderPlatformChanged().AddLambda(
		[PreviewScenePtrWeak = PreviewScenePtr](EShaderPlatform NewShaderPlatform)
		{
			if (TSharedPtr<IPersonaPreviewScene> ScenePtr = PreviewScenePtrWeak.Pin())
			{
				UWorld* World = ScenePtr->GetWorld();
				if (World != nullptr)
				{
					World->ShaderPlatformChanged(NewShaderPlatform);
				}
			}
		});
}

SAnimationEditorViewport::~SAnimationEditorViewport()
{
	UEditorEngine* Editor = (UEditorEngine*)GEngine;
	Editor->OnPreviewShaderPlatformChanged().Remove(PreviewFeatureLevelChangedHandle);
}

void SAnimationEditorViewport::PopulateViewportOverlays(TSharedRef<SOverlay> Overlay)
{
	SEditorViewport::PopulateViewportOverlays(Overlay);

	// add the feature level display widget
	Overlay->AddSlot()
		.VAlign(VAlign_Bottom)
		.HAlign(HAlign_Right)
		.Padding(5.0f)
		[
			BuildShaderPlatformWidget()
		];

	if (TSharedPtr<SAnimationEditorViewportTabBody> ViewportTab = TabBodyPtr.Pin())
	{
		TSharedRef<SVerticalBox> VerticalBox = SNew(SVerticalBox);

		// clang-format off
		Overlay->AddSlot()
			.VAlign(VAlign_Top)
			.HAlign(HAlign_Fill)
		[
			VerticalBox
		];
		// clang-format on

		if (TSharedPtr<IPinnedCommandList> PinnedCommands = ViewportTab->GetPinnedCommands())
		{
			// clang-format off
			VerticalBox->AddSlot()
			.AutoHeight()
			[
				PinnedCommands.ToSharedRef()
			];
			// clang-format on
		}

		// clang-format off
		VerticalBox->AddSlot()
		.AutoHeight()
		.Padding(FMargin(4.0f, 3.0f, 0.0f, 0.0f))
		[
			// Display text (e.g., item being previewed)
			SNew(SRichTextBlock)
			.Visibility(EVisibility::SelfHitTestInvisible)
			.DecoratorStyleSet(&FAppStyle::Get())
			.Text(ViewportTab.Get(), &SAnimationEditorViewportTabBody::GetDisplayString)
			.TextStyle(&FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("AnimViewport.MessageText"))
		];
		// clang-format on
	}
}

TSharedRef<FEditorViewportClient> SAnimationEditorViewport::MakeEditorViewportClient()
{
	using namespace EditorViewportDefs;

	// Create an animation viewport client
	LevelViewportClient = MakeShareable(new FAnimationViewportClient(PreviewScenePtr.Pin().ToSharedRef(), SharedThis(this), AssetEditorToolkitPtr.Pin().ToSharedRef(), ViewportIndex, bShowStats));

	// Done after constructor, as the delegates require the shared pointer to be assigned
	LevelViewportClient->Initialize();

	LevelViewportClient->ViewportType = LVT_Perspective;
	LevelViewportClient->bSetListenerPosition = false;

	const bool bUsingLUFCoordinateSysem = AxisDisplayInfo::GetAxisDisplayCoordinateSystem() == EAxisList::LeftUpForward;

	FVector LocalViewLocation = FVector(DefaultPerspectiveViewLocation.X, 
								(bUsingLUFCoordinateSysem ? -DefaultPerspectiveViewLocation.Y : DefaultPerspectiveViewLocation.Y), 
								DefaultPerspectiveViewLocation.Z);
	FRotator LocalViewRotation = DefaultPerspectiveViewRotation + (bUsingLUFCoordinateSysem ? FRotator(0, -90, 0) : FRotator(0, 0, 0));
	LevelViewportClient->SetInitialViewTransform(LVT_Perspective, LocalViewLocation, LocalViewRotation, DEFAULT_ORTHOZOOM);

	return LevelViewportClient.ToSharedRef();
}

TSharedPtr<SWidget> SAnimationEditorViewport::BuildViewportToolbar()
{
	// Register the viewport toolbar if another viewport hasn't already (it's shared).
	const FName ViewportToolbarName = "AnimationEditor.ViewportToolbar";

	if (!UToolMenus::Get()->IsMenuRegistered(ViewportToolbarName))
	{
		UToolMenu* const ViewportToolbarMenu = UToolMenus::Get()->RegisterMenu(
			ViewportToolbarName, NAME_None /* parent */, EMultiBoxType::SlimHorizontalToolBar
		);

		ViewportToolbarMenu->StyleName = "ViewportToolbar";

		// Add the left-aligned part of the viewport toolbar.
		{
			FToolMenuSection& LeftSection = ViewportToolbarMenu->AddSection("Left");

			LeftSection.AddEntry(UE::UnrealEd::CreateTransformsSubmenu());
			// TODO: Needs specific Select menu for Skel Mesh.
			LeftSection.AddEntry(UE::UnrealEd::CreateSnappingSubmenu());
		}

		// Add the right-aligned part of the viewport toolbar.
		{
			FToolMenuSection& RightSection = ViewportToolbarMenu->AddSection("Right");
			RightSection.Alignment = EToolMenuSectionAlign::Last;

			// Add the "Camera" submenu.
			{
				// Build the menu name our Camera menu will be using so we can extend it.
				RightSection.AddEntry(UE::UnrealEd::CreateCameraSubmenu(UE::UnrealEd::FViewportCameraMenuOptions().ShowAll()));
				const FName SubmenuName = UToolMenus::JoinMenuPaths(ViewportToolbarName, "Camera");
				UE::AnimationEditor::ExtendCameraMenu(SubmenuName);
			}

			// Add the "View Modes" sub menu.
			{
				const FName ViewModesMenuName = "AnimationEditor.ViewportToolbar.ViewModes";

				// Stay backward-compatible with the old viewport toolbar.
				{
					const FName ParentSubmenuName = "UnrealEd.ViewportToolbar.View";
					// Create our parent menu.
					if (!UToolMenus::Get()->IsMenuRegistered(ParentSubmenuName))
					{
						UToolMenus::Get()->RegisterMenu(ParentSubmenuName);
					}

					// Register our ToolMenu here first, before we create the submenu, so we can set our parent.
					UToolMenus::Get()->RegisterMenu(ViewModesMenuName, ParentSubmenuName);
				}

				UE::AnimationEditor::ExtendViewModesSubmenu(ViewModesMenuName);

				RightSection.AddEntry(UE::UnrealEd::CreateViewModesSubmenu());
			}

			RightSection.AddEntry(UE::AnimationEditor::CreateShowSubmenu());
			RightSection.AddEntry(UE::AnimationEditor::CreateLODSubmenu());
			RightSection.AddEntry(UE::AnimationEditor::CreateSkinWeightProfileMenu());
			
			// Add Preview Scene Submenu
			{
				const FName AssetViewerProfileMenuName = "AnimationEditor.ViewportToolbar.AssetViewerProfile";
				RightSection.AddEntry(UE::UnrealEd::CreateAssetViewerProfileSubmenu());
				UE::AdvancedPreviewScene::Menus::ExtendAdvancedPreviewSceneSettings(AssetViewerProfileMenuName);
				UE::AnimationEditor::ExtendPreviewSceneSettingsSubmenu(AssetViewerProfileMenuName);
				UE::UnrealEd::ExtendPreviewSceneSettingsWithTabEntry(AssetViewerProfileMenuName);
			}

			// Add the "Physics" sub menu (only shown when bShowPhysicsMenu is true)
			{
				FToolMenuInsert PhysicsSubmenuInsert("LOD", EToolMenuInsertType::Before);
				UE::AnimationEditor::AddPhysicsMenu(ViewportToolbarName, PhysicsSubmenuInsert);
			}
		}
	}

	FToolMenuContext ViewportToolbarContext;
	{
		if (TSharedPtr<IPersonaPreviewScene> PreviewScene = PreviewScenePtr.Pin())
		{
			ViewportToolbarContext.AppendCommandList(PreviewScene->GetCommandList());
		}

		if (TSharedPtr<SAnimationEditorViewportTabBody> TabBody = TabBodyPtr.Pin())
		{
			ViewportToolbarContext.AppendCommandList(TabBody->GetCommandList());
		}

		ViewportToolbarContext.AppendCommandList(GetCommandList());
		
		// Add extenders
		{
			Extenders.Add(UE::AnimationEditor::GetViewModesLegacyExtenders(TabBodyPtr));
			ViewportToolbarContext.AddExtender(FExtender::Combine(Extenders));
		}

		// Add the UnrealEd viewport toolbar context.
		{
			UUnrealEdViewportToolbarContext* const ContextObject =
				UE::UnrealEd::CreateViewportToolbarDefaultContext(SharedThis(this));

			ContextObject->AssetEditorToolkit = AssetEditorToolkitPtr;
			ContextObject->PreviewSettingsTabId = FPersonaTabs::AdvancedPreviewSceneSettingsID;

			ViewportToolbarContext.AddObject(ContextObject);
		}

		// Add the Anim viewport toolbar context.
		{
			UAnimViewportContext* const ContextObject = NewObject<UAnimViewportContext>();
			ContextObject->ViewportTabBody = TabBodyPtr.Pin();

			ViewportToolbarContext.AddObject(ContextObject);
		}

		// Give the asset editor a chance to extend the context.
		if (TSharedPtr<SAnimationEditorViewportTabBody> Tab = TabBodyPtr.Pin())
		{
			Tab->GetAssetEditorToolkit()->InitToolMenuContext(ViewportToolbarContext);
		}
	}

	return UToolMenus::Get()->GenerateWidget(ViewportToolbarName, ViewportToolbarContext);
}

TSharedPtr<IPreviewProfileController> SAnimationEditorViewport::CreatePreviewProfileController()
{
	return MakeShared<FPreviewProfileController>();
}

void SAnimationEditorViewport::PostUndo( bool bSuccess )
{
	LevelViewportClient->Invalidate();
}

void SAnimationEditorViewport::PostRedo( bool bSuccess )
{
	LevelViewportClient->Invalidate();
}

void SAnimationEditorViewport::OnFocusViewportToSelection()
{
	if (LevelViewportClient)
	{
		TSharedRef<FAnimationViewportClient> AnimViewportClient = StaticCastSharedRef<FAnimationViewportClient>(LevelViewportClient.ToSharedRef());
		AnimViewportClient->OnFocusViewportToSelection();
	}
}

void SAnimationEditorViewport::BindCommands()
{
	SEditorViewport::BindCommands();

	FShowFlagMenuCommands::Get().BindCommands(*CommandList, Client);
	FBufferVisualizationMenuCommands::Get().BindCommands(*CommandList, Client);
	FNaniteVisualizationMenuCommands::Get().BindCommands(*CommandList, Client);

	if (TSharedPtr<SAnimationEditorViewportTabBody> TabBody = TabBodyPtr.Pin())
	{
		if (TSharedPtr<FAssetEditorToolkit> ParentAssetEditor = TabBody->GetAssetEditorToolkit())
		{
			CommandList->Append(ParentAssetEditor->GetToolkitCommands());
		}
	}
}

void SAnimationEditorViewport::OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	SEditorViewport::OnDragEnter(MyGeometry, DragDropEvent);
	if(AssetEditorToolkitPtr.IsValid())
	{
		AssetEditorToolkitPtr.Pin()->OnViewportDragEnter(MyGeometry, DragDropEvent);
	}
}

void SAnimationEditorViewport::OnDragLeave(const FDragDropEvent& DragDropEvent)
{
	SEditorViewport::OnDragLeave(DragDropEvent);
	if(AssetEditorToolkitPtr.IsValid())
	{
		AssetEditorToolkitPtr.Pin()->OnViewportDragLeave(DragDropEvent);
	}
}

FReply SAnimationEditorViewport::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	if(AssetEditorToolkitPtr.IsValid())
	{
		const FReply ReplyFromToolkit = AssetEditorToolkitPtr.Pin()->OnViewportDrop(MyGeometry, DragDropEvent);
		if(ReplyFromToolkit.IsEventHandled())
		{
			return ReplyFromToolkit;
		}
	}
	return SEditorViewport::OnDrop(MyGeometry, DragDropEvent);
}

//////////////////////////////////////////////////////////////////////////
// SAnimationEditorViewportTabBody

SAnimationEditorViewportTabBody::SAnimationEditorViewportTabBody()
	: SelectedTurnTableSpeed(EAnimationPlaybackSpeeds::Normal)
	, SelectedTurnTableMode(EPersonaTurnTableMode::Stopped)
	, SectionsDisplayMode(ESectionDisplayMode::None)
{
	CreatePinnedCommands();
}

bool SAnimationEditorViewportTabBody::CanUseGizmos() const
{
	if (bAlwaysShowTransformToolbar)
	{
		return true;
	}

	class UDebugSkelMeshComponent* Component = GetPreviewScene()->GetPreviewMeshComponent();

	if (Component != NULL)
	{
		if (Component->bForceRefpose)
		{
			return false;
		}
		else if (Component->IsPreviewOn())
		{
			return true;
		}
	}
	
	if (LevelViewportClient.IsValid())
	{
		if(const FEditorModeTools* ModeTools = LevelViewportClient->GetModeTools())
		{
			if(ModeTools->UsesTransformWidget())
			{
				return true;
			}
		}
	}
	
	return false;
}

FText ConcatenateLine(const FText& InText, const FText& InNewLine)
{
	if(InText.IsEmpty())
	{
		return InNewLine;
	}

	return FText::Format(LOCTEXT("ViewportTextNewlineFormatter", "{0}\n{1}"), InText, InNewLine);
}

FText SAnimationEditorViewportTabBody::GetDisplayString() const
{
	class UDebugSkelMeshComponent* Component = GetPreviewScene()->GetPreviewMeshComponent();
	TSharedPtr<IEditableSkeleton> EditableSkeleton = GetPreviewScene()->GetPersonaToolkit()->GetEditableSkeleton();
	FName TargetSkeletonName = (EditableSkeleton.IsValid() && EditableSkeleton->IsSkeletonValid()) ? EditableSkeleton->GetSkeleton().GetFName() : NAME_None;

	FText DefaultText;

	if (Component != NULL)
	{
		if (Component->bForceRefpose)
		{
			DefaultText = LOCTEXT("ReferencePose", "Reference pose");
		}
		else if (Component->IsPreviewOn())
		{
			DefaultText = FText::Format(LOCTEXT("Previewing", "Previewing {0}"), FText::FromString(Component->GetPreviewText()));
		}
		else if (Component->AnimClass != NULL)
		{
			TSharedPtr<FBlueprintEditor> BPEditor = BlueprintEditorPtr.Pin();
			const bool bWarnAboutBoneManip = BPEditor.IsValid() && BPEditor->IsModeCurrent(FPersonaModes::AnimBlueprintEditMode);
			if (bWarnAboutBoneManip)
			{
				DefaultText = FText::Format(LOCTEXT("PreviewingAnimBP_WarnDisabled", "Previewing {0}. \nBone manipulation is disabled in this mode. "), FText::FromString(Component->AnimClass->GetName()));
			}
			else
			{
				DefaultText = FText::Format(LOCTEXT("PreviewingAnimBP", "Previewing {0}"), FText::FromString(Component->AnimClass->GetName()));
			}
		}
		else if (Component->GetSkeletalMeshAsset() == NULL && TargetSkeletonName != NAME_None)
		{
			DefaultText = FText::Format(LOCTEXT("NoMeshFound", "No skeletal mesh found for skeleton '{0}'"), FText::FromName(TargetSkeletonName));
		}

		if (Component->IsDisplayingNaniteFallback())
		{
			DefaultText = ConcatenateLine(LOCTEXT("ShowingFallbackWarning", "<TextBlock.ShadowedTextWarning>(Showing Fallback)</>"), DefaultText);
		}
	}

	if(OnGetViewportText.IsBound())
	{
		DefaultText = ConcatenateLine(DefaultText, OnGetViewportText.Execute(EViewportCorner::TopLeft));
	}

	TSharedPtr<FAnimationViewportClient> AnimViewportClient = StaticCastSharedPtr<FAnimationViewportClient>(LevelViewportClient);

	if(AnimViewportClient->IsShowingMeshStats())
	{
		DefaultText = ConcatenateLine(DefaultText, AnimViewportClient->GetDisplayInfo(AnimViewportClient->IsDetailedMeshStats()));
	}
	else if(AnimViewportClient->IsShowingSelectedNodeStats())
	{
		// Allow edit modes (inc. skeletal control modes) to draw with the canvas, and collect on screen strings to draw later
		if (IAnimationEditContext* PersonaContext = AnimViewportClient->GetModeTools()->GetInteractiveToolsContext()->ContextObjectStore->FindContext<UAnimationEditModeContext>())
		{
			TArray<FText> EditModeDebugText;
			PersonaContext->GetOnScreenDebugInfo(EditModeDebugText);
			for(FText& Text : EditModeDebugText)
			{
				DefaultText = ConcatenateLine(DefaultText, Text);
			}
		}
	}

	if(Component)
	{
		for(const FGetExtendedViewportText& TextDelegate : Component->GetExtendedViewportTextDelegates())
		{
			DefaultText = ConcatenateLine(DefaultText, TextDelegate.Execute());
		}
	}

	return DefaultText;
}

TSharedRef<IPersonaViewportState> SAnimationEditorViewportTabBody::SaveState() const
{
	TSharedRef<FPersonaModeSharedData> State = MakeShareable(new(FPersonaModeSharedData));

	if (const TSharedPtr<FAnimationViewportClient>& AnimViewportClient = GetAnimationViewportClient())
	{
		State->Save(AnimViewportClient.ToSharedRef());
	}
	return State;
}

void SAnimationEditorViewportTabBody::RestoreState(TSharedRef<IPersonaViewportState> InState)
{
	if (const TSharedPtr<FAnimationViewportClient>& AnimViewportClient = GetAnimationViewportClient())
	{
		TSharedRef<FPersonaModeSharedData> State = StaticCastSharedRef<FPersonaModeSharedData>(InState);
		State->Restore(AnimViewportClient.ToSharedRef());
	}
}

FEditorViewportClient& SAnimationEditorViewportTabBody::GetViewportClient() const
{
	return *LevelViewportClient;
}

TSharedRef<IPinnedCommandList> SAnimationEditorViewportTabBody::GetPinnedCommandList() const
{
	return PinnedCommands.ToSharedRef();
}

const TSharedPtr<IPinnedCommandList>& SAnimationEditorViewportTabBody::GetPinnedCommands()
{
	if (!PinnedCommands)
	{
		CreatePinnedCommands();
	}
	return PinnedCommands;
}

TWeakPtr<SWidget> SAnimationEditorViewportTabBody::AddNotification(TAttribute<EMessageSeverity::Type> InSeverity, TAttribute<bool> InCanBeDismissed, const TSharedRef<SWidget>& InNotificationWidget, FPersonaViewportNotificationOptions InOptions)
{
	TSharedPtr<SBorder> ContainingWidget = nullptr;
	TWeakPtr<SWidget> WeakNotificationWidget = InNotificationWidget;

	auto GetPadding = [WeakNotificationWidget]()
	{
		if(WeakNotificationWidget.IsValid())
		{
			return WeakNotificationWidget.Pin()->GetVisibility() == EVisibility::Visible ? FMargin(2.0f) : FMargin(0.0f);
		}

		return FMargin(0.0f);
	};

	TAttribute<EVisibility> GetVisibility(EVisibility::Visible);
	
	if (InOptions.OnGetVisibility.IsSet())
	{
		GetVisibility = InOptions.OnGetVisibility;
	}
	
	TAttribute<const FSlateBrush*> GetBrushForSeverity = TAttribute<const FSlateBrush*>::Create([InSeverity]()
	{
		switch(InSeverity.Get())
		{
		case EMessageSeverity::Error:
			return FAppStyle::GetBrush("AnimViewport.Notification.Error");
		case EMessageSeverity::PerformanceWarning:
		case EMessageSeverity::Warning:
			return FAppStyle::GetBrush("AnimViewport.Notification.Warning");
		default:
		case EMessageSeverity::Info:
			return FAppStyle::GetBrush("AnimViewport.Notification.Message");
		}
	});

	if (InOptions.OnGetBrushOverride.IsSet())
	{
		GetBrushForSeverity = InOptions.OnGetBrushOverride;
	}

	TSharedPtr<SHorizontalBox> BodyBox = nullptr;

	ViewportNotificationsContainer->AddSlot()
	.HAlign(HAlign_Right)
	.AutoHeight()
	.Padding(MakeAttributeLambda(GetPadding))
	[
		SAssignNew(ContainingWidget, SBorder)
		.Visibility(GetVisibility)
		.BorderImage(GetBrushForSeverity)
		[
			SAssignNew(BodyBox, SHorizontalBox)
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				InNotificationWidget
			]
		]
	];

	TWeakPtr<SWidget> WeakContainingWidget = ContainingWidget;
	auto DismissNotification = [this, WeakContainingWidget]()
	{
		if(WeakContainingWidget.IsValid())
		{
			RemoveNotification(WeakContainingWidget.Pin().ToSharedRef());
		}

		return FReply::Handled();
	};

	auto GetDismissButtonVisibility = [InCanBeDismissed]()
	{
		return InCanBeDismissed.Get() ? EVisibility::Visible : EVisibility::Collapsed;
	};

	// add dismiss button
	BodyBox->InsertSlot(0)
	.AutoWidth()
	.HAlign(HAlign_Center)
	.VAlign(VAlign_Top)
	[
		SNew(SButton)
		.Visibility_Lambda(GetDismissButtonVisibility)
		.ButtonStyle(FAppStyle::Get(), "AnimViewport.Notification.CloseButton")
		.ToolTipText(LOCTEXT("DismissNotificationToolTip", "Dismiss this notification."))
		.OnClicked_Lambda(DismissNotification)
	];

	return ContainingWidget;
}

void SAnimationEditorViewportTabBody::RemoveNotification(const TWeakPtr<SWidget>& InContainingWidget)
{
	if(InContainingWidget.IsValid())
	{
		ViewportNotificationsContainer->RemoveSlot(InContainingWidget.Pin().ToSharedRef());
	}
}

void SAnimationEditorViewportTabBody::AddOverlayWidget(TSharedRef<SWidget> InOverlaidWidget, int32 ZOrder)
{
	ViewportWidget->ViewportOverlay->AddSlot(ZOrder)
	[
		InOverlaidWidget
	];
}

void SAnimationEditorViewportTabBody::RemoveOverlayWidget(TSharedRef<SWidget> InOverlaidWidget)
{
	ViewportWidget->ViewportOverlay->RemoveSlot(InOverlaidWidget);
}

void SAnimationEditorViewportTabBody::RefreshViewport()
{
	LevelViewportClient->Invalidate();
}

TSharedPtr<FAssetEditorToolkit> SAnimationEditorViewportTabBody::GetAssetEditorToolkit() const
{
	return AssetEditorToolkitPtr.Pin();
}

bool SAnimationEditorViewportTabBody::IsVisible() const
{
	return ViewportWidget.IsValid();
}

FReply SAnimationEditorViewportTabBody::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (UICommandList.IsValid() && UICommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}

	if (OnKeyDownDelegate.IsBound())
	{
		return OnKeyDownDelegate.Execute(MyGeometry, InKeyEvent);
	}

	return FReply::Unhandled();
}

void SAnimationEditorViewportTabBody::Construct(const FArguments& InArgs, const TSharedRef<class IPersonaPreviewScene>& InPreviewScene, const TSharedRef<class FAssetEditorToolkit>& InAssetEditorToolkit, int32 InViewportIndex)
{
	PreviewScenePtr = StaticCastSharedRef<FAnimationEditorPreviewScene>(InPreviewScene);
	AssetEditorToolkitPtr = InAssetEditorToolkit;
	BlueprintEditorPtr = InArgs._BlueprintEditor;
	bShowTimeline = InArgs._ShowTimeline;
	bAlwaysShowTransformToolbar = InArgs._AlwaysShowTransformToolbar;
	OnInvokeTab = InArgs._OnInvokeTab;
	OnGetViewportText = InArgs._OnGetViewportText;
	ContextName = InArgs._ContextName;
	TimelineDelegates = InArgs._TimelineDelegates;

	// register delegates for change notifications
	InPreviewScene->RegisterOnAnimChanged(FOnAnimChanged::CreateSP(this, &SAnimationEditorViewportTabBody::AnimChanged));
	InPreviewScene->RegisterOnPreviewMeshChanged(FOnPreviewMeshChanged::CreateSP(this, &SAnimationEditorViewportTabBody::HandlePreviewMeshChanged));

	const FSlateFontInfo SmallLayoutFont = FCoreStyle::GetDefaultFontStyle("Regular", 9);

	FAnimViewportMenuCommands::Register();
	FAnimViewportShowCommands::Register();
	FAnimViewportLODCommands::Register();
	FAnimViewportPlaybackCommands::Register();

	// Build toolbar widgets
	UVChannelCombo = SNew(STextComboBox)
		.OptionsSource(&UVChannels)
		.Font(SmallLayoutFont)
		.OnSelectionChanged(this, &SAnimationEditorViewportTabBody::ComboBoxSelectionChanged);

	FAnimationEditorViewportRequiredArgs ViewportArgs(InPreviewScene, SharedThis(this), InAssetEditorToolkit, InViewportIndex);

	ViewportWidget = SNew(SAnimationEditorViewport, ViewportArgs)
		.Extenders(InArgs._Extenders)
		.ContextName(InArgs._ContextName)
		.ShowShowMenu(InArgs._ShowShowMenu)
		.ShowLODMenu(InArgs._ShowLODMenu)
		.ShowPlaySpeedMenu(InArgs._ShowPlaySpeedMenu)
		.ShowStats(InArgs._ShowStats)
		.ShowFloorOptions(InArgs._ShowFloorOptions)
		.ShowTurnTable(InArgs._ShowTurnTable)
		.ShowPhysicsMenu(InArgs._ShowPhysicsMenu);

	TSharedPtr<SVerticalBox> ViewportContainer = nullptr;

	this->ChildSlot
	[
		SAssignNew(ViewportContainer, SVerticalBox)

		// Build our toolbar level toolbar
		+SVerticalBox::Slot()
		.FillHeight(1)
		[
			SNew(SOverlay)

			// The viewport
			+SOverlay::Slot()
			[
				ViewportWidget.ToSharedRef()
			]

			// The 'dirty/in-error' indicator text in the bottom-right corner
			+SOverlay::Slot()
			.Padding(8)
			.VAlign(VAlign_Bottom)
			.HAlign(HAlign_Right)
			[
				SAssignNew(ViewportNotificationsContainer, SVerticalBox)
			]
		]
	];

	if(bShowTimeline && ViewportContainer.IsValid())
	{
		ViewportContainer->AddSlot()
		.AutoHeight()
		[
			SAssignNew(ScrubPanelContainer, SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SAnimationScrubPanel, GetPreviewScene())
				.ViewInputMin(this, &SAnimationEditorViewportTabBody::GetViewMinInput)
				.ViewInputMax(this, &SAnimationEditorViewportTabBody::GetViewMaxInput)
				.bAllowZoom(true)
				.TimelineDelegates(TimelineDelegates)
			]
		];

		UpdateScrubPanel(InPreviewScene->GetPreviewAnimationAsset());
	}

	LevelViewportClient = ViewportWidget->GetViewportClient();

	if (const TSharedPtr<FAnimationViewportClient>& AnimViewportClient = GetAnimationViewportClient())
	{
		// Load the view mode from config
		AnimViewportClient->SetViewMode(AnimViewportClient->ConfigOption
												 ->GetAssetEditorOptions(AssetEditorToolkitPtr.Pin()->GetEditorName())
												 .ViewportConfigs[InViewportIndex]
												 .ViewModeIndex);
	}

	UpdateShowFlagForMeshEdges();


	OnSetTurnTableMode(SelectedTurnTableMode);
	OnSetTurnTableSpeed(SelectedTurnTableSpeed);

	BindCommands();

	PopulateNumUVChannels();

	GetPreviewScene()->OnRecordingStateChanged().AddSP(this, &SAnimationEditorViewportTabBody::AddRecordingNotification);

	AddPostProcessNotification();

	AddMinLODNotification();

	if (TSharedPtr<FAnimationEditorPreviewScene> PreviewScene = PreviewScenePtr.Pin())
	{
		UE::AdvancedPreviewScene::BindDefaultOnSettingsChangedHandler(PreviewScene, LevelViewportClient);
	}
}

void SAnimationEditorViewportTabBody::BindCommands()
{
	FUICommandList_Pinnable& CommandList = *UICommandList;

	//Bind menu commands
	const FAnimViewportMenuCommands& MenuActions = FAnimViewportMenuCommands::Get();

	CommandList.MapAction(
		MenuActions.TogglePauseAnimationOnCameraMove,
		FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::TogglePauseAnimationOnCameraMove),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SAnimationEditorViewportTabBody::GetShouldPauseAnimationOnCameraMove));

	CommandList.MapAction(
		MenuActions.CameraFollowNone,
		FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::SetCameraFollowMode, EAnimationViewportCameraFollowMode::None, FName()),
		FCanExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::CanChangeCameraMode),
		FIsActionChecked::CreateSP(this, &SAnimationEditorViewportTabBody::IsCameraFollowEnabled, EAnimationViewportCameraFollowMode::None));

	CommandList.MapAction(
		MenuActions.CameraFollowBounds,
		FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::SetCameraFollowMode, EAnimationViewportCameraFollowMode::Bounds, FName()),
		FCanExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::CanChangeCameraMode),
		FIsActionChecked::CreateSP(this, &SAnimationEditorViewportTabBody::IsCameraFollowEnabled, EAnimationViewportCameraFollowMode::Bounds));

	CommandList.MapAction(
		MenuActions.CameraFollowRoot,
		FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::SetCameraFollowMode, EAnimationViewportCameraFollowMode::Root, FName()),
		FCanExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::CanChangeCameraMode),
		FIsActionChecked::CreateSP(this, &SAnimationEditorViewportTabBody::IsCameraFollowEnabled, EAnimationViewportCameraFollowMode::Root));

	CommandList.MapAction(
		MenuActions.JumpToDefaultCamera,
		FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::JumpToDefaultCamera),
		FCanExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::HasDefaultCameraSet));

	CommandList.MapAction(
		MenuActions.SaveCameraAsDefault,
		FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::SaveCameraAsDefault),
		FCanExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::CanSaveCameraAsDefault));

	CommandList.MapAction(
		MenuActions.ClearDefaultCamera,
		FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::ClearDefaultCamera),
		FCanExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::HasDefaultCameraSet));

	CommandList.MapAction(
		MenuActions.PreviewSceneSettings,
		FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::OpenPreviewSceneSettings));

	if (const TSharedPtr<FAnimationViewportClient>& AnimationViewportClientPtr = GetAnimationViewportClient())
	{
		if (FAnimationViewportClient* AnimViewportClient = AnimationViewportClientPtr.Get())
		{
			CommandList.MapAction(
				MenuActions.SetCPUSkinning,
				FExecuteAction::CreateSP(AnimViewportClient, &FAnimationViewportClient::ToggleCPUSkinning),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(AnimViewportClient, &FAnimationViewportClient::IsSetCPUSkinningChecked)
			);

			CommandList.MapAction(
				MenuActions.SetShowNormals,
				FExecuteAction::CreateSP(AnimViewportClient, &FAnimationViewportClient::ToggleShowNormals),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(AnimViewportClient, &FAnimationViewportClient::IsSetShowNormalsChecked)
			);

			CommandList.MapAction(
				MenuActions.SetShowTangents,
				FExecuteAction::CreateSP(AnimViewportClient, &FAnimationViewportClient::ToggleShowTangents),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(AnimViewportClient, &FAnimationViewportClient::IsSetShowTangentsChecked)
			);

			CommandList.MapAction(
				MenuActions.SetShowBinormals,
				FExecuteAction::CreateSP(AnimViewportClient, &FAnimationViewportClient::ToggleShowBinormals),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(AnimViewportClient, &FAnimationViewportClient::IsSetShowBinormalsChecked)
			);
		}
	}

	//Bind Show commands
	const FAnimViewportShowCommands& ViewportShowMenuCommands = FAnimViewportShowCommands::Get();

	CommandList.MapAction(
		ViewportShowMenuCommands.ShowBound,
		FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::ShowBound),
		FCanExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::CanShowBound),
		FIsActionChecked::CreateSP(this, &SAnimationEditorViewportTabBody::IsShowBoundEnabled));

	CommandList.MapAction(
		ViewportShowMenuCommands.UseInGameBound,
		FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::UseInGameBound),
		FCanExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::CanUseInGameBound),
		FIsActionChecked::CreateSP(this, &SAnimationEditorViewportTabBody::IsUsingInGameBound));

	CommandList.MapAction(
		ViewportShowMenuCommands.UseFixedBounds,
		FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::UseFixedBounds),
		FCanExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::CanUseFixedBounds),
		FIsActionChecked::CreateSP(this, &SAnimationEditorViewportTabBody::IsUsingFixedBounds));

	CommandList.MapAction(
		ViewportShowMenuCommands.UsePreSkinnedBounds,
		FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::UsePreSkinnedBounds),
		FCanExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::CanUsePreSkinnedBounds),
		FIsActionChecked::CreateSP(this, &SAnimationEditorViewportTabBody::IsUsingPreSkinnedBounds));

	CommandList.MapAction(
		ViewportShowMenuCommands.ShowPreviewMesh,
		FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::ToggleShowPreviewMesh),
		FCanExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::CanShowPreviewMesh),
		FIsActionChecked::CreateSP(this, &SAnimationEditorViewportTabBody::IsShowPreviewMeshEnabled));

	
	CommandList.MapAction(
		ViewportShowMenuCommands.ShowNaniteFallback,
		FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::ToggleShowNaniteFallback),
		FCanExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::CanShowNaniteFallback),
		FIsActionChecked::CreateSP(this, &SAnimationEditorViewportTabBody::IsShowNaniteFallbackEnabled));

	CommandList.MapAction(
		ViewportShowMenuCommands.ShowMorphTargets,
		FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::OnShowMorphTargets),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SAnimationEditorViewportTabBody::IsShowingMorphTargets));

	CommandList.MapAction( 
		ViewportShowMenuCommands.ShowBoneNames,
		FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::OnShowBoneNames),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SAnimationEditorViewportTabBody::IsShowingBoneNames));

	CommandList.MapAction( 
		ViewportShowMenuCommands.ShowBoneColors,
		FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::OnShowBoneColors),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SAnimationEditorViewportTabBody::IsShowingBoneColors));

	CommandList.MapAction( 
		ViewportShowMenuCommands.ShowRawAnimation,
		FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::OnShowRawAnimation),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SAnimationEditorViewportTabBody::IsShowingRawAnimation));

	CommandList.MapAction( 
		ViewportShowMenuCommands.ShowNonRetargetedAnimation,
		FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::OnShowNonRetargetedAnimation),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SAnimationEditorViewportTabBody::IsShowingNonRetargetedPose));

	CommandList.MapAction( 
		ViewportShowMenuCommands.ShowAdditiveBaseBones,
		FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::OnShowAdditiveBase),
		FCanExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::IsPreviewingAnimation),
		FIsActionChecked::CreateSP(this, &SAnimationEditorViewportTabBody::IsShowingAdditiveBase));

	CommandList.MapAction(
		ViewportShowMenuCommands.ShowSourceRawAnimation,
		FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::OnShowSourceRawAnimation),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SAnimationEditorViewportTabBody::IsShowingSourceRawAnimation));

	CommandList.MapAction(
		ViewportShowMenuCommands.ShowBakedAnimation,
		FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::OnShowBakedAnimation),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SAnimationEditorViewportTabBody::IsShowingBakedAnimation));

	//Display info
	CommandList.BeginGroup(TEXT("MeshDisplayInfo"));

	CommandList.MapAction( 
		ViewportShowMenuCommands.ShowDisplayInfoBasic,
		FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::OnShowDisplayInfo, (int32)EDisplayInfoMode::Basic),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SAnimationEditorViewportTabBody::IsShowingMeshInfo, (int32)EDisplayInfoMode::Basic));

	CommandList.MapAction(
		ViewportShowMenuCommands.ShowDisplayInfoDetailed,
		FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::OnShowDisplayInfo, (int32)EDisplayInfoMode::Detailed),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SAnimationEditorViewportTabBody::IsShowingMeshInfo, (int32)EDisplayInfoMode::Detailed));

	CommandList.MapAction(
		ViewportShowMenuCommands.ShowDisplayInfoSkelControls,
		FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::OnShowDisplayInfo, (int32)EDisplayInfoMode::SkeletalControls),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SAnimationEditorViewportTabBody::IsShowingMeshInfo, (int32)EDisplayInfoMode::SkeletalControls));

	CommandList.MapAction(
		ViewportShowMenuCommands.HideDisplayInfo,
		FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::OnShowDisplayInfo, (int32)EDisplayInfoMode::None),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SAnimationEditorViewportTabBody::IsShowingMeshInfo, (int32)EDisplayInfoMode::None));

	CommandList.EndGroup();

	//Material overlay option
	CommandList.BeginGroup(TEXT("MaterialOverlay"));

	CommandList.MapAction(
		ViewportShowMenuCommands.ShowOverlayNone,
		FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::OnShowOverlayNone),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SAnimationEditorViewportTabBody::IsShowingOverlayNone));

	CommandList.MapAction( 
		ViewportShowMenuCommands.ShowBoneWeight,
		FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::OnShowOverlayBoneWeight),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SAnimationEditorViewportTabBody::IsShowingOverlayBoneWeight));

	CommandList.MapAction(
		ViewportShowMenuCommands.ShowMorphTargetVerts,
		FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::OnShowOverlayMorphTargetVert),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SAnimationEditorViewportTabBody::IsShowingOverlayMorphTargetVerts));
	
	CommandList.EndGroup();

	// Show sockets
	CommandList.MapAction( 
		ViewportShowMenuCommands.ShowSockets,
		FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::OnShowSockets),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SAnimationEditorViewportTabBody::IsShowingSockets));

	// Show transform attributes
	CommandList.MapAction(
		ViewportShowMenuCommands.ShowAttributes,
		FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::OnShowAttributes),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SAnimationEditorViewportTabBody::IsShowingAttributes));

	// Set bone drawing mode
	CommandList.BeginGroup(TEXT("BoneDrawingMode"));

	CommandList.MapAction(
		ViewportShowMenuCommands.ShowBoneDrawNone,
		FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::OnSetBoneDrawMode, (int32)EBoneDrawMode::None),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SAnimationEditorViewportTabBody::IsBoneDrawModeSet, (int32)EBoneDrawMode::None));

	CommandList.MapAction(
		ViewportShowMenuCommands.ShowBoneDrawSelected,
		FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::OnSetBoneDrawMode, (int32)EBoneDrawMode::Selected),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SAnimationEditorViewportTabBody::IsBoneDrawModeSet, (int32)EBoneDrawMode::Selected));

	CommandList.MapAction(
		ViewportShowMenuCommands.ShowBoneDrawSelectedAndParents,
		FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::OnSetBoneDrawMode, (int32)EBoneDrawMode::SelectedAndParents),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SAnimationEditorViewportTabBody::IsBoneDrawModeSet, (int32)EBoneDrawMode::SelectedAndParents));

	CommandList.MapAction(
		ViewportShowMenuCommands.ShowBoneDrawSelectedAndChildren,
		FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::OnSetBoneDrawMode, (int32)EBoneDrawMode::SelectedAndChildren),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SAnimationEditorViewportTabBody::IsBoneDrawModeSet, (int32)EBoneDrawMode::SelectedAndChildren));

	CommandList.MapAction(
		ViewportShowMenuCommands.ShowBoneDrawSelectedAndParentsAndChildren,
		FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::OnSetBoneDrawMode, (int32)EBoneDrawMode::SelectedAndParentsAndChildren),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SAnimationEditorViewportTabBody::IsBoneDrawModeSet, (int32)EBoneDrawMode::SelectedAndParentsAndChildren));

	CommandList.MapAction(
		ViewportShowMenuCommands.ShowBoneDrawAll,
		FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::OnSetBoneDrawMode, (int32)EBoneDrawMode::All),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SAnimationEditorViewportTabBody::IsBoneDrawModeSet, (int32)EBoneDrawMode::All));

	CommandList.MapAction(
		ViewportShowMenuCommands.ToggleShowBoneDrawAll,
		FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::ToggleShowAllBones),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SAnimationEditorViewportTabBody::IsBoneDrawModeSet, (int32)EBoneDrawMode::All));

	CommandList.EndGroup();

	// Set bone local axes mode
	CommandList.BeginGroup(TEXT("BoneLocalAxesMode"));

	CommandList.MapAction( 
		ViewportShowMenuCommands.ShowLocalAxesNone,
		FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::OnSetLocalAxesMode, (int32)ELocalAxesMode::None),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SAnimationEditorViewportTabBody::IsLocalAxesModeSet, (int32)ELocalAxesMode::None));
	
	CommandList.MapAction( 
		ViewportShowMenuCommands.ShowLocalAxesSelected,
		FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::OnSetLocalAxesMode, (int32)ELocalAxesMode::Selected),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SAnimationEditorViewportTabBody::IsLocalAxesModeSet, (int32)ELocalAxesMode::Selected));
	
	CommandList.MapAction( 
		ViewportShowMenuCommands.ShowLocalAxesAll,
		FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::OnSetLocalAxesMode, (int32)ELocalAxesMode::All),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SAnimationEditorViewportTabBody::IsLocalAxesModeSet, (int32)ELocalAxesMode::All));

	CommandList.EndGroup();

	//Clothing show options
	CommandList.MapAction( 
		ViewportShowMenuCommands.EnableClothSimulation,
		FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::OnEnableClothSimulation),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SAnimationEditorViewportTabBody::IsClothSimulationEnabled));

	CommandList.MapAction( 
		ViewportShowMenuCommands.ResetClothSimulation,
		FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::OnResetClothSimulation),
		FCanExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::IsClothSimulationEnabled));

	CommandList.MapAction(
		ViewportShowMenuCommands.PauseClothWithAnim,
		FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::OnPauseClothingSimWithAnim),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SAnimationEditorViewportTabBody::IsPausingClothingSimWithAnim));

	CommandList.BeginGroup(TEXT("ClothSectionDisplayMode"));

	CommandList.MapAction(
		ViewportShowMenuCommands.ShowAllSections,
		FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::OnSetSectionsDisplayMode, ESectionDisplayMode::ShowAll),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SAnimationEditorViewportTabBody::IsSectionsDisplayMode, ESectionDisplayMode::ShowAll));

	CommandList.MapAction(
		ViewportShowMenuCommands.ShowOnlyClothSections,
		FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::OnSetSectionsDisplayMode, ESectionDisplayMode::ShowOnlyClothSections),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SAnimationEditorViewportTabBody::IsSectionsDisplayMode, ESectionDisplayMode::ShowOnlyClothSections));

	CommandList.MapAction(
		ViewportShowMenuCommands.HideOnlyClothSections,
		FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::OnSetSectionsDisplayMode, ESectionDisplayMode::HideOnlyClothSections),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SAnimationEditorViewportTabBody::IsSectionsDisplayMode, ESectionDisplayMode::HideOnlyClothSections));

	CommandList.EndGroup();

	CommandList.BeginGroup(TEXT("TimecodeSettings"));

	CommandList.MapAction(
		ViewportShowMenuCommands.ShowTimecode,
		FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::OnToggleShowTimecode),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SAnimationEditorViewportTabBody::IsShowTimecode));

	CommandList.EndGroup();


	GetPreviewScene()->RegisterOnSelectedLODChanged(FOnSelectedLODChanged::CreateSP(this, &SAnimationEditorViewportTabBody::OnLODModelChanged));
	//Bind LOD preview menu commands
	const FAnimViewportLODCommands& ViewportLODMenuCommands = FAnimViewportLODCommands::Get();

	CommandList.BeginGroup(TEXT("LOD"));

	UDebugSkelMeshComponent* PreviewComponent = GetPreviewScene()->GetPreviewMeshComponent();

	if (PreviewComponent)
	{
		//LOD Debug
		CommandList.MapAction(
			ViewportLODMenuCommands.LODDebug,
			FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::OnSetLODTrackDebuggedInstance),
			FCanExecuteAction::CreateLambda([PreviewComponent]() { return PreviewComponent->PreviewInstance ? (bool)PreviewComponent->PreviewInstance->GetDebugSkeletalMeshComponent() : false; }),
			FIsActionChecked::CreateLambda([PreviewComponent]() { return PreviewComponent->IsTrackingAttachedLOD(); }),
			FIsActionButtonVisible::CreateLambda([PreviewComponent]() { return PreviewComponent->PreviewInstance ? (bool)PreviewComponent->PreviewInstance->GetDebugSkeletalMeshComponent() : false; }));

		PreviewComponent->RegisterOnDebugForceLODChangedDelegate(FOnDebugForceLODChanged::CreateSP(this, &SAnimationEditorViewportTabBody::OnDebugForcedLODChanged));
	}

	//LOD Auto
	CommandList.MapAction( 
		ViewportLODMenuCommands.LODAuto,
		FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::OnSetLODModel, 0),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SAnimationEditorViewportTabBody::IsLODModelSelected, 0));

	// LOD 0
	CommandList.MapAction(
		ViewportLODMenuCommands.LOD0,
		FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::OnSetLODModel, 1),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SAnimationEditorViewportTabBody::IsLODModelSelected, 1));

	// all other LODs will be added dynamically

	CommandList.EndGroup();

	CommandList.MapAction(
		ViewportShowMenuCommands.AutoAlignFloorToMesh,
		FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::OnToggleAutoAlignFloor),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SAnimationEditorViewportTabBody::IsAutoAlignFloor));
	
	//Bind LOD preview menu commands
	const FAnimViewportPlaybackCommands& ViewportPlaybackCommands = FAnimViewportPlaybackCommands::Get();

	CommandList.MapAction(
		ViewportShowMenuCommands.MuteAudio,
		FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::OnToggleMuteAudio),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SAnimationEditorViewportTabBody::IsAudioMuted));

	CommandList.MapAction(
		ViewportShowMenuCommands.UseAudioAttenuation,
		FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::OnToggleUseAudioAttenuation),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SAnimationEditorViewportTabBody::IsAudioAttenuationEnabled));

	CommandList.BeginGroup(TEXT("RootMotion"));

	CommandList.MapAction(
		ViewportShowMenuCommands.DoNotProcessRootMotion,
		FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::SetProcessRootMotionMode, EProcessRootMotionMode::Ignore),
		FIsActionChecked::CreateSP(this, &SAnimationEditorViewportTabBody::CanUseProcessRootMotionMode, EProcessRootMotionMode::Ignore),
		FIsActionChecked::CreateSP(this, &SAnimationEditorViewportTabBody::IsProcessRootMotionModeSet, EProcessRootMotionMode::Ignore));

	CommandList.MapAction(
		ViewportShowMenuCommands.ProcessRootMotionLoopAndReset,
		FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::SetProcessRootMotionMode, EProcessRootMotionMode::LoopAndReset),
		FIsActionChecked::CreateSP(this, &SAnimationEditorViewportTabBody::CanUseProcessRootMotionMode, EProcessRootMotionMode::LoopAndReset),
		FIsActionChecked::CreateSP(this, &SAnimationEditorViewportTabBody::IsProcessRootMotionModeSet, EProcessRootMotionMode::LoopAndReset));

	CommandList.MapAction(
		ViewportShowMenuCommands.ProcessRootMotionLoop,
		FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::SetProcessRootMotionMode, EProcessRootMotionMode::Loop),
		FIsActionChecked::CreateSP(this, &SAnimationEditorViewportTabBody::CanUseProcessRootMotionMode, EProcessRootMotionMode::Loop),
		FIsActionChecked::CreateSP(this, &SAnimationEditorViewportTabBody::IsProcessRootMotionModeSet, EProcessRootMotionMode::Loop));

	CommandList.MapAction(
		ViewportShowMenuCommands.DoNotVisualizeRootMotion,
		FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::SetVisualizeRootMotionMode, EVisualizeRootMotionMode::None),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SAnimationEditorViewportTabBody::IsVisualizeRootMotionModeSet, EVisualizeRootMotionMode::None));

	CommandList.MapAction(
		ViewportShowMenuCommands.VisualizeRootMotionTrajectory,
		FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::SetVisualizeRootMotionMode, EVisualizeRootMotionMode::Trajectory),
		FIsActionChecked::CreateSP(this, &SAnimationEditorViewportTabBody::CanVisualizeRootMotion),
		FIsActionChecked::CreateSP(this, &SAnimationEditorViewportTabBody::IsVisualizeRootMotionModeSet, EVisualizeRootMotionMode::Trajectory));

	CommandList.MapAction(
		ViewportShowMenuCommands.VisualizeRootMotionTrajectoryAndOrientation,
		FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::SetVisualizeRootMotionMode, EVisualizeRootMotionMode::TrajectoryAndOrientation),
		FIsActionChecked::CreateSP(this, &SAnimationEditorViewportTabBody::CanVisualizeRootMotion),
		FIsActionChecked::CreateSP(this, &SAnimationEditorViewportTabBody::IsVisualizeRootMotionModeSet, EVisualizeRootMotionMode::TrajectoryAndOrientation));

	CommandList.MapAction(
		ViewportShowMenuCommands.ShowNotificationVisualizations,
		FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::ToggleNotificationVisualizations),
		FIsActionChecked(),
		FIsActionChecked::CreateSP(this, &SAnimationEditorViewportTabBody::IsNotificationVisualizationsEnabled));

	CommandList.MapAction(
		ViewportShowMenuCommands.ShowAssetUserDataVisualizations,
		FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::ToggleAssetUserDataVisualizations),
		FIsActionChecked(),
		FIsActionChecked::CreateSP(this, &SAnimationEditorViewportTabBody::IsAssetUserDataVisualizationsEnabled));

	CommandList.EndGroup();

	CommandList.MapAction(
		ViewportShowMenuCommands.DisablePostProcessBlueprint,
		FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::OnToggleDisablePostProcess),
		FCanExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::CanDisablePostProcess),
		FIsActionChecked::CreateSP(this, &SAnimationEditorViewportTabBody::IsDisablePostProcessChecked));

	CommandList.BeginGroup(TEXT("TurnTableSpeeds"));

	// Turn Table Controls
	for (int32 i = 0; i < int(EAnimationPlaybackSpeeds::NumPlaybackSpeeds); ++i)
	{
		CommandList.MapAction(
			ViewportPlaybackCommands.TurnTableSpeeds[i],
			FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::OnSetTurnTableSpeed, i),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(this, &SAnimationEditorViewportTabBody::IsTurnTableSpeedSelected, i));
	}

	CommandList.EndGroup();

	CommandList.BeginGroup(TEXT("TurnTableMode"));

	CommandList.MapAction(
		ViewportPlaybackCommands.PersonaTurnTablePlay,
		FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::OnSetTurnTableMode, int32(EPersonaTurnTableMode::Playing)),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SAnimationEditorViewportTabBody::IsTurnTableModeSelected, int32(EPersonaTurnTableMode::Playing)));

	CommandList.MapAction(
		ViewportPlaybackCommands.PersonaTurnTablePause,
		FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::OnSetTurnTableMode, int32(EPersonaTurnTableMode::Paused)),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SAnimationEditorViewportTabBody::IsTurnTableModeSelected, int32(EPersonaTurnTableMode::Paused)));

	CommandList.MapAction(
		ViewportPlaybackCommands.PersonaTurnTableStop,
		FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::OnSetTurnTableMode, int32(EPersonaTurnTableMode::Stopped)),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SAnimationEditorViewportTabBody::IsTurnTableModeSelected, int32(EPersonaTurnTableMode::Stopped)));

	CommandList.EndGroup();

	CommandList.MapAction(
		FEditorViewportCommands::Get().FocusViewportToSelection,
		FExecuteAction::CreateSP(this, &SAnimationEditorViewportTabBody::HandleFocusCamera));

	if (const TSharedPtr<FAssetEditorToolkit>& AssetEditorToolkit = GetAssetEditorToolkit())
	{
		TSharedPtr<FUICommandList> ToolkitCommandList =
			ConstCastSharedRef<FUICommandList>(AssetEditorToolkit->GetToolkitCommands());
		ToolkitCommandList->Append(UICommandList->AsShared());
	}

	PinnedCommands->BindCommandList(UICommandList.ToSharedRef());

	if (TSharedPtr<FAnimationEditorPreviewScene> AnimationEditorPreviewScene = PreviewScenePtr.Pin())
	{
		PinnedCommands->BindCommandList(AnimationEditorPreviewScene->GetPinnedCommandList().ToSharedRef());
	}
}

void SAnimationEditorViewportTabBody::OnSetTurnTableSpeed(int32 SpeedIndex)
{
	SelectedTurnTableSpeed = (EAnimationPlaybackSpeeds::Type)SpeedIndex;

	UDebugSkelMeshComponent* PreviewComponent = GetPreviewScene()->GetPreviewMeshComponent();
	if (PreviewComponent)
	{
		const float TurnTableSpeed = (SelectedTurnTableSpeed == EAnimationPlaybackSpeeds::Custom)
			? GetCustomTurnTableSpeed()
			: EAnimationPlaybackSpeeds::Values[SelectedTurnTableSpeed];

		PreviewComponent->TurnTableSpeedScaling = TurnTableSpeed;
	}
}

bool SAnimationEditorViewportTabBody::IsTurnTableSpeedSelected(int32 SpeedIndex) const
{
	return (SelectedTurnTableSpeed == SpeedIndex);
}

void SAnimationEditorViewportTabBody::OnSetTurnTableMode(int32 ModeIndex)
{
	SelectedTurnTableMode = (EPersonaTurnTableMode::Type)ModeIndex;

	UDebugSkelMeshComponent* PreviewComponent = GetPreviewScene()->GetPreviewMeshComponent();
	if (PreviewComponent)
	{
		PreviewComponent->SetTurnTableMode(SelectedTurnTableMode);
	}
}

bool SAnimationEditorViewportTabBody::IsTurnTableModeSelected(int32 ModeIndex) const
{
	return (SelectedTurnTableMode == ModeIndex);
}

int32 SAnimationEditorViewportTabBody::GetLODModelCount() const
{
	UDebugSkelMeshComponent* PreviewComponent = GetPreviewScene()->GetPreviewMeshComponent();
	if( PreviewComponent && PreviewComponent->GetSkeletalMeshAsset())
	{
		return PreviewComponent->GetSkeletalMeshAsset()->GetResourceForRendering()->LODRenderData.Num();
	}
	return 0;
}

void SAnimationEditorViewportTabBody::OnShowMorphTargets()
{
	ForEachDebugMesh([](UDebugSkelMeshComponent* InMesh)
	{
		InMesh->bDisableMorphTarget = !InMesh->bDisableMorphTarget;
		InMesh->MarkRenderStateDirty();
	});
	RefreshViewport();
}

void SAnimationEditorViewportTabBody::OnShowBoneNames()
{
	ForEachDebugMesh([](UDebugSkelMeshComponent* InMesh)
	{
		InMesh->bShowBoneNames = !InMesh->bShowBoneNames;
		InMesh->MarkRenderStateDirty();
	});
	RefreshViewport();

	if (const TSharedPtr<FAnimationViewportClient>& AnimViewportClient = GetAnimationViewportClient())
	{
		AnimViewportClient->UpdateBonesToDraw();
	}
}

void SAnimationEditorViewportTabBody::OnShowRawAnimation()
{
	ForEachDebugMesh([](UDebugSkelMeshComponent* InMesh)
	{
		InMesh->bDisplayRawAnimation = !InMesh->bDisplayRawAnimation;
		InMesh->MarkRenderStateDirty();
	});
	RefreshViewport();
}

void SAnimationEditorViewportTabBody::OnShowNonRetargetedAnimation()
{
	ForEachDebugMesh([](UDebugSkelMeshComponent* InMesh)
	{
		InMesh->bDisplayNonRetargetedPose = !InMesh->bDisplayNonRetargetedPose;
		InMesh->MarkRenderStateDirty();
	});
	RefreshViewport();
}

void SAnimationEditorViewportTabBody::OnShowSourceRawAnimation()
{
	ForEachDebugMesh([](UDebugSkelMeshComponent* InMesh)
	{
		InMesh->bDisplaySourceAnimation = !InMesh->bDisplaySourceAnimation;
		InMesh->MarkRenderStateDirty();
	});
	RefreshViewport();
}

void SAnimationEditorViewportTabBody::OnShowBakedAnimation()
{
	ForEachDebugMesh([](UDebugSkelMeshComponent* InMesh)
	{
		InMesh->bDisplayBakedAnimation = !InMesh->bDisplayBakedAnimation;
		InMesh->MarkRenderStateDirty();
	});
	RefreshViewport();
}

void SAnimationEditorViewportTabBody::OnShowAdditiveBase()
{
	ForEachDebugMesh([](UDebugSkelMeshComponent* InMesh)
	{
		InMesh->bDisplayAdditiveBasePose = !InMesh->bDisplayAdditiveBasePose;
		InMesh->MarkRenderStateDirty();
	});
	RefreshViewport();
}

bool SAnimationEditorViewportTabBody::IsPreviewingAnimation() const
{
	UDebugSkelMeshComponent* PreviewComponent = GetPreviewScene()->GetPreviewMeshComponent();
	return (PreviewComponent && PreviewComponent->PreviewInstance && (PreviewComponent->PreviewInstance == PreviewComponent->GetAnimInstance()));
}

bool SAnimationEditorViewportTabBody::IsShowingMorphTargets() const
{
	UDebugSkelMeshComponent* PreviewComponent = GetPreviewScene()->GetPreviewMeshComponent();
	return PreviewComponent != NULL && PreviewComponent->bDisableMorphTarget == false;
}

bool SAnimationEditorViewportTabBody::IsShowingBoneNames() const
{
	UDebugSkelMeshComponent* PreviewComponent = GetPreviewScene()->GetPreviewMeshComponent();
	return PreviewComponent != NULL && PreviewComponent->bShowBoneNames;
}

void SAnimationEditorViewportTabBody::OnShowBoneColors()
{
	ForEachDebugMesh([](UDebugSkelMeshComponent* InMesh)
	{
		UPersonaOptions* Settings = GetMutableDefault<UPersonaOptions>();
		Settings->bShowBoneColors = !Settings->bShowBoneColors;
		InMesh->MarkRenderStateDirty();
	});
	RefreshViewport();
}

bool SAnimationEditorViewportTabBody::IsShowingBoneColors() const
{
	const UDebugSkelMeshComponent* PreviewComponent = GetPreviewScene()->GetPreviewMeshComponent();
	return PreviewComponent && GetDefault<UPersonaOptions>()->bShowBoneColors;
}

bool SAnimationEditorViewportTabBody::IsShowingRawAnimation() const
{
	UDebugSkelMeshComponent* PreviewComponent = GetPreviewScene()->GetPreviewMeshComponent();
	return PreviewComponent != NULL && PreviewComponent->bDisplayRawAnimation;
}

void SAnimationEditorViewportTabBody::OnToggleDisablePostProcess()
{
	ForEachDebugMesh([](UDebugSkelMeshComponent* InMesh)
	{
		InMesh->ToggleDisablePostProcessBlueprint();
	});
	
	AddPostProcessNotification();
}

bool SAnimationEditorViewportTabBody::CanDisablePostProcess() const
{
	const TArray<UDebugSkelMeshComponent*> PreviewMeshComponents = GetPreviewScene()->GetAllPreviewMeshComponents();
	for (const UDebugSkelMeshComponent* PreviewMeshComponent : PreviewMeshComponents)
	{
		if (PreviewMeshComponent->PostProcessAnimInstance && PreviewMeshComponent->IsVisible())
		{
			return true;
		}
	}
	return false;
}

bool SAnimationEditorViewportTabBody::IsDisablePostProcessChecked()
{
	TArray<UDebugSkelMeshComponent*> PreviewMeshComponents = GetPreviewScene()->GetAllPreviewMeshComponents();
	for (UDebugSkelMeshComponent* PreviewMeshComponent : PreviewMeshComponents)
	{
		if (PreviewMeshComponent->GetDisablePostProcessBlueprint())
		{
			return true;
		}
	}
	
	return false;
}

bool SAnimationEditorViewportTabBody::IsShowingNonRetargetedPose() const
{
	UDebugSkelMeshComponent* PreviewComponent = GetPreviewScene()->GetPreviewMeshComponent();
	return PreviewComponent != NULL && PreviewComponent->bDisplayNonRetargetedPose;
}

bool SAnimationEditorViewportTabBody::IsShowingAdditiveBase() const
{
	UDebugSkelMeshComponent* PreviewComponent = GetPreviewScene()->GetPreviewMeshComponent();
	return PreviewComponent != NULL && PreviewComponent->bDisplayAdditiveBasePose;
}

bool SAnimationEditorViewportTabBody::IsShowingSourceRawAnimation() const
{
	UDebugSkelMeshComponent* PreviewComponent = GetPreviewScene()->GetPreviewMeshComponent();
	return PreviewComponent != NULL && PreviewComponent->bDisplaySourceAnimation;
}

bool SAnimationEditorViewportTabBody::IsShowingBakedAnimation() const
{
	UDebugSkelMeshComponent* PreviewComponent = GetPreviewScene()->GetPreviewMeshComponent();
	return PreviewComponent != NULL && PreviewComponent->bDisplayBakedAnimation;
}

void SAnimationEditorViewportTabBody::OnShowDisplayInfo(int32 DisplayInfoMode)
{
	if (const TSharedPtr<FAnimationViewportClient>& AnimViewportClient = GetAnimationViewportClient())
	{
		AnimViewportClient->OnSetShowMeshStats(DisplayInfoMode);
	}
}

bool SAnimationEditorViewportTabBody::IsShowingMeshInfo(int32 DisplayInfoMode) const
{
	if (const TSharedPtr<FAnimationViewportClient>& AnimViewportClient = GetAnimationViewportClient())
	{
		return AnimViewportClient->GetShowMeshStats() == DisplayInfoMode;
	}

	return false;
}

void SAnimationEditorViewportTabBody::OnShowOverlayNone()
{
	ForEachDebugMesh([](UDebugSkelMeshComponent* PreviewMeshComponent)
	{
		PreviewMeshComponent->SetShowBoneWeight(false);
		PreviewMeshComponent->SetShowMorphTargetVerts(false);
		PreviewMeshComponent->MarkRenderStateDirty();
	});

	UpdateShowFlagForMeshEdges();
	RefreshViewport();
}

bool SAnimationEditorViewportTabBody::IsShowingOverlayNone() const
{
	UDebugSkelMeshComponent* PreviewComponent = GetPreviewScene()->GetPreviewMeshComponent();
	return PreviewComponent != NULL && !PreviewComponent->bDrawBoneInfluences && !PreviewComponent->bDrawMorphTargetVerts;
}

void SAnimationEditorViewportTabBody::OnShowOverlayBoneWeight()
{
	ForEachDebugMesh([](UDebugSkelMeshComponent* PreviewMeshComponent)
	{
		PreviewMeshComponent->SetShowBoneWeight( !PreviewMeshComponent->bDrawBoneInfluences );
		PreviewMeshComponent->MarkRenderStateDirty();
	});
	
	UpdateShowFlagForMeshEdges();
	RefreshViewport();
}

bool SAnimationEditorViewportTabBody::IsShowingOverlayBoneWeight() const
{
	UDebugSkelMeshComponent* PreviewComponent = GetPreviewScene()->GetPreviewMeshComponent();
	return PreviewComponent != NULL && PreviewComponent->bDrawBoneInfluences;
}

void SAnimationEditorViewportTabBody::OnShowOverlayMorphTargetVert()
{
	ForEachDebugMesh([](UDebugSkelMeshComponent* PreviewMeshComponent)
	{
		PreviewMeshComponent->SetShowMorphTargetVerts(!PreviewMeshComponent->bDrawMorphTargetVerts);
		PreviewMeshComponent->MarkRenderStateDirty();
	});

	UpdateShowFlagForMeshEdges();
	RefreshViewport();
}

bool SAnimationEditorViewportTabBody::IsShowingOverlayMorphTargetVerts() const
{
	UDebugSkelMeshComponent* PreviewComponent = GetPreviewScene()->GetPreviewMeshComponent();
	return PreviewComponent != NULL && PreviewComponent->bDrawMorphTargetVerts;
}

void SAnimationEditorViewportTabBody::SetBoneDrawSize(float BoneDrawSize)
{
	if (const TSharedPtr<FAnimationViewportClient>& AnimViewportClient = GetAnimationViewportClient())
	{
		AnimViewportClient->SetBoneDrawSize(BoneDrawSize);
	}
}

float SAnimationEditorViewportTabBody::GetBoneDrawSize() const
{
	if (const TSharedPtr<FAnimationViewportClient>& AnimViewportClient = GetAnimationViewportClient())
	{
		return AnimViewportClient->GetBoneDrawSize();
	}

	return 0.0f;
}

void SAnimationEditorViewportTabBody::SetCustomTurnTableSpeed(float InCustomTurnTableSpeed)
{
	CustomTurnTableSpeed = InCustomTurnTableSpeed;
	OnSetTurnTableSpeed(EAnimationPlaybackSpeeds::Custom);
}

float SAnimationEditorViewportTabBody::GetCustomTurnTableSpeed() const
{
	return CustomTurnTableSpeed;
}

void SAnimationEditorViewportTabBody::OnSetBoneDrawMode(int32 BoneDrawMode)
{
	if (const TSharedPtr<FAnimationViewportClient>& AnimViewportClient = GetAnimationViewportClient())
	{
		AnimViewportClient->SetBoneDrawMode((EBoneDrawMode::Type)BoneDrawMode);
	}
}

bool SAnimationEditorViewportTabBody::IsBoneDrawModeSet(int32 BoneDrawMode) const
{
	if (const TSharedPtr<FAnimationViewportClient>& AnimViewportClient = GetAnimationViewportClient())
	{
		return AnimViewportClient->IsBoneDrawModeSet((EBoneDrawMode::Type)BoneDrawMode);
	}

	return false;
}

void SAnimationEditorViewportTabBody::ToggleShowAllBones()
{
	if (const TSharedPtr<FAnimationViewportClient>& AnimViewportClient = GetAnimationViewportClient())
	{
		const EBoneDrawMode::Type DrawMode = IsBoneDrawModeSet(static_cast<int32>(EBoneDrawMode::All)) ? 
			EBoneDrawMode::None : 
			EBoneDrawMode::All;

		AnimViewportClient->SetBoneDrawMode(DrawMode);
	}
}

void SAnimationEditorViewportTabBody::OnSetLocalAxesMode(int32 LocalAxesMode)
{
	if (const TSharedPtr<FAnimationViewportClient>& AnimViewportClient = GetAnimationViewportClient())
	{
		AnimViewportClient->SetLocalAxesMode((ELocalAxesMode::Type)LocalAxesMode);
	}
}

bool SAnimationEditorViewportTabBody::IsLocalAxesModeSet(int32 LocalAxesMode) const
{
	if (const TSharedPtr<FAnimationViewportClient>& AnimViewportClient = GetAnimationViewportClient())
	{
		return AnimViewportClient->IsLocalAxesModeSet((ELocalAxesMode::Type)LocalAxesMode);
	}

	return false;
}

void SAnimationEditorViewportTabBody::OnShowSockets()
{
	ForEachDebugMesh([](UDebugSkelMeshComponent* PreviewMeshComponent)
	{
		PreviewMeshComponent->bDrawSockets = !PreviewMeshComponent->bDrawSockets;
		PreviewMeshComponent->MarkRenderStateDirty();
	});

	RefreshViewport();
}

bool SAnimationEditorViewportTabBody::IsShowingSockets() const
{
	UDebugSkelMeshComponent* PreviewComponent = GetPreviewScene()->GetPreviewMeshComponent();
	return PreviewComponent != NULL && PreviewComponent->bDrawSockets;
}

void SAnimationEditorViewportTabBody::OnShowAttributes()
{
	ForEachDebugMesh([](UDebugSkelMeshComponent* PreviewMeshComponent)
	{
		PreviewMeshComponent->bDrawAttributes = !PreviewMeshComponent->bDrawAttributes;
		PreviewMeshComponent->MarkRenderStateDirty();
	});
	
	RefreshViewport();
}

bool SAnimationEditorViewportTabBody::IsShowingAttributes() const
{
	UDebugSkelMeshComponent* PreviewComponent = GetPreviewScene()->GetPreviewMeshComponent();
	return PreviewComponent != NULL && PreviewComponent->bDrawAttributes;
}

void SAnimationEditorViewportTabBody::OnToggleAutoAlignFloor()
{
	if (const TSharedPtr<FAnimationViewportClient>& AnimViewportClient = GetAnimationViewportClient())
	{
		AnimViewportClient->OnToggleAutoAlignFloor();
	}
}

bool SAnimationEditorViewportTabBody::IsAutoAlignFloor() const
{
	if (const TSharedPtr<FAnimationViewportClient>& AnimViewportClient = GetAnimationViewportClient())
	{
		return AnimViewportClient->IsAutoAlignFloor();
	}

	return false;
}

void SAnimationEditorViewportTabBody::ShowBound()
{
	if (const TSharedPtr<FAnimationViewportClient>& AnimViewportClient = GetAnimationViewportClient())
	{
		AnimViewportClient->ToggleShowBounds();

		ForEachDebugMesh(
			[AnimViewportClientWeak = AnimViewportClient.ToWeakPtr()](UDebugSkelMeshComponent* PreviewMeshComponent)
			{
				if (const TSharedPtr<FAnimationViewportClient>& AnimViewportClient = AnimViewportClientWeak.Pin())
				{
					PreviewMeshComponent->bDisplayBound = AnimViewportClient->EngineShowFlags.Bounds;
					PreviewMeshComponent->RecreateRenderState_Concurrent();
				}
			}
		);
	}
}

bool SAnimationEditorViewportTabBody::CanShowBound() const
{
	return !GetPreviewScene()->GetAllPreviewMeshComponents().IsEmpty();
}

bool SAnimationEditorViewportTabBody::IsShowBoundEnabled() const
{
	if (const TSharedPtr<FAnimationViewportClient>& AnimViewportClient = GetAnimationViewportClient())
	{
		return AnimViewportClient->IsSetShowBoundsChecked();
	}

	return false;
}

void SAnimationEditorViewportTabBody::ToggleShowPreviewMesh()
{
	const bool bCurrentlyVisible = IsShowPreviewMeshEnabled();
	ForEachDebugMesh([bCurrentlyVisible](UDebugSkelMeshComponent* PreviewMeshComponent)
	{
		PreviewMeshComponent->SetVisibility(!bCurrentlyVisible);
	});
}

bool SAnimationEditorViewportTabBody::CanShowPreviewMesh() const
{
	return !GetPreviewScene()->GetAllPreviewMeshComponents().IsEmpty();
}

bool SAnimationEditorViewportTabBody::IsShowPreviewMeshEnabled() const
{
	TArray<UDebugSkelMeshComponent*> PreviewMeshComponents = GetPreviewScene()->GetAllPreviewMeshComponents();
	for (UDebugSkelMeshComponent* PreviewMeshComponent : PreviewMeshComponents)
	{
		if(PreviewMeshComponent && PreviewMeshComponent->IsVisible())
		{
			return true;
		}
	}
	
	return false;
}

void SAnimationEditorViewportTabBody::ToggleShowNaniteFallback()
{
	const bool bCurrentlyEnabled = IsShowNaniteFallbackEnabled();
	ForEachDebugMesh([bCurrentlyEnabled](UDebugSkelMeshComponent* PreviewMeshComponent)
	{
		PreviewMeshComponent->SetDisplayNaniteFallback(!bCurrentlyEnabled);
	});
}

bool SAnimationEditorViewportTabBody::CanShowNaniteFallback() const
{
	const USkeletalMesh* PreviewMesh = GetPreviewScene()->GetPreviewMesh();
	return PreviewMesh && PreviewMesh->IsNaniteEnabled();
}

bool SAnimationEditorViewportTabBody::IsShowNaniteFallbackEnabled() const
{
	TArray<UDebugSkelMeshComponent*> PreviewMeshComponents = GetPreviewScene()->GetAllPreviewMeshComponents();
	for (const UDebugSkelMeshComponent* PreviewMeshComponent : PreviewMeshComponents)
	{
		if(PreviewMeshComponent && PreviewMeshComponent->IsDisplayingNaniteFallback())
		{
			return true;
		}
	}
	
	return false;
}

void SAnimationEditorViewportTabBody::OnToggleShowTimecode()
{
	GetPreviewScene()->ToggleShowTimecode();
}

bool SAnimationEditorViewportTabBody::IsShowTimecode() const
{
	return GetPreviewScene()->IsShowTimecode();
}

void SAnimationEditorViewportTabBody::UseInGameBound()
{
	ForEachDebugMesh([](UDebugSkelMeshComponent* PreviewMeshComponent)
	{
		PreviewMeshComponent->UseInGameBounds(! PreviewMeshComponent->IsUsingInGameBounds());
	});
}

bool SAnimationEditorViewportTabBody::CanUseInGameBound() const
{
	TArray<UDebugSkelMeshComponent*> PreviewMeshComponents = GetPreviewScene()->GetAllPreviewMeshComponents();
	for (UDebugSkelMeshComponent* PreviewMeshComponent : PreviewMeshComponents)
	{
		if(IsShowBoundEnabled())
		{
			return true;
		}
	}
	
	return false;
}

bool SAnimationEditorViewportTabBody::IsUsingInGameBound() const
{
	TArray<UDebugSkelMeshComponent*> PreviewMeshComponents = GetPreviewScene()->GetAllPreviewMeshComponents();
	for (UDebugSkelMeshComponent* PreviewMeshComponent : PreviewMeshComponents)
	{
		if(PreviewMeshComponent->IsUsingInGameBounds())
		{
			return true;
		}
	}
	return false;
}

void SAnimationEditorViewportTabBody::UseFixedBounds()
{
	ForEachDebugMesh([](UDebugSkelMeshComponent* PreviewMeshComponent)
	{
		PreviewMeshComponent->bComponentUseFixedSkelBounds = !PreviewMeshComponent->bComponentUseFixedSkelBounds;
	});
}

bool SAnimationEditorViewportTabBody::CanUseFixedBounds() const
{
	UDebugSkelMeshComponent* PreviewComponent = GetPreviewScene()->GetPreviewMeshComponent();
	return PreviewComponent != NULL && IsShowBoundEnabled();
}

bool SAnimationEditorViewportTabBody::IsUsingFixedBounds() const
{
	UDebugSkelMeshComponent* PreviewComponent = GetPreviewScene()->GetPreviewMeshComponent();
	return PreviewComponent != NULL && PreviewComponent->bComponentUseFixedSkelBounds;
}

void SAnimationEditorViewportTabBody::UsePreSkinnedBounds()
{
	GetPreviewScene()->ForEachPreviewMesh([](UDebugSkelMeshComponent* PreviewMeshComponent)
	{
		PreviewMeshComponent->UsePreSkinnedBounds(!PreviewMeshComponent->IsUsingPreSkinnedBounds());
	});
}

bool SAnimationEditorViewportTabBody::CanUsePreSkinnedBounds() const
{
	UDebugSkelMeshComponent* PreviewComponent = GetPreviewScene()->GetPreviewMeshComponent();
	return PreviewComponent != NULL && IsShowBoundEnabled();
}

bool SAnimationEditorViewportTabBody::IsUsingPreSkinnedBounds() const
{
	UDebugSkelMeshComponent* PreviewComponent = GetPreviewScene()->GetPreviewMeshComponent();
	return PreviewComponent != NULL && PreviewComponent->IsUsingPreSkinnedBounds();
}

void SAnimationEditorViewportTabBody::HandlePreviewMeshChanged(class USkeletalMesh* OldSkeletalMesh, class USkeletalMesh* NewSkeletalMesh)
{
	PopulateNumUVChannels();

	if (OldSkeletalMesh)
	{
		OldSkeletalMesh->OnPostMeshCached().RemoveAll(this);
	}
}

void SAnimationEditorViewportTabBody::AnimChanged(UAnimationAsset* AnimAsset)
{
	UpdateScrubPanel(AnimAsset);
}

void SAnimationEditorViewportTabBody::ComboBoxSelectionChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo)
{
	const TSharedPtr<FAnimationViewportClient>& AnimViewportClient = GetAnimationViewportClient();
	if (!AnimViewportClient)
	{
		return;
	}

	int32 NewUVSelection = UVChannels.Find(NewSelection) - 1;

	// "None" is index -1 here.
	if ( NewUVSelection < 0 )
	{
		AnimViewportClient->SetDrawUVOverlay(false);
		return;
	}

	AnimViewportClient->SetDrawUVOverlay(true);
	AnimViewportClient->SetUVChannelToDraw(NewUVSelection);

	RefreshViewport();
}

void SAnimationEditorViewportTabBody::PopulateNumUVChannels()
{
	NumUVChannels.Empty();

	if (UDebugSkelMeshComponent* PreviewComponent = GetPreviewScene()->GetPreviewMeshComponent())
	{
		if (FSkeletalMeshRenderData* MeshResource = PreviewComponent->GetSkeletalMeshRenderData())
		{
			int32 NumLods = MeshResource->LODRenderData.Num();
			NumUVChannels.AddZeroed(NumLods);
			for(int32 LOD = 0; LOD < NumLods; ++LOD)
			{
				NumUVChannels[LOD] = MeshResource->LODRenderData[LOD].StaticVertexBuffers.StaticMeshVertexBuffer.GetNumTexCoords();
			}
		}
	}

	PopulateUVChoices();
}

void SAnimationEditorViewportTabBody::PopulateUVChoices()
{
	const TSharedPtr<FAnimationViewportClient>& AnimViewportClient = GetAnimationViewportClient();
	if (!AnimViewportClient)
	{
		return;
	}

	// Fill out the UV channels combo.
	UVChannels.Empty();

	UVChannels.Add(MakeShareable(new FString(NSLOCTEXT("AnimationEditorViewport", "NoUVChannel", "None").ToString())));
	
	if (UDebugSkelMeshComponent* PreviewComponent = GetPreviewScene()->GetPreviewMeshComponent())
	{
		int32 CurrentLOD = FMath::Clamp(PreviewComponent->GetForcedLOD() - 1, 0, NumUVChannels.Num() - 1);

		if (NumUVChannels.IsValidIndex(CurrentLOD))
		{
			for (int32 UVChannelID = 0; UVChannelID < NumUVChannels[CurrentLOD]; ++UVChannelID)
			{
				UVChannels.Add( MakeShareable( new FString( FText::Format( NSLOCTEXT("AnimationEditorViewport", "UVChannel_ID", "UV Channel {0}"), FText::AsNumber( UVChannelID ) ).ToString() ) ) );
			}

			int32 CurrentUVChannel = AnimViewportClient->GetUVChannelToDraw();
			if (!UVChannels.IsValidIndex(CurrentUVChannel))
			{
				CurrentUVChannel = 0;
			}

			AnimViewportClient->SetUVChannelToDraw(CurrentUVChannel);

			if (UVChannelCombo.IsValid() && UVChannels.IsValidIndex(CurrentUVChannel))
			{
				UVChannelCombo->SetSelectedItem(UVChannels[CurrentUVChannel]);
			}
		}
	}
}

void SAnimationEditorViewportTabBody::UpdateScrubPanel(UAnimationAsset* AnimAsset)
{
	// We might not have a scrub panel if we're in animation mode.
	if (ScrubPanelContainer.IsValid())
	{
		ScrubPanelContainer->ClearChildren();
		bool bUseDefaultScrubPanel = true;
		if (UAnimMontage* Montage = Cast<UAnimMontage>(AnimAsset))
		{
			ScrubPanelContainer->AddSlot()
				.AutoHeight()
				[
					SNew(SAnimMontageScrubPanel, GetPreviewScene())
					.ViewInputMin(this, &SAnimationEditorViewportTabBody::GetViewMinInput)
					.ViewInputMax(this, &SAnimationEditorViewportTabBody::GetViewMaxInput)
					.bAllowZoom(true)
				];
			bUseDefaultScrubPanel = false;
		}
		if(bUseDefaultScrubPanel)
		{
			ScrubPanelContainer->AddSlot()
				.AutoHeight()
				[
					SNew(SAnimationScrubPanel, GetPreviewScene())
					.ViewInputMin(this, &SAnimationEditorViewportTabBody::GetViewMinInput)
					.ViewInputMax(this, &SAnimationEditorViewportTabBody::GetViewMaxInput)
					.bAllowZoom(true)
					.bDisplayAnimScrubBarEditing(false)
					.TimelineDelegates(TimelineDelegates)
				];
		}
	}
}

float SAnimationEditorViewportTabBody::GetViewMinInput() const
{
	if(TimelineDelegates.GetPlaybackTimeRangeDelegate.IsBound())
	{
		const TOptional<FVector2f> TimeRange = TimelineDelegates.GetPlaybackTimeRangeDelegate.Execute();
		if(TimeRange.IsSet())
		{
			return TimeRange.GetValue().X;
		}
	}
	
	if (UDebugSkelMeshComponent* PreviewComponent = GetPreviewScene()->GetPreviewMeshComponent())
	{
		UObject* PreviewAsset = GetPreviewScene()->GetPreviewAnimationAsset();
		if (PreviewAsset != NULL)
		{
			return 0.0f;
		}
		else if (PreviewComponent->GetAnimInstance() != NULL)
		{
			return FMath::Max<float>((float)(PreviewComponent->GetAnimInstance()->LifeTimer - 30.0), 0.0f);
		}
	}

	return 0.f; 
}

float SAnimationEditorViewportTabBody::GetViewMaxInput() const
{
	if(TimelineDelegates.GetPlaybackTimeRangeDelegate.IsBound())
	{
		const TOptional<FVector2f> TimeRange = TimelineDelegates.GetPlaybackTimeRangeDelegate.Execute();
		if(TimeRange.IsSet())
		{
			return TimeRange.GetValue().Y;
		}
	}

	UDebugSkelMeshComponent* PreviewComponent = GetPreviewScene()->GetPreviewMeshComponent();
	if (PreviewComponent != NULL)
	{
		UObject* PreviewAsset = GetPreviewScene()->GetPreviewAnimationAsset();
		if ((PreviewAsset != NULL) && (PreviewComponent->PreviewInstance != NULL))
		{
			return PreviewComponent->PreviewInstance->GetLength();
		}
		else if (PreviewComponent->GetAnimInstance() != NULL)
		{
			return static_cast<float>(PreviewComponent->GetAnimInstance()->LifeTimer);
		}
	}

	return 0.f;
}

void SAnimationEditorViewportTabBody::UpdateShowFlagForMeshEdges()
{
	bool bUseOverlayMaterial = false;
	if (UDebugSkelMeshComponent* PreviewComponent = GetPreviewScene()->GetPreviewMeshComponent())
	{
		bUseOverlayMaterial = PreviewComponent->bDrawBoneInfluences || PreviewComponent->bDrawMorphTargetVerts;
	}

	//@TODO: SNOWPOCALYPSE: broke UnlitWithMeshEdges
	bool bShowMeshEdgesViewMode = false;
#if 0
	bShowMeshEdgesViewMode = (CurrentViewMode == EAnimationEditorViewportMode::UnlitWithMeshEdges);
#endif

	LevelViewportClient->EngineShowFlags.SetMeshEdges(bUseOverlayMaterial || bShowMeshEdgesViewMode);
}

int32 SAnimationEditorViewportTabBody::GetLODSelection() const
{
	UDebugSkelMeshComponent* PreviewComponent = GetPreviewScene()->GetPreviewMeshComponent();

	if (PreviewComponent)
	{
		// If we are forcing a LOD level, report the actual LOD level we are displaying
		// as the mesh can potentially change LOD count under the viewport.
		if(PreviewComponent->GetForcedLOD() > 0)
		{
			return PreviewComponent->GetPredictedLODLevel() + 1;
		}
		else
		{
			return PreviewComponent->GetForcedLOD();
		}
	}
	return 0;
}

bool SAnimationEditorViewportTabBody::IsLODModelSelected(int32 LODSelectionType) const
{
	UDebugSkelMeshComponent* PreviewComponent = GetPreviewScene()->GetPreviewMeshComponent();
	if (PreviewComponent && PreviewComponent->IsTrackingAttachedLOD())
	{
		return false;
	}

	return GetLODSelection() == LODSelectionType;
}

bool SAnimationEditorViewportTabBody::IsTrackingAttachedMeshLOD() const
{
	UDebugSkelMeshComponent* PreviewComponent = GetPreviewScene()->GetPreviewMeshComponent();
	if (PreviewComponent)
	{
		return PreviewComponent->IsTrackingAttachedLOD();
	}

	return false;
}

void SAnimationEditorViewportTabBody::OnSetLODModel(int32 LODSelectionType)
{
	UDebugSkelMeshComponent* PreviewComponent = GetPreviewScene()->GetPreviewMeshComponent();
	
	if( PreviewComponent )
	{
		LODSelection = LODSelectionType;
		PreviewComponent->SetDebugForcedLOD(LODSelectionType);
		PreviewComponent->bTrackAttachedInstanceLOD = false;
	}
}

void SAnimationEditorViewportTabBody::OnSetLODTrackDebuggedInstance()
{
	UDebugSkelMeshComponent* PreviewComponent = GetPreviewScene()->GetPreviewMeshComponent();

	if (PreviewComponent)
	{
		PreviewComponent->bTrackAttachedInstanceLOD = true;
	}
}

void SAnimationEditorViewportTabBody::OnLODModelChanged()
{
	UDebugSkelMeshComponent* PreviewComponent = GetPreviewScene()->GetPreviewMeshComponent();

	if (PreviewComponent && LODSelection != PreviewComponent->GetForcedLOD())
	{
		LODSelection = PreviewComponent->GetForcedLOD();
		PopulateUVChoices();
	}
}

void SAnimationEditorViewportTabBody::OnDebugForcedLODChanged()
{
	UDebugSkelMeshComponent* PreviewComponent = GetPreviewScene()->GetPreviewMeshComponent();

	if (PreviewComponent)
	{
		PopulateUVChoices();
		GetPreviewScene()->BroadcastOnSelectedLODChanged();
	}
}

void SAnimationEditorViewportTabBody::OnSetSkinWeightProfile(FName InProfileName, ESkinWeightProfileLayer InLayer)
{
	// Apply the skin weight profile to the component, according to the selected the name, 
	if (PreviewScenePtr.IsValid())
	{
		if (UDebugSkelMeshComponent* MeshComponent = PreviewScenePtr.Pin()->GetPreviewMeshComponent())
		{
			MeshComponent->SetSkinWeightProfile(InProfileName, InLayer);
		}
	}
}

bool SAnimationEditorViewportTabBody::IsSkinWeightProfileSelected(FName InProfileName, ESkinWeightProfileLayer InLayer) const
{
	// Returns true if the given profile on that layer is enabeld. 
	if (PreviewScenePtr.IsValid())
	{
		if (UDebugSkelMeshComponent* MeshComponent = PreviewScenePtr.Pin()->GetPreviewMeshComponent())
		{
			return MeshComponent->GetCurrentSkinWeightProfileName(InLayer) == InProfileName;
		}
	}
	return InProfileName.IsNone();
}

TArray<FName> SAnimationEditorViewportTabBody::GetSelectedProfileNames() const
{
	// Get a list of the selected profile names. If only a single profile is selected, whether on primary or secondary,
	// only one entry is returned. If no profile is selected, a single entry with NAME_None is returned. 
	if (PreviewScenePtr.IsValid())
	{
		if (UDebugSkelMeshComponent* MeshComponent = PreviewScenePtr.Pin()->GetPreviewMeshComponent())
		{
			return MeshComponent->GetCurrentSkinWeightProfileLayerNames();
		}
	}

	TArray<FName> ProfileNames;
	ProfileNames.SetNum(FSkinWeightProfileStack::MaxLayerCount);
	
	return ProfileNames;
}

void SAnimationEditorViewportTabBody::OnBeginSliderMovementFloorOffset()
{
	// This value is saved in a UPROPERTY for the floor mesh, so changes are transactional
	PendingTransaction = MakeUnique<FScopedTransaction>(LOCTEXT("SetFloorOffset", "Set Floor Offset"));
	PinnedCommands->AddCustomWidget(TEXT("FloorOffsetWidget"));
}

void SAnimationEditorViewportTabBody::OnFloorOffsetChanged(float InNewValue)
{
	FAnimationViewportClient& AnimViewportClient = (FAnimationViewportClient&)GetLevelViewportClient();

	AnimViewportClient.SetFloorOffset(InNewValue);

	PinnedCommands->AddCustomWidget(TEXT("FloorOffsetWidget"));
}

void SAnimationEditorViewportTabBody::OnFloorOffsetCommitted(float InNewValue, ETextCommit::Type InCommitType)
{
	if (!PendingTransaction)
	{
		// Create the transaction here if it doesn't already exist. This can happen when changes come via text entry to the slider.
		PendingTransaction = MakeUnique<FScopedTransaction>(LOCTEXT("SetFloorOffset", "Set Floor Offset"));
	}

	FAnimationViewportClient& AnimViewportClient = (FAnimationViewportClient&)GetLevelViewportClient();

	AnimViewportClient.SetFloorOffset(InNewValue);

	PinnedCommands->AddCustomWidget(TEXT("FloorOffsetWidget"));

	PendingTransaction.Reset();
}

void SAnimationEditorViewportTabBody::CreatePinnedCommands()
{
	// Create our pinned commands before we bind commands
	IPinnedCommandListModule& PinnedCommandListModule =
		FModuleManager::LoadModuleChecked<IPinnedCommandListModule>(TEXT("PinnedCommandList"));
	PinnedCommands =
		PinnedCommandListModule.CreatePinnedCommandList((ContextName != NAME_None) ? ContextName : TEXT("PersonaViewport"));
	PinnedCommands->SetStyle(&FAppStyle::Get(), TEXT("ViewportPinnedCommandList"));

	UICommandList = MakeShareable(new FUICommandList_Pinnable);
}

TSharedPtr<FAnimationViewportClient> SAnimationEditorViewportTabBody::GetAnimationViewportClient() const
{
	return StaticCastSharedPtr<FAnimationViewportClient>(LevelViewportClient);
}

void SAnimationEditorViewportTabBody::OpenPreviewSceneSettings()
{
	OnInvokeTab.ExecuteIfBound(FPersonaTabs::AdvancedPreviewSceneSettingsID);
}

void SAnimationEditorViewportTabBody::SetCameraFollowMode(EAnimationViewportCameraFollowMode InCameraFollowMode, FName InBoneName)
{
	if (const TSharedPtr<FAnimationViewportClient>& AnimViewportClient = GetAnimationViewportClient())
	{
		AnimViewportClient->SetCameraFollowMode(InCameraFollowMode, InBoneName);
	}
}

bool SAnimationEditorViewportTabBody::IsCameraFollowEnabled(EAnimationViewportCameraFollowMode InCameraFollowMode) const
{
	if (const TSharedPtr<FAnimationViewportClient>& AnimViewportClient = GetAnimationViewportClient())
	{
		return (AnimViewportClient->GetCameraFollowMode() == InCameraFollowMode);
	}

	return false;
}

void SAnimationEditorViewportTabBody::ToggleRotateCameraToFollowBone()
{
	if (const TSharedPtr<FAnimationViewportClient>& AnimViewportClient = GetAnimationViewportClient())
	{
		AnimViewportClient->ToggleRotateCameraToFollowBone();
	}
}

EAxis::Type SAnimationEditorViewportTabBody::GetBoneAsCameraAxis(FAnimationViewportClient::EDirection Direction) const
{
	if (const TSharedPtr<FAnimationViewportClient>& AnimViewportClient = GetAnimationViewportClient())
	{
		return AnimViewportClient->GetBoneAsCameraAxis(Direction);
	}

	return EAxis::Type::None;
}

UE::Math::TVector<double> SAnimationEditorViewportTabBody::GetBoneAsCameraPositionOffset() const
{
	if (const TSharedPtr<FAnimationViewportClient>& AnimViewportClient = GetAnimationViewportClient())
	{
		return AnimViewportClient->GetBoneAsCameraPositionOffset();
	}

	return UE::Math::TVector<double>::Zero();
}

FRotator SAnimationEditorViewportTabBody::GetBoneAsCameraRotationOffset() const
{
	if (const TSharedPtr<FAnimationViewportClient>& AnimViewportClient = GetAnimationViewportClient())
	{
		return AnimViewportClient->GetBoneAsCameraRotationOffset();
	}

	return FRotator();
}

void SAnimationEditorViewportTabBody::SetBoneAsCameraAxis(FAnimationViewportClient::EDirection Direction, EAxis::Type Axis)
{
	if (const TSharedPtr<FAnimationViewportClient>& AnimViewportClient = GetAnimationViewportClient())
	{
		return AnimViewportClient->SetBoneAsCameraAxis(Direction, Axis);
	}
}

void SAnimationEditorViewportTabBody::SetBoneAsCameraPositionOffset(UE::Math::TVector<double> Offset)
{
	if (const TSharedPtr<FAnimationViewportClient>& AnimViewportClient = GetAnimationViewportClient())
	{
		AnimViewportClient->SetBoneAsCameraPositionOffset(Offset);
	}
}

void SAnimationEditorViewportTabBody::SetBoneAsCameraRotationOffset(const FRotator& Offset)
{
	if (const TSharedPtr<FAnimationViewportClient>& AnimViewportClient = GetAnimationViewportClient())
	{
		AnimViewportClient->SetBoneAsCameraRotationOffset(Offset);
	}
}

bool SAnimationEditorViewportTabBody::GetShouldRotateCameraToFollowBone() const
{
	if (const TSharedPtr<FAnimationViewportClient>& AnimViewportClient = GetAnimationViewportClient())
	{
		return AnimViewportClient->GetShouldRotateCameraToFollowBone();
	}

	return false;
}

void SAnimationEditorViewportTabBody::TogglePauseAnimationOnCameraMove()
{
	GetMutableDefault<UPersonaOptions>()->bPauseAnimationOnCameraMove = !GetMutableDefault<UPersonaOptions>()->bPauseAnimationOnCameraMove;
}

bool SAnimationEditorViewportTabBody::GetShouldPauseAnimationOnCameraMove() const
{
	return GetMutableDefault<UPersonaOptions>()->bPauseAnimationOnCameraMove;
}

FName SAnimationEditorViewportTabBody::GetCameraFollowBoneName() const
{
	if (const TSharedPtr<FAnimationViewportClient>& AnimViewportClient = GetAnimationViewportClient())
	{
		return AnimViewportClient->GetCameraFollowBoneName();
	}

	return NAME_None;
}

void SAnimationEditorViewportTabBody::SaveCameraAsDefault()
{
	if (const TSharedPtr<FAnimationViewportClient>& AnimViewportClient = GetAnimationViewportClient())
	{
		AnimViewportClient->SaveCameraAsDefault();
	}
}

void SAnimationEditorViewportTabBody::ClearDefaultCamera()
{
	if (const TSharedPtr<FAnimationViewportClient>& AnimViewportClient = GetAnimationViewportClient())
	{
		AnimViewportClient->ClearDefaultCamera();
	}
}

void SAnimationEditorViewportTabBody::JumpToDefaultCamera()
{
	if (const TSharedPtr<FAnimationViewportClient>& AnimViewportClient = GetAnimationViewportClient())
	{
		AnimViewportClient->JumpToDefaultCamera();
	}
}

bool SAnimationEditorViewportTabBody::CanSaveCameraAsDefault() const
{
	if (const TSharedPtr<FAnimationViewportClient>& AnimViewportClient = GetAnimationViewportClient())
	{
		return AnimViewportClient->CanSaveCameraAsDefault();
	}

	return false;
}

bool SAnimationEditorViewportTabBody::HasDefaultCameraSet() const
{
	if (const TSharedPtr<FAnimationViewportClient>& AnimViewportClient = GetAnimationViewportClient())
	{
		return (AnimViewportClient->HasDefaultCameraSet());
	}

	return false;
}

bool SAnimationEditorViewportTabBody::CanChangeCameraMode() const
{
	//Not allowed to change camera type when we are in an ortho camera
	return !LevelViewportClient->IsOrtho();
}

void SAnimationEditorViewportTabBody::OnToggleMuteAudio()
{
	if (const TSharedPtr<FAnimationViewportClient>& AnimViewportClient = GetAnimationViewportClient())
	{
		AnimViewportClient->OnToggleMuteAudio();
	}
}

bool SAnimationEditorViewportTabBody::IsAudioMuted() const
{
	if (const TSharedPtr<FAnimationViewportClient>& AnimViewportClient = GetAnimationViewportClient())
	{
		return AnimViewportClient->IsAudioMuted();
	}

	return false;
}

void SAnimationEditorViewportTabBody::OnToggleUseAudioAttenuation()
{
	if (const TSharedPtr<FAnimationViewportClient>& AnimViewportClient = GetAnimationViewportClient())
	{
		AnimViewportClient->OnToggleUseAudioAttenuation();
	}
}

bool SAnimationEditorViewportTabBody::IsAudioAttenuationEnabled() const
{
	if (const TSharedPtr<FAnimationViewportClient>& AnimViewportClient = GetAnimationViewportClient())
	{
		return AnimViewportClient->IsUsingAudioAttenuation();
	}

	return false;
}

void SAnimationEditorViewportTabBody::SetProcessRootMotionMode(EProcessRootMotionMode Mode)
{
	if (UDebugSkelMeshComponent* PreviewComponent = GetPreviewScene()->GetPreviewMeshComponent())
	{
		PreviewComponent->SetProcessRootMotionMode(Mode);
	}
}

bool SAnimationEditorViewportTabBody::IsProcessRootMotionModeSet(EProcessRootMotionMode Mode) const
{
	const UDebugSkelMeshComponent* PreviewComponent = GetPreviewScene()->GetPreviewMeshComponent();
	return PreviewComponent ? (PreviewComponent->GetRequestedProcessRootMotionMode() == Mode) : false;
}

bool SAnimationEditorViewportTabBody::CanUseProcessRootMotionMode(EProcessRootMotionMode Mode) const
{
	if(const UDebugSkelMeshComponent* PreviewComponent = GetPreviewScene()->GetPreviewMeshComponent())
	{
		return PreviewComponent->CanUseProcessRootMotionMode(Mode);
	}

	return false;
}

void SAnimationEditorViewportTabBody::SetVisualizeRootMotionMode(EVisualizeRootMotionMode Mode)
{
	if(UDebugSkelMeshComponent* PreviewComponent = GetPreviewScene()->GetPreviewMeshComponent())
	{
		return PreviewComponent->SetVisualizeRootMotionMode(Mode);
	}
}

bool SAnimationEditorViewportTabBody::IsVisualizeRootMotionModeSet(EVisualizeRootMotionMode Mode) const
{
	if(const UDebugSkelMeshComponent* PreviewComponent = GetPreviewScene()->GetPreviewMeshComponent())
	{
		return PreviewComponent->IsVisualizeRootMotionMode(Mode);
	}

	return false;
}

bool SAnimationEditorViewportTabBody::CanVisualizeRootMotion() const
{
	if(const UDebugSkelMeshComponent* PreviewComponent = GetPreviewScene()->GetPreviewMeshComponent())
	{
		return PreviewComponent->DoesCurrentAssetHaveRootMotion();
	}

	return false;
}

void SAnimationEditorViewportTabBody::ToggleNotificationVisualizations()
{
	if (UDebugSkelMeshComponent* PreviewComponent = GetPreviewScene()->GetPreviewMeshComponent())
	{
		PreviewComponent->SetShowNotificationVisualizations(!PreviewComponent->IsNotificationVisualizationsEnabled());
	}
}

bool SAnimationEditorViewportTabBody::IsNotificationVisualizationsEnabled() const
{
	if (const UDebugSkelMeshComponent* PreviewComponent = GetPreviewScene()->GetPreviewMeshComponent())
	{
		return PreviewComponent->IsNotificationVisualizationsEnabled();
	}
	return false;
}

void SAnimationEditorViewportTabBody::ToggleAssetUserDataVisualizations()
{
	if (UDebugSkelMeshComponent* PreviewComponent = GetPreviewScene()->GetPreviewMeshComponent())
	{
		PreviewComponent->SetShowAssetUserDataVisualizations(!PreviewComponent->IsAssetUserDataVisualizationsEnabled());
	}
}

bool SAnimationEditorViewportTabBody::IsAssetUserDataVisualizationsEnabled() const
{
	if (const UDebugSkelMeshComponent* PreviewComponent = GetPreviewScene()->GetPreviewMeshComponent())
	{
		return PreviewComponent->IsAssetUserDataVisualizationsEnabled();
	}
	return false;
}

bool SAnimationEditorViewportTabBody::IsClothSimulationEnabled() const
{
	UDebugSkelMeshComponent* PreviewComponent = GetPreviewScene()->GetPreviewMeshComponent();

	if( PreviewComponent )
	{
		return !PreviewComponent->bDisableClothSimulation;
	}

	return true;
}

void SAnimationEditorViewportTabBody::OnEnableClothSimulation()
{
	UDebugSkelMeshComponent* PreviewComponent = GetPreviewScene()->GetPreviewMeshComponent();

	if( PreviewComponent )
	{
		PreviewComponent->bDisableClothSimulation = !PreviewComponent->bDisableClothSimulation;

		RefreshViewport();
	}
}

void SAnimationEditorViewportTabBody::OnResetClothSimulation()
{
	UDebugSkelMeshComponent* PreviewComponent = GetPreviewScene()->GetPreviewMeshComponent();

	if( PreviewComponent )
	{
		PreviewComponent->RecreateClothingActors();

		RefreshViewport();
	}
}

bool SAnimationEditorViewportTabBody::IsApplyingClothWind() const
{
	return GetPreviewScene()->IsWindEnabled();
}

void SAnimationEditorViewportTabBody::OnPauseClothingSimWithAnim()
{
	UDebugSkelMeshComponent* PreviewComponent = GetPreviewScene()->GetPreviewMeshComponent();

	if(PreviewComponent)
	{
		PreviewComponent->bPauseClothingSimulationWithAnim = !PreviewComponent->bPauseClothingSimulationWithAnim;

		bool bShouldPause = PreviewComponent->bPauseClothingSimulationWithAnim;

		if(PreviewComponent->IsPreviewOn() && PreviewComponent->PreviewInstance)
		{
			UAnimSingleNodeInstance* PreviewInstance = PreviewComponent->PreviewInstance;
			const bool bPlaying = PreviewInstance->IsPlaying();

			if(!bPlaying && bShouldPause)
			{
				PreviewComponent->SuspendClothingSimulation();
			}
			else if(!bShouldPause && PreviewComponent->IsClothingSimulationSuspended())
			{
				PreviewComponent->ResumeClothingSimulation();
			}
		}
	}
}

bool SAnimationEditorViewportTabBody::IsPausingClothingSimWithAnim()
{
	UDebugSkelMeshComponent* PreviewComponent = GetPreviewScene()->GetPreviewMeshComponent();
	
	if(PreviewComponent)
	{
		return PreviewComponent->bPauseClothingSimulationWithAnim;
	}

	return false;
}

void SAnimationEditorViewportTabBody::SetWindStrength(float SliderPos)
{
	TSharedRef<FAnimationEditorPreviewScene> PreviewScene = GetPreviewScene();

	if ( SliderPos <= 0.0f )
	{
		if ( PreviewScene->IsWindEnabled() )
		{
			PreviewScene->EnableWind(false);
			PreviewScene->SetWindStrength(0.0f);
			RefreshViewport();
		}

		return;
	}

	if ( !PreviewScene->IsWindEnabled() )
	{
		PreviewScene->EnableWind(true);
	}

	GetPreviewScene()->SetWindStrength(SliderPos);

	RefreshViewport();
}

TOptional<float> SAnimationEditorViewportTabBody::GetWindStrengthSliderValue() const
{
	return GetPreviewScene()->GetWindStrength();
}

void SAnimationEditorViewportTabBody::SetGravityScale(float SliderPos)
{
	GetPreviewScene()->SetGravityScale(SliderPos);
	RefreshViewport();
}

float SAnimationEditorViewportTabBody::GetGravityScaleSliderValue() const
{
	return GetPreviewScene()->GetGravityScale();
}

void SAnimationEditorViewportTabBody::OnSetSectionsDisplayMode(ESectionDisplayMode DisplayMode)
{
	UDebugSkelMeshComponent* PreviewComponent = GetPreviewScene()->GetPreviewMeshComponent();

	if (!PreviewComponent)
	{
		return;
	}

	SectionsDisplayMode = DisplayMode;

	switch (SectionsDisplayMode)
	{
	case ESectionDisplayMode::ShowAll:
		// restore to the original states
		PreviewComponent->RestoreClothSectionsVisibility();
		break;
	case ESectionDisplayMode::ShowOnlyClothSections:
		// disable all except clothing sections and shows only cloth sections
		PreviewComponent->ToggleClothSectionsVisibility(true);
		break;
	case ESectionDisplayMode::HideOnlyClothSections:
		// disables only clothing sections
		PreviewComponent->ToggleClothSectionsVisibility(false);
		break;
	}

	RefreshViewport();
}

bool SAnimationEditorViewportTabBody::IsSectionsDisplayMode(ESectionDisplayMode DisplayMode) const
{
	return SectionsDisplayMode == DisplayMode;
}

void SAnimationEditorViewportTabBody::AddRecordingNotification()
{
	if(WeakRecordingNotification.IsValid())
	{
		return;
	}

	auto GetRecordingStateText = [this]()
	{
		if(GetPreviewScene()->IsRecording())
		{
			UAnimSequence* Recording = GetPreviewScene()->GetCurrentRecording();
			const FString& Name = Recording ? Recording->GetName() : TEXT("None");
			float TimeRecorded = GetPreviewScene()->GetCurrentRecordingTime();
			FNumberFormattingOptions NumberOption;
			NumberOption.MaximumFractionalDigits = 2;
			NumberOption.MinimumFractionalDigits = 2;
			return FText::Format(LOCTEXT("AnimRecorder", "Recording '{0}' {1} secs"),
				FText::FromString(Name), FText::AsNumber(TimeRecorded, &NumberOption));
		}

		return FText::GetEmpty();
	};

	auto GetRecordingStateStateVisibility = [this]()
	{
		if (GetPreviewScene()->IsRecording())
		{
			return EVisibility::Visible;
		}

		return EVisibility::Collapsed;
	};

	auto StopRecording = [this]()
	{
		if (GetPreviewScene()->IsRecording())
		{
			GetPreviewScene()->StopRecording();
		}

		return FReply::Handled();
	};

	WeakRecordingNotification = AddNotification(EMessageSeverity::Info,
		true,
		SNew(SHorizontalBox)
		.Visibility_Lambda(GetRecordingStateStateVisibility)
		.ToolTipText(LOCTEXT("RecordingStatusTooltip", "Shows the status of animation recording."))
		+SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.Padding(2.0f, 4.0f)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(0.0f, 0.0f, 4.0f, 0.0f)
			[
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), "AnimViewport.MessageText")
				.Font(FAppStyle::Get().GetFontStyle("FontAwesome.9"))
				.Text(FEditorFontGlyphs::Video_Camera)
			]
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.FillWidth(1.0f)
			[
				SNew(STextBlock)
				.Text_Lambda(GetRecordingStateText)
				.TextStyle(FAppStyle::Get(), "AnimViewport.MessageText")
			]
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2.0f, 0.0f)
		[
			SNew(SButton)
			.ForegroundColor(FSlateColor::UseForeground())
			.ButtonStyle(FAppStyle::Get(), "FlatButton.Success")
			.ToolTipText(LOCTEXT("RecordingInViewportStop", "Stop recording animation."))
			.OnClicked_Lambda(StopRecording)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(0.0f, 0.0f, 4.0f, 0.0f)
				[
					SNew(STextBlock)
					.TextStyle(FAppStyle::Get(), "AnimViewport.MessageText")
					.Font(FAppStyle::Get().GetFontStyle("FontAwesome.9"))
					.Text(FEditorFontGlyphs::Stop)
				]
				+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(STextBlock)
					.TextStyle(FAppStyle::Get(), "AnimViewport.MessageText")
					.Text(LOCTEXT("AnimViewportStopRecordingButtonLabel", "Stop"))
				]
			]
		],
		FPersonaViewportNotificationOptions(TAttribute<EVisibility>::Create(GetRecordingStateStateVisibility))
	);
}

void SAnimationEditorViewportTabBody::AddPostProcessNotification()
{
	if(WeakPostProcessNotification.IsValid())
	{
		return;
	}

	auto GetVisibility = [this]()
	{
		return CanDisablePostProcess() ? EVisibility::Visible : EVisibility::Collapsed;
	};

	auto GetPostProcessGraphName = [this]()
	{
		if (UDebugSkelMeshComponent* PreviewComponent = GetPreviewScene()->GetPreviewMeshComponent())
		{
			if(PreviewComponent->GetSkeletalMeshAsset() && PreviewComponent->GetSkeletalMeshAsset()->GetPostProcessAnimBlueprint() && PreviewComponent->GetSkeletalMeshAsset()->GetPostProcessAnimBlueprint()->ClassGeneratedBy)
			{
				return FText::FromString(PreviewComponent->GetSkeletalMeshAsset()->GetPostProcessAnimBlueprint()->ClassGeneratedBy->GetName());
			}
		}

		return FText::GetEmpty();
	};

	auto DoesPostProcessModifyCurves = [this]()
	{
		if (UDebugSkelMeshComponent* PreviewComponent = GetPreviewScene()->GetPreviewMeshComponent())
		{
			return (PreviewComponent->PostProcessAnimInstance && PreviewComponent->PostProcessAnimInstance->HasActiveCurves());
		}

		return false;
	};

	auto GetText = [this, GetPostProcessGraphName, DoesPostProcessModifyCurves]()
	{
		return IsDisablePostProcessChecked() ? 
			FText::Format(LOCTEXT("PostProcessDisabledText", "Post process Animation Blueprint '{0}' is disabled."), GetPostProcessGraphName()) : 
			FText::Format(LOCTEXT("PostProcessRunningText", "Post process Animation Blueprint '{0}' is running. {1}"), GetPostProcessGraphName(), DoesPostProcessModifyCurves() ? LOCTEXT("PostProcessModifiesCurves", "Post process modifes curves.") : FText::GetEmpty()) ;
	};

	auto GetButtonText = [this]()
	{
		return IsDisablePostProcessChecked() ? LOCTEXT("PostProcessEnableText", "Enable") : LOCTEXT("PostProcessDisableText", "Disable");
	};

	auto GetButtonTooltipText = [this]()
	{
		return IsDisablePostProcessChecked() ? LOCTEXT("PostProcessEnableTooltip", "Enable post process animation blueprint.") : LOCTEXT("PostProcessDisableTooltip", "Disable post process animation blueprint.");
	};

	auto GetButtonIcon = [this]()
	{
		return IsDisablePostProcessChecked() ? FEditorFontGlyphs::Check : FEditorFontGlyphs::Times;
	};

	auto EnablePostProcess = [this]()
	{
		OnToggleDisablePostProcess();
		return FReply::Handled();
	};

	auto EditPostProcess = [this]()
	{
		if (UDebugSkelMeshComponent* PreviewComponent = GetPreviewScene()->GetPreviewMeshComponent())
		{
			if(PreviewComponent->GetSkeletalMeshAsset() && !PreviewComponent->GetSkeletalMeshAsset()->IsCompiling() && PreviewComponent->GetSkeletalMeshAsset()->GetPostProcessAnimBlueprint())
			{
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAssets(TArray<UObject*>({ PreviewComponent->GetSkeletalMeshAsset()->GetPostProcessAnimBlueprint()->ClassGeneratedBy }));
			}
		}

		return FReply::Handled();
	};

	WeakPostProcessNotification = AddNotification(EMessageSeverity::Warning,
		true,
		SNew(SHorizontalBox)
		.Visibility_Lambda(GetVisibility)
		+SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.Padding(4.0f, 4.0f)
		[
			SNew(SHorizontalBox)
			.ToolTipText_Lambda(GetText)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(0.0f, 0.0f, 4.0f, 0.0f)
			[
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), "AnimViewport.MessageText")
				.Font(FAppStyle::Get().GetFontStyle("FontAwesome.9"))
				.Text(FEditorFontGlyphs::Exclamation_Triangle)
			]
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.FillWidth(1.0f)
			[
				SNew(STextBlock)
				.Text_Lambda(GetText)
				.TextStyle(FAppStyle::Get(), "AnimViewport.MessageText")
			]
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2.0f, 0.0f)
		[
			SNew(SButton)
			.ForegroundColor(FSlateColor::UseForeground())
			.ButtonStyle(FAppStyle::Get(), "FlatButton.Success")
			.ToolTipText_Lambda(GetButtonTooltipText)
			.OnClicked_Lambda(EnablePostProcess)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(0.0f, 0.0f, 4.0f, 0.0f)
				[
					SNew(STextBlock)
					.TextStyle(FAppStyle::Get(), "AnimViewport.MessageText")
					.Font(FAppStyle::Get().GetFontStyle("FontAwesome.9"))
					.Text_Lambda(GetButtonIcon)
				]
				+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(STextBlock)
					.TextStyle(FAppStyle::Get(), "AnimViewport.MessageText")
					.Text_Lambda(GetButtonText)
				]
			]
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2.0f, 0.0f)
		[
			SNew(SButton)
			.ForegroundColor(FSlateColor::UseForeground())
			.ButtonStyle(FAppStyle::Get(), "FlatButton")
			.ToolTipText(LOCTEXT("EditPostProcessAnimBPButtonToolTip", "Edit the post process Animation Blueprint."))
			.OnClicked_Lambda(EditPostProcess)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(0.0f, 0.0f, 4.0f, 0.0f)
				[
					SNew(STextBlock)
					.TextStyle(FAppStyle::Get(), "AnimViewport.MessageText")
					.Font(FAppStyle::Get().GetFontStyle("FontAwesome.9"))
					.Text(FEditorFontGlyphs::Pencil)
				]
				+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(STextBlock)
					.TextStyle(FAppStyle::Get(), "AnimViewport.MessageText")
					.Text(LOCTEXT("EditPostProcessAnimBPButtonText", "Edit"))
				]
			]
		],
		FPersonaViewportNotificationOptions(TAttribute<EVisibility>::Create(GetVisibility))
	);
}

void SAnimationEditorViewportTabBody::AddMinLODNotification()
{
	if(WeakMinLODNotification.IsValid())
	{
		return;
	}

	auto GetMinLODNotificationVisibility = [this]()
	{
		if (GetPreviewScene()->GetPreviewMesh() && !GetPreviewScene()->GetPreviewMesh()->IsCompiling() && GetPreviewScene()->GetPreviewMesh()->GetDefaultMinLod() != 0)
		{
			return EVisibility::Visible;
		}

		return EVisibility::Collapsed;
	};

	WeakMinLODNotification = AddNotification(EMessageSeverity::Info,
		true,
		SNew(SHorizontalBox)
		.Visibility_Lambda(GetMinLODNotificationVisibility)
		.ToolTipText(LOCTEXT("MinLODNotificationTooltip", "This asset has a minimum LOD applied."))
		+SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.Padding(2.0f, 4.0f)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(0.0f, 0.0f, 4.0f, 0.0f)
			[
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), "AnimViewport.MessageText")
				.Font(FAppStyle::Get().GetFontStyle("FontAwesome.9"))
				.Text(FEditorFontGlyphs::Level_Down)
			]
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.FillWidth(1.0f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("MinLODNotification", "Min LOD applied"))
				.TextStyle(FAppStyle::Get(), "AnimViewport.MessageText")
			]
		],
		FPersonaViewportNotificationOptions(TAttribute<EVisibility>::Create(GetMinLODNotificationVisibility))
	);
}


void SAnimationEditorViewportTabBody::HandleFocusCamera()
{
	if (const TSharedPtr<FAnimationViewportClient>& AnimViewportClient = GetAnimationViewportClient())
	{
		AnimViewportClient->FocusViewportOnPreviewMesh(false);
	}
}

#undef LOCTEXT_NAMESPACE
