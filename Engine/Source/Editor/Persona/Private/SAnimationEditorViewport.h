// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "PersonaDelegates.h"
#include "IPersonaViewport.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "BlueprintEditor.h"
#include "IPersonaPreviewScene.h"
#include "EditorViewportClient.h"
#include "AnimationEditorViewportClient.h"
#include "AnimationEditorPreviewScene.h"
#include "SEditorViewport.h"
#include "PersonaModule.h"
#include "SNameComboBox.h"
#include "ScopedTransaction.h"

class FUICommandList_Pinnable;
class SAnimViewportToolBar;
class SAnimationEditorViewportTabBody;

struct FAnimationEditorViewportRequiredArgs
{
	FAnimationEditorViewportRequiredArgs(const TSharedRef<class IPersonaPreviewScene>& InPreviewScene, TSharedRef<class SAnimationEditorViewportTabBody> InTabBody, TSharedRef<class FAssetEditorToolkit> InAssetEditorToolkit, int32 InViewportIndex)
		: PreviewScene(InPreviewScene)
		, TabBody(InTabBody)
		, AssetEditorToolkit(InAssetEditorToolkit)
		, ViewportIndex(InViewportIndex)
	{}

	TSharedRef<class IPersonaPreviewScene> PreviewScene;

	TSharedRef<class SAnimationEditorViewportTabBody> TabBody;

	TSharedRef<class FAssetEditorToolkit> AssetEditorToolkit;

	int32 ViewportIndex;
};

//////////////////////////////////////////////////////////////////////////
// SAnimationEditorViewport

enum class ESectionDisplayMode
{
	None = -1,
	ShowAll,
	ShowOnlyClothSections,
	HideOnlyClothSections,
	NumSectionDisplayMode
};

class SAnimationEditorViewport : public SEditorViewport, public FEditorUndoClient
{
public:
	SLATE_BEGIN_ARGS(SAnimationEditorViewport) 
		: _ContextName(NAME_None)
		, _ShowShowMenu(true)
		, _ShowLODMenu(true)
		, _ShowPlaySpeedMenu(true)
		, _ShowStats(true)
		, _ShowFloorOptions(true)
		, _ShowTurnTable(true)
		, _ShowPhysicsMenu(false)
	{}

	SLATE_ARGUMENT(TArray<TSharedPtr<FExtender>>, Extenders)

	SLATE_ARGUMENT(FName, ContextName)

	SLATE_ARGUMENT(bool, ShowShowMenu)

	SLATE_ARGUMENT(bool, ShowLODMenu)

	SLATE_ARGUMENT(bool, ShowPlaySpeedMenu)

	SLATE_ARGUMENT(bool, ShowStats)

	SLATE_ARGUMENT(bool, ShowFloorOptions)

	SLATE_ARGUMENT(bool, ShowTurnTable)

	SLATE_ARGUMENT(bool, ShowPhysicsMenu)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const FAnimationEditorViewportRequiredArgs& InRequiredArgs);
	virtual ~SAnimationEditorViewport();

	/** Get the viewport toolbar widget */
	TSharedPtr<SAnimViewportToolBar> GetViewportToolbar() const { return ViewportToolbar; }

	bool IsPhysicsEditor() const { return bShowPhysicsMenu; }
protected:
	// SEditorViewport interface
	virtual TSharedRef<FEditorViewportClient> MakeEditorViewportClient() override;
	virtual TSharedPtr<SWidget> BuildViewportToolbar() override;
	virtual TSharedPtr<IPreviewProfileController> CreatePreviewProfileController() override;
	virtual void OnFocusViewportToSelection() override;
	virtual void PopulateViewportOverlays(TSharedRef<SOverlay> Overlay) override;
	// End of SEditorViewport interface

	/**  Handle undo/redo by refreshing the viewport */
	virtual void PostUndo( bool bSuccess );
	virtual void PostRedo( bool bSuccess );

	virtual void BindCommands() override;
	virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	virtual void OnDragLeave(const FDragDropEvent& DragDropEvent) override;
	virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;

protected:
	// Viewport client
	TSharedPtr<class FAnimationViewportClient> LevelViewportClient;

	// Pointer to the compound widget that owns this viewport widget
	TWeakPtr<class SAnimationEditorViewportTabBody> TabBodyPtr;

	// The preview scene that we are viewing
	TWeakPtr<class IPersonaPreviewScene> PreviewScenePtr;

	// Handle to the registered OnPreviewFeatureLevelChanged delegate.
	FDelegateHandle PreviewFeatureLevelChangedHandle;

	// The asset editor we are embedded in
	TWeakPtr<class FAssetEditorToolkit> AssetEditorToolkitPtr;

	/** The viewport toolbar */
	TSharedPtr<SAnimViewportToolBar> ViewportToolbar;

	/** Menu extenders */
	TArray<TSharedPtr<FExtender>> Extenders;

	/** Context used for persisting settings */
	FName ContextName;

	/** Viewport index (0-3) */
	int32 ViewportIndex;

	/** Whether to show the 'Show' menu */
	bool bShowShowMenu;

	/** Whether to show the 'LOD' menu */
	bool bShowLODMenu;

	/** Whether to show the 'Play Speed' menu */
	bool bShowPlaySpeedMenu;

	/** Whether we should show stats for this viewport */
	bool bShowStats;

	/** Whether to show options relating to floor height */
	bool bShowFloorOptions;

	/** Whether to show options relating to turntable */
	bool bShowTurnTable;

	/** Whether to show options relating to physics */
	bool bShowPhysicsMenu;

	friend class SAnimationEditorViewportTabBody;
};

//////////////////////////////////////////////////////////////////////////
// SAnimationEditorViewportTabBody

class SAnimationEditorViewportTabBody : public IPersonaViewport
{
public:
	SLATE_BEGIN_ARGS( SAnimationEditorViewportTabBody )
		: _BlueprintEditor()
		, _ContextName(NAME_None)
		, _ShowShowMenu(true)
		, _ShowLODMenu(true)
		, _ShowPlaySpeedMenu(true)
		, _ShowTimeline(true)
		, _ShowStats(true)
		, _AlwaysShowTransformToolbar(false)
		, _ShowFloorOptions(true)
		, _ShowTurnTable(true)
		, _ShowPhysicsMenu(false)
		{}

		SLATE_ARGUMENT(TWeakPtr<FBlueprintEditor>, BlueprintEditor)

		SLATE_ARGUMENT(FOnInvokeTab, OnInvokeTab)

		SLATE_ARGUMENT(TArray<TSharedPtr<FExtender>>, Extenders)

		SLATE_ARGUMENT(FOnGetViewportText, OnGetViewportText)

		SLATE_ARGUMENT(FName, ContextName)

		SLATE_ARGUMENT(bool, ShowShowMenu)

		SLATE_ARGUMENT(bool, ShowLODMenu)

		SLATE_ARGUMENT(bool, ShowPlaySpeedMenu)

		SLATE_ARGUMENT(bool, ShowTimeline)

		SLATE_ARGUMENT(bool, ShowStats)

		SLATE_ARGUMENT(bool, AlwaysShowTransformToolbar)

		SLATE_ARGUMENT(bool, ShowFloorOptions)

		SLATE_ARGUMENT(bool, ShowTurnTable)

		SLATE_ARGUMENT(bool, ShowPhysicsMenu)

		SLATE_ARGUMENT(FAnimationScrubPanelDelegates, TimelineDelegates)
	SLATE_END_ARGS()
public:

	void Construct(const FArguments& InArgs, const TSharedRef<class IPersonaPreviewScene>& InPreviewScene, const TSharedRef<class FAssetEditorToolkit>& InAssetEditorToolkit, int32 InViewportIndex);
	SAnimationEditorViewportTabBody();

	/** IPersonaViewport interface */
	virtual TSharedRef<IPersonaViewportState> SaveState() const override;
	virtual void RestoreState(TSharedRef<IPersonaViewportState> InState) override;
	virtual FEditorViewportClient& GetViewportClient() const override;
	virtual TSharedRef<IPinnedCommandList> GetPinnedCommandList() const override;
	virtual TWeakPtr<SWidget> AddNotification(TAttribute<EMessageSeverity::Type> InSeverity, TAttribute<bool> InCanBeDismissed, const TSharedRef<SWidget>& InNotificationWidget, FPersonaViewportNotificationOptions InOptions) override;
	virtual void RemoveNotification(const TWeakPtr<SWidget>& InContainingWidget) override;
	virtual FPersonaViewportKeyDownDelegate& GetKeyDownDelegate() override { return OnKeyDownDelegate; }
	virtual void AddOverlayWidget( TSharedRef<SWidget> InOverlaidWidget, int32 ZOrder = INDEX_NONE ) override;
	virtual void RemoveOverlayWidget( TSharedRef<SWidget> InOverlaidWidget ) override;


	/** SWidget interface */
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

	void RefreshViewport();

	/**
	 * @return The list of commands on the viewport that are bound to delegates                    
	 */
	const TSharedPtr<FUICommandList_Pinnable>& GetCommandList() const { return UICommandList; }

	/** Handle the skeletal mesh changing */
	void HandlePreviewMeshChanged(class USkeletalMesh* OldSkeletalMesh, class USkeletalMesh* NewSkeletalMesh);

	/** Function to get the number of LOD models associated with the preview skeletal mesh*/
	int32 GetLODModelCount() const;

	/** LOD model selection checking function*/
	bool IsLODModelSelected( int32 LODSelectionType ) const;
	bool IsTrackingAttachedMeshLOD() const;
	int32 GetLODSelection() const;

	/** Function to get anim viewport widget */
	TSharedPtr<class SEditorViewport> GetViewportWidget() const { return ViewportWidget; }

	/** Gets the editor client for this viewport */
	FEditorViewportClient& GetLevelViewportClient()
	{		
		return *LevelViewportClient;
	}

	/** Gets the animation viewport client */
	TSharedPtr<class FAnimationViewportClient> GetAnimationViewportClient() const;

	/** Returns Detail description of what's going with viewport **/
	FText GetDisplayString() const;

	/** Can we use gizmos? */
	bool CanUseGizmos() const;

	/** Function to check whether floor is auto aligned or not */
	bool IsAutoAlignFloor() const;

	void SetWindStrength( float SliderPos );

	/** Function to get slider value which represents wind strength (0 - 1)*/
	TOptional<float> GetWindStrengthSliderValue() const;

	bool IsApplyingClothWind() const;

	/** Show gravity scale */
	void SetGravityScale( float SliderPos );
	float GetGravityScaleSliderValue() const;

	/** Adjustable bone draw size */
	void SetBoneDrawSize(float BoneDrawSize);
	float GetBoneDrawSize() const;

	void SetCustomTurnTableSpeed(float InCustomTurnTableSpeed);
	float GetCustomTurnTableSpeed() const;

	/** Function to set LOD model selection*/
	void OnSetLODModel(int32 LODSelectionType);
	void OnSetLODTrackDebuggedInstance();
	void OnLODModelChanged();
	void OnDebugForcedLODChanged();

	void OnSetSkinWeightProfile(FName InProfileName, ESkinWeightProfileLayer InLayer);
	bool IsSkinWeightProfileSelected(FName InProfileName, ESkinWeightProfileLayer InLayer) const;
	TArray<FName> GetSelectedProfileNames() const;

	void OnBeginSliderMovementFloorOffset();
	void OnFloorOffsetChanged(float InNewValue);
	void OnFloorOffsetCommitted(float InNewValue, ETextCommit::Type InCommitType);

	const TSharedPtr<IPinnedCommandList>& GetPinnedCommands();

	/** Get the preview scene we are viewing */
	TSharedRef<class FAnimationEditorPreviewScene> GetPreviewScene() const { return PreviewScenePtr.Pin().ToSharedRef(); }

	/** Get the asset editor toolkit we are embedded in */
	TSharedPtr<class FAssetEditorToolkit> GetAssetEditorToolkit() const;

private:
	bool IsVisible() const;

	/**
	 * Binds our UI commands to delegates
	 */ 
	void BindCommands();
	
	/** Show Morphtarget of SkeletalMesh **/
	void OnShowMorphTargets();

	bool IsShowingMorphTargets() const;

	/** Show Raw Animation on top of Compressed Animation **/
	void OnShowRawAnimation();

	bool IsShowingRawAnimation() const;

	/** Handlers for the disable post process flag */
	void OnToggleDisablePostProcess();
	bool CanDisablePostProcess() const;
	bool IsDisablePostProcessChecked();

	/** Show non retargeted animation. */
	void OnShowNonRetargetedAnimation();

	bool IsShowingNonRetargetedPose() const;

	/** Show non retargeted animation. */
	void OnShowSourceRawAnimation();

	bool IsShowingSourceRawAnimation() const;

	/** Show non retargeted animation. */
	void OnShowBakedAnimation();

	bool IsShowingBakedAnimation() const;


	/** Additive Base Pose on top of full animation **/
	void OnShowAdditiveBase();

	bool IsShowingAdditiveBase() const;

	bool IsPreviewingAnimation() const;

	/** Function to show/hide bone names */
	void OnShowBoneNames();

	/** Function to check whether bone names are displayed or not */
	bool IsShowingBoneNames() const;

	/** Function to show/hide bone colors */
	void OnShowBoneColors();

	/** Function to check whether bone's are using multiple colors */
	bool IsShowingBoneColors() const;
	
	/** Function to show/hide selected bone weight */
	void OnShowOverlayNone();

	/** Function to check whether bone weights are displayed or not*/
	bool IsShowingOverlayNone() const;

	/** Function to show/hide selected bone weight */
	void OnShowOverlayBoneWeight();

	/** Function to check whether bone weights are displayed or not*/
	bool IsShowingOverlayBoneWeight() const;

	/** Function to show/hide selected morphtarget overlay*/
	void OnShowOverlayMorphTargetVert();

	/** Function to check whether morphtarget overlay is displayed or not*/
	bool IsShowingOverlayMorphTargetVerts() const;
	
	/** Function to set Local axes mode of the specificed type */
	void OnSetBoneDrawMode(int32 BoneDrawMode);

	/** Local axes mode checking function for the specificed type*/
	bool IsBoneDrawModeSet(int32 BoneDrawMode) const;

	/** Toggles showing all bones when not all bones are shown, none otherwise */
	void ToggleShowAllBones();

	/** Function to set Local axes mode of the specificed type */
	void OnSetLocalAxesMode( int32 LocalAxesMode );

	/** Local axes mode checking function for the specificed type*/
	bool IsLocalAxesModeSet( int32 LocalAxesMode ) const;

	/** Function to show/hide socket hit points */
	void OnShowSockets();

	/** Function to check whether socket hit points are displayed or not*/
	bool IsShowingSockets() const;

	/** Function to show/hide extracted transform attributes */
	void OnShowAttributes();

	/** Function to check whether extracted transform attributes are displayed or not */
	bool IsShowingAttributes() const;

	/** Function to show/hide mesh info*/
	void OnShowDisplayInfo(int32 DisplayInfoMode);

	/** Function to check whether mesh info is displayed or not */
	bool IsShowingMeshInfo(int32 DisplayInfoMode) const;

	/** Function to show/hide timecode display. */
	void OnToggleShowTimecode();

	/** Function to check whether timecode is displayed or not */
	bool IsShowTimecode() const;

	/** Toggles floor alignment in the preview scene */
	void OnToggleAutoAlignFloor();

	/** Called to toggle showing of the bounds of the current preview mesh */
	void ShowBound();
	bool CanShowBound() const;
	bool IsShowBoundEnabled() const;

	/** Called to toggle showing of the current preview mesh */
	void ToggleShowPreviewMesh();
	bool CanShowPreviewMesh() const;
	bool IsShowPreviewMeshEnabled() const;

	/** Called to toggle showing the Nanite fallback of the current preview mesh */
	void ToggleShowNaniteFallback();
	bool CanShowNaniteFallback() const;
	bool IsShowNaniteFallbackEnabled() const;

	/** Run a lambda function on each preview mesh in the scene */
	FORCEINLINE_DEBUGGABLE void ForEachDebugMesh(TFunction<void (UDebugSkelMeshComponent*)> PerMeshFunction)
	{
		TArray<UDebugSkelMeshComponent*> PreviewMeshComponents;
		GetPreviewScene()->GetActor()->GetComponents(PreviewMeshComponents, true);
		for (UDebugSkelMeshComponent* PreviewMesh : PreviewMeshComponents)
		{
			PerMeshFunction(PreviewMesh);
		}
	}

	/** Called to toggle using in-game bound on current preview mesh */
	void UseInGameBound();
	bool CanUseInGameBound() const;
	bool IsUsingInGameBound() const;

	/** Called to toggle 'fixed bounds' option to preview */
	void UseFixedBounds();
	bool CanUseFixedBounds() const;
	bool IsUsingFixedBounds() const;

	/** Called to toggle 'pre-skinned' option to preview */
	void UsePreSkinnedBounds();
	bool CanUsePreSkinnedBounds() const;
	bool IsUsingPreSkinnedBounds() const;

	/** Called by UV channel combo box on selection change */
	void ComboBoxSelectionChanged( TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo );

	/** Populates choices for UV Channel combo box for each lod based on current preview asset */
	void PopulateNumUVChannels();

	/** Populates choices for UV Channel combo box */
	void PopulateUVChoices();

	void AnimChanged(UAnimationAsset* AnimAsset);

	/** Open the preview scene settings */
	void OpenPreviewSceneSettings();

	void SaveCameraAsDefault();
	void ClearDefaultCamera();
	void JumpToDefaultCamera();
	bool HasDefaultCameraSet() const;
	bool CanSaveCameraAsDefault() const;

	/** Focus the viewport on the preview mesh */
	void HandleFocusCamera();

	void CreatePinnedCommands();

public:
	/** Called to determine whether the camera mode menu options should be enabled */
	bool CanChangeCameraMode() const;

private:
	/** Function to mute/unmute viewport audio */
	void OnToggleMuteAudio();

	/** Whether audio from the viewport is muted */
	bool IsAudioMuted() const;

	/** Function to enable/disable viewport audio attenuation */
	void OnToggleUseAudioAttenuation();

	/** Whether audio from the viewport is attenuated */
	bool IsAudioAttenuationEnabled() const;

	/** Sets process root motion mode on the debug mesh */
	void SetProcessRootMotionMode(EProcessRootMotionMode Mode);

	/** Checks whether the supplied mode is set on the debug mesh */
	bool IsProcessRootMotionModeSet(EProcessRootMotionMode Mode) const;

	/** Whether the supplied mode can be used */
	bool CanUseProcessRootMotionMode(EProcessRootMotionMode Mode) const;

	void SetVisualizeRootMotionMode(EVisualizeRootMotionMode Mode);
	bool IsVisualizeRootMotionModeSet(EVisualizeRootMotionMode Mode) const;
	bool CanVisualizeRootMotion() const;

	void ToggleNotificationVisualizations();
	bool IsNotificationVisualizationsEnabled() const;

	void ToggleAssetUserDataVisualizations();
	bool IsAssetUserDataVisualizationsEnabled() const;

private:
	/** Selected Turn Table speed  */
	EAnimationPlaybackSpeeds::Type SelectedTurnTableSpeed;
	/** Selected turn table mode */
	EPersonaTurnTableMode::Type SelectedTurnTableMode;
	/** Custom Animation speed in the viewport. Transient setting. */
	float CustomTurnTableSpeed = 1.0f;

	void OnSetTurnTableSpeed(int32 SpeedIndex);
	void OnSetTurnTableMode(int32 ModeIndex);
	bool IsTurnTableModeSelected(int32 ModeIndex) const;

	FPersonaViewportKeyDownDelegate OnKeyDownDelegate;

public:
	/** Setup the camera follow mode */
	void SetCameraFollowMode(EAnimationViewportCameraFollowMode InCameraFollowMode, FName InBoneName);
	bool IsCameraFollowEnabled(EAnimationViewportCameraFollowMode InCameraFollowMode) const;
	FName GetCameraFollowBoneName() const;
	void ToggleRotateCameraToFollowBone();

	/** Gets the axis for a direction when using the Bone As Camera follow mode */
	EAxis::Type GetBoneAsCameraAxis(FAnimationViewportClient::EDirection Direction) const;

	/** Sets the axis for a direction when using the Bone As Camera follow mode */
	void SetBoneAsCameraAxis(FAnimationViewportClient::EDirection Direction, EAxis::Type Axis);

	/** Gets the position offset from the bone origin in world coordiantes for the Bone As Camera follow mode */
	UE::Math::TVector<double> GetBoneAsCameraPositionOffset() const;

	/** Gets the rotation offset from the bone in world coordiantes for the Bone As Camera follow mode */
	FRotator GetBoneAsCameraRotationOffset() const;
	/** Sets the forward axis for the Bone As Camera follow mode */

	/** Sets the position offset from the bone origin in world coordiantes for the Bone As Camera follow mode */
	void SetBoneAsCameraPositionOffset(UE::Math::TVector<double> Offset);

	/** Sets the rotation offset from the bone in world coordiantes for the Bone As Camera follow mode */
	void SetBoneAsCameraRotationOffset(const FRotator& Offset);

	bool GetShouldRotateCameraToFollowBone() const;
	void TogglePauseAnimationOnCameraMove();
	bool GetShouldPauseAnimationOnCameraMove() const;
	bool IsTurnTableSpeedSelected(int32 SpeedIndex) const;

	/** 
	 * clothing show options 
	*/
private:
	/** Enable cloth simulation */
	void OnEnableClothSimulation();
	bool IsClothSimulationEnabled() const;

	/** Reset clothing simulation */
	void OnResetClothSimulation();

	void OnPauseClothingSimWithAnim();
	bool IsPausingClothingSimWithAnim();

	/** Show all sections which means the original state */
	void OnSetSectionsDisplayMode(ESectionDisplayMode DisplayMode);
	bool IsSectionsDisplayMode(ESectionDisplayMode DisplayMode) const;

private:
	/** Weak pointer back to the preview scene we are viewing */
	TWeakPtr<class FAnimationEditorPreviewScene> PreviewScenePtr;

	/** Weak pointer back to asset editor we are embedded in */
	TWeakPtr<class FAssetEditorToolkit> AssetEditorToolkitPtr;

	/** Weak pointer to the blueprint editor we are optionally embedded in */
	TWeakPtr<class FBlueprintEditor> BlueprintEditorPtr;

	/** Whether to show the timeline */
	bool bShowTimeline;

	/** Whether we should always show the transform toolbar for this viewport */
	bool bAlwaysShowTransformToolbar;

	/** Whether to align the camera's rotation to the bone's orientation */
	bool bCameraFollowLockRotation;

	/** Level viewport client */
	TSharedPtr<FEditorViewportClient> LevelViewportClient;

	/** Viewport widget*/
	TSharedPtr<class SAnimationEditorViewport> ViewportWidget;

	/** Toolbar widget */
	TSharedPtr<SHorizontalBox> ToolbarBox;

	/** Commands that are bound to delegates*/
	TSharedPtr<FUICommandList_Pinnable> UICommandList;

	/** Delegate used to invoke tabs in the containing asset editor */
	FOnInvokeTab OnInvokeTab;

	/** Pinned commands widget */
	TSharedPtr<IPinnedCommandList> PinnedCommands;

	/** Transaction to handle scoping updates for sliders we own that transact objects (i.e. floor mesh) */
	TUniquePtr<FScopedTransaction> PendingTransaction;

	FName ContextName;

	FAnimationScrubPanelDelegates TimelineDelegates;
	
public:
	/** UV Channel Selector */
	TSharedPtr< class STextComboBox > UVChannelCombo;
	
private:
	/** Choices for UVChannelCombo */
	TArray< TSharedPtr< FString > > UVChannels;

	/** Num UV Channels at each LOD of Preview Mesh */
	TArray<int32> NumUVChannels;

public:
	/** Box that contains scrub panel */
	TSharedPtr<SVerticalBox> ScrubPanelContainer;

	/** Box that contains notifications */
	TSharedPtr<SVerticalBox> ViewportNotificationsContainer;

	/** Post process notification */
	TWeakPtr<SWidget> WeakPostProcessNotification;

	/** Recording notification */
	TWeakPtr<SWidget> WeakRecordingNotification;

	/** Min LOD notification */
	TWeakPtr<SWidget> WeakMinLODNotification;

	/** Current LOD selection*/
	int32 LODSelection;

	/** Draw All/ Draw only clothing sections/ Hide only clothing sections */
	ESectionDisplayMode SectionsDisplayMode;

	/** Delegate used to get custom viewport text */
	FOnGetViewportText OnGetViewportText;

	/** Get Min/Max Input of value **/
	float GetViewMinInput() const;
	float GetViewMaxInput() const;

	/** Sets The EngineShowFlags.MeshEdges flag on the viewport based on current state */
	void UpdateShowFlagForMeshEdges();

	/** Update scrub panel to reflect viewed animation asset */
	void UpdateScrubPanel(UAnimationAsset* AnimAsset);

	/** Adds a persistent notification to display recording state when recording */
	void AddRecordingNotification();

	/** Adds a persistent notification to display post process graph state */
	void AddPostProcessNotification();

	/** Add a notification to tell the user a min LOD is being applied */
	void AddMinLODNotification();
};
