// Copyright Epic Games, Inc. All Rights Reserved.
#include "SAnimationScrubPanel.h"
#include "Widgets/SBoxPanel.h"
#include "Animation/AnimSequenceBase.h"
#include "Animation/AnimSequence.h"
#include "Animation/AnimBlueprintGeneratedClass.h"
#include "Animation/AnimBlueprint.h"
#include "AnimPreviewInstance.h"
#include "SScrubControlPanel.h"
#include "ScopedTransaction.h"
#include "Animation/BlendSpace.h"
#include "AnimationEditorPreviewScene.h"
#include "Animation/AnimData/IAnimationDataModel.h"
#include "Animation/AnimSequenceHelpers.h"
#include "ViewportToolbar/AnimationEditorMenus.h"
#include "ViewportToolbar/AnimViewportContext.h"
#include "ToolMenus.h"

#define LOCTEXT_NAMESPACE "AnimationScrubPanel"

void SAnimationScrubPanel::Construct( const SAnimationScrubPanel::FArguments& InArgs, const TSharedRef<IPersonaPreviewScene>& InPreviewScene)
{
	bSliderBeingDragged = false;
	LockedSequence = InArgs._LockedSequence;
	OnSetInputViewRange = InArgs._OnSetInputViewRange;
	TimelineDelegates = InArgs._TimelineDelegates;
	ViewInputMinAttribute = InArgs._ViewInputMin;
	ViewInputMaxAttribute = InArgs._ViewInputMax;
	
	PreviewScenePtr = InPreviewScene;

	TSharedPtr<SHorizontalBox> HorizontalBox;

	this->ChildSlot
	[
		SAssignNew(HorizontalBox, SHorizontalBox)
		.AddMetaData<FTagMetaData>(TEXT("AnimScrub.Scrub"))
		+SHorizontalBox::Slot()
		.HAlign(HAlign_Fill) 
		.VAlign(VAlign_Center)
		.FillWidth(1)
		.Padding(0.0f)
		[
			SAssignNew(ScrubControlPanel, SScrubControlPanel)
			.IsEnabled(true)//this, &SAnimationScrubPanel::DoesSyncViewport)
			.Value(this, &SAnimationScrubPanel::GetScrubValue)
			.NumOfKeys(this, &SAnimationScrubPanel::GetNumberOfKeys)
			.SequenceLength(this, &SAnimationScrubPanel::GetSequenceLength)
			.DisplayDrag(this, &SAnimationScrubPanel::GetDisplayDrag)
			.OnValueChanged(this, &SAnimationScrubPanel::OnValueChanged, true)
			.OnBeginSliderMovement(this, &SAnimationScrubPanel::OnBeginSliderMovement)
			.OnEndSliderMovement(this, &SAnimationScrubPanel::OnEndSliderMovement)
			.OnClickedForwardPlay(this, &SAnimationScrubPanel::OnClick_Forward)
			.OnClickedForwardStep(this, &SAnimationScrubPanel::OnClick_Forward_Step)
			.OnClickedForwardEnd(this, &SAnimationScrubPanel::OnClick_Forward_End)
			.OnClickedBackwardPlay(this, &SAnimationScrubPanel::OnClick_Backward)
			.OnClickedBackwardStep(this, &SAnimationScrubPanel::OnClick_Backward_Step)
			.OnClickedBackwardEnd(this, &SAnimationScrubPanel::OnClick_Backward_End)
			.OnClickedToggleLoop(this, &SAnimationScrubPanel::OnClick_ToggleLoop)
			.OnClickedRecord(this, &SAnimationScrubPanel::OnClick_Record)
			.OnGetRecordVisibility(this, &SAnimationScrubPanel::OnGetRecordVisibility)
			.OnGetLooping(this, &SAnimationScrubPanel::IsLoopStatusOn)
			.OnGetPlaybackMode(this, &SAnimationScrubPanel::GetPlaybackMode)
			.OnGetRecording(this, &SAnimationScrubPanel::IsRecording)
			.ViewInputMin(this, &SAnimationScrubPanel::GetViewInputMin)
			.ViewInputMax(this, &SAnimationScrubPanel::GetViewInputMax)
			.bDisplayAnimScrubBarEditing(InArgs._bDisplayAnimScrubBarEditing)
			.OnSetInputViewRange(InArgs._OnSetInputViewRange)
			.OnCropAnimSequence( this, &SAnimationScrubPanel::OnCropAnimSequence )
			.OnAddAnimSequence( this, &SAnimationScrubPanel::OnInsertAnimSequence )
			.OnAppendAnimSequence(this, &SAnimationScrubPanel::OnAppendAnimSequence )
			.OnReZeroAnimSequence( this, &SAnimationScrubPanel::OnReZeroAnimSequence )
			.bAllowZoom(InArgs._bAllowZoom)
			.IsRealtimeStreamingMode(this, &SAnimationScrubPanel::IsRealtimeStreamingMode)
		]
	];
	
	{
		const FName MenuName = "AnimationEditor.Scrub.PlaybackMenu";
		if (!UToolMenus::Get()->IsMenuRegistered(MenuName))
		{
			UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(MenuName);
			Menu->MenuType = EMultiBoxType::SlimHorizontalToolBar;
			
			FToolMenuSection& UnnamedSection = Menu->AddSection(NAME_None);
			UnnamedSection.AddEntry(UE::AnimationEditor::CreatePlaybackSubmenu());
		}
		
		FToolMenuContext MenuContext;
		{
			MenuContext.AppendCommandList(InPreviewScene->GetCommandList());
			UAnimViewportContext* const ContextObject = NewObject<UAnimViewportContext>();
			ContextObject->PersonaPreviewScene = InPreviewScene;
			MenuContext.AddObject(ContextObject);
		}

		// clang-format off
		HorizontalBox->AddSlot()
		.HAlign(HAlign_Right)
		.AutoWidth()
		[
			UToolMenus::Get()->GenerateWidget(MenuName, MenuContext)
		];
		// clang-format on
	}
}

FReply SAnimationScrubPanel::OnClick_Forward_Step()
{
	SetPlaybackMode(EPlaybackMode::Stopped);

	if(TimelineDelegates.StepForwardDelegate.IsBound())
	{
		if(TimelineDelegates.StepForwardDelegate.Execute())
		{
			return FReply::Handled();
		}
	}
	
	UDebugSkelMeshComponent* SMC = GetPreviewScene()->GetPreviewMeshComponent();

	if (UAnimSingleNodeInstance* PreviewInstance = GetPreviewInstance())
	{
		bool bShouldStepCloth = FMath::Abs(PreviewInstance->GetLength() - PreviewInstance->GetCurrentTime()) > SMALL_NUMBER;

		PreviewInstance->StepForward();

		if(SMC && bShouldStepCloth)
		{
			SMC->bPerformSingleClothingTick = true;
		}
	}
	else if (SMC)
	{
		// BlendSpaces and Animation Blueprints combine animations so there's no such thing as a frame. However, 1/30 is a sensible/common rate.
		const float FixedFrameRate = 30.0f;

		// Advance a single frame, leaving it paused afterwards
		SMC->GlobalAnimRateScale = 1.0f;
		GetPreviewScene()->Tick(1.0f / FixedFrameRate); 
		SMC->GlobalAnimRateScale = 0.0f;
	}

	return FReply::Handled();
}

FReply SAnimationScrubPanel::OnClick_Forward_End()
{
	SetPlaybackMode(EPlaybackMode::Stopped);
	OnValueChanged(GetViewInputMax(), false);
	return FReply::Handled();
}

FReply SAnimationScrubPanel::OnClick_Backward_Step()
{
	SetPlaybackMode(EPlaybackMode::Stopped);

	if(TimelineDelegates.StepBackwardDelegate.IsBound())
	{
		if(TimelineDelegates.StepBackwardDelegate.Execute())
		{
			return FReply::Handled();
		}
	}

	UAnimSingleNodeInstance* PreviewInstance = GetPreviewInstance();
	UDebugSkelMeshComponent* SMC = GetPreviewScene()->GetPreviewMeshComponent();
	if (PreviewInstance)
	{
		bool bShouldStepCloth = PreviewInstance->GetCurrentTime() > SMALL_NUMBER;

		PreviewInstance->StepBackward();

		if(SMC && bShouldStepCloth)
		{
			SMC->bPerformSingleClothingTick = true;
		}
	}
	return FReply::Handled();
}

FReply SAnimationScrubPanel::OnClick_Backward_End()
{
	SetPlaybackMode(EPlaybackMode::Stopped);
	OnValueChanged(GetViewInputMin(), false);
	return FReply::Handled();
}

FReply SAnimationScrubPanel::OnClick_Forward()
{
	UAnimSingleNodeInstance* PreviewInstance = GetPreviewInstance();
	UDebugSkelMeshComponent* SMC = GetPreviewScene()->GetPreviewMeshComponent();

	const EPlaybackMode::Type PlaybackMode = GetPlaybackMode();
	if(PlaybackMode == EPlaybackMode::PlayingForward)
	{
		SetPlaybackMode(EPlaybackMode::Stopped);

		if(SMC && SMC->bPauseClothingSimulationWithAnim)
		{
			SMC->SuspendClothingSimulation();
		}
	}
	else
	{
		if ( GetScrubValue() >= GetSequenceLength() )
		{
			OnValueChanged(0.0f, false);
		}
		SetPlaybackMode(EPlaybackMode::PlayingForward);
		
		if(SMC && SMC->bPauseClothingSimulationWithAnim)
		{
			SMC->ResumeClothingSimulation();
		}
	}
	
	if(SMC && !PreviewInstance)
	{
		SMC->GlobalAnimRateScale = (SMC->GlobalAnimRateScale > 0.0f) ? 0.0f : 1.0f;
	}

	return FReply::Handled();
}

FReply SAnimationScrubPanel::OnClick_Backward()
{
	const EPlaybackMode::Type PlaybackMode = GetPlaybackMode();
	if(PlaybackMode == EPlaybackMode::PlayingForward)
	{
		SetPlaybackMode(EPlaybackMode::Stopped);
	}
	else
	{
		if ( GetScrubValue() >= GetSequenceLength() )
		{
			OnValueChanged(0.0f, false);
		}
		SetPlaybackMode(EPlaybackMode::PlayingReverse);
	}
	return FReply::Handled();
}

FReply SAnimationScrubPanel::OnClick_ToggleLoop()
{
	if(TimelineDelegates.GetIsLoopingDelegate.IsBound() && TimelineDelegates.SetIsLoopingDelegate.IsBound())
	{
		const TOptional<bool> PreviousState = TimelineDelegates.GetIsLoopingDelegate.Execute();
		if(PreviousState.IsSet())
		{
			if(TimelineDelegates.SetIsLoopingDelegate.Execute(!PreviousState.GetValue()))
			{
				return FReply::Handled();
			}
		}
	}
	
	UAnimSingleNodeInstance* PreviewInstance = GetPreviewInstance();
	if (PreviewInstance)
	{
		bool bIsLooping = PreviewInstance->IsLooping();
		PreviewInstance->SetLooping(!bIsLooping);
	}
	return FReply::Handled();
}

FReply SAnimationScrubPanel::OnClick_Record()
{
	StaticCastSharedRef<FAnimationEditorPreviewScene>(GetPreviewScene())->RecordAnimation();

	return FReply::Handled();
}

EVisibility SAnimationScrubPanel::OnGetRecordVisibility() const
{
	if(TimelineDelegates.GetRecordingVisibilityDelegate.IsBound())
	{
		const TOptional<EVisibility> Visibility = TimelineDelegates.GetRecordingVisibilityDelegate.Execute();
		if(Visibility.IsSet())
		{
			return Visibility.GetValue();
		}
	}
	return EVisibility::Visible;
}

bool SAnimationScrubPanel::IsLoopStatusOn() const
{
	if(TimelineDelegates.GetIsLoopingDelegate.IsBound())
	{
		const TOptional<bool> PreviousState = TimelineDelegates.GetIsLoopingDelegate.Execute();
		if(PreviousState.IsSet())
		{
			return PreviousState.GetValue();
		}
	}
	
	UAnimSingleNodeInstance* PreviewInstance = GetPreviewInstance();
	return (PreviewInstance && PreviewInstance->IsLooping());
}

EPlaybackMode::Type SAnimationScrubPanel::GetPlaybackMode() const
{
	if(TimelineDelegates.GetPlaybackModeDelegate.IsBound())
	{
		const TOptional<int32> Mode = TimelineDelegates.GetPlaybackModeDelegate.Execute();
		if(Mode.IsSet())
		{
			return static_cast<EPlaybackMode::Type>(Mode.GetValue());
		}
	}
	if (UAnimSingleNodeInstance* PreviewInstance = GetPreviewInstance())
	{
		if (PreviewInstance->IsPlaying())
		{
			return PreviewInstance->IsReverse() ? EPlaybackMode::PlayingReverse : EPlaybackMode::PlayingForward;
		}
		return EPlaybackMode::Stopped;
	}
	else if (UDebugSkelMeshComponent* SMC = GetPreviewScene()->GetPreviewMeshComponent())
	{
		return (SMC->GlobalAnimRateScale > 0.0f) ? EPlaybackMode::PlayingForward : EPlaybackMode::Stopped;
	}
	
	return EPlaybackMode::Stopped;
}

bool SAnimationScrubPanel::IsRecording() const
{
	if(TimelineDelegates.IsRecordingActiveDelegate.IsBound())
	{
		const TOptional<bool> State = TimelineDelegates.IsRecordingActiveDelegate.Execute();
		if(State.IsSet())
		{
			return State.GetValue();
		}
	}
	return StaticCastSharedRef<FAnimationEditorPreviewScene>(GetPreviewScene())->IsRecording();
}

bool SAnimationScrubPanel::IsRealtimeStreamingMode() const
{
	UAnimSingleNodeInstance* PreviewInstance = GetPreviewInstance();
	return ( ! (PreviewInstance && PreviewInstance->GetCurrentAsset()) );
}

float SAnimationScrubPanel::GetViewInputMin() const
{
	if(TimelineDelegates.GetPlaybackTimeRangeDelegate.IsBound())
	{
		const TOptional<FVector2f> TimeRange = TimelineDelegates.GetPlaybackTimeRangeDelegate.Execute();
		if(TimeRange.IsSet())
		{
			return TimeRange.GetValue().X;
		}
	}
	return ViewInputMinAttribute.Get();
}

float SAnimationScrubPanel::GetViewInputMax() const
{
	if(TimelineDelegates.GetPlaybackTimeRangeDelegate.IsBound())
	{
		const TOptional<FVector2f> TimeRange = TimelineDelegates.GetPlaybackTimeRangeDelegate.Execute();
		if(TimeRange.IsSet())
		{
			return TimeRange.GetValue().Y;
		}
	}
	return ViewInputMaxAttribute.Get();
}

void SAnimationScrubPanel::OnValueChanged(float NewValue, bool bFireNotifies)
{
	if(TimelineDelegates.SetPlaybackTimeDelegate.IsBound())
	{
		if(TimelineDelegates.SetPlaybackTimeDelegate.Execute(NewValue, true))
		{
			return;
		}
	}
	
	if (UAnimSingleNodeInstance* PreviewInstance = GetPreviewInstance())
	{
		PreviewInstance->SetPosition(NewValue, bFireNotifies);
	}
	else
	{
		UAnimInstance* Instance;
		FAnimBlueprintDebugData* DebugData;
		if (GetAnimBlueprintDebugData(/*out*/ Instance, /*out*/ DebugData))
		{
			DebugData->SetSnapshotIndexByTime(Instance, NewValue);
		}
	}
}

// make sure viewport is freshes
void SAnimationScrubPanel::OnBeginSliderMovement()
{
	bSliderBeingDragged = true;

	if (UAnimSingleNodeInstance* PreviewInstance = GetPreviewInstance())
	{
		SetPlaybackMode(EPlaybackMode::Stopped);
	}
}

void SAnimationScrubPanel::OnEndSliderMovement(float NewValue)
{
	bSliderBeingDragged = false;
}

uint32 SAnimationScrubPanel::GetNumberOfKeys() const
{
	if(TimelineDelegates.GetNumberOfKeysDelegate.IsBound())
	{
		const TOptional<uint32> NumberOfKeys = TimelineDelegates.GetNumberOfKeysDelegate.Execute();
		if(NumberOfKeys.IsSet())
		{
			return NumberOfKeys.GetValue();
		}
	}
	if (DoesSyncViewport())
	{
		UAnimSingleNodeInstance* PreviewInstance = GetPreviewInstance();
		float Length = PreviewInstance->GetLength();
		// if anim sequence, use correct num frames
		int32 NumKeys = (int32) (Length/0.0333f); 
		if (PreviewInstance->GetCurrentAsset())
		{
			if (PreviewInstance->GetCurrentAsset()->IsA(UAnimSequenceBase::StaticClass()))
			{
				NumKeys = CastChecked<UAnimSequenceBase>(PreviewInstance->GetCurrentAsset())->GetNumberOfSampledKeys();
			}
			else if(PreviewInstance->GetCurrentAsset()->IsA(UBlendSpace::StaticClass()))
			{
				// Blendspaces dont display frame notches, so just return 0 here
				NumKeys = 0;
			}
		}
		return NumKeys;
	}
	else if (LockedSequence)
	{
		return LockedSequence->GetNumberOfSampledKeys();
	}
	else
	{
		UAnimInstance* Instance;
		FAnimBlueprintDebugData* DebugData;
		if (GetAnimBlueprintDebugData(/*out*/ Instance, /*out*/ DebugData))
		{
			return DebugData->GetSnapshotLengthInFrames();
		}
	}

	return 1;
}

float SAnimationScrubPanel::GetSequenceLength() const
{
	if(TimelineDelegates.GetPlaybackTimeRangeDelegate.IsBound())
	{
		const TOptional<FVector2f> TimeRange = TimelineDelegates.GetPlaybackTimeRangeDelegate.Execute();
		if(TimeRange.IsSet())
		{
			return TimeRange.GetValue().Y - TimeRange.GetValue().X;
		}
	}
	
	if (DoesSyncViewport())
	{
		UAnimSingleNodeInstance* PreviewInstance = GetPreviewInstance();
		return PreviewInstance->GetLength();
	}
	else if (LockedSequence)
	{
		return LockedSequence->GetPlayLength();
	}
	else
	{
		UAnimInstance* Instance;
		FAnimBlueprintDebugData* DebugData;
		if (GetAnimBlueprintDebugData(/*out*/ Instance, /*out*/ DebugData))
		{
			return static_cast<float>(Instance->LifeTimer);
		}
	}

	return 0.f;
}

bool SAnimationScrubPanel::DoesSyncViewport() const
{
	UAnimSingleNodeInstance* PreviewInstance = GetPreviewInstance();

	return (( LockedSequence==nullptr && PreviewInstance ) || ( LockedSequence && PreviewInstance && PreviewInstance->GetCurrentAsset() == LockedSequence ));
}

void SAnimationScrubPanel::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	if (bSliderBeingDragged)
	{
		GetPreviewScene()->InvalidateViews();
	}
}

class UAnimSingleNodeInstance* SAnimationScrubPanel::GetPreviewInstance() const
{
	UDebugSkelMeshComponent* PreviewMeshComponent = GetPreviewScene()->GetPreviewMeshComponent();
	return PreviewMeshComponent && PreviewMeshComponent->IsPreviewOn()? PreviewMeshComponent->PreviewInstance : nullptr;
}

float SAnimationScrubPanel::GetScrubValue() const
{
	if(TimelineDelegates.GetPlaybackTimeDelegate.IsBound())
	{
		const TOptional<float> Time = TimelineDelegates.GetPlaybackTimeDelegate.Execute();
		if(Time.IsSet())
		{
			return Time.GetValue();
		}
	}
	
	if (DoesSyncViewport())
	{
		UAnimSingleNodeInstance* PreviewInstance = GetPreviewInstance();
		if (PreviewInstance)
		{
			return PreviewInstance->GetCurrentTime(); 
		}
	}
	else
	{
		UAnimInstance* Instance;
		FAnimBlueprintDebugData* DebugData;
		if (GetAnimBlueprintDebugData(/*out*/ Instance, /*out*/ DebugData))
		{
			return static_cast<float>(Instance->CurrentLifeTimerScrubPosition);
		}
	}

	return 0.f;
}

void SAnimationScrubPanel::ReplaceLockedSequence(class UAnimSequenceBase* NewLockedSequence)
{
	LockedSequence = NewLockedSequence;
}

UAnimInstance* SAnimationScrubPanel::GetAnimInstanceWithBlueprint() const
{
	if (UDebugSkelMeshComponent* DebugComponent = GetPreviewScene()->GetPreviewMeshComponent())
	{
		UAnimInstance* Instance = DebugComponent->GetAnimInstance();

		if ((Instance != nullptr) && (Instance->GetClass()->ClassGeneratedBy != nullptr))
		{
			return Instance;
		}
	}

	return nullptr;
}

bool SAnimationScrubPanel::GetAnimBlueprintDebugData(UAnimInstance*& Instance, FAnimBlueprintDebugData*& DebugInfo) const
{
	Instance = GetAnimInstanceWithBlueprint();

	if (Instance != nullptr)
	{
		// Avoid updating the instance if we're replaying the past
		if (UAnimBlueprintGeneratedClass* AnimBlueprintClass = Cast<UAnimBlueprintGeneratedClass>(Instance->GetClass()))
		{
			if (UAnimBlueprint* Blueprint = Cast<UAnimBlueprint>(AnimBlueprintClass->ClassGeneratedBy))
			{
				if (Blueprint->GetObjectBeingDebugged() == Instance)
				{
					DebugInfo = &(AnimBlueprintClass->GetAnimBlueprintDebugData());
					return true;
				}
			}
		}
	}

	return false;
}

void SAnimationScrubPanel::OnCropAnimSequence( bool bFromStart, float CurrentTime )
{
	UAnimSingleNodeInstance* PreviewInstance = GetPreviewInstance();
	if(PreviewInstance)
	{
		float Length = PreviewInstance->GetLength();
		if (PreviewInstance->GetCurrentAsset())
		{
			UAnimSequence* AnimSequence = Cast<UAnimSequence>( PreviewInstance->GetCurrentAsset() );
			if( AnimSequence )
			{
				const FScopedTransaction Transaction( LOCTEXT("CropAnimSequence", "Crop Animation Sequence") );

				//Call modify to restore slider position
				PreviewInstance->Modify();

				//Call modify to restore anim sequence current state
				AnimSequence->Modify();

				const FFrameTime AssetFrameTime = AnimSequence->GetSamplingFrameRate().AsFrameTime(CurrentTime);
				const FFrameNumber RoundedAssetFrame = AssetFrameTime.RoundToFrame();
				const TRange<FFrameNumber> TrimRange(TRangeBound<FFrameNumber>::Inclusive(bFromStart ? 0 : RoundedAssetFrame),
				bFromStart ? TRangeBound<FFrameNumber>::Exclusive(RoundedAssetFrame) : TRangeBound<FFrameNumber>::Exclusive(AnimSequence->GetNumberOfSampledKeys()));

				// Trim off the user-selected part of the raw anim data.
				UE::Anim::AnimationData::Trim(AnimSequence, TrimRange);

				//Resetting slider position to the first frame
				PreviewInstance->SetPosition( 0.0f, false );

				OnSetInputViewRange.ExecuteIfBound(0, AnimSequence->GetPlayLength());
			}
		}
	}
}

void SAnimationScrubPanel::OnAppendAnimSequence( bool bFromStart, int32 NumOfFrames )
{
	UAnimSingleNodeInstance* PreviewInstance = GetPreviewInstance();
	if(PreviewInstance && PreviewInstance->GetCurrentAsset())
	{
		UAnimSequence* AnimSequence = Cast<UAnimSequence>(PreviewInstance->GetCurrentAsset());
		if(AnimSequence)
		{
			const FScopedTransaction Transaction(LOCTEXT("InsertAnimSequence", "Insert Animation Sequence"));

			//Call modify to restore slider position
			PreviewInstance->Modify();

			//Call modify to restore anim sequence current state
			AnimSequence->Modify();

			// Crop the raw anim data.
			int32 StartFrame = (bFromStart)? 0 : AnimSequence->GetDataModel()->GetNumberOfFrames() - 1;
			int32 EndFrame = StartFrame + NumOfFrames;
			int32 CopyFrame = StartFrame;
			UE::Anim::AnimationData::DuplicateKeys(AnimSequence, StartFrame, NumOfFrames, CopyFrame);

			OnSetInputViewRange.ExecuteIfBound(0, AnimSequence->GetPlayLength());
		}
	}
}

void SAnimationScrubPanel::OnInsertAnimSequence( bool bBefore, int32 CurrentFrame )
{
	UAnimSingleNodeInstance* PreviewInstance = GetPreviewInstance();
	if(PreviewInstance && PreviewInstance->GetCurrentAsset())
	{
		UAnimSequence* AnimSequence = Cast<UAnimSequence>(PreviewInstance->GetCurrentAsset());
		if(AnimSequence)
		{
			const FScopedTransaction Transaction(LOCTEXT("InsertAnimSequence", "Insert Animation Sequence"));

			//Call modify to restore slider position
			PreviewInstance->Modify();

			//Call modify to restore anim sequence current state
			AnimSequence->Modify();

			// Duplicate specified key
			const int32 StartFrame = (bBefore)? CurrentFrame : CurrentFrame + 1;
			UE::Anim::AnimationData::DuplicateKeys(AnimSequence, StartFrame, 1, CurrentFrame);

			OnSetInputViewRange.ExecuteIfBound(0, AnimSequence->GetPlayLength());
		}
	}
}

void SAnimationScrubPanel::OnReZeroAnimSequence(int32 FrameIndex)
{
	UAnimSingleNodeInstance* PreviewInstance = GetPreviewInstance();
	if(PreviewInstance)
	{
		UDebugSkelMeshComponent* PreviewSkelComp = GetPreviewScene()->GetPreviewMeshComponent();

		if (PreviewInstance->GetCurrentAsset() && PreviewSkelComp )
		{
			if(UAnimSequence* AnimSequence = Cast<UAnimSequence>( PreviewInstance->GetCurrentAsset()))
			{
				if (const USkeleton* Skeleton = AnimSequence->GetSkeleton())
				{
					const FName RootBoneName = Skeleton->GetReferenceSkeleton().GetBoneName(0);

					if(AnimSequence->GetDataModel()->IsValidBoneTrackName(RootBoneName))
					{
						TArray<FVector3f> PosKeys;
						TArray<FQuat4f> RotKeys;
						TArray<FVector3f> ScaleKeys;

						TArray<FTransform> BoneTransforms;
						AnimSequence->GetDataModel()->GetBoneTrackTransforms(RootBoneName, BoneTransforms);

						PosKeys.SetNum(BoneTransforms.Num());
						RotKeys.SetNum(BoneTransforms.Num());
						ScaleKeys.SetNum(BoneTransforms.Num());

						// Find vector that would translate current root bone location onto origin.
						FVector FrameTransform = FVector::ZeroVector;
						if (FrameIndex == INDEX_NONE)
						{
							// Use current transform
							FrameTransform = PreviewSkelComp->GetComponentSpaceTransforms()[0].GetLocation();
						}
						else if(BoneTransforms.IsValidIndex(FrameIndex))
						{
							// Use transform at frame
							FrameTransform = BoneTransforms[FrameIndex].GetLocation();
						}

						FVector ApplyTranslation = -1.f * FrameTransform;

						// Convert into world space
						const FVector WorldApplyTranslation = PreviewSkelComp->GetComponentTransform().TransformVector(ApplyTranslation);
						ApplyTranslation = PreviewSkelComp->GetComponentTransform().InverseTransformVector(WorldApplyTranslation);

						for(int32 KeyIndex = 0; KeyIndex < BoneTransforms.Num(); KeyIndex++)
						{
							PosKeys[KeyIndex] = FVector3f(BoneTransforms[KeyIndex].GetLocation() + ApplyTranslation);
							RotKeys[KeyIndex] = FQuat4f(BoneTransforms[KeyIndex].GetRotation());
							ScaleKeys[KeyIndex] = FVector3f(BoneTransforms[KeyIndex].GetScale3D());
						}

						IAnimationDataController& Controller = AnimSequence->GetController();
						Controller.SetBoneTrackKeys(RootBoneName, PosKeys, RotKeys, ScaleKeys);
					}
				}
			}
		}
	}
}

bool SAnimationScrubPanel::GetDisplayDrag() const
{
	if(TimelineDelegates.GetPlaybackTimeDelegate.IsBound())
	{
		if(TimelineDelegates.GetPlaybackTimeDelegate.Execute().IsSet())
		{
			return true;
		}
	}
	
	UAnimSingleNodeInstance* PreviewInstance = GetPreviewInstance();
	if (PreviewInstance && PreviewInstance->GetCurrentAsset())
	{
		return true;
	}

	return false;
}

void SAnimationScrubPanel::SetPlaybackMode(EPlaybackMode::Type InMode)
{
	if(TimelineDelegates.SetPlaybackModeDelegate.IsBound())
	{
		if(TimelineDelegates.SetPlaybackModeDelegate.Execute(InMode))
		{
			return;
		}
	}
	
	if (UAnimSingleNodeInstance* PreviewInstance = GetPreviewInstance())
	{
		switch(InMode)
		{
			case EPlaybackMode::Stopped:
			{
				PreviewInstance->SetPlaying(false);
				break;
			}
			case EPlaybackMode::PlayingForward:
			{
				PreviewInstance->SetReverse(false);
				PreviewInstance->SetPlaying(true);
				break;
			}
			case EPlaybackMode::PlayingReverse:
			{
				PreviewInstance->SetReverse(true);
				PreviewInstance->SetPlaying(true);
				break;
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
