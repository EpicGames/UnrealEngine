// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "IPersonaPreviewScene.h"
#include "Input/Reply.h"
#include "Widgets/SCompoundWidget.h"
#include "SScrubWidget.h"
#include "ITransportControl.h"
#include "PersonaModule.h"

class SScrubControlPanel;
class UAnimationAsset;
class UAnimInstance;
class UAnimSequenceBase;
struct FAnimBlueprintDebugData;

class SAnimationScrubPanel : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SAnimationScrubPanel)
		: _LockedSequence()
	{}
		/** If you'd like to lock to one asset for this scrub control, give this**/
		SLATE_ARGUMENT(UAnimSequenceBase*, LockedSequence)
		/** View Input range **/
		SLATE_ATTRIBUTE( float, ViewInputMin )
		SLATE_ATTRIBUTE( float, ViewInputMax )
		SLATE_ARGUMENT( bool, bDisplayAnimScrubBarEditing)
		SLATE_EVENT( FOnSetInputViewRange, OnSetInputViewRange )
		/** Called when an anim sequence is cropped before/after a selected frame */
		SLATE_EVENT( FOnCropAnimSequence, OnCropAnimSequence )
		/** Called to zero out selected frame's translation from origin */
		SLATE_EVENT( FSimpleDelegate, OnReZeroAnimSequence )
		SLATE_ARGUMENT( bool, bAllowZoom )
		SLATE_ARGUMENT( FAnimationScrubPanelDelegates, TimelineDelegates )
	SLATE_END_ARGS()

	/**
	 * Construct the widget
	 * 
	 * @param InArgs   A declaration from which to construct the widget
	 */
	void Construct( const FArguments& InArgs, const TSharedRef<IPersonaPreviewScene>& InPreviewScene );

	// SWidget interface
	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;
	// End of SWidget interface

	void ReplaceLockedSequence(class UAnimSequenceBase * NewLockedSequence);
protected:
	TWeakPtr<IPersonaPreviewScene> PreviewScenePtr;
	FOnSetInputViewRange OnSetInputViewRange;

	bool bSliderBeingDragged;

	// notifiers 
	virtual FReply OnClick_Forward_Step();
	virtual FReply OnClick_Forward_End();
	virtual FReply OnClick_Backward_Step();
	virtual FReply OnClick_Backward_End();

	virtual FReply OnClick_Forward();
	virtual FReply OnClick_Backward();

	virtual FReply OnClick_ToggleLoop();

	virtual FReply OnClick_Record();

	virtual EVisibility OnGetRecordVisibility() const;

	void AnimChanged(UAnimationAsset * AnimAsset);

	virtual void OnValueChanged(float NewValue, bool bFireNotifies);

	/** Function to crop animation sequence before/after selected frame */
	void OnCropAnimSequence( bool bFromStart, float CurrentTime );
	void OnAppendAnimSequence( bool bFromStart, int32 NumOfFrames );
	void OnInsertAnimSequence( bool bBefore, int32 CurrentFrame );

	/** 
	 * Sets the root bone to be at the origin at the specified frame.
	 * If FrameIndex is INDEX_NONE then the current frame is used.
	 */
	void OnReZeroAnimSequence(int32 FrameIndex);

	// make sure viewport is freshes
	void OnBeginSliderMovement();
	void OnEndSliderMovement(float NewValue);

	EPlaybackMode::Type GetPlaybackMode() const;
	bool IsRecording() const;
	bool IsLoopStatusOn() const;
	bool IsRealtimeStreamingMode() const;
	float GetViewInputMin() const;
	float GetViewInputMax() const;

	float GetScrubValue() const;
	class UAnimSingleNodeInstance* GetPreviewInstance() const;

	TSharedPtr<class SScrubControlPanel> ScrubControlPanel;
	class UAnimSequenceBase* LockedSequence;
	FAnimationScrubPanelDelegates TimelineDelegates;
	TAttribute<float> ViewInputMinAttribute;
	TAttribute<float> ViewInputMaxAttribute;

	/** Do I need to sync with viewport? **/
	bool DoesSyncViewport() const;
	uint32 GetNumberOfKeys() const;
	float GetSequenceLength() const;

	// Returns a UAnimInstance that came from a blueprint, or NULL (even if the UAnimInstance is not null, but it didn't come from a blueprint)
	UAnimInstance* GetAnimInstanceWithBlueprint() const;

	// Returns the debug data if the current preview is of an anim blueprint that is the selected debug object, or NULL
	bool GetAnimBlueprintDebugData(UAnimInstance*& Instance, FAnimBlueprintDebugData*& DebugInfo) const;

	TSharedRef<IPersonaPreviewScene> GetPreviewScene() const { return PreviewScenePtr.Pin().ToSharedRef(); }

	bool GetDisplayDrag() const;

	void SetPlaybackMode(EPlaybackMode::Type InMode);
};
