// Copyright Epic Games, Inc. All Rights Reserved.


#include "SAnimationSequenceBrowser.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandList.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SMenuAnchor.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Animation/AnimationAsset.h"
#include "Animation/AnimSequenceBase.h"
#include "Animation/AnimSequence.h"

#include "Styling/AppStyle.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "IPersonaPreviewScene.h"
#include "PersonaModule.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SViewport.h"
#include "EditorReimportHandler.h"
#include "FileHelpers.h"

#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "SSkeletonWidget.h"
#include "Toolkits/GlobalEditorCommonCommands.h"
#include "FrontendFilterBase.h"
#include "Slate/SceneViewport.h"
#include "AnimPreviewInstance.h"
#include "ObjectEditorUtils.h"
#include "IPersonaToolkit.h"
#include "IAnimationEditorModule.h"
#include "Sound/SoundWave.h"
#include "Components/AudioComponent.h"
#include "Misc/ConfigCacheIni.h"
#include "Framework/Application/SlateApplication.h"
#include "Preferences/PersonaOptions.h"
#include "Settings/SkeletalMeshEditorSettings.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "SSkeletonAnimNotifies.h"
#include "SAnimCurveViewer.h"
#include "IEditableSkeleton.h"
#include "SAnimCurvePicker.h"
#include "ToolMenus.h"
#include "AnimationSequenceBrowserMenuContexts.h"
#include "SAnimAttributePicker.h"
#include "Viewports.h"
#include "SceneInterface.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "SEnumCombo.h"

#define LOCTEXT_NAMESPACE "SequenceBrowser"

const FString SAnimationSequenceBrowser::SettingsIniSection = TEXT("SequenceBrowser");

/** A filter that displays animations that are additive */
class FFrontendFilter_AdditiveAnimAssets : public FFrontendFilter
{
public:
	FFrontendFilter_AdditiveAnimAssets(TSharedPtr<FFrontendFilterCategory> InCategory) : FFrontendFilter(InCategory) {}

	// FFrontendFilter implementation
	virtual FString GetName() const override { return TEXT("AdditiveAnimAssets"); }
	virtual FText GetDisplayName() const override
	{
		return FText::Format(LOCTEXT("FFrontendFilter_AdditiveAnimAssets", "Additive Type ({0})"), ModeDisplayText);
	}
	virtual FText GetToolTipText() const override { return LOCTEXT("FFrontendFilter_AdditiveAnimAssetsToolTip", "Show only animations that match a given additive type."); }

	virtual void ModifyContextMenu(FMenuBuilder& MenuBuilder) override
	{
		MenuBuilder.BeginSection(TEXT("FilterMode"), LOCTEXT("FilterMode", "Filter Mode"));
		{
			TSharedRef<SWidget> ComboBox = SNew(SEnumComboBox, StaticEnum<EAdditiveTypeFilterMode>())
				.CurrentValue_Lambda([this]()
				{
					return static_cast<int32>(Mode);
				})
				.OnEnumSelectionChanged_Lambda([this](uint32 NewValue, ESelectInfo::Type)
				{
					Mode = static_cast<EAdditiveTypeFilterMode>(NewValue);
					ModeDisplayText = StaticEnum<EAdditiveTypeFilterMode>()->GetDisplayNameTextByValue(NewValue);
					
					switch (Mode)
					{
					case EAdditiveTypeFilterMode::NonAdditive:
						ModeTagString = StaticEnum<EAdditiveAnimationType>()->GetNameStringByValue(AAT_None);
						bInvertComparison = false;
						break;
					case EAdditiveTypeFilterMode::AnyAdditive:
						ModeTagString = StaticEnum<EAdditiveAnimationType>()->GetNameStringByValue(AAT_None);
						bInvertComparison = true;
						break;
					case EAdditiveTypeFilterMode::LocalSpaceBase:
						ModeTagString = StaticEnum<EAdditiveAnimationType>()->GetNameStringByValue(AAT_LocalSpaceBase);
						bInvertComparison = false;
						break;
					case EAdditiveTypeFilterMode::RotationOffsetMeshSpace:
						ModeTagString = StaticEnum<EAdditiveAnimationType>()->GetNameStringByValue(AAT_RotationOffsetMeshSpace);
						bInvertComparison = false;
					}
					
					OnChanged().Broadcast();
				});
				
			MenuBuilder.AddWidget(ComboBox, LOCTEXT("Mode", "Mode"));
		}
		MenuBuilder.EndSection();
	}

	// IFilter implementation
	virtual bool PassesFilter(FAssetFilterType InItem) const override
	{
		FAssetData ItemAssetData;
		if (InItem.Legacy_TryGetAssetData(ItemAssetData))
		{
			const FString TagValue = ItemAssetData.GetTagValueRef<FString>(GET_MEMBER_NAME_CHECKED(UAnimSequence, AdditiveAnimType));
			return !TagValue.IsEmpty() && (bInvertComparison ^ TagValue.Equals(ModeTagString));
		}
		return false;
	}
	
	EAdditiveTypeFilterMode Mode = EAdditiveTypeFilterMode::AnyAdditive;
	bool bInvertComparison = true;
	FString ModeTagString = StaticEnum<EAdditiveAnimationType>()->GetNameStringByValue(AAT_None);
	FText ModeDisplayText = StaticEnum<EAdditiveTypeFilterMode>()->GetDisplayNameTextByValue(static_cast<int32>(EAdditiveTypeFilterMode::AnyAdditive));
};

/** A filter that displays animations that use a skeleton notify */
class FFrontendFilter_SkeletonNotify : public FFrontendFilter
{
public:
	FFrontendFilter_SkeletonNotify(TSharedPtr<FFrontendFilterCategory> InCategory, TSharedPtr<IEditableSkeleton> InEditableSkeleton)
		: FFrontendFilter(InCategory) 
		, EditableSkeleton(InEditableSkeleton)
	{}

	// FFrontendFilter implementation
	virtual FString GetName() const override 
	{
		return TEXT("SkeletonNotifyAssets"); 
	}

	virtual FText GetDisplayName() const override 
	{ 
		return NotifyString.IsEmpty() ? LOCTEXT("FFrontendFilter_SkeletonNotifyAssetsEmpty", "Uses Skeleton Notify...") : FText::Format(LOCTEXT("FFrontendFilter_SkeletonNotifyAssets", "Uses Skeleton Notify: {0}"), FText::FromString(NotifyString));
	}

	virtual FText GetToolTipText() const override
	{
		return NotifyString.IsEmpty() ? LOCTEXT("FFrontendFilter_SkeletonNotifyAssetsEmptyTooltip", "Show assets that contains a (specified) skeleton notify") : FText::Format(LOCTEXT("FFrontendFilter_SkeletonNotifyAssetsTooltip", "Show assets that use skeleton notify '{0}'"), FText::FromString(NotifyString));
	}

	virtual void ModifyContextMenu(FMenuBuilder& MenuBuilder) override
	{
		MenuBuilder.BeginSection(TEXT("NotifySection"), LOCTEXT("NotifySectionHeading", "Choose Notify"));

		MenuBuilder.AddMenuEntry(LOCTEXT("AllNotifyFilterName", "Any Animation Notify"),
			LOCTEXT("AllNotifyFilterName_ToolTip", "Consider all Animation Notifies, rather than a specific one, for filtering."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this]()
				{
					if (!NotifyString.IsEmpty())
					{
						NotifyString.Empty();
					}

					OnChanged().Broadcast();
				}), 
				FCanExecuteAction::CreateLambda([this]()
				{
					return !NotifyString.IsEmpty();
				}),
				FGetActionCheckState::CreateLambda([this]()
				{
					return NotifyString.IsEmpty() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				})
			), 
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
		
		TSharedRef<SWidget> Widget =
			SNew(SBox)
			.HeightOverride(300.0f)
			.WidthOverride(200.0f)
			[
				SNew(SSkeletonAnimNotifies)
				.IsPicker(true)
				.ShowNotifies(true)
				.ShowSyncMarkers(false)
				.ShowCompatibleSkeletonAssets(true)
				.ShowOtherAssets(true)
				.EditableSkeleton(EditableSkeleton.Pin())
				.OnItemSelected_Lambda([this](const FName& InNotifyName)
				{
					FSlateApplication::Get().DismissAllMenus();
					NotifyString = InNotifyName.ToString();
					OnChanged().Broadcast();
				})
			];

		MenuBuilder.AddWidget(Widget, FText(), true);

		MenuBuilder.EndSection();
	}

	virtual FLinearColor GetColor() const override
	{
		return FLinearColor(0.0f, 0.4f, 0.6f, 1);
	}

	// IFilter implementation
	virtual bool PassesFilter(FAssetFilterType InItem) const override
	{
		FAssetData ItemAssetData;
		if (InItem.Legacy_TryGetAssetData(ItemAssetData))
		{
			const FString TagValue = ItemAssetData.GetTagValueRef<FString>(USkeleton::AnimNotifyTag);
			if (!TagValue.IsEmpty())
			{
				if (!NotifyString.IsEmpty())
				{
					// parse notifies
					TArray<FString> NotifyValues;
					if (TagValue.ParseIntoArray(NotifyValues, *USkeleton::AnimNotifyTagDelimiter, true) > 0)
					{
						for (const FString& NotifyValue : NotifyValues)
						{
							if (NotifyValue == NotifyString)
							{
								return true;
							}
						}
					}
				}

				return NotifyString.IsEmpty();
			}
		}

		return false;
	}

	void SetNotifyFilter(const FName& InNotifyFilter)
	{
		NotifyString = InNotifyFilter.ToString();
		OnChanged().Broadcast();
		SetActive(true);
	}

public:
	/** Notify string to use when filtering */
	FString NotifyString;

	/** Editable skeleton used to filter available notifies */
	TWeakPtr<IEditableSkeleton> EditableSkeleton;
};

/** A filter that displays animations that use a skeleton sync marker */
class FFrontendFilter_SkeletonSyncMarker : public FFrontendFilter
{
public:
	FFrontendFilter_SkeletonSyncMarker(TSharedPtr<FFrontendFilterCategory> InCategory, TSharedPtr<IEditableSkeleton> InEditableSkeleton)
		: FFrontendFilter(InCategory) 
		, EditableSkeleton(InEditableSkeleton)
	{}

	// FFrontendFilter implementation
	virtual FString GetName() const override 
	{
		return TEXT("SkeletonSyncMarkerAssets"); 
	}

	virtual FText GetDisplayName() const override 
	{ 
		return SyncMarkerString.IsEmpty() ? LOCTEXT("FFrontendFilter_SkeletonSyncMarkerAssetsEmpty", "Uses Sync Marker...") : FText::Format(LOCTEXT("FFrontendFilter_SkeletonSyncMarkerAssets", "Uses Sync Marker: {0}"), FText::FromString(SyncMarkerString));
	}

	virtual FText GetToolTipText() const override
	{
		return SyncMarkerString.IsEmpty() ? LOCTEXT("FFrontendFilter_SkeletonSyncMarkerAssetsEmptyTooltip", "Show assets that contains a (specified) sync marker") : FText::Format(LOCTEXT("FFrontendFilter_SkeletonSyncMarkerAssetsTooltip", "Show assets that use sync marker '{0}'"), FText::FromString(SyncMarkerString));
	}

	virtual void ModifyContextMenu(FMenuBuilder& MenuBuilder) override
	{
		MenuBuilder.BeginSection(TEXT("SyncMarkerSection"), LOCTEXT("SyncMarkerSectionHeading", "Choose Sync Marker"));

		MenuBuilder.AddMenuEntry(LOCTEXT("AllSyncMarkerFilterName", "Any Animation Sync Marker"),
			LOCTEXT("AllSyncMarkerFilterName_ToolTip", "Consider all Sync Markers, rather than a specific one, for filtering."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this]()
				{
					if (!SyncMarkerString.IsEmpty())
					{
						SyncMarkerString.Empty();
					}

					OnChanged().Broadcast();
				}), 
				FCanExecuteAction::CreateLambda([this]()
				{
					return !SyncMarkerString.IsEmpty();
				}),
				FGetActionCheckState::CreateLambda([this]()
				{
					return SyncMarkerString.IsEmpty() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				})
			), 
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
		
		TSharedRef<SWidget> Widget =
			SNew(SBox)
			.HeightOverride(300.0f)
			.WidthOverride(200.0f)
			[
				SNew(SSkeletonAnimNotifies)
				.IsPicker(true)
				.ShowNotifies(false)
				.ShowSyncMarkers(true)
				.ShowCompatibleSkeletonAssets(true)
				.ShowOtherAssets(true)
				.EditableSkeleton(EditableSkeleton.Pin())
				.OnItemSelected_Lambda([this](const FName& InNotifyName)
				{
					FSlateApplication::Get().DismissAllMenus();
					SyncMarkerString = InNotifyName.ToString();
					OnChanged().Broadcast();
				})
			];

		MenuBuilder.AddWidget(Widget, FText(), true);

		MenuBuilder.EndSection();
	}

	virtual FLinearColor GetColor() const override
	{
		return FLinearColor(0.62f, 0.43f, 0.0f, 1);
	}

	// IFilter implementation
	virtual bool PassesFilter(FAssetFilterType InItem) const override
	{
		FAssetData ItemAssetData;
		if (InItem.Legacy_TryGetAssetData(ItemAssetData))
		{
			const FString TagValue = ItemAssetData.GetTagValueRef<FString>(USkeleton::AnimSyncMarkerTag);
			if (!TagValue.IsEmpty())
			{
				if (!SyncMarkerString.IsEmpty())
				{
					// Parse sync markers
					TArray<FString> SyncMarkerValues;
					if (TagValue.ParseIntoArray(SyncMarkerValues, *USkeleton::AnimSyncMarkerTagDelimiter, true) > 0)
					{
						for (const FString& SyncMarkerValue : SyncMarkerValues)
						{
							if (SyncMarkerValue == SyncMarkerString)
							{
								return true;
							}
						}
					}
				}

				return SyncMarkerString.IsEmpty();
			}
		}

		return false;
	}

	void SetSyncMarkerFilter(const FName& InNotifyFilter)
	{
		SyncMarkerString = InNotifyFilter.ToString();
		OnChanged().Broadcast();
		SetActive(true);
	}

public:
	/** Sync marker string to use when filtering */
	FString SyncMarkerString;

	/** Editable skeleton used to filter available sync markers */
	TWeakPtr<IEditableSkeleton> EditableSkeleton;
};

/** A filter that displays animations that use an attribute */
class FFrontendFilter_Attributes : public FFrontendFilter
{
public:
	FFrontendFilter_Attributes(TSharedPtr<FFrontendFilterCategory> InCategory, TSharedPtr<IEditableSkeleton> InEditableSkeleton)
		: FFrontendFilter(InCategory)
		, EditableSkeleton(InEditableSkeleton)
	{}

	// FFrontendFilter implementation
	virtual FString GetName() const override
	{
		return TEXT("AttributeAssets");
	}

	virtual FText GetDisplayName() const override
	{
		return AttributeName.IsEmpty()
			? LOCTEXT("FFrontendFilter_AttributeAssetsEmpty", "Uses Attributes")
			: FText::Format(LOCTEXT("FFrontendFilter_AttributeAssets", "Uses Attribute: {0}"), FText::FromString(AttributeName));
	}

	virtual FText GetToolTipText() const override
	{
		return AttributeName.IsEmpty()
			? LOCTEXT("FFrontendFilter_AttributeAssetsEmptyTooltip", "Show assets that contains a (specified) attribute")
			: FText::Format(LOCTEXT("FFrontendFilter_AttributeAssetsTooltip", "Show assets that use attribute '{0}'"), FText::FromString(AttributeName));
	}

	virtual void ModifyContextMenu(FMenuBuilder& MenuBuilder) override
	{
		MenuBuilder.BeginSection(TEXT("AttributeSection"), LOCTEXT("AttributeSectionHeading", "Choose Attribute"));
		
		MenuBuilder.AddMenuEntry(LOCTEXT("AllAttributeFilterName", "Any attribute"),
			LOCTEXT("AllAttributeFilterName_ToolTip", "Consider all animation attributes for filtering."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this]()
				{
					AttributeName.Empty();
					OnChanged().Broadcast();
				}), 
				FCanExecuteAction::CreateLambda([this]()
				{
					return !AttributeName.IsEmpty();
				}),
				FGetActionCheckState::CreateLambda([this]()
				{
					return AttributeName.IsEmpty() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				})
			), 
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
		
		TSharedRef<SWidget> Widget = 
			SNew(SBox)
			.HeightOverride(300.0f)
			.WidthOverride(200.0f)
			[
				SNew(SAnimAttributePicker, &EditableSkeleton->GetSkeleton())
				.bEnableMultiSelect(false)
				.OnAttributesPicked_Lambda([this](const TConstArrayView<FAnimationAttributeIdentifier> InAttributes)
				{
					check(InAttributes.Num() == 1);
					FSlateApplication::Get().DismissAllMenus();
					AttributeName = InAttributes[0].GetName().ToString();
					OnChanged().Broadcast();
				})
			];

		MenuBuilder.AddWidget(Widget, FText(), true);

		MenuBuilder.EndSection();
	}

	virtual FLinearColor GetColor() const override
	{
		return FLinearColor(0.6f, 0.0f, 0.2f, 1.0f);
	}

	// IFilter implementation
	virtual bool PassesFilter(FAssetFilterType InItem) const override
	{
		FAssetData ItemAssetData;
		if (InItem.Legacy_TryGetAssetData(ItemAssetData))
		{
			const FString TagValue = ItemAssetData.GetTagValueRef<FString>(USkeleton::AttributeTag);
			if (!TagValue.IsEmpty())
			{
				
				FAnimationAttributeIdentifierArray AttributesIdentifierArray;
				FAnimationAttributeIdentifierArray::StaticStruct()->ImportText(*TagValue, &AttributesIdentifierArray, nullptr, PPF_None, GLog, FAnimationAttributeIdentifierArray::StaticStruct()->GetName());
				
				if (AttributeName.IsEmpty() && !AttributesIdentifierArray.AttributeIdentifiers.IsEmpty())
				{
					return true;
				}
				
				for (const FAnimationAttributeIdentifier& AttrId : AttributesIdentifierArray.AttributeIdentifiers)
				{
					if (AttrId.GetName().ToString() == AttributeName)
					{
						return true;
					}
				}
			}
		}

		return false;
	}

public:
	/** Attribute name string to use when filtering */
	FString AttributeName;

	/** Editable skeleton used to access available attributes */
	TSharedPtr<IEditableSkeleton> EditableSkeleton;
};


/** A filter that displays animations that use a curve */
class FFrontendFilter_Curves : public FFrontendFilter
{
public:
	FFrontendFilter_Curves(TSharedPtr<FFrontendFilterCategory> InCategory, TSharedPtr<IEditableSkeleton> InEditableSkeleton)
		: FFrontendFilter(InCategory)
		, EditableSkeleton(InEditableSkeleton)
	{}

	// FFrontendFilter implementation
	virtual FString GetName() const override
	{
		return TEXT("CurveAssets");
	}

	virtual FText GetDisplayName() const override
	{
		return CurveString.IsEmpty() ? LOCTEXT("FFrontendFilter_CurveAssetsEmpty", "Uses Curve...") : FText::Format(LOCTEXT("FFrontendFilter_CurveAssets", "Uses Curve: {0}"), FText::FromString(CurveString));
	}

	virtual FText GetToolTipText() const override
	{
		return CurveString.IsEmpty() ? LOCTEXT("FFrontendFilter_CurveAssetsEmptyTooltip", "Show assets that contains a (specified) curve") : FText::Format(LOCTEXT("FFrontendFilter_CurveAssetsTooltip", "Show assets that use curve '{0}'"), FText::FromString(CurveString));
	}

	virtual void ModifyContextMenu(FMenuBuilder& MenuBuilder) override
	{
		MenuBuilder.BeginSection(TEXT("CurveSection"), LOCTEXT("CurveSectionHeading", "Choose Curve"));
		
		MenuBuilder.AddMenuEntry(LOCTEXT("AllCurveFilterName", "Any Animation Curve"),
			LOCTEXT("AllCurveFilterName_ToolTip", "Consider all Animation Curves, rather than a specific one, for filtering."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this]()
				{
					if (!CurveString.IsEmpty())
					{
						CurveString.Empty();
					}

					OnChanged().Broadcast();
				}), 
				FCanExecuteAction::CreateLambda([this]()
				{
					return !CurveString.IsEmpty();
				}),
				FGetActionCheckState::CreateLambda([this]()
				{
					return CurveString.IsEmpty() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				})
			), 
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
		
		TSharedRef<SWidget> Widget = 
			SNew(SBox)
			.HeightOverride(300.0f)
			.WidthOverride(200.0f)
			[
				SNew(SAnimCurvePicker, &EditableSkeleton->GetSkeleton())
				.OnCurvesPicked_Lambda([this](const TConstArrayView<FName> InCurveNames)
				{
					check(InCurveNames.Num() == 1);
					FSlateApplication::Get().DismissAllMenus();
					CurveString = InCurveNames[0].ToString();
					OnChanged().Broadcast();
				})
			];

		MenuBuilder.AddWidget(Widget, FText(), true);

		MenuBuilder.EndSection();
	}

	virtual FLinearColor GetColor() const override
	{
		return FLinearColor(0.0f, 0.6f, 0.2f, 1);
	}

	// IFilter implementation
	virtual bool PassesFilter(FAssetFilterType InItem) const override
	{
		FAssetData ItemAssetData;
		if (InItem.Legacy_TryGetAssetData(ItemAssetData))
		{
			const FString TagValue = ItemAssetData.GetTagValueRef<FString>(USkeleton::CurveNameTag);
			if (!TagValue.IsEmpty())
			{
				if (!CurveString.IsEmpty())
				{
					// parse curves
					TArray<FString> CurveValues;
					if (TagValue.ParseIntoArray(CurveValues, *USkeleton::CurveTagDelimiter, true) > 0)
					{
						for (const FString& CurveValue : CurveValues)
						{
							if (CurveValue == CurveString)
							{
								return true;
							}
						}
					}
				}

				return CurveString.IsEmpty();
			}
		}

		return false;
	}

	void SetCurveFilter(const FName& InCurveFilter)
	{
		CurveString = InCurveFilter.ToString();
		OnChanged().Broadcast();
		SetActive(true);
	}

public:
	/** Curve string to use when filtering */
	FString CurveString;

	/** Editable skeleton used to access available curves */
	TSharedPtr<IEditableSkeleton> EditableSkeleton;
};

/** A filter that displays sound waves */
class FFrontendFilter_SoundWaves : public FFrontendFilter
{
public:
	FFrontendFilter_SoundWaves(TSharedPtr<FFrontendFilterCategory> InCategory) : FFrontendFilter(InCategory) {}

	// FFrontendFilter implementation
	virtual FString GetName() const override { return TEXT("ShowSoundWaves"); }
	virtual FText GetDisplayName() const override { return LOCTEXT("FFrontendFilter_SoundWaves", "Show Sound Waves"); }
	virtual FText GetToolTipText() const override { return LOCTEXT("FFrontendFilter_SoundWavesToolTip", "Show sound waves."); }
	virtual bool IsInverseFilter() const override { return true; }

	// IFilter implementation
	virtual bool PassesFilter(FAssetFilterType InItem) const override
	{
		FAssetData ItemAssetData;
		if (InItem.Legacy_TryGetAssetData(ItemAssetData))
		{
			return !ItemAssetData.IsInstanceOf(USoundWave::StaticClass());
		}
		return false;
	}
};

/** A filter that shows specific folders */
class FFrontendFilter_Folder : public FFrontendFilter
{
public:
	FFrontendFilter_Folder(TSharedPtr<FFrontendFilterCategory> InCategory, int32 InFolderIndex, FSimpleDelegate InOnActiveStateChanged) 
		: FFrontendFilter(InCategory)
		, FolderIndex(InFolderIndex)
		, OnActiveStateChanged(InOnActiveStateChanged)
	{}

	// FFrontendFilter implementation
	virtual FString GetName() const override
	{ 
		return FString::Printf(TEXT("ShowFolder%d"), FolderIndex); 
	}

	virtual FText GetDisplayName() const override
	{
		return Folder.IsEmpty() ? FText::Format(LOCTEXT("FolderFormatInvalid", "Show Specified Folder {0}"), FText::AsNumber(FolderIndex + 1)) : FText::Format(LOCTEXT("FolderFormatValid", "Folder: {0}"), FText::FromString(Folder));
	}

	virtual FText GetToolTipText() const override
	{ 
		return Folder.IsEmpty() ? LOCTEXT("FFrontendFilter_FolderToolTip", "Show assets in a specified folder") : FText::Format(LOCTEXT("FolderFormatValidToolTip", "Show assets in folder: {0}"), FText::FromString(Folder)); 
	}

	virtual FLinearColor GetColor() const override
	{ 
		return FLinearColor(0.6f, 0.6f, 0.0f, 1);
	}

	virtual void ModifyContextMenu(FMenuBuilder& MenuBuilder) override
	{
		MenuBuilder.BeginSection(TEXT("FolderSection"), LOCTEXT("FolderSectionHeading", "Choose Folder"));

		FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
		FPathPickerConfig PathPickerConfig;
		PathPickerConfig.DefaultPath = Folder;
		PathPickerConfig.bAllowContextMenu = false;
		PathPickerConfig.OnPathSelected = FOnPathSelected::CreateLambda([this](const FString& InPath) 
		{ 
			Folder = InPath; 
			FSlateApplication::Get().DismissAllMenus(); 
			OnActiveStateChanged.ExecuteIfBound();
		});
		
		TSharedRef<SWidget> FolderWidget = 
			SNew(SBox)
			.HeightOverride(300.0f)
			.WidthOverride(200.0f)
			[
				ContentBrowserModule.Get().CreatePathPicker(PathPickerConfig)
			];

		MenuBuilder.AddWidget(FolderWidget, FText(), true);

		MenuBuilder.EndSection();
	}

	virtual void SaveSettings(const FString& IniFilename, const FString& IniSection, const FString& SettingsString) const override
	{
		GConfig->SetString(*IniSection, *(SettingsString + TEXT(".Folder")), *Folder, IniFilename);
	}

	virtual void LoadSettings(const FString& IniFilename, const FString& IniSection, const FString& SettingsString) override
	{
		GConfig->GetString(*IniSection, *(SettingsString + TEXT(".Folder")), Folder, IniFilename);
	}

	virtual bool PassesFilter(FAssetFilterType InItem) const override
	{
		// Always pass this as a frontend filter, it acts as a backend filter
		return true;
	}

	virtual void ActiveStateChanged(bool bEnable)
	{
		bEnabled = bEnable;
		OnActiveStateChanged.ExecuteIfBound();
	}

public:
	/** Folder string to use when filtering */
	FString Folder;

	/** The index of this filter, for uniquely identifying this filter */
	int32 FolderIndex;

	/** Delegate fired to refresh the filter */
	FSimpleDelegate OnActiveStateChanged;

	/** Whether this filter is currently enabled */
	bool bEnabled;
};

////////////////////////////////////////////////////

const int32 SAnimationSequenceBrowser::MaxAssetsHistory = 10;

SAnimationSequenceBrowser::~SAnimationSequenceBrowser()
{
	if(PreviewComponent)
	{
		for(int32 ComponentIdx = PreviewComponent->GetAttachChildren().Num() - 1 ; ComponentIdx >= 0 ; --ComponentIdx)
		{
			USceneComponent* Component = PreviewComponent->GetAttachChildren()[ComponentIdx];
			if(Component)
			{
				CleanupPreviewSceneComponent(Component);
			}
		}
		check(PreviewComponent->GetAttachChildren().Num() == 0);
	}
}

void SAnimationSequenceBrowser::OnRequestOpenAssets(const TArray<FAssetData>& SelectedAssets, bool bFromHistory)
{
	if (SelectedAssets.Num() == 1)
	{
		OnRequestOpenAsset(SelectedAssets[0], bFromHistory);
	}
}

void SAnimationSequenceBrowser::OnRequestOpenAsset(const FAssetData& AssetData, bool bFromHistory)
{
	if (UObject* RawAsset = AssetData.GetAsset())
	{
		if (UAnimationAsset* AnimationAsset = Cast<UAnimationAsset>(RawAsset))
		{
			if (FSlateApplication::Get().GetModifierKeys().IsShiftDown())
			{
				IAnimationEditorModule& AnimationEditorModule = FModuleManager::LoadModuleChecked<IAnimationEditorModule>("AnimationEditor");
				AnimationEditorModule.CreateAnimationEditor(EToolkitMode::Standalone, nullptr, AnimationAsset);
			}
			else
			{
				if (!bFromHistory)
				{
					AddAssetToHistory(AssetData);
				}
				OnOpenNewAsset.ExecuteIfBound(AnimationAsset);
			}
		}
		else if (USoundWave* SoundWave = Cast<USoundWave>(RawAsset))
		{
			PlayPreviewAudio(SoundWave);
		}
	}
}

TSharedPtr<SWidget> SAnimationSequenceBrowser::OnGetAssetContextMenu(const TArray<FAssetData>& SelectedAssets)
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	const FName SequenceBrowserContextMenuName = "AnimationSequenceBrowser.SequenceContextMenu";
	if (!ToolMenus->IsMenuRegistered(SequenceBrowserContextMenuName))
	{
		ToolMenus->RegisterMenu(SequenceBrowserContextMenuName);
	}
	UToolMenu* Menu = ToolMenus->FindMenu(SequenceBrowserContextMenuName);

	Menu->AddDynamicSection("SequenceContextMenuDynamic", FNewToolMenuDelegate::CreateLambda([this, SelectedAssets](UToolMenu* InMenu)
	{
		bool bHasSelectedAnimSequence = false;
		bool bHasSelectedAnimAsset = false;
		if (SelectedAssets.Num())
		{
			for (auto Iter = SelectedAssets.CreateConstIterator(); Iter; ++Iter)
			{
				UObject* Asset = Iter->GetAsset();
				if (Cast<UAnimSequence>(Asset))
				{
					bHasSelectedAnimSequence = true;
				}
				if (Cast<UAnimationAsset>(Asset))
				{
					bHasSelectedAnimAsset = true;
				}
			}
		}

		if (bHasSelectedAnimSequence)
		{
			FToolMenuSection& Section = InMenu->AddSection("SequenceOptions", LOCTEXT("AssetHeading", "Asset"));

			Section.AddMenuEntry("ApplyCompression",
				LOCTEXT("ApplyCompression", "Apply Compression"),
				LOCTEXT("ApplyCompression_ToolTip", "Apply a compression scheme from the options given to the selected animations"),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP(this, &SAnimationSequenceBrowser::OnApplyCompression, SelectedAssets),
					FCanExecuteAction()
				)
			);

			Section.AddMenuEntry("ExportAnimationsToFBX",
				LOCTEXT("ExportAnimationsToFBX", "Export to FBX"),
				LOCTEXT("ExportAnimationsToFBX_ToolTip", "Export Animation(s) To FBX"),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP(this, &SAnimationSequenceBrowser::OnExportToFBX, SelectedAssets),
					FCanExecuteAction()
				)
			);

			Section.AddMenuEntry("AddLoopingInterpolation",
				LOCTEXT("AddLoopingInterpolation", "Add Looping Interpolation"),
				LOCTEXT("AddLoopingInterpolation_ToolTip", "Add an extra frame at the end of the animation to create better looping"),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP(this, &SAnimationSequenceBrowser::OnAddLoopingInterpolation, SelectedAssets),
					FCanExecuteAction()
				)
			);

			Section.AddMenuEntry("ReimportAnimation",
				LOCTEXT("ReimportAnimation", "Reimport Animation"),
				LOCTEXT("ReimportAnimation_ToolTip", "Reimport current animation."),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP(this, &SAnimationSequenceBrowser::OnReimportAnimation, SelectedAssets),
					FCanExecuteAction()
				)
			);
		}

		if (SelectedAssets.Num() == 1 && SelectedAssets[0].IsInstanceOf(USoundWave::StaticClass()))
		{
			FToolMenuSection& Section = InMenu->AddSection("SequenceAudioOptions", LOCTEXT("AssetHeading", "Asset"));

			Section.AddMenuEntry("PlayAudio",
				LOCTEXT("PlayAudio", "Play Audio"),
				LOCTEXT("PlayAudio_ToolTip", "Play this audio asset as a preview"),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP(this, &SAnimationSequenceBrowser::HandlePlayAudio, SelectedAssets[0]),
					FCanExecuteAction()
				)
			);

			UAudioComponent* AudioComponent = PersonaToolkitPtr.Pin()->GetPreviewScene()->GetActor()->FindComponentByClass<UAudioComponent>();
			if (AudioComponent)
			{
				if (AudioComponent->IsPlaying())
				{
					Section.AddMenuEntry("StopAudio",
						LOCTEXT("StopAudio", "Stop Audio"),
						LOCTEXT("StopAudio_ToolTip", "Stop the currently playing preview audio"),
						FSlateIcon(),
						FUIAction(
							FExecuteAction::CreateSP(this, &SAnimationSequenceBrowser::HandleStopAudio),
							FCanExecuteAction()
						)
					);
				}
			}
		}

		{
			FToolMenuSection& Section = InMenu->FindOrAddSection("SequenceOptions", LOCTEXT("AssetHeading", "Asset"));

			Section.AddMenuEntry("SetCurrentPreviewMesh",
				LOCTEXT("SetCurrentPreviewMesh", "Set Current Preview Mesh"),
				LOCTEXT("SetCurrentPreviewMesh_ToolTip", "Set current preview mesh to be used when previewed by this asset. This only applies when you open Persona using this asset."),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP(this, &SAnimationSequenceBrowser::OnSetCurrentPreviewMesh, SelectedAssets),
					FCanExecuteAction()
				)
			);
		}
		
		{
			FToolMenuSection& Section = InMenu->AddSection("SequenceGeneralOptions", LOCTEXT("AnimationSequenceGeneralOptions", "Options"));

			Section.AddMenuEntry("SaveSelectedAssets",
				LOCTEXT("SaveSelectedAssets", "Save"),
				LOCTEXT("SaveSelectedAssets_ToolTip", "Save the selected assets"),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Level.SaveIcon16x"),
				FUIAction(
					FExecuteAction::CreateSP(this, &SAnimationSequenceBrowser::SaveSelectedAssets, SelectedAssets),
					FCanExecuteAction::CreateSP(this, &SAnimationSequenceBrowser::CanSaveSelectedAssets, SelectedAssets)
				));

			Section.AddMenuEntry("OpenSequenceInNewWindow",
				LOCTEXT("AnimSequenceBase_OpenInNewWindow", "Open In New Window"),
				LOCTEXT("AnimSequenceBase_OpenInNewWindowTooltip", "Will always open asset in a new window, and not re-use existing window. (Shift+Double-Click)"),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "ContentBrowser.AssetActions.OpenInExternalEditor"),
				FUIAction(
					FExecuteAction::CreateSP(this, &SAnimationSequenceBrowser::OpenInNewWindow, SelectedAssets),
					FCanExecuteAction()
				));

			Section.AddMenuEntry(FGlobalEditorCommonCommands::Get().FindInContentBrowser);
		}
	}));

	UAnimationSequenceBrowserContextMenuContext* ContextObject = NewObject<UAnimationSequenceBrowserContextMenuContext>();
	ContextObject->OwningAnimSequenceBrowser = SharedThis(this);
	ContextObject->SelectedObjects.Reset();
	for (auto Iter = SelectedAssets.CreateConstIterator(); Iter; ++Iter)
	{
		UObject* Asset = Iter->GetAsset();
		if (Asset != nullptr)
		{
			ContextObject->SelectedObjects.Add(Asset);
		}	
	}

	FToolMenuContext MenuContext(Commands, nullptr, ContextObject);
	return ToolMenus->GenerateWidget(SequenceBrowserContextMenuName, MenuContext);
}

void SAnimationSequenceBrowser::OpenInNewWindow(TArray<FAssetData> AnimationAssets)
{
	for (auto Iter = AnimationAssets.CreateConstIterator(); Iter; ++Iter)
	{
		UObject* Asset = Iter->GetAsset();
		UAnimationAsset* AnimationAsset = Cast<UAnimationAsset>(Asset);
		if (AnimationAsset)
		{
			IAnimationEditorModule& AnimationEditorModule = FModuleManager::LoadModuleChecked<IAnimationEditorModule>("AnimationEditor");
			AnimationEditorModule.CreateAnimationEditor(EToolkitMode::Standalone, nullptr, AnimationAsset);
		}
	}
}

void SAnimationSequenceBrowser::FindInContentBrowser()
{
	TArray<FAssetData> CurrentSelection = GetCurrentSelectionDelegate.Execute();
	if (CurrentSelection.Num() > 0)
	{
		FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
		ContentBrowserModule.Get().SyncBrowserToAssets(CurrentSelection);
	}
}

bool SAnimationSequenceBrowser::CanFindInContentBrowser() const
{
	TArray<FAssetData> CurrentSelection = GetCurrentSelectionDelegate.Execute();
	return CurrentSelection.Num() > 0;
}

void SAnimationSequenceBrowser::GetSelectedPackages(const TArray<FAssetData>& Assets, TArray<UPackage*>& OutPackages) const
{
	for (int32 AssetIdx = 0; AssetIdx < Assets.Num(); ++AssetIdx)
	{
		UPackage* Package = FindPackage(nullptr, *Assets[AssetIdx].PackageName.ToString());

		if ( Package )
		{
			OutPackages.Add(Package);
		}
	}
}

void SAnimationSequenceBrowser::SaveSelectedAssets(TArray<FAssetData> ObjectsToSave) const
{
	TArray<UPackage*> PackagesToSave;
	GetSelectedPackages(ObjectsToSave, PackagesToSave);

	const bool bCheckDirty = false;
	const bool bPromptToSave = false;
	const FEditorFileUtils::EPromptReturnCode Return = FEditorFileUtils::PromptForCheckoutAndSave(PackagesToSave, bCheckDirty, bPromptToSave);
}

bool SAnimationSequenceBrowser::CanSaveSelectedAssets(TArray<FAssetData> ObjectsToSave) const
{
	TArray<UPackage*> Packages;
	GetSelectedPackages(ObjectsToSave, Packages);
	// Don't offer save option if none of the packages are loaded
	return Packages.Num() > 0;
}

void SAnimationSequenceBrowser::OnApplyCompression(TArray<FAssetData> SelectedAssets)
{
	if ( SelectedAssets.Num() > 0 )
	{
		TArray<TWeakObjectPtr<UAnimSequence>> AnimSequences;
		for(const FAssetData& Asset : SelectedAssets)
		{
			if(UAnimSequence* AnimSequence = Cast<UAnimSequence>(Asset.GetAsset()) )
			{
				AnimSequences.Add( MakeWeakObjectPtr(AnimSequence) );
			}
		}

		FPersonaModule& PersonaModule = FModuleManager::GetModuleChecked<FPersonaModule>("Persona");
		PersonaModule.ApplyCompression(AnimSequences, true);
	}
}

void SAnimationSequenceBrowser::OnExportToFBX(TArray<FAssetData> SelectedAssets)
{
	if (SelectedAssets.Num() > 0)
	{
		TArray<TWeakObjectPtr<UAnimSequence>> AnimSequences;
		for(const FAssetData& Asset : SelectedAssets)
		{
			if(UAnimSequence* AnimSequence = Cast<UAnimSequence>(Asset.GetAsset()))
			{
				// we only shows anim sequence that belong to this skeleton
				AnimSequences.Add(MakeWeakObjectPtr(AnimSequence));
			}
		}

		FPersonaModule& PersonaModule = FModuleManager::GetModuleChecked<FPersonaModule>("Persona");
		PersonaModule.ExportToFBX(AnimSequences, PersonaToolkitPtr.Pin()->GetPreviewScene()->GetPreviewMeshComponent()->GetSkeletalMeshAsset());
	}
}

void SAnimationSequenceBrowser::OnSetCurrentPreviewMesh(TArray<FAssetData> SelectedAssets)
{
	if(SelectedAssets.Num() > 0)
	{
		USkeletalMesh* PreviewMesh = PersonaToolkitPtr.Pin()->GetPreviewScene()->GetPreviewMeshComponent()->GetSkeletalMeshAsset();
		if (PreviewMesh)
		{
			for(auto Iter = SelectedAssets.CreateIterator(); Iter; ++Iter)
			{
				UAnimationAsset * AnimAsset = Cast<UAnimationAsset>(Iter->GetAsset());
				if (AnimAsset)
				{
					AnimAsset->SetPreviewMesh(PreviewMesh);
				}
			}
		}
	}
}

void SAnimationSequenceBrowser::OnAddLoopingInterpolation(TArray<FAssetData> SelectedAssets)
{
	if(SelectedAssets.Num() > 0)
	{
		TArray<TWeakObjectPtr<UAnimSequence>> AnimSequences;
		for(const FAssetData& Asset : SelectedAssets)
		{
			if(UAnimSequence* AnimSequence = Cast<UAnimSequence>(Asset.GetAsset()))
			{
				// we only shows anim sequence that belong to this skeleton
				AnimSequences.Add(MakeWeakObjectPtr(AnimSequence));
			}
		}

		FPersonaModule& PersonaModule = FModuleManager::GetModuleChecked<FPersonaModule>("Persona");
		PersonaModule.AddLoopingInterpolation(AnimSequences);
	}
}

void SAnimationSequenceBrowser::OnReimportAnimation(TArray<FAssetData> SelectedAssets)
{
	if (SelectedAssets.Num() > 0)
	{
		TArray<UObject *> CopyOfSelectedAssets;
		for (auto Iter = SelectedAssets.CreateIterator(); Iter; ++Iter)
		{
			if (UAnimSequence* AnimSequence = Cast<UAnimSequence>(Iter->GetAsset()))
			{
				CopyOfSelectedAssets.Add(AnimSequence);
			}
		}
		FReimportManager::Instance()->ValidateAllSourceFileAndReimport(CopyOfSelectedAssets);
	}
}

bool SAnimationSequenceBrowser::CanShowColumnForAssetRegistryTag(FName AssetType, FName TagName) const
{
	return !AssetRegistryTagsToIgnore.Contains(TagName);
}

void SAnimationSequenceBrowser::Construct(const FArguments& InArgs, const TSharedRef<IPersonaToolkit>& InPersonaToolkit)
{
	PersonaToolkitPtr = InPersonaToolkit;
	OnOpenNewAsset = InArgs._OnOpenNewAsset;
	bShowHistory = InArgs._ShowHistory;

	Commands = MakeShareable(new FUICommandList());
	Commands->MapAction(FGlobalEditorCommonCommands::Get().FindInContentBrowser, FUIAction(
		FExecuteAction::CreateSP(this, &SAnimationSequenceBrowser::FindInContentBrowser),
		FCanExecuteAction::CreateSP(this, &SAnimationSequenceBrowser::CanFindInContentBrowser)
		));

	CurrentAssetHistoryIndex = INDEX_NONE;
	bTriedToCacheOrginalAsset = false;

	bIsActiveTimerRegistered = false;

	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

	CreateAssetTooltipResources();

	// Configure filter for asset picker
	Filter.bRecursiveClasses = true;
	Filter.ClassPaths.Add(UAnimationAsset::StaticClass()->GetClassPathName());
	Filter.ClassPaths.Add(USoundWave::StaticClass()->GetClassPathName());

	FAssetPickerConfig Config;
	Config.Filter = Filter;
	Config.InitialAssetViewType = EAssetViewType::Column;
	Config.bAddFilterUI = true;
	Config.bShowPathInColumnView = true;
	Config.bSortByPathInColumnView = true;

	// Configure response to click and double-click
	Config.OnAssetDoubleClicked = FOnAssetDoubleClicked::CreateSP(this, &SAnimationSequenceBrowser::OnRequestOpenAsset, false);
	Config.OnGetAssetContextMenu = FOnGetAssetContextMenu::CreateSP(this, &SAnimationSequenceBrowser::OnGetAssetContextMenu);
	Config.OnAssetTagWantsToBeDisplayed = FOnShouldDisplayAssetTag::CreateSP(this, &SAnimationSequenceBrowser::CanShowColumnForAssetRegistryTag);
	Config.OnAssetEnterPressed = FOnAssetEnterPressed::CreateSP(this, &SAnimationSequenceBrowser::OnRequestOpenAssets, false);
	Config.SyncToAssetsDelegates.Add(&SyncToAssetsDelegate);
	Config.OnShouldFilterAsset = FOnShouldFilterAsset::CreateSP(this, &SAnimationSequenceBrowser::HandleFilterAsset);
	Config.GetCurrentSelectionDelegates.Add(&GetCurrentSelectionDelegate);
	Config.SetFilterDelegates.Add(&SetFilterDelegate);
	Config.bFocusSearchBoxWhenOpened = false;
	Config.DefaultFilterMenuExpansion = EAssetTypeCategories::Animation;

	Config.SaveSettingsName = SettingsIniSection;

	TSharedPtr<FFrontendFilterCategory> AnimCategory = MakeShareable( new FFrontendFilterCategory(LOCTEXT("ExtraAnimationFilters", "Anim Filters"), LOCTEXT("ExtraAnimationFiltersTooltip", "Filter assets by all filters in this category.")) );
	Config.ExtraFrontendFilters.Add( MakeShareable(new FFrontendFilter_AdditiveAnimAssets(AnimCategory)) );
	TSharedPtr<FFrontendFilterCategory> AudioCategory = MakeShareable(new FFrontendFilterCategory(LOCTEXT("AudioFilters", "Audio Filters"), LOCTEXT("AudioFiltersTooltip", "Filter audio assets.")));
	Config.ExtraFrontendFilters.Add( MakeShareable(new FFrontendFilter_SoundWaves(AudioCategory)) );
	TSharedPtr<FFrontendFilterCategory> FolderCategory = MakeShareable(new FFrontendFilterCategory(LOCTEXT("FolderFilters", "Folder Filters"), LOCTEXT("FolderFiltersTooltip", "Filter by folders.")));
	const uint32 NumFilters = GetDefault<UPersonaOptions>()->NumFolderFiltersInAssetBrowser;
	for(uint32 FilterIndex = 0; FilterIndex < NumFilters; ++FilterIndex)
	{
		TSharedRef<FFrontendFilter_Folder> FolderFilter = MakeShared<FFrontendFilter_Folder>(FolderCategory, FilterIndex, 
			FSimpleDelegate::CreateLambda([this]()
			{
				Filter.PackagePaths.Empty();

				for(TSharedPtr<FFrontendFilter_Folder> CurrentFolderFilter : FolderFilters)
				{
					if(CurrentFolderFilter->bEnabled)
					{
						Filter.PackagePaths.Add(*CurrentFolderFilter->Folder);
					}
				}

				SetFilterDelegate.ExecuteIfBound(Filter);
			}));
		FolderFilters.Add(FolderFilter);
		Config.ExtraFrontendFilters.Add(FolderFilter);
	}

	SkeletonNotifyFilter = MakeShareable(new FFrontendFilter_SkeletonNotify(AnimCategory, InPersonaToolkit->GetEditableSkeleton()));
	Config.ExtraFrontendFilters.Add(SkeletonNotifyFilter.ToSharedRef());
	Config.ExtraFrontendFilters.Add(MakeShareable(new FFrontendFilter_Curves(AnimCategory, InPersonaToolkit->GetEditableSkeleton())));
	Config.ExtraFrontendFilters.Add(MakeShareable(new FFrontendFilter_SkeletonSyncMarker(AnimCategory, InPersonaToolkit->GetEditableSkeleton())));
	Config.ExtraFrontendFilters.Add(MakeShareable(new FFrontendFilter_Attributes(AnimCategory, InPersonaToolkit->GetEditableSkeleton())));
	
	Config.OnIsAssetValidForCustomToolTip = FOnIsAssetValidForCustomToolTip::CreateLambda([](const FAssetData& AssetData) {return AssetData.IsAssetLoaded(); });
	Config.OnGetCustomAssetToolTip = FOnGetCustomAssetToolTip::CreateSP(this, &SAnimationSequenceBrowser::CreateCustomAssetToolTip);
	Config.OnVisualizeAssetToolTip = FOnVisualizeAssetToolTip::CreateSP(this, &SAnimationSequenceBrowser::OnVisualizeAssetToolTip);
	Config.OnAssetToolTipClosing = FOnAssetToolTipClosing::CreateSP( this, &SAnimationSequenceBrowser::OnAssetToolTipClosing );

	// hide all asset registry columns by default (we only really want the name and path)
	UObject* AnimSequenceDefaultObject = UAnimSequence::StaticClass()->GetDefaultObject();
	FAssetRegistryTagsContextData TagsContext(AnimSequenceDefaultObject, EAssetRegistryTagsCaller::Uncategorized);
	AnimSequenceDefaultObject->GetAssetRegistryTags(TagsContext);
	for (const TPair<FName,UObject::FAssetRegistryTag>& TagPair : TagsContext.Tags)
	{
		Config.HiddenColumnNames.Add(TagPair.Key.ToString());
	}

	// Also hide the type column by default (but allow users to enable it, so don't use bShowTypeInColumnView)
	Config.HiddenColumnNames.Add(TEXT("Class"));

	static const FName DefaultForegroundName("DefaultForeground");

	TSharedRef< SMenuAnchor > BackMenuAnchorPtr = SNew(SMenuAnchor)
		.Placement(MenuPlacement_BelowAnchor)
		.OnGetMenuContent(this, &SAnimationSequenceBrowser::CreateHistoryMenu, true)
		[
			SNew(SButton)
			.OnClicked(this, &SAnimationSequenceBrowser::OnGoBackInHistory)
			.ForegroundColor(FAppStyle::GetSlateColor(DefaultForegroundName))
			.ButtonStyle(FAppStyle::Get(), "FlatButton")
			.ContentPadding(FMargin(1, 0))
			.IsEnabled(this, &SAnimationSequenceBrowser::CanStepBackwardInHistory)
			.ToolTipText(LOCTEXT("Backward_Tooltip", "Step backward in the asset history. Right click to see full history."))
			[
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), "ContentBrowser.TopBar.Font")
				.Font(FAppStyle::Get().GetFontStyle("FontAwesome.11"))
				.Text(FText::FromString(FString(TEXT("\xf060"))) /*fa-arrow-left*/)
			]
		];

	TSharedRef< SMenuAnchor > FwdMenuAnchorPtr = SNew(SMenuAnchor)
		.Placement(MenuPlacement_BelowAnchor)
		.OnGetMenuContent(this, &SAnimationSequenceBrowser::CreateHistoryMenu, false)
		[
			SNew(SButton)
			.OnClicked(this, &SAnimationSequenceBrowser::OnGoForwardInHistory)
			.ForegroundColor(FAppStyle::GetSlateColor(DefaultForegroundName))
			.ButtonStyle(FAppStyle::Get(), "FlatButton")
			.ContentPadding(FMargin(1, 0))
			.IsEnabled(this, &SAnimationSequenceBrowser::CanStepForwardInHistory)
			.ToolTipText(LOCTEXT("Forward_Tooltip", "Step forward in the asset history. Right click to see full history."))
			[
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), "ContentBrowser.TopBar.Font")
				.Font(FAppStyle::Get().GetFontStyle("FontAwesome.11"))
				.Text(FText::FromString(FString(TEXT("\xf061"))) /*fa-arrow-right*/)
			]
		];

	this->ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SBorder)
			.Visibility(this, &SAnimationSequenceBrowser::GetHistoryVisibility)
			.Padding(FMargin(3))
			.BorderImage( FAppStyle::GetBrush("ToolPanel.GroupBorder") )
		[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SBorder)
						.OnMouseButtonDown(this, &SAnimationSequenceBrowser::OnMouseDownHistory, TWeakPtr<SMenuAnchor>(BackMenuAnchorPtr))
				.BorderImage( FAppStyle::GetBrush("NoBorder") )
				[
					BackMenuAnchorPtr
				]
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SBorder)
						.OnMouseButtonDown(this, &SAnimationSequenceBrowser::OnMouseDownHistory, TWeakPtr<SMenuAnchor>(FwdMenuAnchorPtr))
				.BorderImage( FAppStyle::GetBrush("NoBorder") )
				[
					FwdMenuAnchorPtr
				]
			]
		]
			]
		]
		+SVerticalBox::Slot()
		.FillHeight(1.f)
		[
			SNew(SBorder)
			.Padding(FMargin(3))
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				ContentBrowserModule.Get().CreateAssetPicker(Config)
			]
		]
	];

	// Create the ignore set for asset registry tags
	// Making Skeleton to be private, and now GET_MEMBER_NAME_CHECKED doesn't work
	AssetRegistryTagsToIgnore.Add(TEXT("Skeleton"));
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	AssetRegistryTagsToIgnore.Add(TEXT("SequenceLength"));
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	AssetRegistryTagsToIgnore.Add(GET_MEMBER_NAME_CHECKED(UAnimSequenceBase, RateScale));
}

FReply SAnimationSequenceBrowser::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (Commands->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void SAnimationSequenceBrowser::AddAssetToHistory(const FAssetData& AssetData)
{
	CacheOriginalAnimAssetHistory();

	if (CurrentAssetHistoryIndex == AssetHistory.Num() - 1)
	{
		// History added to the end
		if (AssetHistory.Num() == MaxAssetsHistory)
		{
			// If max history entries has been reached
			// remove the oldest history
			AssetHistory.RemoveAt(0);
		}
	}
	else
	{
		// Clear out any history that is in front of the current location in the history list
		AssetHistory.RemoveAt(CurrentAssetHistoryIndex + 1, AssetHistory.Num() - (CurrentAssetHistoryIndex + 1), EAllowShrinking::Yes);
	}

	AssetHistory.Add(AssetData);
	CurrentAssetHistoryIndex = AssetHistory.Num() - 1;
}

FReply SAnimationSequenceBrowser::OnMouseDownHistory( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, TWeakPtr< SMenuAnchor > InMenuAnchor )
{
	if(MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		InMenuAnchor.Pin()->SetIsOpen(true);
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

TSharedRef<SWidget> SAnimationSequenceBrowser::CreateHistoryMenu(bool bInBackHistory) const
{
	FMenuBuilder MenuBuilder(true, nullptr);
	if(bInBackHistory)
	{
		int32 HistoryIdx = CurrentAssetHistoryIndex - 1;
		while( HistoryIdx >= 0 )
		{
			const FAssetData& AssetData = AssetHistory[ HistoryIdx ];

			if(AssetData.IsValid())
			{
				const FText DisplayName = FText::FromName(AssetData.AssetName);
				const FText Tooltip = FText::FromString( AssetData.GetObjectPathString() );

				MenuBuilder.AddMenuEntry(DisplayName, Tooltip, FSlateIcon(), 
					FUIAction(
					FExecuteAction::CreateRaw(const_cast<SAnimationSequenceBrowser*>(this), &SAnimationSequenceBrowser::GoToHistoryIndex, HistoryIdx)
					), 
					NAME_None, EUserInterfaceActionType::Button);
			}

			--HistoryIdx;
		}
	}
	else
	{
		int32 HistoryIdx = CurrentAssetHistoryIndex + 1;
		while( HistoryIdx < AssetHistory.Num() )
		{
			const FAssetData& AssetData = AssetHistory[ HistoryIdx ];

			if(AssetData.IsValid())
			{
				const FText DisplayName = FText::FromName(AssetData.AssetName);
				const FText Tooltip = FText::FromString(AssetData.GetObjectPathString());

				MenuBuilder.AddMenuEntry(DisplayName, Tooltip, FSlateIcon(), 
					FUIAction(
					FExecuteAction::CreateRaw(const_cast<SAnimationSequenceBrowser*>(this), &SAnimationSequenceBrowser::GoToHistoryIndex, HistoryIdx)
					), 
					NAME_None, EUserInterfaceActionType::Button);
			}

			++HistoryIdx;
		}
	}

	return MenuBuilder.MakeWidget();
}

bool SAnimationSequenceBrowser::CanStepBackwardInHistory() const
{
	int32 HistoryIdx = CurrentAssetHistoryIndex - 1;
	while( HistoryIdx >= 0 )
	{
		if(AssetHistory[HistoryIdx].IsValid())
		{
			return true;
		}

		--HistoryIdx;
	}
	return false;
}

bool SAnimationSequenceBrowser::CanStepForwardInHistory() const
{
	int32 HistoryIdx = CurrentAssetHistoryIndex + 1;
	while( HistoryIdx < AssetHistory.Num() )
	{
		if(AssetHistory[HistoryIdx].IsValid())
		{
			return true;
		}

		++HistoryIdx;
	}
	return false;
}

FReply SAnimationSequenceBrowser::OnGoForwardInHistory()
{
	while( CurrentAssetHistoryIndex < AssetHistory.Num() - 1)
	{
		++CurrentAssetHistoryIndex;

		if( AssetHistory[CurrentAssetHistoryIndex].IsValid() )
		{
			GoToHistoryIndex(CurrentAssetHistoryIndex);
			break;
		}
	}
	return FReply::Handled();
}

FReply SAnimationSequenceBrowser::OnGoBackInHistory()
{
	while( CurrentAssetHistoryIndex > 0 )
	{
		--CurrentAssetHistoryIndex;

		if( AssetHistory[CurrentAssetHistoryIndex].IsValid() )
		{
			GoToHistoryIndex(CurrentAssetHistoryIndex);
			break;
		}
	}
	return FReply::Handled();
}

void SAnimationSequenceBrowser::GoToHistoryIndex(int32 InHistoryIdx)
{
	if(AssetHistory[InHistoryIdx].IsValid())
	{
		CurrentAssetHistoryIndex = InHistoryIdx;
		OnRequestOpenAsset(AssetHistory[InHistoryIdx], /**bFromHistory=*/true);
	}
}

void SAnimationSequenceBrowser::CacheOriginalAnimAssetHistory()
{
	/** If we have nothing in the AssetHistory see if we can store 
	anything for where we currently are as we can't do this on construction */
	if (!bTriedToCacheOrginalAsset)
	{
		bTriedToCacheOrginalAsset = true;

		if(AssetHistory.Num() == 0)
		{
			USkeleton* DesiredSkeleton = PersonaToolkitPtr.Pin()->GetSkeleton();

			if(UObject* PreviewAsset = PersonaToolkitPtr.Pin()->GetPreviewScene()->GetPreviewAnimationAsset())
			{
				FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
				FAssetData AssetData = AssetRegistryModule.Get().GetAssetByObjectPath(FSoftObjectPath(PreviewAsset));
				AssetHistory.Add(AssetData);
				CurrentAssetHistoryIndex = AssetHistory.Num() - 1;
			}
		}
	}
}

void SAnimationSequenceBrowser::SelectAsset(UAnimationAsset * AnimAsset)
{
	FAssetData AssetData(AnimAsset);

	if (AssetData.IsValid())
	{
		TArray<FAssetData> CurrentSelection = GetCurrentSelectionDelegate.Execute();

		if ( !CurrentSelection.Contains(AssetData) )
		{
			TArray<FAssetData> AssetsToSelect;
			AssetsToSelect.Add(AssetData);

			SyncToAssetsDelegate.Execute(AssetsToSelect);
		}
	}
}

void SAnimationSequenceBrowser::FilterBySkeletonNotify(const FName& InNotifyName)
{
	check(SkeletonNotifyFilter.IsValid());

	SkeletonNotifyFilter->SetNotifyFilter(InNotifyName);
}

void SAnimationSequenceBrowser::AddToHistory(UAnimationAsset * AnimAsset)
{
	if (AnimAsset)
	{
		FAssetData AssetData(AnimAsset);
		AddAssetToHistory(AssetData);
	}
}

TSharedRef<SToolTip> SAnimationSequenceBrowser::CreateCustomAssetToolTip(FAssetData& AssetData)
{
	// Make a list of tags to show
	TArray<UObject::FAssetRegistryTag> Tags;
	UClass* AssetClass = FindObject<UClass>(AssetData.AssetClassPath);
	check(AssetClass);
	UObject* DefaultObject = AssetClass->GetDefaultObject();
	FAssetRegistryTagsContextData TagsContext(DefaultObject, EAssetRegistryTagsCaller::Uncategorized);
	DefaultObject->GetAssetRegistryTags(TagsContext);

	TArray<FName> TagsToShow;
	FName NameSkeleton(TEXT("Skeleton"));
	for (const TPair<FName, UObject::FAssetRegistryTag>& TagPair : TagsContext.Tags)
	{
		if(TagPair.Key != NameSkeleton && TagPair.Value.Type != UObject::FAssetRegistryTag::TT_Hidden)
		{
			TagsToShow.Add(TagPair.Key);
		}
	}

	// Add asset registry tags to a text list; except skeleton as that is implied in Persona
	TSharedRef<SVerticalBox> DescriptionBox = SNew(SVerticalBox);
	for(TPair<FName, FAssetTagValueRef> TagPair : AssetData.TagsAndValues)
	{
		if(TagsToShow.Contains(TagPair.Key))
		{
			// Check for DisplayName metadata
			FText DisplayName;
			if (FProperty* Field = FindFProperty<FProperty>(AssetClass, TagPair.Key))
			{
				DisplayName = Field->GetDisplayNameText();
			}
			else
			{
				DisplayName = FText::FromName(TagPair.Key);
			}

			DescriptionBox->AddSlot()
			.AutoHeight()
			.Padding(0,0,5,0)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text(FText::Format(LOCTEXT("AssetTagKey", "{0}: "), DisplayName))
					.ColorAndOpacity(FSlateColor::UseSubduedForeground())
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text(TagPair.Value.AsText())
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			];
		}
	}

	DescriptionBox->AddSlot()
		.AutoHeight()
		.Padding(0,0,5,0)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("AssetBrowser_FolderPathLabel", "Folder :"))
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(FText::FromName(AssetData.PackagePath))
				.ColorAndOpacity(FSlateColor::UseForeground())
				.WrapTextAt(300.f)
			]
		];

	TSharedPtr<SHorizontalBox> ContentBox = nullptr;
	TSharedRef<SToolTip> ToolTipWidget = SNew(SToolTip)
	.TextMargin(1.f)
	.BorderImage(FAppStyle::GetBrush("ContentBrowser.TileViewTooltip.ToolTipBorder"))
	[
		SNew(SBorder)
		.Padding(6.f)
		.BorderImage(FAppStyle::GetBrush("ContentBrowser.TileViewTooltip.NonContentBorder"))
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0,0,0,4)
			[
				SNew(SBorder)
				.Padding(6.f)
				.BorderImage(FAppStyle::GetBrush("ContentBrowser.TileViewTooltip.ContentBorder"))
				[
					SNew(SBox)
					.HAlign(HAlign_Left)
					[
						SNew(STextBlock)
						.Text(FText::FromName(AssetData.AssetName))
						.Font(FAppStyle::GetFontStyle("ContentBrowser.TileViewTooltip.NameFont"))
					]
				]
			]
		
			+ SVerticalBox::Slot()
			[
				SAssignNew(ContentBox, SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SBorder)
					.Padding(6.f)
					.Visibility(AssetClass->IsChildOf<UAnimationAsset>() ? EVisibility::Visible : EVisibility::Collapsed)
					.BorderImage(FAppStyle::GetBrush("ContentBrowser.TileViewTooltip.ContentBorder"))
					[
						SNew(SOverlay)
						+SOverlay::Slot()
						.VAlign(VAlign_Center)
						.HAlign(HAlign_Center)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("NoPreviewMesh", "No Preview Mesh"))
						]

						+ SOverlay::Slot()
						.VAlign(VAlign_Center)
						.HAlign(HAlign_Center)
						[
							ViewportWidget.ToSharedRef()
						]
					]
				]
			]
		]
	];

	// add an extra section to the tooltip for it.
	ContentBox->AddSlot()
	.Padding(AssetClass->IsChildOf<UAnimationAsset>() ? 4.f : 0, 0, 0, 0)
	[
		SNew(SBorder)
		.Padding(6.f)
		.BorderImage(FAppStyle::GetBrush("ContentBrowser.TileViewTooltip.ContentBorder"))
		[
			DescriptionBox
		]
	];

	return ToolTipWidget;
}

void SAnimationSequenceBrowser::CreateAssetTooltipResources()
{
	SAssignNew(ViewportWidget, SViewport)
		.EnableGammaCorrection(false)
		.ViewportSize(FVector2D(128, 128));

	ViewportClient = MakeShareable(new FAnimationAssetViewportClient(PreviewScene));
	SceneViewport = FSceneViewport::Create(ViewportClient, ViewportWidget);
	PreviewComponent = NewObject<UDebugSkelMeshComponent>();

	// Client options
	ViewportClient->ViewportType = LVT_Perspective;
	ViewportClient->bSetListenerPosition = false;
	// Default view until we need to show the viewport
	ViewportClient->SetViewLocation(EditorViewportDefs::DefaultPerspectiveViewLocation);
	ViewportClient->SetViewRotation(EditorViewportDefs::DefaultPerspectiveViewRotation);

	ViewportClient->SetRealtime(true);
	ViewportClient->SetViewMode(VMI_Lit);
	ViewportClient->ToggleOrbitCamera(true);
	ViewportClient->VisibilityDelegate.BindSP(this, &SAnimationSequenceBrowser::IsToolTipPreviewVisible);

	// Setup the preview component to ensure an animation will update when requested
	PreviewComponent->VisibilityBasedAnimTickOption = EVisibilityBasedAnimTickOption::AlwaysTickPoseAndRefreshBones;
	PreviewScene.AddComponent(PreviewComponent, FTransform::Identity);

	const USkeletalMeshEditorSettings* Options = GetDefault<USkeletalMeshEditorSettings>();

	PreviewScene.SetLightDirection(Options->AnimPreviewLightingDirection);
	PreviewScene.SetLightColor(Options->AnimPreviewDirectionalColor);
	PreviewScene.SetLightBrightness(Options->AnimPreviewLightBrightness);
}

bool SAnimationSequenceBrowser::OnVisualizeAssetToolTip(const TSharedPtr<SWidget>& TooltipContent, FAssetData& AssetData)
{
	// Resolve the asset
	USkeletalMesh* MeshToUse = nullptr;
	UClass* AssetClass = FindObject<UClass>(AssetData.AssetClassPath);
	if(AssetClass->IsChildOf(UAnimationAsset::StaticClass()) && AssetData.IsAssetLoaded() && AssetData.GetAsset())
	{
		// Set up the viewport to show the asset. Catching the visualize allows us to use
		// one viewport between all of the assets in the sequence browser.
		UAnimationAsset* Asset = StaticCast<UAnimationAsset*>(AssetData.GetAsset());
		USkeleton* Skeleton = Asset->GetSkeleton();
		if(Skeleton)
		{
			MeshToUse = Skeleton->GetAssetPreviewMesh(Asset);
		}
		
		if(MeshToUse)
		{
			if(PreviewComponent->GetSkeletalMeshAsset() != MeshToUse)
			{
				PreviewComponent->SetSkeletalMesh(MeshToUse);
			}

			PreviewComponent->EnablePreview(true, Asset);
			PreviewComponent->PreviewInstance->PlayAnim(true);

			const FBoxSphereBounds MeshImportedBounds = MeshToUse->GetImportedBounds();
			const float HalfFov = FMath::DegreesToRadians(ViewportClient->ViewFOV) / 2.0f;
			const float TargetDist = static_cast<float>(MeshImportedBounds.SphereRadius / FMath::Tan(HalfFov));

			ViewportClient->SetViewRotation(FRotator(0.0f, -45.0f, 0.0f));
			ViewportClient->SetViewLocationForOrbiting(FVector(0.0f, 0.0f, MeshImportedBounds.BoxExtent.Z / 2.0f), TargetDist);

			ViewportWidget->SetVisibility(EVisibility::Visible);
			
			// Update the preview as long as the tooltip is visible
			if ( !bIsActiveTimerRegistered )
			{
				bIsActiveTimerRegistered = true;
				RegisterActiveTimer(0.f, FWidgetActiveTimerDelegate::CreateSP(this, &SAnimationSequenceBrowser::UpdateTootipPreview));
			}
		}
		else
		{
			ViewportWidget->SetVisibility(EVisibility::Hidden);
		}
	}

	// We return false here as we aren't visualizing the tooltip - just detecting when it is about to be shown.
	// We still want slate to draw it.
	return false;
}

void SAnimationSequenceBrowser::OnAssetToolTipClosing()
{
}

void SAnimationSequenceBrowser::CleanupPreviewSceneComponent(USceneComponent* Component)
{
	if(Component)
	{
		for(int32 ComponentIdx = Component->GetAttachChildren().Num() - 1 ; ComponentIdx >= 0 ; --ComponentIdx)
		{
			USceneComponent* ChildComponent = Component->GetAttachChildren()[ComponentIdx];
			CleanupPreviewSceneComponent(ChildComponent);
		}
		check(Component->GetAttachChildren().Num() == 0);
		Component->DestroyComponent();
	}
}

EActiveTimerReturnType SAnimationSequenceBrowser::UpdateTootipPreview( double InCurrentTime, float InDeltaTime )
{
	if (PreviewComponent && IsToolTipPreviewVisible() && ViewportWidget->IsParentValid())
	{
		// Tick the world to update preview viewport for tooltips
		PreviewComponent->GetScene()->GetWorld()->Tick( LEVELTICK_All, InDeltaTime );
	}
	else
	{
		bIsActiveTimerRegistered = false;
		return EActiveTimerReturnType::Stop;
	}

	return EActiveTimerReturnType::Continue;
}

bool SAnimationSequenceBrowser::IsToolTipPreviewVisible()
{
	bool bVisible = false;

	if(ViewportWidget.IsValid())
	{
		bVisible = ViewportWidget->GetVisibility() == EVisibility::Visible;
	}
	return bVisible;
}

EVisibility SAnimationSequenceBrowser::GetHistoryVisibility() const
{
	return bShowHistory ? EVisibility::Visible : EVisibility::Collapsed;
}

bool SAnimationSequenceBrowser::HandleFilterAsset(const FAssetData& InAssetData) const
{
	if (InAssetData.IsInstanceOf(UAnimationAsset::StaticClass()))
	{
		const USkeleton* DesiredSkeleton = PersonaToolkitPtr.Pin()->GetSkeleton();
		if (DesiredSkeleton)
		{
			return !DesiredSkeleton->IsCompatibleForEditor(InAssetData);
		}
	}

	return false;
}

void SAnimationSequenceBrowser::HandlePlayAudio(FAssetData InAssetData)
{
	PlayPreviewAudio(Cast<USoundWave>(InAssetData.GetAsset()));
}

void SAnimationSequenceBrowser::HandleStopAudio()
{
	UAudioComponent* AudioComponent = PersonaToolkitPtr.Pin()->GetPreviewScene()->GetActor()->FindComponentByClass<UAudioComponent>();
	if (AudioComponent)
	{
		AudioComponent->Stop();
	}
}

void SAnimationSequenceBrowser::PlayPreviewAudio(USoundWave* InSoundWave)
{
	if (InSoundWave)
	{
		UAudioComponent* AudioComponent = PersonaToolkitPtr.Pin()->GetPreviewScene()->GetActor()->FindComponentByClass<UAudioComponent>();
		if (AudioComponent)
		{
			// If we are playing this soundwave, stop
			if (AudioComponent->IsPlaying() && AudioComponent->Sound == InSoundWave)
			{
				AudioComponent->Stop();
			}
			else
			{
				AudioComponent->Stop();
				AudioComponent->SetSound(InSoundWave);
				AudioComponent->Play();
			}
		}
	}
}

FAnimationAssetViewportClient::FAnimationAssetViewportClient(FPreviewScene& InPreviewScene)
	: FEditorViewportClient(nullptr, &InPreviewScene)
{
	SetViewMode(VMI_Lit);

	// Always composite editor objects after post processing in the editor
	EngineShowFlags.SetCompositeEditorPrimitives(true);
	EngineShowFlags.DisableAdvancedFeatures();

	// Setup defaults for the common draw helper.
	DrawHelper.bDrawPivot = false;
	DrawHelper.bDrawWorldBox = false;
	DrawHelper.bDrawKillZ = false;
	DrawHelper.bDrawGrid = true;
	DrawHelper.GridColorAxis = FColor(70, 70, 70);
	DrawHelper.GridColorMajor = FColor(40, 40, 40);
	DrawHelper.GridColorMinor = FColor(20, 20, 20);
	DrawHelper.PerspectiveGridSize = UE_OLD_HALF_WORLD_MAX1;
	bDrawAxes = false;
}

FSceneInterface* FAnimationAssetViewportClient::GetScene() const
{
	return PreviewScene->GetScene();
}

FLinearColor FAnimationAssetViewportClient::GetBackgroundColor() const
{
	return FLinearColor(0.8f, 0.85f, 0.85f);
}


#undef LOCTEXT_NAMESPACE
