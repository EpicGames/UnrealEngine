// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "PreviewScene.h"
#include "EditorViewportClient.h"
#include "AssetRegistry/AssetData.h"
#include "Widgets/SToolTip.h"
#include "PersonaDelegates.h"
#include "ContentBrowserDelegates.h"
#include "EditorAnimUtils.h"
#include "IAnimationSequenceBrowser.h"
#include "AssetRegistry/ARFilter.h"

#include "SAnimationSequenceBrowser.generated.h"

class FSceneViewport;
class FUICommandList;
class IPersonaToolkit;
class SMenuAnchor;
class SViewport;
class UAnimationAsset;
class UDebugSkelMeshComponent;
class USoundWave;
class FFrontendFilter_Folder;
class FFrontendFilter_SkeletonNotify;

UENUM()
enum class EAdditiveTypeFilterMode : uint8
{
	NonAdditive UMETA(DisplayName = "Non-Additive"),
	AnyAdditive UMETA(DisplayName = "Any Additive"),
	LocalSpaceBase UMETA(DisplayName = "Local Space"),
	RotationOffsetMeshSpace UMETA(DisplayName = "Mesh Space"),
};

//////////////////////////////////////////////////////////////////////////
// FAnimationAssetViewportClient
class FAnimationAssetViewportClient : public FEditorViewportClient
{
public:
	FAnimationAssetViewportClient(FPreviewScene& InPreviewScene);

	// FEditorViewportClient interface
	virtual FSceneInterface* GetScene() const override;
	virtual FLinearColor GetBackgroundColor() const override;
	// End of FEditorViewportClient interface
};

//////////////////////////////////////////////////////////////////////////
// SAnimationSequenceBrowser

class SAnimationSequenceBrowser : public IAnimationSequenceBrowser
{
public:
	SLATE_BEGIN_ARGS(SAnimationSequenceBrowser)
		: _ShowHistory(false)
	{}

	SLATE_ARGUMENT(bool, ShowHistory)

	SLATE_ARGUMENT(FOnOpenNewAsset, OnOpenNewAsset)

	SLATE_END_ARGS()
public:
	void Construct(const FArguments& InArgs, const TSharedRef<class IPersonaToolkit>& InPersonaToolkit);

	void OnRequestOpenAssets(const TArray<FAssetData>& SelectedAssets, bool bFromHistory);

	void OnRequestOpenAsset(const FAssetData& AssetData, bool bFromHistory);

	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

	virtual ~SAnimationSequenceBrowser();

	/** IAnimationSequenceBrowser interface */
	virtual void SelectAsset(UAnimationAsset * AnimAsset) override;
	virtual void AddToHistory(UAnimationAsset* AnimAsset) override;
	virtual void FilterBySkeletonNotify(const FName& InNotifyName) override;
	
	/** Delegate that handles creation of context menu */
	TSharedPtr<SWidget> OnGetAssetContextMenu(const TArray<FAssetData>& SelectedAssets);

	/** Delegate to open animation assets each in a new window */
	void OpenInNewWindow(TArray<FAssetData> AnimationAssets);

	/** Delegate to handle "Find in Content Browser" context menu option */
	void FindInContentBrowser();

	/** Delegate to handle enabling the "Find in Content Browser" context menu option */
	bool CanFindInContentBrowser() const;

	/** Delegate to handle "Save" context menu option */
	void SaveSelectedAssets(TArray<FAssetData> ObjectsToSave) const;

	/** Delegate to handle enabling the "Save" context menu option */
	bool CanSaveSelectedAssets(TArray<FAssetData> ObjectsToSave) const;

	/** Delegate to handle compression context menu option 
		Applies a chosen compression method to the selected assets */
	void OnApplyCompression(TArray<FAssetData> SelectedAssets);

	/** Delegate to handle Export FBX context menu option */
	void OnExportToFBX(TArray<FAssetData> SelectedAssets);

	/** Delegate to handle Add looping interpolation context menu option */
	void OnAddLoopingInterpolation(TArray<FAssetData> SelectedAssets);

	/** Delegate to handle set current preview mesh context menu option */
	void OnSetCurrentPreviewMesh(TArray<FAssetData> SelectedAssets);

	/** Delegate to handle reimport selected animations */
	void OnReimportAnimation(TArray<FAssetData> SelectedAssets);

	/** This will allow duplicate the current object, and remap to new skeleton 
	 *	Only allowed for AnimSequence 
	 */
	void OnCreateCopy(TArray<FAssetData> Selected);

protected:
	bool CanShowColumnForAssetRegistryTag(FName AssetType, FName TagName) const;
	
	/** Populate supplied OutPackages with the packages for the supplied Assets array */
	void GetSelectedPackages(const TArray<FAssetData>& Assets, TArray<UPackage*>& OutPackages) const;

	/** Adds the supplied asset to the asset history */
	void AddAssetToHistory(const FAssetData& AssetData);

	void CacheOriginalAnimAssetHistory();

	/** Single step forward in history */
	FReply OnGoForwardInHistory();

	/** Single step back in history */
	FReply OnGoBackInHistory();

	/** Jumps immediately to an index in the history if valid */
	void GoToHistoryIndex(int32 InHistoryIdx);

	/** Returns TRUE if stepping backward in history is allowed */
	bool CanStepBackwardInHistory() const;

	/** Returns TRUE if stepping forward in history is allowed */
	bool CanStepForwardInHistory() const;

	/**
	 * Mouse down callback to display a history menu
	 *
	 * @param InMenuAnchor		This is the anchor the menu will use for positioning
	 */
	FReply OnMouseDownHistory( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, TWeakPtr< SMenuAnchor > InMenuAnchor );

	/** 
	 * Callback to create the history menu.
	 *
	 * @param bInBackHistory		TRUE if the back history is requested, FALSE if the forward history is
	 *
	 * @return						The menu widget displaying all available history
	 */
	TSharedRef<SWidget> CreateHistoryMenu(bool bInBackHistory) const;

	/**
	 * Create the viewport and required objects to control the viewport used in tooltips
	 */
	void CreateAssetTooltipResources();

	/**
	 * Create a custom asset view tooltip for the provided asset
	 */
	TSharedRef<SToolTip> CreateCustomAssetToolTip(FAssetData& AssetData);

	/**
	 * Called as a tooltip is about to show;
	 */
	bool OnVisualizeAssetToolTip(const TSharedPtr<SWidget>& TooltipContent, FAssetData& AssetData);

	/**
	* Called as a tooltip is closing
	*/
	void OnAssetToolTipClosing();

	/**
	 * Cleaup a component from the preview scene along with any attached children
	 */
	void CleanupPreviewSceneComponent(USceneComponent* Component);

	/**
	 * Has the animation asset preview tool tip been visualized
	 */
	bool IsToolTipPreviewVisible();

	/** Returns visible when not in a Blueprint mode (anim mode, etc...) */
	EVisibility GetHistoryVisibility() const;

	/** Perform additional filtering */
	bool HandleFilterAsset(const FAssetData& InAssetData) const;

	/** Handle playing audio from the right-click menu */
	void HandlePlayAudio(FAssetData InAssetData);

	/** Handle stopping audio from the right-click menu */
	void HandleStopAudio();

	/** Play the specified sound on the preview audio component */
	void PlayPreviewAudio(USoundWave* InSoundWave);

protected:
	/**
	 * The actual viewport widget
	 */	
	TSharedPtr<SViewport> ViewportWidget;

	/**
	 * The scene viewport data
	 */
	TSharedPtr<FSceneViewport> SceneViewport;

	/**
	 * Custom viewport client used for tooltip previews
	 */
	TSharedPtr<FAnimationAssetViewportClient> ViewportClient;

	/**
	 * Skeletal component to preview the animation asset on
	 */
	UDebugSkelMeshComponent* PreviewComponent;

	/**
	 * The scene to show in the asset previews
	 */
	FPreviewScene PreviewScene;

	/**
	 * Commands handled by this widget
	 */
	TSharedPtr<FUICommandList> Commands;

	/** The persona toolkit we are using */
	TWeakPtr<class IPersonaToolkit> PersonaToolkitPtr;

	// Set of tags to prevent creating details view columns for (infrequently used)
	TSet<FName> AssetRegistryTagsToIgnore;

	// List of recently opened assets
	TArray<FAssetData> AssetHistory;

	// Current position in the asset history
	int32 CurrentAssetHistoryIndex;

	// Max assets to save in history
	static const int32 MaxAssetsHistory;

	// Track if we have tried to cache the first asset we were playing
	bool bTriedToCacheOrginalAsset;

	/** Whether to show the history widgets */
	bool bShowHistory;

	/** Delegate called to open a new asset for editing */
	FOnOpenNewAsset OnOpenNewAsset;

	// delegate to sync the asset picker to selected assets
	FSyncToAssetsDelegate SyncToAssetsDelegate;
	FGetCurrentSelectionDelegate GetCurrentSelectionDelegate;

	/** Delegate used to set the AR filter after the fact */
	FSetARFilterDelegate SetFilterDelegate;

	/** Keep the AR filter around so we can modify it */
	FARFilter Filter;

	/** All the folder filters we have */
	TArray<TSharedPtr<FFrontendFilter_Folder>> FolderFilters;

	/** The skeleton notify filter */
	TSharedPtr<FFrontendFilter_SkeletonNotify> SkeletonNotifyFilter;

private:
	/** Updates the animation preview in the tooltip */
	EActiveTimerReturnType UpdateTootipPreview( double InCurrentTime, float InDeltaTime );

	/** Whether the active timer is currently registered */
	uint8 bIsActiveTimerRegistered : 1;

public:
	/** The section of EditorPerProjectUserSettings in which to save settings */
	static const FString SettingsIniSection;
};
