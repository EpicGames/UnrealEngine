// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAssetFamilyShortcutBar.h"
#include "Styling/SlateTypes.h"
#include "Widgets/Input/SCheckBox.h"
#include "Modules/ModuleManager.h"
#include "Framework/Commands/UIAction.h"
#include "Textures/SlateIcon.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Framework/MultiBox/MultiBoxDefs.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Layout/SSeparator.h"

#include "Styling/AppStyle.h"
#include "IAssetFamily.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "WorkflowOrientedApp/WorkflowCentricApplication.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SComboButton.h"
#include "Framework/Application/SlateApplication.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Styling/ToolBarStyle.h"
#include "AssetToolsModule.h"
#include "IAssetTools.h"
#include "PersonaAssetFamilyManager.h"

#define LOCTEXT_NAMESPACE "SAssetFamilyShortcutBar"

namespace AssetShortcutConstants
{
	const int32 ThumbnailSize = 40;
	const int32 ThumbnailSizeSmall = 16;
}

class SAssetShortcut : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAssetShortcut)
	{}

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<class FWorkflowCentricApplication>& InHostingApp, const TSharedRef<class IAssetFamily>& InAssetFamily, const FAssetData& InAssetData, const TSharedRef<FAssetThumbnailPool>& InThumbnailPool)
	{
		AssetData = InAssetData;
		AssetFamily = InAssetFamily;
		HostingApp = InHostingApp;
		ThumbnailPoolPtr = InThumbnailPool;
		bPackageDirty = false;

		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
		AssetRegistryModule.Get().OnFilesLoaded().AddSP(this, &SAssetShortcut::HandleFilesLoaded);
		AssetRegistryModule.Get().OnAssetAdded().AddSP(this, &SAssetShortcut::HandleAssetAdded);
		AssetRegistryModule.Get().OnAssetRemoved().AddSP(this, &SAssetShortcut::HandleAssetRemoved);
		AssetRegistryModule.Get().OnAssetRenamed().AddSP(this, &SAssetShortcut::HandleAssetRenamed);

		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OnAssetEditorRequestedOpen().AddSP(this, &SAssetShortcut::HandleAssetOpened);
		AssetFamily->GetOnAssetOpened().AddSP(this, &SAssetShortcut::HandleAssetOpened);

		AssetThumbnail = MakeShareable(new FAssetThumbnail(InAssetData, AssetShortcutConstants::ThumbnailSize, AssetShortcutConstants::ThumbnailSize, InThumbnailPool));
		AssetThumbnailSmall = MakeShareable(new FAssetThumbnail(InAssetData, AssetShortcutConstants::ThumbnailSizeSmall, AssetShortcutConstants::ThumbnailSizeSmall, InThumbnailPool));

		TArray<FAssetData> Assets;
		InAssetFamily->FindAssetsOfType(InAssetData.GetClass(), Assets);
		bMultipleAssetsExist = Assets.Num() > 1;
		AssetDirtyBrush = FAppStyle::Get().GetBrush("Icons.DirtyBadge");

		const FToolBarStyle& ToolBarStyle = FAppStyle::Get().GetWidgetStyle<FToolBarStyle>("ToolBar");

		ChildSlot
		[
			SNew(SHorizontalBox)

			// This is the fat button for when there are not multiple options
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			[

				SAssignNew(CheckBox, SCheckBox)
				.Style(FAppStyle::Get(), "SegmentedCombo.ButtonOnly")
				.OnCheckStateChanged(this, &SAssetShortcut::HandleOpenAssetShortcut)
				.IsChecked(this, &SAssetShortcut::GetCheckState)
				.Visibility(this, &SAssetShortcut::GetSoloButtonVisibility)
				.ToolTipText(this, &SAssetShortcut::GetButtonTooltip)
				.Padding(0.0f)
				[
					SNew(SOverlay)

					+ SOverlay::Slot()
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Center)
					.Padding(FMargin(28.f, 4.f))
					[
						SNew(SImage)
						.ColorAndOpacity(this, &SAssetShortcut::GetAssetTint)
						.Image(this, &SAssetShortcut::GetAssetIcon)
					]

					+ SOverlay::Slot()
					.VAlign(VAlign_Bottom)
					.HAlign(HAlign_Right)
					.Padding(FMargin(2.f, 2.f))
					[
						SNew(SImage)
						.ColorAndOpacity(FSlateColor::UseForeground())
						.Image(this, &SAssetShortcut::GetDirtyImage)
					]
				]
			]

			// This is the left half of the button / combo pair for when there are multiple options
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			[
				SAssignNew(CheckBox, SCheckBox)
				.Style(FAppStyle::Get(), "SegmentedCombo.Left")
				.OnCheckStateChanged(this, &SAssetShortcut::HandleOpenAssetShortcut)
				.IsChecked(this, &SAssetShortcut::GetCheckState)
				.Visibility(this, &SAssetShortcut::GetComboButtonVisibility)
				.ToolTipText(this, &SAssetShortcut::GetButtonTooltip)
				.Padding(0.0f)
				[
					SNew(SOverlay)

					+ SOverlay::Slot()
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Center)
					.Padding(FMargin(16.f, 4.f))
					[
						SNew(SImage)
						.ColorAndOpacity(this, &SAssetShortcut::GetAssetTint)
						.Image(this, &SAssetShortcut::GetAssetIcon)
					]

					+ SOverlay::Slot()
					.VAlign(VAlign_Bottom)
					.HAlign(HAlign_Right)
					.Padding(FMargin(2.f, 2.f))
					[
						SNew(SImage)
						.ColorAndOpacity(FSlateColor::UseForeground())
						.Image(this, &SAssetShortcut::GetDirtyImage)
					]
				]
			]
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SSeparator)
				.Visibility(this, &SAssetShortcut::GetComboVisibility)
				.Thickness(1.0f)
				.Orientation(EOrientation::Orient_Vertical)
			]
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SComboButton)
				.Visibility(this, &SAssetShortcut::GetComboVisibility)
				.ContentPadding(FMargin(7.f, 0.f))
				.ForegroundColor(FSlateColor::UseForeground())
				.ComboButtonStyle(&FAppStyle::Get(), "SegmentedCombo.Right")
				.OnGetMenuContent(this, &SAssetShortcut::HandleGetMenuContent)
				.ToolTipText(LOCTEXT("AssetComboTooltip", "Find other assets of this type and perform asset operations.\nShift-Click to open in new window."))
			]
		];

		EnableToolTipForceField(true);

		DirtyStateTimerHandle = RegisterActiveTimer(1.0f / 10.0f, FWidgetActiveTimerDelegate::CreateSP(this, &SAssetShortcut::HandleRefreshDirtyState));
	}

	~SAssetShortcut()
	{
		if (FModuleManager::Get().IsModuleLoaded(TEXT("AssetRegistry")))
		{
			IAssetRegistry* AssetRegistry = FModuleManager::GetModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).TryGet();
			if (AssetRegistry)
			{
				AssetRegistry->OnFilesLoaded().RemoveAll(this);
				AssetRegistry->OnAssetAdded().RemoveAll(this);
				AssetRegistry->OnAssetRemoved().RemoveAll(this);
				AssetRegistry->OnAssetRenamed().RemoveAll(this);
			}
		}

		AssetFamily->GetOnAssetOpened().RemoveAll(this);
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OnAssetEditorRequestedOpen().RemoveAll(this);
		UnRegisterActiveTimer(DirtyStateTimerHandle.ToSharedRef());
	}

	void HandleOpenAssetShortcut(ECheckBoxState InState)
	{
		if(AssetData.IsValid())
		{
			if (UObject* AssetObject = AssetData.GetAsset())
			{
				TArray<UObject*> Assets;
				Assets.Add(AssetObject);
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAssets(Assets);
			}
			else
			{
				UE_LOGF(LogAnimation, Error, "Asset cannot be opened: %ls", *AssetData.GetObjectPathString());
			}
		}
	}

	FText GetAssetText() const
	{
		return AssetFamily->GetAssetTypeDisplayName(AssetData.GetClass());
	}

	const FSlateBrush* GetAssetIcon() const 
	{
		return AssetFamily->GetAssetTypeDisplayIcon(AssetData.GetClass());	
	}

	FSlateColor GetAssetTint() const
	{
		if (GetCheckState() == ECheckBoxState::Checked)
		{
			return FSlateColor::UseForeground();
		}
		return AssetFamily->GetAssetTypeDisplayTint(AssetData.GetClass());
	}

	ECheckBoxState GetCheckState() const
	{
		if(HostingApp.IsValid())
		{
			const TArray<UObject*>* Objects = HostingApp.Pin()->GetObjectsCurrentlyBeingEdited();
			if (Objects != nullptr)
			{
				for (UObject* Object : *Objects)
				{
					if (Object->GetPathName().Compare(AssetData.GetObjectPathString(), ESearchCase::IgnoreCase) == 0)
					{
						return ECheckBoxState::Checked;
					}
				}
			}
		}
		return ECheckBoxState::Unchecked;
	}

	FSlateColor GetAssetTextColor() const
	{
		static const FName InvertedForeground("InvertedForeground");
		return GetCheckState() == ECheckBoxState::Checked || CheckBox->IsHovered() ? FAppStyle::GetSlateColor(InvertedForeground) : FSlateColor::UseForeground();
	}

	TSharedRef<SWidget> HandleGetMenuContent()
	{
		FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

		const bool bInShouldCloseWindowAfterMenuSelection = true;
		FMenuBuilder MenuBuilder(bInShouldCloseWindowAfterMenuSelection, nullptr);

		MenuBuilder.BeginSection("AssetActions", LOCTEXT("AssetActionsSection", "Asset Actions"));
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("ShowInContentBrowser", "Show In Content Browser"),
				LOCTEXT("ShowInContentBrowser_ToolTip", "Show this asset in the content browser."),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Search"),
				FUIAction(FExecuteAction::CreateSP(this, &SAssetShortcut::HandleShowInContentBrowser)));
		}
		MenuBuilder.EndSection();

		if (bMultipleAssetsExist)
		{
			MenuBuilder.BeginSection("AssetSelection", LOCTEXT("AssetSelectionSection", "Select Asset"));
			{
				FAssetPickerConfig AssetPickerConfig;

				UClass* FilterClass = AssetFamily->GetAssetFamilyClass(AssetData.GetClass());
				if (FilterClass != nullptr)
				{
					AssetPickerConfig.Filter.ClassPaths.Add(FilterClass->GetClassPathName());
					AssetPickerConfig.Filter.bRecursiveClasses = true;
				}

				AssetPickerConfig.SelectionMode = ESelectionMode::SingleToggle;
				AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateSP(this, &SAssetShortcut::HandleAssetSelectedFromPicker);
				AssetPickerConfig.OnShouldFilterAsset = FOnShouldFilterAsset::CreateSP(this, &SAssetShortcut::HandleFilterAsset);
				AssetPickerConfig.bAllowNullSelection = false;
				AssetPickerConfig.ThumbnailLabel = EThumbnailLabel::ClassName;
				AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;
				AssetPickerConfig.InitialAssetSelection = AssetData;

				MenuBuilder.AddWidget(
					SNew(SBox)
					.WidthOverride(300.f)
					.HeightOverride(600.f)
					[
						ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
					],
					FText(), true);
			}
			MenuBuilder.EndSection();
		}
		
		return MenuBuilder.MakeWidget();
	}

	void HandleAssetSelectedFromPicker(const struct FAssetData& InAssetData)
	{
		if (InAssetData.IsValid())
		{
			FSlateApplication::Get().DismissAllMenus();

			TArray<UObject*> Assets;
			Assets.Add(InAssetData.GetAsset());
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAssets(Assets);
		}
		else if(AssetData.IsValid())
		{
			FSlateApplication::Get().DismissAllMenus();

			// Assume that as we are set to 'toggle' mode with no 'none' selection allowed, we are selecting the currently selected item
			TArray<UObject*> Assets;
			Assets.Add(AssetData.GetAsset());
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAssets(Assets);
		}
	}

	bool HandleFilterAsset(const struct FAssetData& InAssetData)
	{
		return !AssetFamily->IsAssetCompatible(InAssetData);
	}

	EVisibility GetSoloButtonVisibility() const
	{
		return AssetData.IsValid() && !bMultipleAssetsExist ? EVisibility::Visible : EVisibility::Collapsed;
	}

	EVisibility GetComboButtonVisibility() const
	{
		return AssetData.IsValid() && bMultipleAssetsExist ? EVisibility::Visible : EVisibility::Collapsed;
	}

	EVisibility GetComboVisibility() const
	{
		return bMultipleAssetsExist && AssetData.IsValid() ? EVisibility::Visible : EVisibility::Collapsed;
	}

	void HandleFilesLoaded()
	{
		TArray<FAssetData> Assets;
		AssetFamily->FindAssetsOfType(AssetData.GetClass(), Assets);
		bMultipleAssetsExist = Assets.Num() > 1;
	}

	void HandleAssetRemoved(const FAssetData& InAssetData)
	{
		if (AssetFamily->IsAssetCompatible(InAssetData))
		{
			TArray<FAssetData> Assets;
			AssetFamily->FindAssetsOfType(AssetData.GetClass(), Assets);
			bMultipleAssetsExist = Assets.Num() > 1;
		}
	}

	void HandleAssetRenamed(const FAssetData& InAssetData, const FString& InOldObjectPath)
	{
		if (AssetFamily->IsAssetCompatible(InAssetData))
		{
			if (InOldObjectPath == AssetData.GetObjectPathString())
			{
				AssetData = InAssetData;

				RegenerateThumbnail();
			}
		}
	}

	void HandleAssetAdded(const FAssetData& InAssetData)
	{
		const IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();
		if(!AssetRegistry.IsLoadingAssets())
		{
			if (AssetFamily->IsAssetCompatible(InAssetData))
			{
				TArray<FAssetData> Assets;
				AssetFamily->FindAssetsOfType(AssetData.GetClass(), Assets);
				bMultipleAssetsExist = Assets.Num() > 1;
			}
		}
	}

	void HandleShowInContentBrowser()
	{
		FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
		TArray<FAssetData> Assets;
		Assets.Add(AssetData);
		ContentBrowserModule.Get().SyncBrowserToAssets(Assets);
	}

	void HandleAssetOpened(UObject* InAsset)
	{
		const FAssetData InAssetData(InAsset);

		if (AssetFamily.IsValid() && AssetFamily->IsAssetCompatible(InAssetData))
		{
			if (AssetFamily->FindAssetOfType(InAsset->GetClass()) == InAssetData)
			{
				RefreshAsset();
			}
		}
	}

	EVisibility GetThumbnailVisibility() const
	{
		return FMultiBoxSettings::UseSmallToolBarIcons.Get() ? EVisibility::Collapsed : EVisibility::Visible;
	}

	EVisibility GetSmallThumbnailVisibility() const
	{
		return FMultiBoxSettings::UseSmallToolBarIcons.Get() ? EVisibility::Visible : EVisibility::Collapsed;
	}

	const FSlateBrush* GetDirtyImage() const
	{
		return bPackageDirty ? AssetDirtyBrush : nullptr;
	}

	void RefreshAsset()
	{
		if(HostingApp.IsValid())
		{
			// if this is the asset being edited by our hosting asset editor, don't switch it
			bool bAssetBeingEdited = false;
			const TArray<UObject*>* Objects = HostingApp.Pin()->GetObjectsCurrentlyBeingEdited();
			if (Objects != nullptr)
			{
				for (UObject* Object : *Objects)
				{
					if (FAssetData(Object) == AssetData)
					{
						bAssetBeingEdited = true;
						break;
					}
				}
			}

			// switch to new asset if needed
			if (!bAssetBeingEdited)
			{
				FAssetData NewAssetData = AssetFamily->FindAssetOfType(AssetData.GetClass());
				if (NewAssetData != AssetData)
				{
					AssetData = NewAssetData;

					RegenerateThumbnail();
				}
			}
		}
	}

	void RegenerateThumbnail()
	{
		if(AssetData.IsValid())
		{
			AssetThumbnail = MakeShareable(new FAssetThumbnail(AssetData, AssetShortcutConstants::ThumbnailSize, AssetShortcutConstants::ThumbnailSize, ThumbnailPoolPtr.Pin()));
			AssetThumbnailSmall = MakeShareable(new FAssetThumbnail(AssetData, AssetShortcutConstants::ThumbnailSizeSmall, AssetShortcutConstants::ThumbnailSizeSmall, ThumbnailPoolPtr.Pin()));
		}
	}

	EActiveTimerReturnType HandleRefreshDirtyState(double InCurrentTime, float InDeltaTime)
	{
		if (AssetData.IsAssetLoaded())
		{
			if (!AssetPackage.IsValid())
			{
				AssetPackage = AssetData.GetPackage();
			}

			if (AssetPackage.IsValid())
			{
				bPackageDirty = AssetPackage->IsDirty();
			}
		}

		return EActiveTimerReturnType::Continue;
	}

	FText GetButtonTooltip() const
	{
		return FText::Format(LOCTEXT("AssetTooltipFormat", "{0}\n{1}"), FText::FromName(AssetData.AssetName), FText::FromString(AssetData.GetFullName()));
	}

private:
	/** The current asset data for this widget */
	FAssetData AssetData;

	/** Cache the package of the object for checking dirty state */
	TWeakObjectPtr<UPackage> AssetPackage;

	/** Timer handle used to updating dirty state */
	TSharedPtr<class FActiveTimerHandle> DirtyStateTimerHandle;

	/** The asset family we are working with */
	TSharedPtr<class IAssetFamily> AssetFamily;

	/** Our asset thumbnails */
	TSharedPtr<FAssetThumbnail> AssetThumbnail;
	TSharedPtr<FAssetThumbnail> AssetThumbnailSmall;

	/** The asset editor we are embedded in */
	TWeakPtr<class FWorkflowCentricApplication> HostingApp;

	/** Thumbnail pool */
	TWeakPtr<FAssetThumbnailPool> ThumbnailPoolPtr;

	/** Check box */
	TSharedPtr<SCheckBox> CheckBox;

	/** Cached dirty brush */
	const FSlateBrush* AssetDirtyBrush;

	/** Whether there are multiple (>1) of this asset type in existence */
	bool bMultipleAssetsExist;

	/** Cache the package's dirty state */
	bool bPackageDirty;

	/** Flag for handling deferred refreshes */
	bool bDeferAssetAdded;
};

void SAssetFamilyShortcutBar::Construct(const FArguments& InArgs, const TSharedRef<FWorkflowCentricApplication>& InHostingApp, const TSharedRef<IAssetFamily>& InAssetFamily)
{
	WeakHostingApp = InHostingApp;
	AssetFamily = InAssetFamily;

	ThumbnailPool = MakeShareable(new FAssetThumbnailPool(16));

	InAssetFamily->GetOnAssetFamilyChanged().AddSP(this, &SAssetFamilyShortcutBar::OnAssetFamilyChanged);

	HorizontalBox = SNew(SHorizontalBox);

	BuildShortcuts();

	ChildSlot
	[
		HorizontalBox.ToSharedRef()
	];
}

void SAssetFamilyShortcutBar::BuildShortcuts()
{
	TArray<UClass*> AssetTypes;
	AssetFamily->GetAssetTypes(AssetTypes);

	for (UClass* Class : AssetTypes)
	{
		FAssetData AssetData = AssetFamily->FindAssetOfType(Class);
		HorizontalBox->AddSlot()
		.AutoWidth()
		.Padding(0.0f, 4.0f, 16.0f, 4.0f)
		[
			SNew(SAssetShortcut, WeakHostingApp.Pin().ToSharedRef(), AssetFamily.ToSharedRef(), AssetData, ThumbnailPool.ToSharedRef())
			.Visibility_Lambda([Class]()
			{
				IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();

				bool bIsVisible = false;
				TWeakPtr<IAssetTypeActions> AssetTypeActions = AssetTools.GetAssetTypeActionsForClass(Class);
				if (AssetTypeActions.IsValid())
				{
					if (const UClass* SupportedClass = AssetTypeActions.Pin()->GetSupportedClass())
					{
						bIsVisible = AssetTools.GetAssetClassPathPermissionList(EAssetClassAction::ViewAsset)->PassesFilter(SupportedClass->GetClassPathName().ToString());
					}
				}
				return bIsVisible ? EVisibility::Visible : EVisibility::Collapsed;
			})
		];
	}
}

void SAssetFamilyShortcutBar::OnAssetFamilyChanged()
{
	HorizontalBox->ClearChildren();

	const TArray<UObject*>* CurrentObjects = WeakHostingApp.Pin()->GetObjectsCurrentlyBeingEdited();
	if(CurrentObjects && CurrentObjects->Num() > 0 && (*CurrentObjects)[0])
	{
		AssetFamily->GetOnAssetFamilyChanged().RemoveAll((this));
		AssetFamily.Reset();
		AssetFamily = FPersonaAssetFamilyManager::Get().CreatePersonaAssetFamily((*CurrentObjects)[0]);
		AssetFamily->GetOnAssetFamilyChanged().AddSP(this, &SAssetFamilyShortcutBar::OnAssetFamilyChanged);

		BuildShortcuts();
	}
}

#undef LOCTEXT_NAMESPACE
