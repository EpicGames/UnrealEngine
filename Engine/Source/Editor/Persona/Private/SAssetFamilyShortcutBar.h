// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "AssetThumbnail.h"

class SHorizontalBox;
class FWorkflowCentricApplication;

class SAssetFamilyShortcutBar : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAssetFamilyShortcutBar)
	{}
		
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<class FWorkflowCentricApplication>& InHostingApp, const TSharedRef<class IAssetFamily>& InAssetFamily);

private:
	void BuildShortcuts();

	void OnAssetFamilyChanged();
	
	/** The thumbnail pool for displaying asset shortcuts */
	TSharedPtr<class FAssetThumbnailPool> ThumbnailPool;

	/** Container widget for shortcuts */
	TSharedPtr<SHorizontalBox> HorizontalBox;

	/** Asset family we represent */
	TSharedPtr<IAssetFamily> AssetFamily;

	/** App we are embedded in */
	TWeakPtr<FWorkflowCentricApplication> WeakHostingApp;
};
