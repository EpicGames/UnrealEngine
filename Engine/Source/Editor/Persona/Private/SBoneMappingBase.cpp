// Copyright Epic Games, Inc. All Rights Reserved.


#include "SBoneMappingBase.h"
#include "Misc/MessageDialog.h"
#include "Modules/ModuleManager.h"
#include "Widgets/SWindow.h"
#include "ReferenceSkeleton.h"
#include "Editor.h"
#include "Styling/AppStyle.h"
#include "Widgets/Input/SButton.h"
#include "AssetNotifications.h"
#include "BoneSelectionWidget.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "SSkeletonWidget.h"
#include "IEditableSkeleton.h"

class FPersona;

#define LOCTEXT_NAMESPACE "SBoneMappingBase"

static const FName ColumnId_NodeNameLabel( "Node Name" );
static const FName ColumnID_BoneNameLabel( "Bone" );

void SBoneMappingListRow::Construct( const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView )
{
	Item = InArgs._Item;
	BoneMappingListView = InArgs._BoneMappingListView;
	OnBoneMappingChanged = InArgs._OnBoneMappingChanged;
	OnGetBoneMapping = InArgs._OnGetBoneMapping;
	OnGetReferenceSkeleton = InArgs._OnGetReferenceSkeleton;
	OnGetFilteredText = InArgs._OnGetFilteredText;

	check( Item.IsValid() );

	SMultiColumnTableRow< FDisplayedBoneMappingInfoPtr >::Construct( FSuperRowType::FArguments(), InOwnerTableView );
}

TSharedRef< SWidget > SBoneMappingListRow::GenerateWidgetForColumn( const FName& ColumnName )
{
	if ( ColumnName == ColumnId_NodeNameLabel )
	{
		TSharedPtr< SInlineEditableTextBlock > InlineWidget;
		TSharedRef< SWidget > NewWidget = 
			SNew( SVerticalBox )

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding( 0.0f, 4.0f )
			.VAlign( VAlign_Center )
			[
				SAssignNew(InlineWidget, SInlineEditableTextBlock)
				.Text( FText::FromString(Item->GetDisplayName()) )
				.HighlightText( this, &SBoneMappingListRow::GetFilterText )
				.IsReadOnly(true)
				.IsSelected(this, &SMultiColumnTableRow< FDisplayedBoneMappingInfoPtr >::IsSelectedExclusively)
			];

		return NewWidget;
	}
	else
	{
		// show bone list
		// Encase the SSpinbox in an SVertical box so we can apply padding. Setting ItemHeight on the containing SListView has no effect :-(
		return
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.0f, 1.0f)
			.VAlign(VAlign_Center)
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				[
					SNew(SBoneSelectionWidget)
					.ToolTipText(FText::Format(LOCTEXT("BoneSelectinWidget", "Select Bone for node {0}"), FText::FromString(Item->GetDisplayName())))
					.OnBoneSelectionChanged(this, &SBoneMappingListRow::OnBoneSelectionChanged)
					.OnGetSelectedBone(this, &SBoneMappingListRow::GetSelectedBone)
					.OnGetReferenceSkeleton(OnGetReferenceSkeleton)
					.bShowVirtualBones(false)
				]

				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SButton)
					.OnClicked(FOnClicked::CreateSP(this, &SBoneMappingListRow::OnClearButtonClicked))
					.Text(FText::FromString(TEXT("x")))
				]
			];
	}
}

FReply SBoneMappingListRow::OnClearButtonClicked()
{
	if(OnBoneMappingChanged.IsBound())
	{
		OnBoneMappingChanged.Execute(Item->GetNodeName(), NAME_None);
	}

	return FReply::Handled();
}

void SBoneMappingListRow::OnBoneSelectionChanged(FName Name)
{
	if (OnBoneMappingChanged.IsBound())
	{
		OnBoneMappingChanged.Execute(Item->GetNodeName(), Name);
	}
}

FName SBoneMappingListRow::GetSelectedBone(bool& bMultipleValues) const
{
	if (OnGetBoneMapping.IsBound())
	{
		return OnGetBoneMapping.Execute(Item->GetNodeName());
	}

	return NAME_None;
}

FText SBoneMappingListRow::GetFilterText() const
{
	if (OnGetFilteredText.IsBound())
	{
		return OnGetFilteredText.Execute();
	}

	return FText::GetEmpty();
}

//////////////////////////////////////////////////////////////////////////
// SBoneMappingBase

void SBoneMappingBase::Construct(const FArguments& InArgs, FSimpleMulticastDelegate& InOnPostUndo)
{
	OnGetReferenceSkeletonDelegate = InArgs._OnGetReferenceSkeleton;
	OnGetBoneMappingDelegate = InArgs._OnGetBoneMapping;
	OnBoneMappingChangedDelegate = InArgs._OnBoneMappingChanged;
	OnCreateBoneMappingDelegate = InArgs._OnCreateBoneMapping;

	InOnPostUndo.Add(FSimpleDelegate::CreateSP( this, &SBoneMappingBase::PostUndo ) );
	
	ChildSlot
	[
		SNew( SVerticalBox )

		// now show bone mapping
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0,2)
		[
			SNew(SHorizontalBox)
			// Filter entry
			+SHorizontalBox::Slot()
			.FillWidth( 1 )
			[
				SAssignNew( NameFilterBox, SSearchBox )
				.SelectAllTextWhenFocused( true )
				.OnTextChanged( this, &SBoneMappingBase::OnFilterTextChanged )
				.OnTextCommitted( this, &SBoneMappingBase::OnFilterTextCommitted )
			]
		]

		+ SVerticalBox::Slot()
		.FillHeight( 1.0f )		// This is required to make the scrollbar work, as content overflows Slate containers by default
		[
			SAssignNew( BoneMappingListView, SBoneMappingListType )
			.ListItemsSource( &BoneMappingList )
			.OnGenerateRow( this, &SBoneMappingBase::GenerateBoneMappingRow )
			.HeaderRow
			(
				SNew( SHeaderRow )
				+ SHeaderRow::Column( ColumnId_NodeNameLabel )
				.DefaultLabel( LOCTEXT( "BoneMappingBase_SourceNameLabel", "Source" ) )
				.FixedWidth(150.f)

				+ SHeaderRow::Column( ColumnID_BoneNameLabel )
				.DefaultLabel( LOCTEXT( "BoneMappingBase_TargetNameLabel", "Target" ) )
			)
		]
	];

	RefreshBoneMappingList();
}

void SBoneMappingBase::OnFilterTextChanged( const FText& SearchText )
{
	// need to make sure not to have the same text go
	// otherwise, the widget gets recreated multiple times causing 
	// other issue
	if (FilterText.CompareToCaseIgnored(SearchText) != 0)
	{
		FilterText = SearchText;
		RefreshBoneMappingList();
	}
}

void SBoneMappingBase::OnFilterTextCommitted( const FText& SearchText, ETextCommit::Type CommitInfo )
{
	// Just do the same as if the user typed in the box
	OnFilterTextChanged( SearchText );
}

TSharedRef<ITableRow> SBoneMappingBase::GenerateBoneMappingRow(TSharedPtr<FDisplayedBoneMappingInfo> InInfo, const TSharedRef<STableViewBase>& OwnerTable)
{
	check( InInfo.IsValid() );

	return
		SNew(SBoneMappingListRow, OwnerTable)
		.Item(InInfo)
		.BoneMappingListView(BoneMappingListView)
		.OnBoneMappingChanged(OnBoneMappingChangedDelegate)
		.OnGetBoneMapping(OnGetBoneMappingDelegate)
		.OnGetReferenceSkeleton(OnGetReferenceSkeletonDelegate)
		.OnGetFilteredText(this, &SBoneMappingBase::GetFilterText);
}

void SBoneMappingBase::RefreshBoneMappingList()
{
	OnCreateBoneMappingDelegate.ExecuteIfBound(FilterText.ToString(), BoneMappingList);

	BoneMappingListView->RequestListRefresh();
}

void SBoneMappingBase::PostUndo()
{
	RefreshBoneMappingList();
}
#undef LOCTEXT_NAMESPACE

