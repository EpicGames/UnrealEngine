// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCreateClothingSettingsPanel.h"

#include "PropertyEditorModule.h"
#include "Modules/ModuleManager.h"
#include "IStructureDetailsView.h"
#include "IDetailsView.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "DetailLayoutBuilder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SButton.h"
#include "DetailCategoryBuilder.h"
#include "DetailWidgetRow.h"
#include "PropertyCustomizationHelpers.h"
#include "ClothingAsset.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "PhysicsEngine/PhysicsAsset.h"

#define LOCTEXT_NAMESPACE "CreateClothSettings"

void SCreateClothingSettingsPanel::Construct(const FArguments& InArgs)
{
	check(InArgs._LodIndex != INDEX_NONE && InArgs._SectionIndex != INDEX_NONE);

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	TSharedPtr<IStructureDetailsView> StructureDetailsView;

	OnCreateDelegate = InArgs._OnCreateRequested;

	bIsSubImport = InArgs._bIsSubImport;

	BuildParams.LodIndex = InArgs._LodIndex;
	BuildParams.SourceSection = InArgs._SectionIndex;

	FDetailsViewArgs DetailsViewArgs;
	{
		DetailsViewArgs.bAllowSearch = false;
		DetailsViewArgs.bHideSelectionTip = true;
		DetailsViewArgs.bLockable = false;
		DetailsViewArgs.bSearchInitialKeyFocus = true;
		DetailsViewArgs.bUpdatesFromSelection = false;
		DetailsViewArgs.NotifyHook = nullptr;
		DetailsViewArgs.bShowOptions = true;
		DetailsViewArgs.bShowModifiedPropertiesOption = false;
		DetailsViewArgs.bShowScrollBar = false;
	}

	FStructureDetailsViewArgs StructureViewArgs;
	{
		StructureViewArgs.bShowObjects = true;
		StructureViewArgs.bShowAssets = true;
		StructureViewArgs.bShowClasses = true;
		StructureViewArgs.bShowInterfaces = true;
	}

	StructureDetailsView = PropertyEditorModule.CreateStructureDetailView(DetailsViewArgs, StructureViewArgs, nullptr);

	StructureDetailsView->GetDetailsView()->SetGenericLayoutDetailsDelegate(FOnGetDetailCustomizationInstance::CreateStatic(&FClothCreateSettingsCustomization::MakeInstance, InArgs._Mesh, InArgs._bIsSubImport));

	BuildParams.AssetName = InArgs._MeshName + TEXT("_Clothing");
	BuildParams.PhysicsAsset = InArgs._Mesh->GetPhysicsAsset();

	FStructOnScope* Struct = new FStructOnScope(FSkeletalMeshClothBuildParams::StaticStruct(), (uint8*)&BuildParams);
	StructureDetailsView->SetStructureData(MakeShareable(Struct));
	
	this->ChildSlot
	[
		SNew(SBox)
		.MinDesiredWidth(300.0f)
		[
			SNew(SVerticalBox)

			+SVerticalBox::Slot()
			.MaxHeight(500.0f)
			.Padding(2.f)
			[
				StructureDetailsView->GetWidget()->AsShared()
			]

			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2.f)
			.HAlign(HAlign_Right)
			[
				SNew(SUniformGridPanel)
				.SlotPadding(2.f)
				+SUniformGridPanel::Slot(0, 0)
				[
					SNew(SButton)
					.Text(LOCTEXT("Label_Create", "Create"))
					.OnClicked(this, &SCreateClothingSettingsPanel::OnCreateClicked)
					.ToolTipText(this, &SCreateClothingSettingsPanel::GetCreateButtonTooltip)
					.IsEnabled(this, &SCreateClothingSettingsPanel::CanCreateClothing)
				]
			]
		]
	];
}

FText SCreateClothingSettingsPanel::GetCreateButtonTooltip() const
{
	if(CanCreateClothing())
	{
		if(bIsSubImport)
		{
			return LOCTEXT("CreateTooltip_NewLod", "Create new simulation mesh for the specified asset and LOD.");
		}
		else
		{
			return LOCTEXT("CreateTooltip_NewAsset", "Create new clothing asset from selected section.");
		}
	}
	else if(bIsSubImport)
	{
		return LOCTEXT("CreateTooltip_NewLodInvalid", "Select an asset and LOD level to create a simulation mesh.");
	}

	return FText::GetEmpty();
}

bool SCreateClothingSettingsPanel::CanCreateClothing() const
{
	// If we're importing a LOD, make sure a target has been selected
	if(bIsSubImport)
	{
		return BuildParams.TargetAsset.IsValid() && BuildParams.TargetLod != INDEX_NONE;
	}

	// No restriction for new assets
	return true;
}

FReply SCreateClothingSettingsPanel::OnCreateClicked()
{
	OnCreateDelegate.ExecuteIfBound(BuildParams);

	return FReply::Handled();
}

TSharedRef<SWidget> FClothCreateSettingsCustomization::OnGetTargetAssetMenu()
{
	FMenuBuilder Builder(true, nullptr, TSharedPtr<FExtender>(), true);

	Builder.BeginSection(TEXT("TargetAssetDropdown"), LOCTEXT("TargetAssetMenuHeader", "Available Assets"));
	{
		if (const USkeletalMesh* const Mesh = MeshPtr.Get())
		{
			const int32 NumCloths = Mesh->GetMeshClothingAssets().Num();
			for (int32 ClothIndex = 0; ClothIndex < NumCloths; ++ClothIndex)
			{
				if (const UClothingAssetCommon* const ClothingAsset = Cast<const UClothingAssetCommon>(Mesh->GetMeshClothingAssets()[ClothIndex]))
				{
					FUIAction Action;
					Action.ExecuteAction = FExecuteAction::CreateSP(this, &FClothCreateSettingsCustomization::OnAssetSelected, ClothIndex);
					Builder.AddMenuEntry(FText::FromString(ClothingAsset->GetName()), LOCTEXT("TargetAsset_Tooltip", "Select this clothing as the target to import to."), FSlateIcon(), Action);
				}
			}
		}
	}
	Builder.EndSection();

	return Builder.MakeWidget();
}

FText FClothCreateSettingsCustomization::GetTargetAssetText() const
{
	check(ParamsStruct);

	if(UClothingAssetBase* Target = ParamsStruct->TargetAsset.Get())
	{
		return FText::FromString(Target->GetName());
	}
	else
	{
		return LOCTEXT("SelectAssetPrompt", "Select target clothing...");
	}
}

void FClothCreateSettingsCustomization::OnAssetSelected(int32 InMeshClothingIndex)
{
	check(ParamsStruct);

	if(USkeletalMesh* Mesh = MeshPtr.Get())
	{
		if(Mesh->GetMeshClothingAssets().IsValidIndex(InMeshClothingIndex) &&
		   Mesh->GetMeshClothingAssets()[InMeshClothingIndex])
		{
			if(ParamsStruct->TargetAsset != Mesh->GetMeshClothingAssets()[InMeshClothingIndex])
			{
				ParamsStruct->TargetAsset = Mesh->GetMeshClothingAssets()[InMeshClothingIndex];
				ParamsStruct->TargetLod = INDEX_NONE;
			}
		}
	}
}

TSharedRef<SWidget> FClothCreateSettingsCustomization::OnGetTargetLodMenu()
{
	check(ParamsStruct);

	const USkeletalMesh* const Mesh = MeshPtr.Get();

	FMenuBuilder Builder(true, nullptr, TSharedPtr<FExtender>(), true);

	Builder.BeginSection(TEXT("TargetLodDropdown"), LOCTEXT("TargetLodMenuHeader", "Available LODs"));
	{
		if(const UClothingAssetCommon* const ClothingAsset = Cast<const UClothingAssetCommon>(ParamsStruct->TargetAsset.Get()))
		{
			const int32 NumLodEntries = ClothingAsset->GetNumLods();
			for(int32 LodEntryIndex = 0; LodEntryIndex < NumLodEntries + 1; ++LodEntryIndex)
			{
				FText EntryText;
				FText EntryTooltip;

				if(LodEntryIndex < NumLodEntries)
				{
					EntryText = FText::Format(LOCTEXT("LodEntryTextReplace", "Replace LOD {0}"), FText::AsNumber(LodEntryIndex));
					EntryTooltip = FText::Format(LOCTEXT("ImportLodReplaceTooltip", "Replace the simulation mesh in LOD {0} of {1} with this section."), FText::AsNumber(LodEntryIndex), FText::FromString(ClothingAsset->GetName()));
				}
				else
				{
					EntryText = FText::Format(LOCTEXT("LodEntryTextAdd", "Add LOD {0}"), FText::AsNumber(LodEntryIndex));
					EntryTooltip = LOCTEXT("ImportLodNewTooltip", "Use the selected section to add a as a new LOD");
				}

				FUIAction Action;
				Action.ExecuteAction = FExecuteAction::CreateSP(this, &FClothCreateSettingsCustomization::OnLodSelected, LodEntryIndex);
				Builder.AddMenuEntry(EntryText, EntryTooltip, FSlateIcon(), Action);
			}
		}
	}
	Builder.EndSection();

	return Builder.MakeWidget();
}

FText FClothCreateSettingsCustomization::GetTargetLodText() const
{
	check(ParamsStruct);

	if(!CanSelectLod())
	{
		return LOCTEXT("LodMenuSelectAsset", "Select an Asset");
	}

	if(ParamsStruct->TargetLod == INDEX_NONE)
	{
		return LOCTEXT("SelectALod", "Select a LOD");
	}

	return FText::Format(LOCTEXT("LodDropdownEntry", "LOD {0}"), FText::AsNumber(ParamsStruct->TargetLod));
}

void FClothCreateSettingsCustomization::OnLodSelected(int32 InLodIndex)
{
	check(ParamsStruct);

	ParamsStruct->TargetLod = InLodIndex;
}

bool FClothCreateSettingsCustomization::CanSelectLod() const
{
	check(ParamsStruct);

	return ParamsStruct->TargetAsset.IsValid();
}

void FClothCreateSettingsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	// Make sure we actually get a valid struct before continuing
	TArray<TSharedPtr<FStructOnScope>> Structs;
	DetailBuilder.GetStructsBeingCustomized(Structs);

	if(Structs.Num() == 0)
	{
		// Nothing being customized
		return;
	}

	const UStruct* Struct = Structs[0]->GetStruct();
	if(!Struct || Struct != FSkeletalMeshClothBuildParams::StaticStruct())
	{
		// Invalid struct
		return;
	}

	// Get ptr to our actual type
	ParamsStruct = (FSkeletalMeshClothBuildParams*)Structs[0]->GetStructMemory();

	TSharedPtr<IPropertyHandle> LodIndexProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(FSkeletalMeshClothBuildParams, LodIndex), (UClass*)FSkeletalMeshClothBuildParams::StaticStruct());
	TSharedPtr<IPropertyHandle> SectionIndexProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(FSkeletalMeshClothBuildParams, SourceSection), (UClass*)FSkeletalMeshClothBuildParams::StaticStruct());

	check(LodIndexProperty.IsValid() && SectionIndexProperty.IsValid());

	LodIndexProperty->MarkHiddenByCustomization();
	SectionIndexProperty->MarkHiddenByCustomization();

	TSharedPtr<IPropertyHandle> TargetAssetProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(FSkeletalMeshClothBuildParams, TargetAsset), (UClass*)FSkeletalMeshClothBuildParams::StaticStruct());
	TSharedPtr<IPropertyHandle> TargetLodProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(FSkeletalMeshClothBuildParams, TargetLod), (UClass*)FSkeletalMeshClothBuildParams::StaticStruct());
	
	check(TargetAssetProperty.IsValid() && TargetLodProperty.IsValid());
	TargetAssetProperty->MarkHiddenByCustomization();
	TargetLodProperty->MarkHiddenByCustomization();

	if(bIsSubImport)
	{
		// Asset name makes no sense for LODs, so hide it
		TSharedPtr<IPropertyHandle> AssetNameProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(FSkeletalMeshClothBuildParams, AssetName), (UClass*)FSkeletalMeshClothBuildParams::StaticStruct());
		if(AssetNameProperty.IsValid())
		{
			AssetNameProperty->MarkHiddenByCustomization();
		}

		// Physics mesh doesn't make sense for LODs
		TSharedPtr<IPropertyHandle> PhysicsAssetProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(FSkeletalMeshClothBuildParams, PhysicsAsset), (UClass*)FSkeletalMeshClothBuildParams::StaticStruct());
		if(AssetNameProperty.IsValid())
		{
			PhysicsAssetProperty->MarkHiddenByCustomization();
		}

		IDetailCategoryBuilder& TargetCategory = DetailBuilder.EditCategory(TEXT("Target"));

		FDetailWidgetRow& AssetRow = TargetCategory.AddCustomRow(LOCTEXT("Asset_FilterString", "Target Asset"));

		AssetRow.NameContent()
		[
			TargetAssetProperty->CreatePropertyNameWidget()
		];
		AssetRow.ValueContent()
		[
			SNew(SComboButton)
			.OnGetMenuContent(this, &FClothCreateSettingsCustomization::OnGetTargetAssetMenu)
			.ContentPadding(2.f)
			.ButtonStyle(FAppStyle::Get(), "PropertyEditor.AssetComboStyle")
			.ForegroundColor(FAppStyle::GetColor("PropertyEditor.AssetName.ColorAndOpacity"))
			.CollapseMenuOnParentFocus(true)
			.ButtonContent()
			[
				SNew(STextBlock)
				.Text(this, &FClothCreateSettingsCustomization::GetTargetAssetText)
				.TextStyle(FAppStyle::Get(), "PropertyEditor.AssetClass")
				.Font(DetailBuilder.GetDetailFont())
			]
		];

		FDetailWidgetRow& LodRow = TargetCategory.AddCustomRow(LOCTEXT("Lod_FilterString", "Target Lod"));

		LodRow.NameContent()
		[
			LodIndexProperty->CreatePropertyNameWidget()
		];
		LodRow.ValueContent()
		[
			SNew(SComboButton)
			.IsEnabled(this, &FClothCreateSettingsCustomization::CanSelectLod)
			.OnGetMenuContent(this, &FClothCreateSettingsCustomization::OnGetTargetLodMenu)
			.ContentPadding(2.f)
			.CollapseMenuOnParentFocus(true)
			.ButtonContent()
			[
				SNew(STextBlock)
				.Text(this, &FClothCreateSettingsCustomization::GetTargetLodText)
				.Font(DetailBuilder.GetDetailFont())
			]
		];
	}
	else
	{
		// Specifics for non sub-import
		TSharedPtr<IPropertyHandle> RemapProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(FSkeletalMeshClothBuildParams, bRemapParameters), (UClass*)FSkeletalMeshClothBuildParams::StaticStruct());
		if(RemapProperty.IsValid())
		{
			RemapProperty->MarkHiddenByCustomization();
		}
	}
}

#undef LOCTEXT_NAMESPACE
