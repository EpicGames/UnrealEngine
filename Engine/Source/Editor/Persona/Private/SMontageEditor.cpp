// Copyright Epic Games, Inc. All Rights Reserved.


#include "SMontageEditor.h"
#include "Misc/MessageDialog.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Editor.h"

#include "Animation/DebugSkelMeshComponent.h"
#include "Animation/EditorCompositeSection.h"
#include "IDocumentation.h"

#include "SAnimTimingPanel.h"
#include "SAnimNotifyPanel.h"
#include "SAnimMontageScrubPanel.h"
#include "SAnimMontagePanel.h"
#include "SAnimMontageSectionsPanel.h"
#include "ScopedTransaction.h"
#include "AnimPreviewInstance.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Images/SImage.h"
#include "Factories/AnimMontageFactory.h"
#include "Editor.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "AnimTimeline/AnimModel_AnimMontage.h"
#include "AnimTimeline/SAnimTimeline.h"

#define LOCTEXT_NAMESPACE "AnimSequenceEditor"

//////////////////////////////////////////////////////////////////////////
// SMontageEditor

TSharedRef<SWidget> SMontageEditor::CreateDocumentAnchor()
{
	return IDocumentation::Get()->CreateAnchor(TEXT("Engine/Animation/AnimMontage"));
}

void SMontageEditor::Construct(const FArguments& InArgs, const FMontageEditorRequiredArgs& InRequiredArgs)
{
	MontageObj = InArgs._Montage;
	check(MontageObj);

	AnimModel = MakeShared<FAnimModel_AnimMontage>(InRequiredArgs.PreviewScene, InRequiredArgs.EditableSkeleton, InRequiredArgs.CommandList, MontageObj);
	AnimModel->OnSelectObjects = FOnObjectsSelected::CreateSP(this, &SAnimEditorBase::OnSelectionChanged);
	AnimModel->OnSectionsChanged = InArgs._OnSectionsChanged;
	AnimModel->OnInvokeTab = InArgs._OnInvokeTab;

	AnimModel->OnEditCurves = FOnEditCurves::CreateLambda([this, InOnEditCurves = InArgs._OnEditCurves](UAnimSequenceBase* InAnimSequence, const TArray<IAnimationEditor::FCurveEditInfo>& InCurveInfo, const TSharedPtr<ITimeSliderController>& InExternalTimeSliderController)
	{
		InOnEditCurves.ExecuteIfBound(InAnimSequence, InCurveInfo, TimelineWidget->GetTimeSliderController());
	});

	AnimModel->Initialize();

	MontageObj->RegisterOnMontageChanged(UAnimMontage::FOnMontageChanged::CreateSP(AnimModel.Get(), &FAnimModel_AnimMontage::RefreshTracks));

	SAnimEditorBase::Construct( SAnimEditorBase::FArguments()
		.OnObjectsSelected(InArgs._OnObjectsSelected)
		.AnimModel(AnimModel), 
		InRequiredArgs.PreviewScene );
}

SMontageEditor::~SMontageEditor()
{
	if (MontageObj)
	{
		MontageObj->UnregisterOnMontageChanged(this);
	}
}

#undef LOCTEXT_NAMESPACE

