// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Animation/AnimMontage.h"
#include "SAnimEditorBase.h"

class SAnimMontagePanel;
class SAnimMontageScrubPanel;
class SAnimMontageSectionsPanel;
class SAnimNotifyPanel;
class SAnimTimingPanel;
class FAnimModel_AnimMontage;

struct FMontageEditorRequiredArgs
{
	FMontageEditorRequiredArgs(const TSharedRef<class IPersonaPreviewScene>& InPreviewScene, const TSharedRef<class IEditableSkeleton>& InEditableSkeleton, FSimpleMulticastDelegate& InOnSectionsChanged, const TSharedRef<FUICommandList>& InCommandList)
		: PreviewScene(InPreviewScene)
		, EditableSkeleton(InEditableSkeleton)
		, OnSectionsChanged(InOnSectionsChanged)
		, CommandList(InCommandList)
	{}

	TSharedRef<class IPersonaPreviewScene> PreviewScene;
	TSharedRef<class IEditableSkeleton> EditableSkeleton;
	FSimpleMulticastDelegate& OnSectionsChanged;
	TSharedRef<FUICommandList> CommandList;
};

//////////////////////////////////////////////////////////////////////////
// SMontageEditor

/** Overall animation montage editing widget. This mostly contains functions for editing the UAnimMontage.

	SMontageEditor will create the SAnimMontagePanel which is mostly responsible for setting up the UI 
	portion of the Montage tool and registering callbacks to the SMontageEditor to do the actual editing.
	
*/
class SMontageEditor : public SAnimEditorBase
{
public:
	SLATE_BEGIN_ARGS( SMontageEditor )
		: _Montage(NULL)
		{}

	SLATE_EVENT(FOnInvokeTab, OnInvokeTab)
	SLATE_EVENT(FSimpleDelegate, OnCurvesChanged)
	SLATE_EVENT(FSimpleDelegate, OnSectionsChanged)
	SLATE_ARGUMENT( UAnimMontage*, Montage )
	SLATE_EVENT(FOnObjectsSelected, OnObjectsSelected)
	SLATE_EVENT(FOnEditCurves, OnEditCurves)

	SLATE_END_ARGS()

private:
	TSharedPtr<FAnimModel_AnimMontage> AnimModel;

public:
	~SMontageEditor();

	void Construct(const FArguments& InArgs, const FMontageEditorRequiredArgs& InRequiredArgs);

	void SetMontageObj(UAnimMontage * NewMontage);
	UAnimMontage * GetMontageObj() const { return MontageObj; }

	virtual UAnimationAsset* GetEditorObject() const override { return GetMontageObj(); }

private:
	/** Pointer to the animation sequence being edited */
	// @todo fix this
	UAnimMontage* MontageObj;


protected:
	//~ Begin SAnimEditorBase Interface
	virtual TSharedRef<SWidget> CreateDocumentAnchor() override;
	//~ End SAnimEditorBase Interface
};
