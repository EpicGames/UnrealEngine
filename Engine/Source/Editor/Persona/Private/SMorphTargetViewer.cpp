// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMorphTargetViewer.h"

#include "Animation/AnimInstance.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "Animation/MorphTarget.h"
#include "Engine/RendererSettings.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "GPUSkinCache.h"
#include "HAL/PlatformApplicationMisc.h"
#include "IPersonaPreviewScene.h"
#include "InterchangeMeshUtilities.h"
#include "InterchangeManager.h"
#include "Misc/App.h"
#include "Rendering/SkeletalMeshLODImporterData.h"
#include "ScopedTransaction.h"
#include "SkeletalMeshAttributes.h"
#include "SkeletalRenderPublic.h"
#include "SkinnedAssetCompiler.h"
#include "SRenameMorphTargetDialog.h"
#include "Textures/SlateIcon.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Input/SSpinBox.h"
#include "Editor.h"

#define LOCTEXT_NAMESPACE "SMorphTargetViewer"

static const FName ColumnId_MorphTargetNameLabel( "MorphTargetName" );
static const FName ColumnID_MorphTargetWeightLabel( "Weight" );
static const FName ColumnID_MorphTargetEditLabel( "Edit" );
static const FName ColumnID_MorphTargetVertCountLabel( "NumberOfVerts" );


//////////////////////////////////////////////////////////////////////////
// SMorphTargetListRow

typedef TSharedPtr< FDisplayedMorphTargetInfo > FDisplayedMorphTargetInfoPtr;

class SMorphTargetListRow
	: public SMultiColumnTableRow< FDisplayedMorphTargetInfoPtr >
{
public:

	SLATE_BEGIN_ARGS( SMorphTargetListRow ) {}

	/** The item for this row **/
	SLATE_ARGUMENT( FDisplayedMorphTargetInfoPtr, Item )

		/* The SMorphTargetViewer that we push the morph target weights into */
		SLATE_ARGUMENT( class SMorphTargetViewer*, MorphTargetViewer )

		/* Widget used to display the list of morph targets */
		SLATE_ARGUMENT( TSharedPtr<SMorphTargetListType>, MorphTargetListView )

		SLATE_END_ARGS()

		void Construct( const FArguments& InArgs, const TSharedRef<IPersonaPreviewScene>& InPreviewScene, const TSharedRef<STableViewBase>& OwnerTableView );

	/** Overridden from SMultiColumnTableRow.  Generates a widget for this column of the tree row. */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn( const FName& ColumnName ) override;

private:

	/**
	* Called when the user begins/ends dragging the slider on the SSpinBox
	*/
	void OnBeginSlideMorphTargetWeight();
	void OnEndSlideMorphTargetWeight(float Value);
	
	/**
	* Called when the user changes the value of the SSpinBox
	*
	* @param NewWeight - The new number the SSpinBox is set to
	*
	*/
	void OnMorphTargetWeightChanged( float NewWeight );
	
	/**
	* Called when the user types the value and enters
	*
	* @param NewWeight - The new number the SSpinBox is set to
	*
	*/
	void OnMorphTargetWeightValueCommitted( float NewWeight, ETextCommit::Type CommitType);
	
	/**
	* Called to know if we enable or disable the weight sliders
	*/
	bool IsMorphTargetWeightSliderEnabled() const;
	
	/**
	* Show the tooltip for the weight widget
	*/
	FText GetMorphTargetWeightSliderToolTip() const;

	/** Auto fill check call back functions */
	void OnMorphTargetAutoFillChecked(ECheckBoxState InState);
	ECheckBoxState IsMorphTargetAutoFillChangedChecked() const;
	
	/**
	* Returns the weight of this morph target
	*
	* @return SearchText - The new number the SSpinBox is set to
	*
	*/
	float GetWeight() const;

	/* The SMorphTargetViewer that we push the morph target weights into */
	SMorphTargetViewer* MorphTargetViewer;

	/** Widget used to display the list of morph targets */
	TSharedPtr<SMorphTargetListType> MorphTargetListView;

	/** The name and weight of the morph target */
	FDisplayedMorphTargetInfoPtr	Item;

	/** Preview scene - we invalidate this etc. */
	TWeakPtr<IPersonaPreviewScene> PreviewScenePtr;
};

void SMorphTargetListRow::Construct( const FArguments& InArgs, const TSharedRef<IPersonaPreviewScene>& InPreviewScene, const TSharedRef<STableViewBase>& InOwnerTableView )
{
	Item = InArgs._Item;
	MorphTargetViewer = InArgs._MorphTargetViewer;
	MorphTargetListView = InArgs._MorphTargetListView;
	PreviewScenePtr = InPreviewScene;

	check( Item.IsValid() );

	SMultiColumnTableRow< FDisplayedMorphTargetInfoPtr >::Construct( FSuperRowType::FArguments(), InOwnerTableView );
}

TSharedRef< SWidget > SMorphTargetListRow::GenerateWidgetForColumn( const FName& ColumnName )
{
	if ( ColumnName == ColumnId_MorphTargetNameLabel )
	{
		FText SourceFilenamesTooltip;
		FText MorphNameText = FText::FromName(Item->Name);
		FText CarriageReturn = FText::FromString(TEXT("\n"));

		for (int32 LodIndex = 0; LodIndex < Item->MorphLodInfos.Num(); ++LodIndex)
		{
			const FDisplayedMorphTargetInfo::FMorphLodInfo& MorphLodInfo = Item->MorphLodInfos[LodIndex];
			const bool bLodCustomImported = !MorphLodInfo.SourceFilename.IsEmpty();
			const bool bIsValidLodMorph = MorphLodInfo.bIsValidLodMorph;
			const int32 GeneratedFromLodIndex = MorphLodInfo.GeneratedFromLodIndex;
			if (LodIndex == 0)
			{
				if(!bIsValidLodMorph)
				{
					MorphNameText = FText::Format(LOCTEXT("MorphRowNameInvalid", "{0} (Empty or invalid Morph Data)"), MorphNameText);
				}
				else if (bLodCustomImported)
				{
					MorphNameText = FText::Format(LOCTEXT("MorphRowNameCustomImport", "{0} (Imported by File)"), MorphNameText);
					SourceFilenamesTooltip = FText::Format(LOCTEXT("Lod0Tooltip_ImportByFile", "This morph target was imported from source filename: {0}"), FText::FromString(MorphLodInfo.SourceFilename));
				}
				else if(GeneratedFromLodIndex != INDEX_NONE)
				{
					//LOD 0 could be generated only from it's own import data (inline generated)
					ensure(GeneratedFromLodIndex == 0);
					MorphNameText = FText::Format(LOCTEXT("MorphRowNameGeneratedFromLOD", "{0} (Generated From Source Data)"), MorphNameText);
					SourceFilenamesTooltip = LOCTEXT("Lod0Tooltip_GeneratedFromLOD", "This morph target was generated from is base mesh source data.");
				}
				else if(MorphLodInfo.bIsGeneratedByEngine)
				{
					MorphNameText = FText::Format(LOCTEXT("MorphRowNameGenerated", "{0} (Generated By Engine)"), MorphNameText);
					SourceFilenamesTooltip = LOCTEXT("Lod0Tooltip_Generated", "This morph target was generated by an engine tool.");
				}
				else
				{
					MorphNameText = FText::Format(LOCTEXT("MorphRowNameImportWithLodGeometry", "{0} (Imported with LOD geometry)"), MorphNameText);
					SourceFilenamesTooltip = LOCTEXT("Lod0Tooltip_ImportWithLodGeometry", "This morph target was imported with the LOD geometry.");
				}
			}
			else if (bIsValidLodMorph) //Do not add invalid morph to the tooltip
			{
				if (bLodCustomImported)
				{
					SourceFilenamesTooltip = FText::Format(LOCTEXT("LodXTooltip_ImportByFile", "{0}{1}LOD {2} was imported from source filename: {3}"), SourceFilenamesTooltip, CarriageReturn, FText::AsNumber(LodIndex), FText::FromString(MorphLodInfo.SourceFilename));
				}
				else if (GeneratedFromLodIndex != INDEX_NONE)
				{
					SourceFilenamesTooltip = FText::Format(LOCTEXT("LodXTooltip_GeneratedFromLOD", "{0}{1}LOD {2} was generated from a lower LOD {3}"), SourceFilenamesTooltip, CarriageReturn, FText::AsNumber(LodIndex), GeneratedFromLodIndex);
				}
				else if(MorphLodInfo.bIsGeneratedByEngine)
				{
					SourceFilenamesTooltip = FText::Format(LOCTEXT("LodXTooltip_Generated", "{0}{1}LOD {2} was generated by an engine tool"), SourceFilenamesTooltip, CarriageReturn, FText::AsNumber(LodIndex));
				}
				else
				{
					SourceFilenamesTooltip = FText::Format(LOCTEXT("LodXTooltip_ImportWithLodGeometry", "{0}{1}LOD {2} was imported with the LOD geometry"), SourceFilenamesTooltip, CarriageReturn, FText::AsNumber(LodIndex));
				}
			}
		}

		return
			SNew( SVerticalBox )

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding( 0.0f, 4.0f )
			.VAlign( VAlign_Center )
			[
				SNew( STextBlock )
				.Text( MorphNameText )
				.ToolTipText(SourceFilenamesTooltip)
				.HighlightText( MorphTargetViewer->GetFilterText() )
			];
	}
	else if ( ColumnName == ColumnID_MorphTargetWeightLabel )
	{
		// Encase the SSpinbox in an SVertical box so we can apply padding. Setting ItemHeight on the containing SListView has no effect :-(
		return
			SNew( SVerticalBox )

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding( 0.0f, 1.0f )
			.VAlign( VAlign_Center )
			[
				SNew( SSpinBox<float> )
				.MinSliderValue(-1.f)
				.MaxSliderValue(1.f)
				.Value( this, &SMorphTargetListRow::GetWeight )
				.OnBeginSliderMovement(this, &SMorphTargetListRow::OnBeginSlideMorphTargetWeight)
				.OnEndSliderMovement(this, &SMorphTargetListRow::OnEndSlideMorphTargetWeight)
				.OnValueChanged( this, &SMorphTargetListRow::OnMorphTargetWeightChanged )
				.OnValueCommitted( this, &SMorphTargetListRow::OnMorphTargetWeightValueCommitted )
				.IsEnabled(this, &SMorphTargetListRow::IsMorphTargetWeightSliderEnabled)
				.ToolTipText(this, &SMorphTargetListRow::GetMorphTargetWeightSliderToolTip)
			];
	}
	else if ( ColumnName == ColumnID_MorphTargetEditLabel )
	{
		return
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.0f, 1.0f)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			[
				SNew(SCheckBox)
				.OnCheckStateChanged(this, &SMorphTargetListRow::OnMorphTargetAutoFillChecked)
				.IsChecked(this, &SMorphTargetListRow::IsMorphTargetAutoFillChangedChecked)
			];
	}
	else
	{
		return
				SNew(SVerticalBox)

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(4.0f, 4.0f)
				.VAlign(VAlign_Center)
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Right)
					[
						SNew(STextBlock)
						.Text(FText::AsNumber(Item->NumberOfVerts))
						.HighlightText(MorphTargetViewer->GetFilterText())
					]
				];
	}
}

void SMorphTargetListRow::OnBeginSlideMorphTargetWeight()
{
	GEditor->BeginTransaction(LOCTEXT("OverrideMorphTargetWeight", "Override Morph Target Weight"));
}

void SMorphTargetListRow::OnEndSlideMorphTargetWeight(float Value)
{
	GEditor->EndTransaction();
}

void SMorphTargetListRow::OnMorphTargetAutoFillChecked(ECheckBoxState InState)
{
	Item->bAutoFillData = InState == ECheckBoxState::Checked;

	if (Item->bAutoFillData)
	{
		// clear value so that it can be filled up
		MorphTargetViewer->AddMorphTargetOverride(Item->Name, 0.f, true);
	}
	else
	{
		// Setting value, add the override
		MorphTargetViewer->AddMorphTargetOverride(Item->Name, Item->Weight, false);
	}
}

ECheckBoxState SMorphTargetListRow::IsMorphTargetAutoFillChangedChecked() const
{
	return (Item->bAutoFillData)? ECheckBoxState::Checked: ECheckBoxState::Unchecked;
}

void SMorphTargetListRow::OnMorphTargetWeightChanged( float NewWeight )
{
	// First change this item...
	// the delta feature is a bit confusing when debugging morphtargets, and you're not sure why it's changing, so I'm disabling it for now. 
	// I think in practice, you want each morph target to move independentaly. It is very unlikely you'd like to move multiple things together. 

	const float MorphTargetMaxBlendWeight = UE::SkeletalRender::Settings::GetMorphTargetMaxBlendWeight();
	NewWeight = FMath::Clamp(NewWeight, -MorphTargetMaxBlendWeight, MorphTargetMaxBlendWeight);

#if 0 
	float Delta = NewWeight - GetWeight();
#endif
	Item->Weight = NewWeight;
	Item->bAutoFillData = false;

	MorphTargetViewer->AddMorphTargetOverride( Item->Name, Item->Weight, false );

	PreviewScenePtr.Pin()->InvalidateViews();

#if 0 
	TArray< TSharedPtr< FDisplayedMorphTargetInfo > > SelectedRows = MorphTargetListView->GetSelectedItems();

	// ...then any selected rows need changing by the same delta
	for ( auto ItemIt = SelectedRows.CreateIterator(); ItemIt; ++ItemIt )
	{
		TSharedPtr< FDisplayedMorphTargetInfo > RowItem = ( *ItemIt );

		if ( RowItem != Item ) // Don't do "this" row again if it's selected
		{
			RowItem->Weight = FMath::Clamp(RowItem->Weight + Delta, -MorphTargetMaxBlendWeight, MorphTargetMaxBlendWeight);
			RowItem->bAutoFillData = false;
			MorphTargetViewer->AddMorphTargetOverride( RowItem->Name, RowItem->Weight, false );
		}
	}
#endif
}

void SMorphTargetListRow::OnMorphTargetWeightValueCommitted( float NewWeight, ETextCommit::Type CommitType)
{
	if (CommitType == ETextCommit::OnEnter || CommitType == ETextCommit::OnUserMovedFocus)
	{
		const float MorphTargetMaxBlendWeight = UE::SkeletalRender::Settings::GetMorphTargetMaxBlendWeight();
		NewWeight = FMath::Clamp(NewWeight, -MorphTargetMaxBlendWeight, MorphTargetMaxBlendWeight);

		Item->Weight = NewWeight;
		Item->bAutoFillData = false;

		MorphTargetViewer->AddMorphTargetOverride(Item->Name, Item->Weight, false);

		TArray< TSharedPtr< FDisplayedMorphTargetInfo > > SelectedRows = MorphTargetListView->GetSelectedItems();

		// ...then any selected rows need changing by the same delta
		for(auto ItemIt = SelectedRows.CreateIterator(); ItemIt; ++ItemIt)
		{
			TSharedPtr< FDisplayedMorphTargetInfo > RowItem = (*ItemIt);

			if(RowItem != Item) // Don't do "this" row again if it's selected
			{
				RowItem->Weight = NewWeight;
				RowItem->bAutoFillData = false;
				MorphTargetViewer->AddMorphTargetOverride(RowItem->Name, RowItem->Weight, false);
			}
		}

		PreviewScenePtr.Pin()->InvalidateViews();
	}
}

bool SMorphTargetListRow::IsMorphTargetWeightSliderEnabled() const
{
	const uint32 CVarMorphTargetModeValue = GetDefault<URendererSettings>()->bUseGPUMorphTargets;
	return FGPUSkinCache::IsGPUSkinCacheEnabled(GMaxRHIFeatureLevel) ? CVarMorphTargetModeValue > 0 : true;
}

FText SMorphTargetListRow::GetMorphTargetWeightSliderToolTip() const
{
	if (!IsMorphTargetWeightSliderEnabled())
	{
		return LOCTEXT("MorphTargetWeightSliderTooltip", "When using skin cache, the morph target must use the GPU to affect the mesh");
	}
	return FText();
}

float SMorphTargetListRow::GetWeight() const 
{ 
	if (Item->bAutoFillData)
	{
		float CurrentWeight = 0.f;

		USkeletalMeshComponent* SkelComp = PreviewScenePtr.Pin()->GetPreviewMeshComponent();
		UAnimInstance* AnimInstance = (SkelComp) ? SkelComp->GetAnimInstance() : nullptr;
		if (AnimInstance)
		{
			// make sure if they have value that's not same as saved value
			const TMap<FName, float>& MorphCurves = AnimInstance->GetAnimationCurveList(EAnimCurveType::MorphTargetCurve);
			const float* CurrentWeightPtr = MorphCurves.Find(Item->Name);
			if (CurrentWeightPtr)
			{
				CurrentWeight = *CurrentWeightPtr;
			}
		}
		return CurrentWeight;
	}
	else
	{
		if (USkeletalMeshComponent* SkelComp = PreviewScenePtr.Pin()->GetPreviewMeshComponent())
		{
			if (const float* Weight = SkelComp->GetMorphTargetCurves().Find(Item->Name))
			{
				return *Weight;
			}
		}
	}

	return 0;
}
//////////////////////////////////////////////////////////////////////////
// SMorphTargetViewer

void SMorphTargetViewer::Construct(const FArguments& InArgs, const TSharedRef<IPersonaPreviewScene>& InPreviewScene, FSimpleMulticastDelegate& OnPostUndo)
{
	PreviewScenePtr = InPreviewScene;

	SkeletalMesh = InPreviewScene->GetPreviewMeshComponent()->GetSkeletalMeshAsset();
	InPreviewScene->RegisterOnPreviewMeshChanged( FOnPreviewMeshChanged::CreateSP( this, &SMorphTargetViewer::OnPreviewMeshChanged ) );
	InPreviewScene->RegisterOnMorphTargetsChanged(FSimpleDelegate::CreateSP(this, &SMorphTargetViewer::OnMorphTargetsChanged));
	OnPostUndo.Add(FSimpleDelegate::CreateSP(this, &SMorphTargetViewer::OnPostUndo));

	const FText SkeletalMeshName = SkeletalMesh ? FText::FromString( SkeletalMesh->GetName() ) : LOCTEXT( "MorphTargetMeshNameLabel", "No Skeletal Mesh Present" );

	SkeletalMesh->GetOnMeshChanged().AddSP(this, &SMorphTargetViewer::OnMeshChanged);

	ChildSlot
	[
		SNew( SVerticalBox )
		
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew( STextBlock )
			.Text( SkeletalMeshName )
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0,2)
		[
			SNew(SHorizontalBox)
			// Import morph target
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew( SButton )
				.OnClicked(this, &SMorphTargetViewer::OnImportMorphTargetButton)
				[
					SNew(STextBlock)
					.ToolTipText(LOCTEXT("ImportCustomMorphTargetButtonTooltip", "Import a new morph target from a file."))
					.Text(LOCTEXT("ImportCustomMorphTargetButtonText", "Import Morph Target"))
				]
			]
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0,2)
		[
			SNew(SHorizontalBox)
			// Filter entry
			+SHorizontalBox::Slot()
			.FillWidth( 1 )
			[
				SAssignNew( NameFilterBox, SSearchBox )
				.SelectAllTextWhenFocused( true )
				.OnTextChanged( this, &SMorphTargetViewer::OnFilterTextChanged )
				.OnTextCommitted( this, &SMorphTargetViewer::OnFilterTextCommitted )
			]
		]

		+ SVerticalBox::Slot()
		.FillHeight( 1.0f )		// This is required to make the scrollbar work, as content overflows Slate containers by default
		[
			SAssignNew( MorphTargetListView, SMorphTargetListType )
			.ListItemsSource( &MorphTargetList )
			.OnGenerateRow( this, &SMorphTargetViewer::GenerateMorphTargetRow )
			.OnContextMenuOpening( this, &SMorphTargetViewer::OnGetContextMenuContent )
			.OnSelectionChanged( this, &SMorphTargetViewer::OnRowsSelectedChanged )
			.HeaderRow
			(
				SNew( SHeaderRow )
				+ SHeaderRow::Column( ColumnId_MorphTargetNameLabel )
				.DefaultLabel( LOCTEXT( "MorphTargetNameLabel", "Morph Target Name" ) )

				+ SHeaderRow::Column( ColumnID_MorphTargetWeightLabel )
				.DefaultLabel( LOCTEXT( "MorphTargetWeightLabel", "Weight" ) )

				+ SHeaderRow::Column(ColumnID_MorphTargetEditLabel)
				.DefaultLabel(LOCTEXT("MorphTargetEditLabel", "Auto"))

				+ SHeaderRow::Column( ColumnID_MorphTargetVertCountLabel )
				.DefaultLabel( LOCTEXT("MorphTargetVertCountLabel", "Vert Count") )
			)
		]
	];

	CreateMorphTargetList();
}

TArray<FName> SMorphTargetViewer::GetSelectedMorphTargetNames() const
{
	TArray< TSharedPtr< FDisplayedMorphTargetInfo > > SelectedRows = MorphTargetListView->GetSelectedItems();

	TArray<FName> SelectedMorphTargetNames;
	for (auto ItemIt = SelectedRows.CreateIterator(); ItemIt; ++ItemIt)
	{
		TSharedPtr< FDisplayedMorphTargetInfo > RowItem = (*ItemIt);
		SelectedMorphTargetNames.AddUnique(RowItem->Name);
	}

	return SelectedMorphTargetNames;
}

TSharedRef<SWidget> SMorphTargetViewer::AsWidget()
{
	return this->AsShared();
}

void SMorphTargetViewer::OnPreviewMeshChanged(class USkeletalMesh* OldPreviewMesh, class USkeletalMesh* NewPreviewMesh)
{
	if (SkeletalMesh)
	{
		SkeletalMesh->GetOnMeshChanged().RemoveAll( this );
	}
	
	SkeletalMesh = NewPreviewMesh;
	
	if (SkeletalMesh)
	{
		SkeletalMesh->GetOnMeshChanged().AddSP(this, &SMorphTargetViewer::OnMeshChanged);
	}
	CreateMorphTargetList( NameFilterBox->GetText().ToString() );
}

void SMorphTargetViewer::OnMorphTargetsChanged()
{
	CreateMorphTargetList(NameFilterBox->GetText().ToString());
}

void SMorphTargetViewer::OnFilterTextChanged( const FText& SearchText )
{
	FilterText = SearchText;

	CreateMorphTargetList( SearchText.ToString() );
}

void SMorphTargetViewer::OnFilterTextCommitted( const FText& SearchText, ETextCommit::Type CommitInfo )
{
	// Just do the same as if the user typed in the box
	OnFilterTextChanged( SearchText );
}

TSharedRef<ITableRow> SMorphTargetViewer::GenerateMorphTargetRow(TSharedPtr<FDisplayedMorphTargetInfo> InInfo, const TSharedRef<STableViewBase>& OwnerTable)
{
	check( InInfo.IsValid() );

	return
		SNew( SMorphTargetListRow, PreviewScenePtr.Pin().ToSharedRef(), OwnerTable )
		.Item( InInfo )
		.MorphTargetViewer( this )
		.MorphTargetListView( MorphTargetListView );
}

TSharedPtr<SWidget> SMorphTargetViewer::OnGetContextMenuContent() const
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder( bShouldCloseWindowAfterMenuSelection, NULL);

	MenuBuilder.BeginSection("MorphTargetAction", LOCTEXT( "MorphsAction", "Selected Item Actions" ) );
	{
		TArray< TSharedPtr< FDisplayedMorphTargetInfo > > SelectedRows = MorphTargetListView->GetSelectedItems();
		const int32 SelectionCount = SelectedRows.Num();

		if (SelectionCount > 0)
		{
			const int32 LodCount = SkeletalMesh->GetLODNum();
			bool bShowImportMenu = false;
			struct FLodMorphTargetImportMenuInfo
			{
				bool bShowImportMenu = true;
				bool bShowReimportMenu = true;
				bool bShowReimportWithNewFileMenu = true;
				bool IsMenuShow() const { return bShowImportMenu || bShowReimportMenu || bShowReimportWithNewFileMenu; }
				void HideMenus()
				{
					bShowImportMenu = false;
					bShowReimportMenu = false;
					bShowReimportWithNewFileMenu = false;
				}
			};
			TMap<int32, FLodMorphTargetImportMenuInfo> MenuInforPerLods;
			for (int32 LodIndex = 0; LodIndex < LodCount; ++LodIndex)
			{
				FLodMorphTargetImportMenuInfo& MenuInfo = MenuInforPerLods.FindOrAdd(LodIndex);
				if (!SkeletalMesh->HasMeshDescription(LodIndex))
				{
					MenuInfo.HideMenus();
					continue;
				}

				for (int32 RowIndex = 0; RowIndex < SelectionCount; ++RowIndex)
				{
					UMorphTarget* MorphTarget = SkeletalMesh->FindMorphTarget(SelectedRows[RowIndex]->Name);
					if (MorphTarget)
					{
						//Look at the lod 0 to see if the morph target is an imported morph target
						constexpr int32 LodIndex0 = 0;
						if (!MorphTarget->IsCustomImported(LodIndex0))
						{
							MenuInfo.bShowImportMenu = false;
							MenuInfo.bShowReimportMenu = false;
							MenuInfo.bShowReimportWithNewFileMenu = false;
						}
						else
						{
							const bool bIsCustomImportedLod = MorphTarget->IsCustomImported(LodIndex);
							MenuInfo.bShowImportMenu &= (SelectionCount == 1 && !bIsCustomImportedLod);
							MenuInfo.bShowReimportMenu &= bIsCustomImportedLod;
							MenuInfo.bShowReimportWithNewFileMenu &= (SelectionCount == 1 && bIsCustomImportedLod);
						}
					}
				}
				bShowImportMenu |= MenuInfo.IsMenuShow();
			}

			if (bShowImportMenu)
			{
				//Create the import menu for every lods
				for (int32 LodIndex = 0; LodIndex < LodCount; ++LodIndex)
				{
					const FLodMorphTargetImportMenuInfo& MenuInfo = MenuInforPerLods.FindOrAdd(LodIndex);

					//We can import a morph only if the lod is custom imported
					if (MenuInfo.IsMenuShow())
					{
						FText SubMenuLabel = FText::Format(LOCTEXT("LodSubMenu", "LOD {0}"), FText::AsNumber(LodIndex));
						MenuBuilder.AddSubMenu(SubMenuLabel,
							FText(),
							FNewMenuDelegate::CreateLambda([this, LodIndex, MenuInfo](FMenuBuilder& InSubMenuBuilder)
								{
									FUIAction Action;
									
									//Import Morph target
									if (MenuInfo.bShowImportMenu)
									{
										Action.ExecuteAction = FExecuteAction::CreateSP(const_cast<SMorphTargetViewer*>(this), &SMorphTargetViewer::OnReimportMorphTargets, LodIndex);
										Action.CanExecuteAction = nullptr;
										const FText Label = LOCTEXT("ImportMorphTargetLabel", "Import");
										const FText ToolTipText = LOCTEXT("ImportMorphTargetTooltip", "Import all selected custom imported morph target");
										InSubMenuBuilder.AddMenuEntry(Label, ToolTipText, FSlateIcon(), Action);
									}
									//Reimport Morph target
									if (MenuInfo.bShowReimportMenu)
									{
										Action.ExecuteAction = FExecuteAction::CreateSP(const_cast<SMorphTargetViewer*>(this), &SMorphTargetViewer::OnReimportMorphTargets, LodIndex);
										Action.CanExecuteAction = nullptr;
										const FText Label = LOCTEXT("ReimportMorphTargetLabel", "Reimport");
										const FText ToolTipText = LOCTEXT("ReimportMorphTargetTooltip", "Reimport all selected custom imported morph target");
										InSubMenuBuilder.AddMenuEntry(Label, ToolTipText, FSlateIcon(), Action);
									}

									//Reimport Morph target with new file
									if (MenuInfo.bShowReimportWithNewFileMenu)
									{
										Action.ExecuteAction = FExecuteAction::CreateSP(const_cast<SMorphTargetViewer*>(this), &SMorphTargetViewer::OnReimportMorphTargetsWithNewFile, LodIndex);
										Action.CanExecuteAction = nullptr;
										const FText Label = LOCTEXT("ReimportWithNewFileMorphTargetLabel", "Reimport With New File");
										const FText ToolTipText = LOCTEXT("ReimportWithNewFileMorphTargetTooltip", "Ask a file and re-import every selected morph target.");
										InSubMenuBuilder.AddMenuEntry(Label, ToolTipText, FSlateIcon(), Action);
									}
								})
						);
					}
				}

				MenuBuilder.AddMenuSeparator();
			}

			//Basic morph target context menu
			{
				FUIAction Action;

				//Rename morph target
				{
					Action.ExecuteAction = FExecuteAction::CreateSP(const_cast<SMorphTargetViewer*>(this), &SMorphTargetViewer::OnRenameMorphTargets);
					Action.CanExecuteAction = nullptr;
					const FText Label = LOCTEXT("RenameMorphTargetLabel", "Rename");
					const FText ToolTipText = LOCTEXT("RenameMorphTargetTooltip", "Rename the selected morph targets");
					MenuBuilder.AddMenuEntry(Label, ToolTipText, FSlateIcon(), Action);
				}

				//Delete morph target
				{
					Action.ExecuteAction = FExecuteAction::CreateSP(const_cast<SMorphTargetViewer*>(this), &SMorphTargetViewer::OnDeleteMorphTargets);
					Action.CanExecuteAction = FCanExecuteAction::CreateSP(this, &SMorphTargetViewer::CanPerformDelete);
					const FText Label = LOCTEXT("DeleteMorphTargetButtonLabel", "Delete");
					const FText ToolTipText = LOCTEXT("DeleteMorphTargetButtonTooltip", "Deletes the selected morph targets.");
					MenuBuilder.AddMenuEntry(Label, ToolTipText, FSlateIcon(), Action);
				}

				//Copy morph target name
				{
					Action.ExecuteAction = FExecuteAction::CreateSP(const_cast<SMorphTargetViewer*>(this), &SMorphTargetViewer::OnCopyMorphTargetNames);
					Action.CanExecuteAction = nullptr;
					const FText Label = LOCTEXT("CopyMorphTargetNamesButtonLabel", "Copy Names");
					const FText ToolTipText = LOCTEXT("CopyMorphTargetNamesButtonTooltip", "Copy the names of selected morph targets to clipboard");
					MenuBuilder.AddMenuEntry(Label, ToolTipText, FSlateIcon(), Action);
				}
			}
		}
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void SMorphTargetViewer::CreateMorphTargetList( const FString& SearchText )
{
	TArray<FName> SelectedMorphTargets = GetSelectedMorphTargetNames();
	
	MorphTargetList.Empty();

	if ( SkeletalMesh )
	{
		UDebugSkelMeshComponent* MeshComponent = PreviewScenePtr.Pin()->GetPreviewMeshComponent();
		TArray<TObjectPtr<UMorphTarget>>& MorphTargets = SkeletalMesh->GetMorphTargets();

		bool bDoFiltering = !SearchText.IsEmpty();

		for ( int32 I = 0; I < MorphTargets.Num(); ++I )
		{
			if ( bDoFiltering && !MorphTargets[I]->GetName().Contains( SearchText ) )
			{
				continue; // Skip items that don't match our filter
			}

			int32 NumberOfVerts = (MorphTargets[I]->GetMorphLODModels().Num() > 0)? MorphTargets[I]->GetMorphLODModels()[0].Vertices.Num() : 0;

			TArray<FDisplayedMorphTargetInfo::FMorphLodInfo> MorphLodInfos;
			
			MorphLodInfos.AddDefaulted(SkeletalMesh->GetLODNum());
			for (int32 LodIndex = 0; LodIndex < SkeletalMesh->GetLODNum(); ++LodIndex)
			{
				//Only query the data if the morph lod has some data (is valid)
				MorphLodInfos[LodIndex].bIsValidLodMorph = MorphTargets[I]->HasDataForLOD(LodIndex);
				if(MorphLodInfos[LodIndex].bIsValidLodMorph)
				{
					const bool bIsGeneratedByEngine = MorphTargets[I]->IsGeneratedByEngine(LodIndex);
					// Morph target generated by the LOD reduction should be tag properly in the UI
					// We will keep the generated lod index to be able to output it in the UI
					if(LodIndex > 0 && bIsGeneratedByEngine && SkeletalMesh->IsReductionActive(LodIndex))
					{
						FSkeletalMeshOptimizationSettings ReductionSettings = SkeletalMesh->GetReductionSettings(LodIndex);
						const int32 ReductionBaseLodIndex = ReductionSettings.BaseLOD;
						MorphLodInfos[LodIndex].GeneratedFromLodIndex = ReductionBaseLodIndex;
					}
					MorphLodInfos[LodIndex].SourceFilename = MorphTargets[I]->GetCustomImportedSourceFilename(LodIndex);
					MorphLodInfos[LodIndex].bIsGeneratedByEngine = bIsGeneratedByEngine;
				}
			}
			TSharedRef<FDisplayedMorphTargetInfo> Info = FDisplayedMorphTargetInfo::Make( MorphTargets[I]->GetFName(), NumberOfVerts, MorphLodInfos);
			if(MeshComponent)
			{
				const float *CurveValPtr = MeshComponent->GetMorphTargetCurves().Find( MorphTargets[I]->GetFName() );
				if(CurveValPtr)
				{
					Info.Get().Weight = (*CurveValPtr);
					Info.Get().bAutoFillData = false;
				}
			}

			MorphTargetList.Add( Info );
		}
	}

	for (const FDisplayedMorphTargetInfoPtr& Item : MorphTargetList)
	{
		if (SelectedMorphTargets.Contains(Item->Name))
		{
			MorphTargetListView->SetItemSelection(Item, true);
		}
	}
	
	
	NotifySelectionChange();
	MorphTargetListView->RequestListRefresh();
}

void SMorphTargetViewer::AddMorphTargetOverride( FName& Name, float Weight, bool bRemoveZeroWeight )
{
	UDebugSkelMeshComponent* Mesh = PreviewScenePtr.Pin()->GetPreviewMeshComponent();

	if ( Mesh )
	{
		GEditor->BeginTransaction(LOCTEXT("MorphTargetOverrideChanged", "Changed Morph Target Override"));
		Mesh->SetFlags(RF_Transactional);
		Mesh->Modify();
		Mesh->SetMorphTarget( Name, Weight, bRemoveZeroWeight );
		GEditor->EndTransaction();
	}
}

bool SMorphTargetViewer::CanPerformDelete() const
{
	TArray< TSharedPtr< FDisplayedMorphTargetInfo > > SelectedRows = MorphTargetListView->GetSelectedItems();
	return SelectedRows.Num() > 0;
}

void SMorphTargetViewer::OnRenameMorphTargets()
{
	auto RenameMorphTarget = [this](UMorphTarget* SelectMorphTarget)
		{
			TSharedRef <SRenameMorphTargetDialog> RenameWidgetDialog = SNew(SRenameMorphTargetDialog)
				.SkeletalMesh(SkeletalMesh)
				.MorphTarget(SelectMorphTarget);

			TSharedRef<SWindow> RenameWindowDialog =
				SNew(SWindow)
				.Title(LOCTEXT("RenameMorphTargetWindowTitle", "Rename Morph target"))
				.SizingRule(ESizingRule::Autosized)
				.SupportsMaximize(false)
				.SupportsMinimize(false);

			RenameWindowDialog->SetContent(SNew(SBox)
				.MinDesiredWidth(320.0f)
				[
					RenameWidgetDialog
				]);
			TSharedPtr<SWindow> CurrentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
			FSlateApplication::Get().AddModalWindow(RenameWindowDialog, CurrentWindow);
		};

	{
		FScopedSkeletalMeshPostEditChange PostEditChangeScope(SkeletalMesh);
		TArray< TSharedPtr< FDisplayedMorphTargetInfo > > SelectedRows = MorphTargetListView->GetSelectedItems();

		for (int32 RowIndex = 0; RowIndex < SelectedRows.Num(); ++RowIndex)
		{
			UMorphTarget* MorphTarget = SkeletalMesh->FindMorphTarget(SelectedRows[RowIndex]->Name);
			if (MorphTarget)
			{
				RenameMorphTarget(MorphTarget);
			}
		}
	}

	//Wait until the skeletal mesh compilation is done
	FSkinnedAssetCompilingManager::Get().FinishCompilation({ SkeletalMesh });

	CreateMorphTargetList(NameFilterBox->GetText().ToString());
}

void SMorphTargetViewer::OnDeleteMorphTargets()
{
	TArray< TSharedPtr< FDisplayedMorphTargetInfo > > SelectedRows = MorphTargetListView->GetSelectedItems();

	// Clean up override usage
	TArray<FName> MorphTargetNames;
	for (int32 RowIndex = 0; RowIndex < SelectedRows.Num(); ++RowIndex)
	{
		UMorphTarget* MorphTarget = SkeletalMesh->FindMorphTarget(SelectedRows[RowIndex]->Name);
		if(MorphTarget)
		{
			AddMorphTargetOverride(SelectedRows[RowIndex]->Name, 0.0f, true);
			MorphTargetNames.Add(SelectedRows[RowIndex]->Name);
		}
	}

	//Scope a skeletal mesh build
	{
		FScopedSkeletalMeshPostEditChange ScopePostEditChange(SkeletalMesh);
		// Remove from mesh
		SkeletalMesh->RemoveMorphTargets(MorphTargetNames);
	}
	//Wait until the skeletal mesh compilation is done
	FSkinnedAssetCompilingManager::Get().FinishCompilation({ SkeletalMesh });

	CreateMorphTargetList( NameFilterBox->GetText().ToString() );
}

void SMorphTargetViewer::OnCopyMorphTargetNames()
{
	FString CopyText;

	TArray< TSharedPtr< FDisplayedMorphTargetInfo > > SelectedRows = MorphTargetListView->GetSelectedItems();
	for (int RowIndex = 0; RowIndex < SelectedRows.Num(); ++RowIndex)
	{
		UMorphTarget* MorphTarget = SkeletalMesh->FindMorphTarget(SelectedRows[RowIndex]->Name);
		if (MorphTarget)
		{
			CopyText += FString::Printf(TEXT("%s\r\n"), *MorphTarget->GetName());
		}
	}

	if(!CopyText.IsEmpty())
	{
		FPlatformApplicationMisc::ClipboardCopy(*CopyText);
	}
}

FReply SMorphTargetViewer::OnImportMorphTargetButton()
{
	constexpr int32 LodIndex0 = 0;
	constexpr bool bWithNewFileTrue = true;
	constexpr bool bRecreateMorphTargetListTrue = true;
	InternalImportMorphTarget(LodIndex0, bWithNewFileTrue, nullptr, bRecreateMorphTargetListTrue);
	
	return FReply::Handled();
}

void SMorphTargetViewer::OnReimportMorphTargets(int32 LodIndex)
{
	constexpr bool bWithNewFileFalse = false;
	TArray< TSharedPtr< FDisplayedMorphTargetInfo > > SelectedRows = MorphTargetListView->GetSelectedItems();
	for (int RowIndex = 0; RowIndex < SelectedRows.Num(); ++RowIndex)
	{
		UMorphTarget* MorphTarget = SkeletalMesh->FindMorphTarget(SelectedRows[RowIndex]->Name);
		if (MorphTarget)
		{
			constexpr bool bRecreateMorphTargetListFalse = false;
			InternalImportMorphTarget(LodIndex, bWithNewFileFalse, MorphTarget, bRecreateMorphTargetListFalse);
		}
	}
	CreateMorphTargetList();
}

void SMorphTargetViewer::OnReimportMorphTargetsWithNewFile(int32 LodIndex)
{
	constexpr bool bWithNewFileTrue = true;
	TArray< TSharedPtr< FDisplayedMorphTargetInfo > > SelectedRows = MorphTargetListView->GetSelectedItems();
	for (int RowIndex = 0; RowIndex < SelectedRows.Num(); ++RowIndex)
	{
		UMorphTarget* MorphTarget = SkeletalMesh->FindMorphTarget(SelectedRows[RowIndex]->Name);
		if (MorphTarget)
		{
			constexpr bool bRecreateMorphTargetListFalse = false;
			InternalImportMorphTarget(LodIndex, bWithNewFileTrue, MorphTarget, bRecreateMorphTargetListFalse);
		}
	}
	CreateMorphTargetList();
}

void SMorphTargetViewer::InternalImportMorphTarget(int32 LodIndex, bool bWithNewFile, UMorphTarget* ReimportMorphTarget, bool bRecreateMorphTargetList)
{
	FString Filename;

	bool bInternalWithNewFile = bWithNewFile || !ReimportMorphTarget || !ReimportMorphTarget->IsCustomImported(LodIndex);
	if (bInternalWithNewFile)
	{
		FText PickerTitle = FText::Format(NSLOCTEXT("SMorphTargetViewer", "OnImportNewMorphTarget_PickerTitle", "Choose a file to import a morph target for LOD{0}"), FText::AsNumber(LodIndex));

		if (!UInterchangeMeshUtilities::ShowMeshFilePicker(Filename, PickerTitle))
		{
			return;
		}
	}
	else if(ensure(ReimportMorphTarget && ReimportMorphTarget->IsCustomImported(LodIndex)))
	{
		Filename = ReimportMorphTarget->GetCustomImportedSourceFilename(LodIndex);
	}

	constexpr bool bAsyncFalse = false;
	UInterchangeManager& InterchangeManager = UInterchangeManager::GetInterchangeManager();
	const UInterchangeSourceData* SourceData = InterchangeManager.CreateSourceData(Filename);
	//Import a new morph target
	TFuture<bool> FutureResult = UInterchangeMeshUtilities::ImportMorphTarget(SkeletalMesh, LodIndex, SourceData, bAsyncFalse, ReimportMorphTarget ? ReimportMorphTarget->GetName() : FString());
	ensure(FutureResult.IsReady());

	if (bRecreateMorphTargetList)
	{
		CreateMorphTargetList();
	}
}

SMorphTargetViewer::~SMorphTargetViewer()
{
	if (PreviewScenePtr.IsValid())
	{
		UDebugSkelMeshComponent* Mesh = PreviewScenePtr.Pin()->GetPreviewMeshComponent();

		if (Mesh)
		{
			Mesh->ClearMorphTargets();
		}
	}
}

void SMorphTargetViewer::OnPostUndo()
{
	CreateMorphTargetList();
	NotifySelectionChange();
}

void SMorphTargetViewer::OnMeshChanged()
{
	CreateMorphTargetList();
	NotifySelectionChange();
}

void SMorphTargetViewer::NotifySelectionChange() const
{
	TArray<FName> SelectedMorphTargetNames = GetSelectedMorphTargetNames();

	// stil have to call this even if empty, otherwise it won't clear it
	PreviewMorphTargets(SelectedMorphTargetNames);
}

void SMorphTargetViewer::OnRowsSelectedChanged(TSharedPtr<FDisplayedMorphTargetInfo> Item, ESelectInfo::Type SelectInfo)
{
	NotifySelectionChange();
}

void SMorphTargetViewer::PreviewMorphTargets(const TArray<FName>& SelectedMorphTargetNames) const
{
	UDebugSkelMeshComponent* PreviewComponent = PreviewScenePtr.Pin()->GetPreviewMeshComponent();
	if (PreviewComponent)
	{
		PreviewComponent->MorphTargetOfInterests.Reset();

		if (SelectedMorphTargetNames.Num() > 0)
		{
			if (SkeletalMesh)
			{
				for (const FName& MorphTargetName : SelectedMorphTargetNames)
				{
					int32 MorphtargetIdx;
					UMorphTarget* MorphTarget = SkeletalMesh->FindMorphTargetAndIndex(MorphTargetName, MorphtargetIdx);
					if (MorphTarget != nullptr)
					{
						PreviewComponent->MorphTargetOfInterests.AddUnique(MorphTarget);
					}
				}
			}

			PreviewScenePtr.Pin()->InvalidateViews();
			PreviewComponent->PostInitMeshObject(PreviewComponent->MeshObject);
		}
	}
}


#undef LOCTEXT_NAMESPACE

