// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IMorphTargetViewer.h"
#include "SlateFwd.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "IPersonaPreviewScene.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"

class USkeletalMesh;
class UMorphTarget;

//////////////////////////////////////////////////////////////////////////
// FDisplayedMorphTargetInfo

class FDisplayedMorphTargetInfo
{
public:
	FName Name;
	float Weight;
	bool bAutoFillData;
	int32 NumberOfVerts;
	struct FMorphLodInfo
	{
		FString SourceFilename;
		bool bIsGeneratedByEngine = false;
		int32 GeneratedFromLodIndex = INDEX_NONE;
		bool bIsValidLodMorph = false;
	};
	TArray<FMorphLodInfo> MorphLodInfos;

	/** Static function for creating a new item, but ensures that you can only have a TSharedRef to one */
	static TSharedRef<FDisplayedMorphTargetInfo> Make(const FName& Source, int32 InNumberOfVerts, const TArray<FMorphLodInfo>& InMorphLodInfos)
	{
		return MakeShareable(new FDisplayedMorphTargetInfo(Source, InNumberOfVerts, InMorphLodInfos));
	}

protected:
	/** Hidden constructor, always use Make above */
	FDisplayedMorphTargetInfo(const FName& InSource, int32 InNumberOfVerts, const TArray<FMorphLodInfo>& InMorphLodInfos)
		: Name( InSource )
		, Weight( 0 )
		, bAutoFillData(true)
		, NumberOfVerts(InNumberOfVerts)
		, MorphLodInfos(InMorphLodInfos)
	{}

	/** Hidden constructor, always use Make above */
	FDisplayedMorphTargetInfo() {}
};

typedef SListView< TSharedPtr<FDisplayedMorphTargetInfo> > SMorphTargetListType;

//////////////////////////////////////////////////////////////////////////
// SMorphTargetViewer

class SMorphTargetViewer : public SCompoundWidget, public IMorphTargetViewer
{
public:
	SLATE_BEGIN_ARGS( SMorphTargetViewer )
	{}
	
	SLATE_END_ARGS()

	/**
	* Slate construction function
	*
	* @param InArgs - Arguments passed from Slate
	*
	*/
	void Construct( const FArguments& InArgs, const TSharedRef<class IPersonaPreviewScene>& InPreviewScene, FSimpleMulticastDelegate& OnPostUndo );

	/** IMorphTargetViewer overrides */
	virtual TArray<FName> GetSelectedMorphTargetNames() const override;
	virtual TSharedRef<SWidget> AsWidget() override;

	/**
	* Destructor - resets the morph targets
	*
	*/
	virtual ~SMorphTargetViewer();

	/**
	* Is registered with Persona to handle when its preview mesh is changed.
	*
	* @param NewPreviewMesh - The new preview mesh being used by Persona
	*
	*/
	void OnPreviewMeshChanged(class USkeletalMesh* OldPreviewMesh, class USkeletalMesh* NewPreviewMesh);

	/**
	* Is registered with Persona to handle when its preview mesh morph targets has changed.
	*/
	void OnMorphTargetsChanged();

	/**
	* Filters the SListView when the user changes the search text box (NameFilterBox)
	*
	* @param SearchText - The text the user has typed
	*
	*/
	void OnFilterTextChanged( const FText& SearchText );

	/**
	* Filters the SListView when the user hits enter or clears the search box
	* Simply calls OnFilterTextChanged
	*
	* @param SearchText - The text the user has typed
	* @param CommitInfo - Not used
	*
	*/
	void OnFilterTextCommitted( const FText& SearchText, ETextCommit::Type CommitInfo );

	/**
	* Create a widget for an entry in the tree from an info
	*
	* @param InInfo - Shared pointer to the morph target we're generating a row for
	* @param OwnerTable - The table that owns this row
	*
	* @return A new Slate widget, containing the UI for this row
	*/
	TSharedRef<ITableRow> GenerateMorphTargetRow(TSharedPtr<FDisplayedMorphTargetInfo> InInfo, const TSharedRef<STableViewBase>& OwnerTable);

	/**
	* Adds a morph target override or updates the weight for an existing one
	*
	* @param Name - Name of the morph target we want to override
	* @param Weight - How much of this morph target to apply (0.0 - 1.0)
	*
	*/
	void AddMorphTargetOverride( FName& Name, float Weight, bool bRemoveZeroWeight );

	/**
	* Tells the AnimInstance to reset all of its morph target curves
	*
	*/
	void ResetMorphTargets();
	
	/**
	* Provides state to the IsEnabled property of the delete morph targets button
	*
	*/
	bool CanPerformDelete() const;

	/**
	* Handler for the rename morph targets button
	*/
	void OnRenameMorphTargets();

	/**
	* Handler for the delete morph targets button
	*/
	void OnDeleteMorphTargets();

	/** Handler for copying morph taret names */
	void OnCopyMorphTargetNames();

	/** Handler for importing a new morph target */
	FReply OnImportMorphTargetButton();
	void OnReimportMorphTargets(int32 LodIndex);
	void OnReimportMorphTargetsWithNewFile(int32 LodIndex);

	void InternalImportMorphTarget(int32 LodIndex, bool bWithNewFile, UMorphTarget* ReimportMorphTarget, bool bRecreateMorphTargetList);



	/**
	* Accessor so our rows can grab the filtertext for highlighting
	*
	*/
	FText& GetFilterText() { return FilterText; }

	/**
	 * Refreshes the morph target list after an undo
	 */
	void OnPostUndo();

	/**
	 * Refreshes the morph target list after a mesh changed
	 */
	void OnMeshChanged();

private:

	/** Handler for context menus */
	TSharedPtr<SWidget> OnGetContextMenuContent() const;

	/** Handler for row selection change */
	void OnRowsSelectedChanged(TSharedPtr<FDisplayedMorphTargetInfo> Item, ESelectInfo::Type SelectInfo);

	/**
	* Clears and rebuilds the table, according to an optional search string
	*
	* @param SearchText - Optional search string
	*
	*/
	void CreateMorphTargetList( const FString& SearchText = FString() );

	// Sets the selected morph target
	void PreviewMorphTargets(const TArray<FName>& SelectedMorphTargetNames) const;

	/** Pointer back to the Persona that owns us */
	TWeakPtr<class IPersonaPreviewScene> PreviewScenePtr;

	/** Box to filter to a specific morph target name */
	TSharedPtr<SSearchBox>	NameFilterBox;

	/** Widget used to display the list of morph targets */
	TSharedPtr<SMorphTargetListType> MorphTargetListView;

	/** A list of morph targets. Used by the MorphTargetListView. */
	TArray< TSharedPtr<FDisplayedMorphTargetInfo> > MorphTargetList;

	/** The skeletal mesh that we grab the morph targets from */
	USkeletalMesh* SkeletalMesh;

	/** Current text typed into NameFilterBox */
	FText FilterText;

	/** notify selection change to Persona */
	void NotifySelectionChange() const;
};
