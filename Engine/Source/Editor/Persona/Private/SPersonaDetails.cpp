// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPersonaDetails.h"

#include "DetailsViewArgs.h"
#include "IDetailsView.h"
#include "Layout/Children.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "SlotBase.h"
#include "Widgets/SBoxPanel.h"

void SPersonaDetails::Construct(const FArguments& InArgs)
{
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bHideSelectionTip = true;

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

	TSharedRef<SVerticalBox> Content = SNew(SVerticalBox);

	if (InArgs._TopContent.IsValid())
	{
		Content->AddSlot()
		.AutoHeight()
		[
			InArgs._TopContent.ToSharedRef()
		];
	}

	Content->AddSlot()
	.FillHeight(1.0f)
	[
		DetailsView.ToSharedRef()
	];

	if (InArgs._BottomContent.IsValid())
	{
		Content->AddSlot()
		.AutoHeight()
		[
			InArgs._BottomContent.ToSharedRef()
		];
	}

	ChildSlot
	[
		Content
	];
}
