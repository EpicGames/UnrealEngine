// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class SWidget;

class SPersonaDetails : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SPersonaDetails) {}

	/** Optional content to display above the details panel */
	SLATE_ARGUMENT(TSharedPtr<SWidget>, TopContent)

	/** Optional content to display below the details panel */
	SLATE_ARGUMENT(TSharedPtr<SWidget>, BottomContent)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	TSharedPtr<class IDetailsView> DetailsView;
};
