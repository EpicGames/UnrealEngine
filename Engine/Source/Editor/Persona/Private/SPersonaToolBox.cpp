// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPersonaToolBox.h"

#include "EditorModeManager.h"
#include "EdMode.h"
#include "PersonaTabs.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Tools/UEdMode.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Layout/SUniformWrapPanel.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Input/SSegmentedControl.h"

#define LOCTEXT_NAMESPACE "PersonaToolbox"

FToolBoxSummoner::FToolBoxSummoner(TSharedPtr<FPersonaAssetEditorToolkit> InPersonaEditorToolkit) : FWorkflowTabFactory(FPersonaTabs::ToolboxID, InPersonaEditorToolkit), PersonaAssetEditorToolkit(InPersonaEditorToolkit)
{
	TabLabel = LOCTEXT("ToolboxTabTitle", "Toolbox");
	TabIcon = FSlateIcon(FAppStyle::Get().GetStyleSetName(), "LevelEditor.Tabs.Modes");

	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("ToolBoxMenu", "Toolbox");
	ViewMenuTooltip = LOCTEXT("ToolBoxMenu_ToolTip", "Used to display Edit Mode widget.");	
}

TSharedRef<SDockTab> FToolBoxSummoner::SpawnTab(const FWorkflowTabSpawnInfo& Info) const
{
	TSharedRef<SDockTab> DockTab = FWorkflowTabFactory::SpawnTab(Info);

	const TWeakPtr<FPersonaAssetEditorToolkit> WeakToolkit = PersonaAssetEditorToolkit;	
	DockTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateLambda([WeakToolkit](TSharedRef<SDockTab> DockTab)
	{
		if (WeakToolkit.IsValid())
		{
			WeakToolkit.Pin()->GetEditorModeManager().ActivateDefaultMode();
		}
	}));

	return DockTab;
}

TSharedRef<SWidget> FToolBoxSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	return SNew(SBox)
		.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("ToolboxTab")))
		[
			SNew(SPersonaToolbox, PersonaAssetEditorToolkit.Pin().ToSharedRef())
		];
}

bool FToolBoxSummoner::CanSpawnTab(const FSpawnTabArgs& SpawnArgs, TWeakPtr<FTabManager> WeakTabManager) const
{
	return FWorkflowTabFactory::CanSpawnTab(SpawnArgs, WeakTabManager) && PersonaAssetEditorToolkit.Pin()->GetHostedToolkit().IsValid();
}

SPersonaToolbox::~SPersonaToolbox()
{
	if (StatusBarMessageHandle.IsValid() && PersonaEditor.IsValid())
	{
		GEditor->GetEditorSubsystem<UStatusBarSubsystem>()->PopStatusBarMessage(PersonaEditor.Pin()->GetToolMenuName(), StatusBarMessageHandle);
		StatusBarMessageHandle.Reset();
	}

	if (ModeToolBarContainer)
	{
		ModeToolBarContainer->SetContent(SNullWidget::NullWidget);
	}
	if (ModeToolHeader)
	{
		ModeToolHeader->SetContent(SNullWidget::NullWidget);
	}
	if (InlineContentHolder)
	{
		InlineContentHolder->SetContent(SNullWidget::NullWidget);
	}

	if (PersonaEditor.IsValid())
	{
		PersonaEditor.Pin()->GetOnAttachToolkit().RemoveAll(this);
		PersonaEditor.Pin()->GetOnDetachToolkit().RemoveAll(this);
	}
	
	PersonaEditor.Reset();
}

void SPersonaToolbox::Construct(const FArguments& InArgs, const TSharedRef<FPersonaAssetEditorToolkit>& InOwningEditor)
{
	PersonaEditor = InOwningEditor;
	// Hook into delegates to handle attaching and detaching of toolkits
	InOwningEditor->GetOnAttachToolkit().AddSP(this, &SPersonaToolbox::AttachToolkit);
	InOwningEditor->GetOnDetachToolkit().AddSP(this, &SPersonaToolbox::DetachToolkit);

	// Modeled after FModeToolkit::CreatePrimaryModePanel
	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		.Padding(0.0f)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Left)
			[
				SAssignNew(ModeToolBarContainer, SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				.Padding(FMargin(4, 0, 0, 0))
			]

			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.Padding(0.0, 8.0, 0.0, 0.0)
				.AutoHeight()
				[
					SAssignNew(ModeToolHeader, SBorder)
					.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				]
				+ SVerticalBox::Slot()
				.FillHeight(1)
				[
					SAssignNew(InlineContentHolder, SBorder)
					.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
					.Visibility(this, &SPersonaToolbox::GetInlineContentHolderVisibility)
				]
			]
		]
	];

	// Directly attach toolkit if one is being hosted
	if (InOwningEditor->GetHostedToolkit().IsValid())
	{
		AttachToolkit(InOwningEditor->GetHostedToolkit().ToSharedRef());
	}
}

void SPersonaToolbox::AttachToolkit(const TSharedRef<IToolkit>& InToolkit)
{
	UpdateInlineContent(InToolkit, InToolkit->GetInlineContent());
}

void SPersonaToolbox::DetachToolkit(const TSharedRef<IToolkit>& InToolkit)
{
	UpdateInlineContent(nullptr, SNullWidget::NullWidget);
}

void SPersonaToolbox::SetOwningTab(const TSharedRef<SDockTab>& InOwningTab)
{
	OwningTab = InOwningTab;
}

void SPersonaToolbox::UpdateInlineContent(const TSharedPtr<IToolkit>& Toolkit, TSharedPtr<SWidget> InlineContent)
{
	// The display name that the owning tab should have as its label
	FText TabName;

	// The icon that should be displayed in the parent tab
	const FSlateBrush* TabIcon = nullptr;

	if (StatusBarMessageHandle.IsValid() && PersonaEditor.IsValid())
	{
		GEditor->GetEditorSubsystem<UStatusBarSubsystem>()->PopStatusBarMessage(PersonaEditor.Pin()->GetToolMenuName(), StatusBarMessageHandle);
		StatusBarMessageHandle.Reset();	

		const TWeakPtr<FModeToolkit> ModeToolkit = StaticCastSharedPtr<FModeToolkit>(Toolkit);
		if (ModeToolkit.IsValid())
		{
			TabName = ModeToolkit.Pin()->GetEditorModeDisplayName();
			TabIcon = ModeToolkit.Pin()->GetEditorModeIcon().GetSmallIcon();

			UpdatePalette(ModeToolkit);
			
			// Show the name of the active tool in the statusbar.
			// FIXME: We should also be showing Ctrl/Shift/Alt LMB/RMB shortcuts.
			StatusBarMessageHandle = GEditor->GetEditorSubsystem<UStatusBarSubsystem>()->PushStatusBarMessage(
					PersonaEditor.Pin()->GetToolMenuName(),
					TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(ModeToolkit.Pin()->AsShared(), &FModeToolkit::GetActiveToolDisplayName)));
			
		}
	}
	else
	{
		TabName = NSLOCTEXT("Persona", "ToolboxTab", "Toolbox");
		TabIcon = FAppStyle::Get().GetBrush("LevelEditor.Tabs.Modes");
	}

	if (InlineContent.IsValid() && InlineContentHolder.IsValid())
	{
		InlineContentHolder->SetContent(InlineContent.ToSharedRef());
	}

	const TSharedPtr<SDockTab> OwningTabPinned = OwningTab.Pin();
	if (OwningTabPinned.IsValid())
	{
		OwningTabPinned->SetLabel(TabName);
		OwningTabPinned->SetTabIcon(TabIcon);
	}
}

void SPersonaToolbox::UpdatePalette(const TWeakPtr<FModeToolkit>& InWeakToolkit) const
{
	if (!InWeakToolkit.IsValid())
	{
		return;
	}

	if (const TSharedPtr<FModeToolkit> Toolkit = InWeakToolkit.Pin())
	{
		if (Toolkit->HasIntegratedToolPalettes())
		{
			TSharedRef<SSegmentedControl<FName>> PaletteTabBox = SNew(SSegmentedControl<FName>)
			.UniformPadding(FMargin(8.f, 3.f))
			.Value_Lambda([InWeakToolkit]()
			{
				return InWeakToolkit.IsValid() ? InWeakToolkit.Pin()->GetCurrentPalette() : NAME_None;
			})
			.OnValueChanged_Lambda([InWeakToolkit](const FName& Palette)
			{
				if (InWeakToolkit.IsValid())
				{
					InWeakToolkit.Pin()->SetCurrentPalette(Palette);
				}
			});

			// Only show if there's more than one entry.
			PaletteTabBox->SetVisibility(TAttribute<EVisibility>::Create(
				TAttribute<EVisibility>::FGetter::CreateLambda([PaletteTabBox]() -> EVisibility { 
					return PaletteTabBox->NumSlots() > 1 ? EVisibility::Visible : EVisibility::Collapsed;
				})));
			

			const TSharedPtr<FModeToolkit> ModeToolkit = InWeakToolkit.Pin();
			
			// Also build the toolkit here
			TArray<FName> PaletteNames;
			ModeToolkit->GetToolPaletteNames(PaletteNames);

			const TSharedRef<SWidgetSwitcher> PaletteSwitcher = SNew(SWidgetSwitcher)
				.WidgetIndex_Lambda( [PaletteNames, InWeakToolkit] () -> int32
				{
					if (InWeakToolkit.IsValid())
					{
						int32 FoundIndex;
						if (PaletteNames.Find(InWeakToolkit.Pin()->GetCurrentPalette(), FoundIndex))
						{
							return FoundIndex;	
						}
					}
					return 0;
				});
					
			const TSharedPtr<FUICommandList> CommandList = ModeToolkit->GetToolkitCommands();
			for(const FName& Palette : PaletteNames)
			{
				const FName ToolbarCustomizationName = ModeToolkit->GetEditorMode() ?
					ModeToolkit->GetEditorMode()->GetModeInfo().ToolbarCustomizationName :
					ModeToolkit->GetScriptableEditorMode()->GetModeInfo().ToolbarCustomizationName;
				FUniformToolBarBuilder ModeToolbarBuilder(CommandList, FMultiBoxCustomization(ToolbarCustomizationName));
				ModeToolbarBuilder.SetStyle(&FAppStyle::Get(), "PaletteToolBar");

				ModeToolkit->BuildToolPalette(Palette, ModeToolbarBuilder);

				TSharedRef<SWidget> PaletteWidget = ModeToolbarBuilder.MakeWidget();
				constexpr bool bRebuildChildren = false;
				PaletteTabBox->AddSlot(Palette, false)
				.Text(ModeToolkit->GetToolPaletteDisplayName(Palette));

				PaletteSwitcher->AddSlot()
				[
					PaletteWidget
				];
			}

			PaletteTabBox->RebuildChildren();

			if (ModeToolHeader)
			{
				ModeToolHeader->SetContent
				(
					SNew(SVerticalBox)
					+SVerticalBox::Slot()
					.Padding(8.f, 0.f, 0.f, 8.f)
					.AutoHeight()
					.HAlign(HAlign_Center)
					[
						PaletteTabBox
					]
					+SVerticalBox::Slot()
					.AutoHeight()
					[
						PaletteSwitcher
					]
				);
			}
		}
		else
		{
			if (ModeToolHeader)
			{
				ModeToolHeader->SetContent(SNullWidget::NullWidget);
			}
			if (ModeToolBarContainer)
			{
				ModeToolBarContainer->SetContent(SNullWidget::NullWidget);
			}
		}

		if (InlineContentHolder.IsValid())
		{
			if (const TSharedPtr<SWidget> InlineContent = Toolkit->GetInlineContent())
			{
				InlineContentHolder->SetContent(
				SNew(SScrollBox)
				+ SScrollBox::Slot()
				[
					InlineContent.ToSharedRef()
				]);
			}
		}
	}
}
struct 
EVisibility SPersonaToolbox::GetInlineContentHolderVisibility() const
{
	return InlineContentHolder->GetContent() == SNullWidget::NullWidget ? EVisibility::Collapsed : EVisibility::Visible;
}

#undef LOCTEXT_NAMESPACE // "PersonaToolbox"