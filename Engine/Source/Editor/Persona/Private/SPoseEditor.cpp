// Copyright Epic Games, Inc. All Rights Reserved.


#include "SPoseEditor.h"
#include "AnimPreviewInstance.h"
#include "Misc/MessageDialog.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Layout/SSplitter.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "ScopedTransaction.h"
#include "Widgets/Input/SSearchBox.h"
#include "Animation/AnimSingleNodeInstance.h"
#include "UObject/UObjectIterator.h"
#include "HAL/PlatformApplicationMisc.h"

#include "Widgets/Text/SInlineEditableTextBlock.h"

#include "Framework/Commands/GenericCommands.h"
#include "PoseEditorCommands.h"

#define LOCTEXT_NAMESPACE "AnimPoseEditor"

static const FName ColumnId_PoseNameLabel("Pose Name");
static const FName ColumnID_PoseWeightLabel("Weight");
static const FName ColumnId_CurveNameLabel("Curve Name");
static const FName ColumnID_CurveValueLabel("Curve Value");

const float MaxPoseWeight = 1.f;

//////////////////////////////////////////////////////////////////////////
// SPoseEditor

void SPoseEditor::Construct(const FArguments& InArgs, const TSharedRef<class IPersonaToolkit>& InPersonaToolkit, const TSharedRef<IEditableSkeleton>& InEditableSkeleton, const TSharedRef<class IPersonaPreviewScene>& InPreviewScene)
{
	PoseAssetObj = InArgs._PoseAsset;
	check(PoseAssetObj);

	SAnimEditorBase::Construct(SAnimEditorBase::FArguments()
		.DisplayAnimTimeline(false),
		InPreviewScene);

	NonScrollEditorPanels->AddSlot()
 	.FillHeight(1)
 	.Padding(0, 10)
	[
 		SNew( SPoseViewer, InPersonaToolkit, InEditableSkeleton, InPreviewScene)
		.PoseAsset(PoseAssetObj)
 	];
}


//////////////////////////////////////////////////////////////////////////
// SPoseListRow

FText SPoseListRow::GetName() const
{
	return (FText::FromName(Item->Name));
}

void SPoseListRow::OnNameCommitted(const FText& InText, ETextCommit::Type InCommitType) const
{
	// for now only allow enter
	// because it is important to keep the unique names per pose
	if (InCommitType == ETextCommit::OnEnter)
	{
		FName NewName = FName(*InText.ToString());
		FName OldName = Item->Name;

		if (PoseViewerPtr.IsValid() && PoseViewerPtr.Pin()->ModifyName(OldName, NewName))
		{
			Item->Name = NewName;
		}
	}
}

bool SPoseListRow::OnVerifyNameChanged(const FText& InText, FText& OutErrorMessage)
{
	bool bVerifyName = false;

	FName NewName = FName(*InText.ToString());

	if (NewName == NAME_None)
	{
		OutErrorMessage = LOCTEXT("EmptyPoseName", "Poses must have a name!");
	}

	if (PoseViewerPtr.IsValid())
	{
		UPoseAsset* PoseAsset = PoseViewerPtr.Pin()->PoseAssetPtr.Get();
		if (PoseAsset != nullptr)
		{
			if (PoseAsset->ContainsPose(NewName))
			{
				OutErrorMessage = LOCTEXT("NameAlreadyUsedByTheSameAsset", "The name is used by another pose within the same asset. Please choose another name.");
			}
			else
			{
				bVerifyName = true;
			}
		}
	}

	return bVerifyName;
}


void SPoseListRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, const TSharedRef<IPersonaPreviewScene>& InPreviewScene)
{
	Item = InArgs._Item;
	PoseViewerPtr = InArgs._PoseViewer;
	FilterText = InArgs._FilterText;
	PreviewScenePtr = InPreviewScene;

	check(Item.IsValid());

	SMultiColumnTableRow< TSharedPtr<FDisplayedPoseInfo> >::Construct(FSuperRowType::FArguments(), InOwnerTableView);
}

TSharedRef< SWidget > SPoseListRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	if (ColumnName == ColumnId_PoseNameLabel)
	{
		TSharedPtr< SInlineEditableTextBlock > InlineWidget;

		TSharedRef<SWidget> NameWidget =
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(5.0f)
			.VAlign(VAlign_Center)
			[
				SAssignNew(InlineWidget, SInlineEditableTextBlock)
				.Text(this, &SPoseListRow::GetName)
				.HighlightText(FilterText)
				.ToolTipText(LOCTEXT("PoseName_ToolTip", "Modify Pose Name - Make sure this name is unique among all curves per skeleton."))
				.MaximumLength(NAME_SIZE-1)
				.OnVerifyTextChanged(this, &SPoseListRow::OnVerifyNameChanged)
				.OnTextCommitted(this, &SPoseListRow::OnNameCommitted)
			];

		Item->OnRenameRequested.BindSP(InlineWidget.Get(), &SInlineEditableTextBlock::EnterEditingMode);

		return NameWidget;
	}
	else 
	{
		// Encase the SSpinbox in an SVertical box so we can apply padding. Setting ItemHeight on the containing SListView has no effect :-(
		return
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(5.0f)
			.VAlign(VAlign_Center)
			[
				SNew(SSpinBox<float>)
				.MinSliderValue(-1.f)
				.MaxSliderValue(1.f)
				.MinValue(-MaxPoseWeight)
				.MaxValue(MaxPoseWeight)
				.Value(this, &SPoseListRow::GetWeight)
				.OnValueChanged(this, &SPoseListRow::OnPoseWeightChanged)
				.OnValueCommitted(this, &SPoseListRow::OnPoseWeightValueCommitted)
				.IsEnabled(this, &SPoseListRow::CanChangeWeight)
			];
	}
}

void SPoseListRow::OnPoseWeightChanged(float NewWeight)
{
	Item->Weight = NewWeight;

	if (PoseViewerPtr.IsValid())
	{
		TSharedPtr<SPoseViewer> PoseViewer = PoseViewerPtr.Pin();
		PoseViewer->AddCurveOverride(Item->Name, Item->Weight);

		PreviewScenePtr.Pin()->InvalidateViews();
	}
}

void SPoseListRow::OnPoseWeightValueCommitted(float NewWeight, ETextCommit::Type CommitType)
{
	if (CommitType == ETextCommit::OnEnter || CommitType == ETextCommit::OnUserMovedFocus)
	{
		float NewValidWeight = FMath::Clamp(NewWeight, -MaxPoseWeight, MaxPoseWeight);
		OnPoseWeightChanged(NewValidWeight);
	}
}

float SPoseListRow::GetWeight() const
{
	return Item->Weight;
}

bool SPoseListRow::CanChangeWeight() const
{
	if (PoseViewerPtr.IsValid())
	{
		return PoseViewerPtr.Pin()->IsBasePose(Item->Name) == false;
	}
	else
	{
		return false;
	}
}

//////////////////////////////////////////////////////////////////////////
// SCurveListRow

FText SCurveListRow::GetName() const
{
	return (FText::FromName(Item->Name));
}

FText SCurveListRow::GetValue() const
{
	FText ValueText;
	if (PoseViewerPtr.IsValid())
	{
		TSharedPtr<SPoseViewer> PoseViewer = PoseViewerPtr.Pin();

		// Get pose asset
		UPoseAsset* PoseAsset = PoseViewer->PoseAssetPtr.Get();
		if (PoseAsset != nullptr)
		{
			// Get selected row (only show values if only one selected)
			TArray< TSharedPtr<FDisplayedPoseInfo> > SelectedRows = PoseViewer->PoseListView->GetSelectedItems();
			if (SelectedRows.Num() == 1)
			{
				TSharedPtr<FDisplayedPoseInfo> PoseInfo = SelectedRows[0];

				// Get pose index that we have selected
				int32 PoseIndex = PoseAsset->GetPoseIndexByName(PoseInfo->Name);
				int32 CurveIndex = PoseAsset->GetCurveIndexByName(Item->Name);

				float CurveValue = 0.f;
				bool bSuccess = PoseAsset->GetCurveValue(PoseIndex, CurveIndex, CurveValue);

				if (bSuccess)
				{
					ValueText = FText::FromString(FString::Printf(TEXT("%f"), CurveValue));
				}
			}
		}
	}
	return ValueText;
}


void SCurveListRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
{
	Item = InArgs._Item;
	PoseViewerPtr = InArgs._PoseViewer;

	check(Item.IsValid());

	SMultiColumnTableRow< TSharedPtr<FDisplayedCurveInfo> >::Construct(FSuperRowType::FArguments(), InOwnerTableView);
}

TSharedRef< SWidget > SCurveListRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	// for now we have one colume
	if (ColumnName == ColumnId_CurveNameLabel)
	{
		return
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(5.0f)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(this, &SCurveListRow::GetName)
			];
	}
	else
	{
		return
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(5.0f)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(this, &SCurveListRow::GetValue)
			];
	}
}

//////////////////////////////////////////////////////////////////////////
// SPoseViewer

void SPoseViewer::Construct(const FArguments& InArgs, const TSharedRef<IPersonaToolkit>& InPersonaToolkit, const TSharedRef<IEditableSkeleton>& InEditableSkeleton, const TSharedRef<IPersonaPreviewScene>& InPreviewScene)
{
	PreviewScenePtr = InPreviewScene;
	PersonaToolkitPtr = InPersonaToolkit;
	EditableSkeletonPtr = InEditableSkeleton;
	PoseAssetPtr = InArgs._PoseAsset;
	
	NewPoseName = UPoseAsset::GetUniquePoseName(PoseAssetPtr.Get());

	InPreviewScene->RegisterOnPreviewMeshChanged(FOnPreviewMeshChanged::CreateSP(this, &SPoseViewer::OnPreviewMeshChanged));

	OnDelegatePoseListChangedDelegateHandle = PoseAssetPtr->RegisterOnPoseListChanged(UPoseAsset::FOnPoseListChanged::CreateSP(this, &SPoseViewer::OnPoseAssetModified));

	// Register and bind all our menu commands
	FPoseEditorCommands::Register();
	BindCommands();

	ChildSlot
	[
		SNew(SSplitter)
		.Orientation(EOrientation::Orient_Horizontal)

		// Pose List
		+SSplitter::Slot()
		.Value(1)
		[
			SNew(SBox)
			.Padding(5)
			[
				SNew(SVerticalBox)

				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0, 2)
				[
					SNew(SHorizontalBox)
					// Filter entry
					+ SHorizontalBox::Slot()
					.FillWidth(1)
					[
						SAssignNew(NameFilterBox, SSearchBox)
						.SelectAllTextWhenFocused(true)
						.OnTextChanged(this, &SPoseViewer::OnFilterTextChanged)
						.OnTextCommitted(this, &SPoseViewer::OnFilterTextCommitted)
					]
				]

				+ SVerticalBox::Slot()
				.FillHeight(1)
				.Padding(0, 2)
				[
					SAssignNew(PoseListView, SPoseListType)
					.ListItemsSource(&PoseList)
					.OnGenerateRow(this, &SPoseViewer::GeneratePoseRow)
					.OnContextMenuOpening(this, &SPoseViewer::OnGetContextMenuContent)
					.OnMouseButtonDoubleClick(this, &SPoseViewer::OnListDoubleClick)
					.HeaderRow
					(
						SNew(SHeaderRow)
						+ SHeaderRow::Column(ColumnId_PoseNameLabel)
						.DefaultLabel(LOCTEXT("PoseNameLabel", "Pose Name"))

						+ SHeaderRow::Column(ColumnID_PoseWeightLabel)
						.DefaultLabel(LOCTEXT("PoseWeightLabel", "Weight"))
						)
				]
			]
		]

		// Curve List
		+SSplitter::Slot()
		.Value(1)
		[
			SNew(SBorder)
			.Padding(8)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.DarkGroupBorder"))
			[
				SAssignNew(CurveListView, SCurveListType)
				.ListItemsSource(&CurveList)
				.OnGenerateRow(this, &SPoseViewer::GenerateCurveRow)
				.OnContextMenuOpening(this, &SPoseViewer::OnGetContextMenuContentForCurveList)
				.HeaderRow
				(
					SNew(SHeaderRow)
					+ SHeaderRow::Column(ColumnId_CurveNameLabel)
					.DefaultLabel(LOCTEXT("CurveNameLabel", "Curve Name"))

					+ SHeaderRow::Column(ColumnID_CurveValueLabel)
					.DefaultLabel(LOCTEXT("CurveValueLabel", "Value"))
				)
			]

		]
	];

	CreatePoseList();
	CreateCurveList();
}

void SPoseViewer::OnPreviewMeshChanged(class USkeletalMesh* OldPreviewMesh, class USkeletalMesh* NewPreviewMesh)
{
	CreatePoseList(NameFilterBox->GetText().ToString());
	CreateCurveList(NameFilterBox->GetText().ToString());
}

void SPoseViewer::OnFilterTextChanged(const FText& SearchText)
{
	FilterText = SearchText;

	CreatePoseList(SearchText.ToString());
	CreateCurveList(SearchText.ToString());
}

void SPoseViewer::OnFilterTextCommitted(const FText& SearchText, ETextCommit::Type CommitInfo)
{
	// Just do the same as if the user typed in the box
	OnFilterTextChanged(SearchText);
}

TSharedRef<ITableRow> SPoseViewer::GeneratePoseRow(TSharedPtr<FDisplayedPoseInfo> InInfo, const TSharedRef<STableViewBase>& OwnerTable)
{
	check(InInfo.IsValid());

	return
		SNew(SPoseListRow, OwnerTable, PreviewScenePtr.Pin().ToSharedRef())
		.Item(InInfo)
		.PoseViewer(SharedThis(this))
		.FilterText(GetFilterText());
}

TSharedRef<ITableRow> SPoseViewer::GenerateCurveRow(TSharedPtr<FDisplayedCurveInfo> InInfo, const TSharedRef<STableViewBase>& OwnerTable)
{
	check(InInfo.IsValid());

	return
		SNew(SCurveListRow, OwnerTable)
		.Item(InInfo)
		.PoseViewer(SharedThis(this));
}

bool SPoseViewer::IsPoseSelected() const
{
	// @todo: make sure not to delete base Curve
	TArray< TSharedPtr< FDisplayedPoseInfo > > SelectedRows = PoseListView->GetSelectedItems();
	return SelectedRows.Num() > 0;
}

bool SPoseViewer::IsSinglePoseSelected() const
{
	// @todo: make sure not to delete base Curve
	TArray< TSharedPtr< FDisplayedPoseInfo > > SelectedRows = PoseListView->GetSelectedItems();
	return SelectedRows.Num() == 1;
}

bool SPoseViewer::IsCurveSelected() const
{
	// @todo: make sure not to delete base pose
	TArray< TSharedPtr< FDisplayedCurveInfo > > SelectedRows = CurveListView->GetSelectedItems();
	return SelectedRows.Num() > 0;
}

// Restart Animation state for all instnace that belong to the current Skeleton
void RestartAnimations(const USkeleton* CurrentSkeleton) 
{
	for (FThreadSafeObjectIterator Iter(USkeletalMeshComponent::StaticClass()); Iter; ++Iter)
	{
		USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(*Iter);
		if (SkeletalMeshComponent->GetSkeletalMeshAsset() && SkeletalMeshComponent->GetSkeletalMeshAsset()->GetSkeleton() == CurrentSkeleton)
		{
			SkeletalMeshComponent->InitAnim(true);
		}
	}
}

void SPoseViewer::OnDeletePoses()
{
	TArray< TSharedPtr< FDisplayedPoseInfo > > SelectedRows = PoseListView->GetSelectedItems();

	FScopedTransaction Transaction(LOCTEXT("DeletePoses", "Delete Poses"));
	PoseAssetPtr.Get()->Modify();

	TArray<FName> PosesToDelete;
	for (int RowIndex = 0; RowIndex < SelectedRows.Num(); ++RowIndex)
	{
		PosesToDelete.Add(SelectedRows[RowIndex]->Name);
	}

	PoseAssetPtr.Get()->DeletePoses(PosesToDelete);

	// reinit animation
	RestartAnimations(&(EditableSkeletonPtr.Pin()->GetSkeleton()));
	RestartPreviewComponent();

	CreatePoseList(NameFilterBox->GetText().ToString());
}

void SPoseViewer::OnRenamePose()
{
	TArray< TSharedPtr< FDisplayedPoseInfo > > SelectedRows = PoseListView->GetSelectedItems();
	if (SelectedRows.Num() > 0)
	{
		TSharedPtr<FDisplayedPoseInfo> SelectedRow = SelectedRows[0];
		if (SelectedRow.IsValid())
		{
			SelectedRow->OnRenameRequested.ExecuteIfBound();
		}
	}
}

void SPoseViewer::OnDeleteCurves()
{
	TArray< TSharedPtr< FDisplayedCurveInfo > > SelectedRows = CurveListView->GetSelectedItems();

	FScopedTransaction Transaction(LOCTEXT("DeleteCurves", "Delete Curves"));
	PoseAssetPtr.Get()->Modify();

	TArray<FName> CurvesToDelete;
	for (int RowIndex = 0; RowIndex < SelectedRows.Num(); ++RowIndex)
	{
		CurvesToDelete.Add(SelectedRows[RowIndex]->Name);
	}

	PoseAssetPtr.Get()->DeleteCurves(CurvesToDelete);

	CreateCurveList(NameFilterBox->GetText().ToString());
}

void SPoseViewer::OnPastePoseNamesFromClipBoard(bool bSelectedOnly)
{
	FString PastedString;

	FPlatformApplicationMisc::ClipboardPaste(PastedString);

	if (PastedString.IsEmpty() == false)
	{
		TArray<FString> ListOfNames;
		PastedString.ParseIntoArrayLines(ListOfNames);

		if (ListOfNames.Num() > 0)
		{
			TArray<FName> PosesToRename;
			if (bSelectedOnly)
			{
				TArray< TSharedPtr< FDisplayedPoseInfo > > SelectedRows = PoseListView->GetSelectedItems();

				for (int RowIndex = 0; RowIndex < SelectedRows.Num(); ++RowIndex)
				{
					PosesToRename.Add(SelectedRows[RowIndex]->Name);
				}
			}
			else
			{
				for (auto& PoseItem : PoseList)
				{
					PosesToRename.Add(PoseItem->Name);
				}
			}

			if (PosesToRename.Num() > 0)
			{
				FScopedTransaction Transaction(LOCTEXT("PasteNames", "Paste Curve Names"));
				PoseAssetPtr.Get()->Modify();

				int32 TotalItemCount = FMath::Min(PosesToRename.Num(), ListOfNames.Num());

				for (int32 PoseIndex = 0; PoseIndex < TotalItemCount; ++PoseIndex)
				{ 
					ModifyName(PosesToRename[PoseIndex], FName(*ListOfNames[PoseIndex]), true);
				}

				CreatePoseList(NameFilterBox->GetText().ToString());
			}
		}
	}

}

FReply SPoseViewer::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (UICommandList.IsValid() && UICommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

void SPoseViewer::BindCommands()
{
	// This should not be called twice on the same instance
	check(!UICommandList.IsValid());
	UICommandList = MakeShareable(new FUICommandList);
	FUICommandList& CommandList = *UICommandList;

	// Grab the list of menu commands to bind...
	const FPoseEditorCommands& PoseEditorCommands = FPoseEditorCommands::Get();

	// ...and bind them all

	CommandList.MapAction(
		FGenericCommands::Get().Rename,
		FExecuteAction::CreateSP(this, &SPoseViewer::OnRenamePose),
		FCanExecuteAction::CreateSP(this, &SPoseViewer::IsSinglePoseSelected));

	CommandList.MapAction(
		FGenericCommands::Get().Delete,
		FExecuteAction::CreateSP(this, &SPoseViewer::OnDeletePoses),
		FCanExecuteAction::CreateSP(this, &SPoseViewer::IsPoseSelected));

	CommandList.MapAction(
		FGenericCommands::Get().Paste,
		FExecuteAction::CreateSP(this, &SPoseViewer::OnPastePoseNamesFromClipBoard, true),
		FCanExecuteAction::CreateSP(this, &SPoseViewer::IsPoseSelected));

	CommandList.MapAction(
		PoseEditorCommands.PasteAllNames,
		FExecuteAction::CreateSP(this, &SPoseViewer::OnPastePoseNamesFromClipBoard, false),
		FCanExecuteAction());

	CommandList.MapAction(
		PoseEditorCommands.UpdatePoseToCurrent,
		FExecuteAction::CreateSP(this, &SPoseViewer::UpdateSelectedPoseWithCurrent),
		FCanExecuteAction(),
		FGetActionCheckState(),
		FIsActionButtonVisible::CreateLambda([this]()
		{
			const TArray<TSharedPtr<FDisplayedPoseInfo>> SelectedRows = PoseListView->GetSelectedItems();
			return SelectedRows.Num() == 1;
		}));

	CommandList.MapAction(
		PoseEditorCommands.AddPoseFromCurrent,
		FExecuteAction::CreateSP(this, &SPoseViewer::AddPoseWithCurrent),
		FCanExecuteAction::CreateLambda([this]() -> bool
		{
			if (const UPoseAsset* PoseAsset = PoseAssetPtr.Get())
			{
				FText Temp;
				return PoseAsset->SourceAnimation == nullptr && IsNewPoseNameValid(Temp);
			}
			return false;
		}),
		FGetActionCheckState(),
		FIsActionButtonVisible::CreateLambda([this]() -> bool
		{
			if (const UPoseAsset* PoseAsset = PoseAssetPtr.Get())
			{
				return PoseAsset->SourceAnimation == nullptr;
			}
			return false;
		}));

	CommandList.MapAction(
		PoseEditorCommands.AddPoseFromReference,
		FExecuteAction::CreateSP(this, &SPoseViewer::AddPoseWithReference),
		FCanExecuteAction::CreateLambda([this]() -> bool
		{
			if (const UPoseAsset* PoseAsset = PoseAssetPtr.Get())
			{
				FText Temp;
				return PoseAsset->SourceAnimation == nullptr && IsNewPoseNameValid(Temp);
			}
			return false;
		}),
		FGetActionCheckState(),
		FIsActionButtonVisible::CreateLambda([this]() -> bool
		{
			if (const UPoseAsset* PoseAsset = PoseAssetPtr.Get())
			{
				return PoseAsset->SourceAnimation == nullptr;
			}
			return false;
		}));
}


TSharedPtr<SWidget> SPoseViewer::OnGetContextMenuContent()
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, UICommandList);

	const FPoseEditorCommands& PoseEditorCommands = FPoseEditorCommands::Get();

	MenuBuilder.AddMenuEntry(PoseEditorCommands.PasteAllNames);

	MenuBuilder.BeginSection("PoseAction", LOCTEXT("SelectedItems", "Selected Item Actions"));
	{
		MenuBuilder.AddMenuEntry(PoseEditorCommands.UpdatePoseToCurrent);
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Delete, NAME_None, LOCTEXT("DeletePoseButtonLabel", "Delete"), LOCTEXT("DeletePoseButtonTooltip", "Delete the selected pose(s)"));
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Rename, NAME_None, LOCTEXT("RenamePoseButtonLabel", "Rename"), LOCTEXT("RenamePoseButtonTooltip", "Renames the selected pose"));
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Paste, NAME_None, LOCTEXT("PastePoseNamesButtonLabel", "Paste Selected"), LOCTEXT("PastePoseNamesButtonTooltip", "Paste the selected pose names from clipboard"));
	}
	MenuBuilder.EndSection();

	if (const UPoseAsset* PoseAsset = PoseAssetPtr.Get())
	{
		if (PoseAsset->SourceAnimation == nullptr)
		{
			MenuBuilder.BeginSection("AddPoseAction", LOCTEXT("PosesSection", "Poses"));
			{				
				MenuBuilder.AddSubMenu(LOCTEXT("AddPoseSubMenu", "Add Pose"), LOCTEXT("PosesSubMenuToolTip", "Adding new Pose Related Actions"), FNewMenuDelegate::CreateLambda([this]( FMenuBuilder& SubMenuBuilder)
				{
					SubMenuBuilder.AddMenuEntry(FPoseEditorCommands::Get().AddPoseFromCurrent, NAME_None, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "Persona.AssetClass.Animation"));
					SubMenuBuilder.AddMenuEntry(FPoseEditorCommands::Get().AddPoseFromReference, NAME_None, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "Persona.AssetClass.Skeleton"));
					SubMenuBuilder.AddVerifiedEditableText(LOCTEXT("NewPoseLabel", "Pose Name"), LOCTEXT("NewPoseTooltip", "Tooltip"), FSlateIcon(),
					TAttribute<FText>::CreateLambda([this]() { return FText::FromName(NewPoseName); }),
					FOnVerifyTextChanged::CreateLambda([this](const FText& NewText, FText& OutMessage)-> bool
					{
						NewPoseName = FName(*NewText.ToString());
						return IsNewPoseNameValid(OutMessage);
					}));
				}));
			}
			MenuBuilder.EndSection();
		}
	}
	
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

bool SPoseViewer::IsNewPoseNameValid(FText& OutReason) const
{
	if (const UPoseAsset* PoseAsset = PoseAssetPtr.Get())
	{
		if (PoseAsset->ContainsPose(NewPoseName))
		{
			OutReason = LOCTEXT("NewPoseAlreadyExistsMessage", "Pose with this name already exists");
			return false;
		}
	}

	if(!NewPoseName.IsValidObjectName(OutReason))
	{
		return false;
	}

	if (NewPoseName == NAME_None)
	{
		OutReason = LOCTEXT("NameNonePoseName", "Pose name cannot be empty or None");
		return false;
	}
	
	return true;
}


TSharedPtr<SWidget> SPoseViewer::OnGetContextMenuContentForCurveList() const
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, NULL);

	MenuBuilder.BeginSection("CurveAction", LOCTEXT("CurveActions", "Selected Item Actions"));
	{
		FUIAction Action = FUIAction(FExecuteAction::CreateSP(const_cast<SPoseViewer*>(this), &SPoseViewer::OnDeleteCurves),
			FCanExecuteAction::CreateSP(this, &SPoseViewer::IsCurveSelected));
		const FText MenuLabel = LOCTEXT("DeleteCurveButtonLabel", "Delete");
		const FText MenuToolTip = LOCTEXT("DeleteCurveButtonTooltip", "Deletes the selected animation curve.");
		MenuBuilder.AddMenuEntry(MenuLabel, MenuToolTip, FSlateIcon(), Action);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void SPoseViewer::OnListDoubleClick(TSharedPtr<FDisplayedPoseInfo> InItem)
{
	if(InItem.IsValid())
	{
		const float CurrentWeight = InItem->Weight;

		// Clear all preview poses
		for (TSharedPtr<FDisplayedPoseInfo>& Pose : PoseList)
		{
			Pose->Weight = 0.f;
			AddCurveOverride(Pose->Name, 0.f);
		}

		// If current weight was already at 1.0, do nothing (we are setting it to zero)
		if (!FMath::IsNearlyEqual(CurrentWeight, 1.f))
		{
			// Otherwise set to 1.0
			InItem->Weight = 1.f;
			AddCurveOverride(InItem->Name, 1.f);
		}

		// Force update viewport
		PreviewScenePtr.Pin()->InvalidateViews();
	}
}


void SPoseViewer::CreatePoseList(const FString& SearchText)
{
	PoseList.Empty();

	if (PoseAssetPtr.IsValid())
	{
		UPoseAsset* PoseAsset = PoseAssetPtr.Get();

		TArray<FName> PoseNames = PoseAsset->GetPoseFNames();
		if (PoseNames.Num() > 0)
		{
			bool bDoFiltering = !SearchText.IsEmpty();

			for (const FName& PoseName : PoseNames)
			{
				if (bDoFiltering && !PoseName.ToString().Contains(SearchText))
				{
					continue; // Skip items that don't match our filter
				}

				const TSharedRef<FDisplayedPoseInfo> Info = FDisplayedPoseInfo::Make(PoseName);
				float* Weight = OverrideCurves.Find(PoseName);
				if (Weight)
				{
					Info->Weight = *Weight;
				}

				PoseList.Add(Info);
			}
		}
	}

	PoseListView->RequestListRefresh();
}

void SPoseViewer::CreateCurveList(const FString& SearchText)
{
	CurveList.Empty();

	if (PoseAssetPtr.IsValid())
	{
		UPoseAsset* PoseAsset = PoseAssetPtr.Get();

		for (const FName& CurveName : PoseAsset->GetCurveFNames())
		{
			const TSharedRef<FDisplayedCurveInfo> Info = FDisplayedCurveInfo::Make(CurveName);
			CurveList.Add(Info);
		}
	}

	CurveListView->RequestListRefresh();
}

void SPoseViewer::AddCurveOverride(const FName& Name, float Weight)
{
	float& Value = OverrideCurves.FindOrAdd(Name);
	Value = Weight;

	UAnimSingleNodeInstance* SingleNodeInstance = Cast<UAnimSingleNodeInstance>(GetAnimInstance());
	if (SingleNodeInstance)
	{
		SingleNodeInstance->SetPreviewCurveOverride(Name, Value, false);
	}
}

void SPoseViewer::RemoveCurveOverride(FName& Name)
{
	OverrideCurves.Remove(Name);

	UAnimSingleNodeInstance* SingleNodeInstance = Cast<UAnimSingleNodeInstance>(GetAnimInstance());
	if (SingleNodeInstance)
	{
		SingleNodeInstance->SetPreviewCurveOverride(Name, 0.f, true);
	}
}

SPoseViewer::~SPoseViewer()
{
	if (PreviewScenePtr.IsValid())
	{
		PreviewScenePtr.Pin()->UnregisterOnPreviewMeshChanged(this);
		PreviewScenePtr.Pin()->UnregisterOnAnimChanged(this);
	}

	if (PoseAssetPtr.IsValid())
	{
		PoseAssetPtr->UnregisterOnPoseListChanged(OnDelegatePoseListChangedDelegateHandle);
	}
}

void SPoseViewer::RestartPreviewComponent()
{
	// it needs reinitialization of animation system
	// so that pose blender can reinitialize names and so on correctly
	if (PreviewScenePtr.IsValid())
	{
		UDebugSkelMeshComponent* PreviewComponent = PreviewScenePtr.Pin()->GetPreviewMeshComponent();
		if (PreviewComponent)
		{
			// we can't wait until next tick for this in this case
			// since animation is initialized here
			// so we calculate curve here
			PreviewComponent->RecalcRequiredCurves();
			PreviewComponent->InitAnim(true);
			for (auto Iter = OverrideCurves.CreateConstIterator(); Iter; ++Iter)
			{
				// refresh curve names that are active
				AddCurveOverride(Iter.Key(), Iter.Value());
			}
		}
	}
}

void SPoseViewer::OnPoseAssetModified()
{
	CreatePoseList(NameFilterBox->GetText().ToString());
	CreateCurveList(NameFilterBox->GetText().ToString());
	RestartPreviewComponent();
}

void SPoseViewer::ApplyCustomCurveOverride(UAnimInstance* AnimInstance) const
{
	for (auto Iter = OverrideCurves.CreateConstIterator(); Iter; ++Iter)
	{
		// @todo we might want to save original curve flags? or just change curve to apply flags only
		AnimInstance->AddCurveValue(Iter.Key(), Iter.Value());
	}
}

UAnimInstance* SPoseViewer::GetAnimInstance() const
{
	return PreviewScenePtr.Pin()->GetPreviewMeshComponent()->GetAnimInstance();
}

bool SPoseViewer::ModifyName(FName OldName, FName NewName, bool bSilence)
{
	if(PoseAssetPtr.Get()->ContainsPose(NewName))
	{
		// warn users
		// if so, verify if this name is still okay
		if (!bSilence)
		{
			const EAppReturnType::Type Response = FMessageDialog::Open(EAppMsgType::YesNo, LOCTEXT("UseSameNameConfirm", "The name already exists. Would you like to reuse the name? This can cause conflict of curve data."));

			if (Response == EAppReturnType::No)
			{
				return false;
			}
		}
	}

	FScopedTransaction Transaction(LOCTEXT("RenamePoses", "Rename Pose"));
	PoseAssetPtr.Get()->Modify();
	
	// I think this might have to be delegate of the top window
	if (PoseAssetPtr.Get()->ModifyPoseName(OldName, NewName) == false)
	{
		return false;
	}

	// now refresh pose data
	float* Value = OverrideCurves.Find(OldName);
	if (Value)
	{
		AddCurveOverride(NewName, *Value);
		RemoveCurveOverride(OldName);
	}

	return true;
}

bool SPoseViewer::IsBasePose(FName PoseName) const
{
	if (PoseAssetPtr.IsValid() && PoseAssetPtr->IsValidAdditive())
	{
		int32 PoseIndex = PoseAssetPtr->GetPoseIndexByName(PoseName);
		return (PoseIndex == PoseAssetPtr->GetBasePoseIndex());
	}

	return false;
}

void SPoseViewer::UpdateSelectedPoseWithCurrent()
{
	TArray<TSharedPtr<FDisplayedPoseInfo>> SelectedRows = PoseListView->GetSelectedItems();
	UPoseAsset* PoseAsset = PoseAssetPtr.Get();
	
	if (SelectedRows.Num() == 1 && PoseAsset && PreviewScenePtr.IsValid())
	{
		UDebugSkelMeshComponent* PreviewComponent = PreviewScenePtr.Pin()->GetPreviewMeshComponent();
		const USkeleton* Skeleton = PoseAsset->GetSkeleton();
		if (PreviewComponent && Skeleton)
		{
			const FName PoseName = SelectedRows[0]->Name;

			FScopedTransaction Transaction(LOCTEXT("UpdatePose", "Update Pose from Viewport"));
			PoseAsset->Modify();			
			PoseAsset->AddOrUpdatePose(PoseName, PreviewComponent, false);

			// Reset bone modifiers in case the user has created a pose using them - resetting them to the 'base' pose
			if(UAnimPreviewInstance* PreviewInstance = Cast<UAnimPreviewInstance>(PreviewComponent->GetAnimInstance()))
			{
				PreviewInstance->ResetModifiedBone();
			}

			// Reinitialize animation preview
			RestartAnimations(Skeleton);
			RestartPreviewComponent();
		}
	}
}

void SPoseViewer::AddPoseWithCurrent()
{
	if(UPoseAsset* PoseAsset = PoseAssetPtr.Get())
	{
		UDebugSkelMeshComponent* PreviewComponent = PreviewScenePtr.Pin()->GetPreviewMeshComponent();
		USkeleton* Skeleton = PoseAsset->GetSkeleton();
		if (PreviewComponent && Skeleton)
		{
			PoseAsset->AddOrUpdatePose(NewPoseName, PreviewComponent);
			NewPoseName = UPoseAsset::GetUniquePoseName(PoseAsset);
		}
	}
}

void SPoseViewer::AddPoseWithReference()
{
	if(UPoseAsset* PoseAsset = PoseAssetPtr.Get())
	{
		const UDebugSkelMeshComponent* PreviewComponent = PreviewScenePtr.Pin()->GetPreviewMeshComponent();
		USkeleton* Skeleton = PoseAsset->GetSkeleton();
		if (PreviewComponent && Skeleton)
		{
			PoseAsset->AddReferencePose(NewPoseName, PreviewComponent->GetReferenceSkeleton());
			NewPoseName = UPoseAsset::GetUniquePoseName(PoseAsset);
		}
	}	
}

#undef LOCTEXT_NAMESPACE