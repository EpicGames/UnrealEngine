// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "IPersonaPreviewScene.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "IEditableSkeleton.h"
#include "IPersonaToolkit.h"
#include "Widgets/Views/SListView.h"
#include "SAnimEditorBase.h"
#include "Animation/PoseAsset.h"
#include "Animation/AnimInstance.h"

class SPoseViewer;
class UAnimSingleNodeInstance;

//////////////////////////////////////////////////////////////////////////
// FDisplayedPoseInfo

class FDisplayedPoseInfo
{
public:
	FName Name;
	float Weight;

	/** Static function for creating a new item, but ensures that you can only have a TSharedRef to one */
	static TSharedRef<FDisplayedPoseInfo> Make(const FName& Source)
	{
		return MakeShareable(new FDisplayedPoseInfo(Source));
	}

	/** Delegate for when the context menu requests a rename */
	DECLARE_DELEGATE(FOnRenameRequested);
	FOnRenameRequested OnRenameRequested;

protected:
	/** Hidden constructor, always use Make above */
	FDisplayedPoseInfo(const FName& InSource)
		: Name(InSource)
		, Weight(0)
	{}

	/** Hidden constructor, always use Make above */
	FDisplayedPoseInfo() {}
};

typedef SListView< TSharedPtr<FDisplayedPoseInfo> > SPoseListType;

//////////////////////////////////////////////////////////////////////////
// SPoseListRow

class SPoseListRow : public SMultiColumnTableRow< TSharedPtr<FDisplayedPoseInfo> >
{
public:

	SLATE_BEGIN_ARGS(SPoseListRow) {}

		/** The item for this row **/
		SLATE_ARGUMENT(TSharedPtr<FDisplayedPoseInfo>, Item)

		/* The SPoseViewer that we push the morph target weights into */
		SLATE_ARGUMENT(TWeakPtr<SPoseViewer>, PoseViewer)

		/** Filter text typed by the user into the parent tree's search widget */
		SLATE_ARGUMENT(FText, FilterText);

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTableView, const TSharedRef<IPersonaPreviewScene>& InPreviewScene);

	/** Overridden from SMultiColumnTableRow.  Generates a widget for this column of the tree row. */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

private:

	/**
	* Called when the user changes the value of the SSpinBox
	*
	* @param NewWeight - The new number the SSpinBox is set to
	*
	*/
	void OnPoseWeightChanged(float NewWeight);
	/**
	* Called when the user types the value and enters
	*
	* @param NewWeight - The new number the SSpinBox is set to
	*
	*/
	void OnPoseWeightValueCommitted(float NewWeight, ETextCommit::Type CommitType);

	/** Delegate to get labels root text from settings */
	FText GetName() const;

	/** Delegate to commit labels root text to settings */
	void OnNameCommitted(const FText& InText, ETextCommit::Type InCommitType) const;

	bool OnVerifyNameChanged(const FText& InText, FText& OutErrorMessage);


	/**
	* Returns the weight of this morph target
	*
	* @return SearchText - The new number the SSpinBox is set to
	*
	*/
	float GetWeight() const;

	bool CanChangeWeight() const;

	/* The SPoseViewer that we push the pose weights into */
	TWeakPtr<SPoseViewer> PoseViewerPtr;

	/** The name and weight of the morph target */
	TSharedPtr<FDisplayedPoseInfo>	Item;

	/** Text the user typed into the search box - used for text highlighting */
	FText FilterText;

	/** The preview scene we are viewing */
	TWeakPtr<class IPersonaPreviewScene> PreviewScenePtr;
};

//////////////////////////////////////////////////////////////////////////
// FDisplayedCurveInfo

class FDisplayedCurveInfo
{
public:
	FName Name;

	/** Static function for creating a new item, but ensures that you can only have a TSharedRef to one */
	static TSharedRef<FDisplayedCurveInfo> Make(const FName& Source)
	{
		return MakeShareable(new FDisplayedCurveInfo(Source));
	}

protected:
	/** Hidden constructor, always use Make above */
	FDisplayedCurveInfo(const FName& InSource)
		: Name(InSource)
	{}

	/** Hidden constructor, always use Make above */
	FDisplayedCurveInfo() {}
};

//////////////////////////////////////////////////////////////////////////
// SCurveListRow

typedef SListView< TSharedPtr<FDisplayedCurveInfo> > SCurveListType;


class SCurveListRow : public SMultiColumnTableRow< TSharedPtr<FDisplayedCurveInfo> >
{
public:

	SLATE_BEGIN_ARGS(SCurveListRow) {}

		/** The item for this row **/
		SLATE_ARGUMENT(TSharedPtr<FDisplayedCurveInfo>, Item)

		/* The SPoseViewer that we push the morph target weights into */
		SLATE_ARGUMENT(TWeakPtr<SPoseViewer>, PoseViewer)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTableView);

	/** Overridden from SMultiColumnTableRow.  Generates a widget for this column of the tree row. */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

private:

	/** Delegate to get labels root text from settings */
	FText GetName() const;

	/** Delegate to get weight of curve in selected pose */
	FText GetValue() const;

	/** The name and weight of the morph target */
	TSharedPtr<FDisplayedCurveInfo>	Item;

	/* The SPoseViewer that we push the morph target weights into */
	TWeakPtr<SPoseViewer> PoseViewerPtr;
};


//////////////////////////////////////////////////////////////////////////
// SPoseViewer

class SPoseViewer : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SPoseViewer)
	{}

	SLATE_ARGUMENT(TWeakObjectPtr< UPoseAsset >, PoseAsset)

	SLATE_END_ARGS()

	/**
	* Slate construction function
	*
	* @param InArgs - Arguments passed from Slate
	*
	*/
	void Construct(const FArguments& InArgs, const TSharedRef<class IPersonaToolkit>& InPersonaToolkit, const TSharedRef<IEditableSkeleton>& InEditableSkeleton, const TSharedRef<class IPersonaPreviewScene>& InPreviewScene);

	/**
	* Destructor - resets the animation curve
	*
	*/
	virtual ~SPoseViewer();

	/** SWidget interface */
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent);

	/**
	* Is registered with Persona to handle when its preview mesh is changed.
	*
	* @param NewPreviewMesh - The new preview mesh being used by Persona
	*
	*/
	void OnPreviewMeshChanged(class USkeletalMesh* OldPreviewMesh, class USkeletalMesh* NewPreviewMesh);

	/**
	* Filters the SListView when the user changes the search text box (NameFilterBox)
	*
	* @param SearchText - The text the user has typed
	*
	*/
	void OnFilterTextChanged(const FText& SearchText);

	/**
	* Filters the SListView when the user hits enter or clears the search box
	* Simply calls OnFilterTextChanged
	*
	* @param SearchText - The text the user has typed
	* @param CommitInfo - Not used
	*
	*/
	void OnFilterTextCommitted(const FText& SearchText, ETextCommit::Type CommitInfo);

	/**
	* Create a widget for an entry in the tree from an info
	*
	* @param InInfo - Shared pointer to the morph target we're generating a row for
	* @param OwnerTable - The table that owns this row
	*
	* @return A new Slate widget, containing the UI for this row
	*/
	TSharedRef<ITableRow> GeneratePoseRow(TSharedPtr<FDisplayedPoseInfo> InInfo, const TSharedRef<STableViewBase>& OwnerTable);
	TSharedRef<ITableRow> GenerateCurveRow(TSharedPtr<FDisplayedCurveInfo> InInfo, const TSharedRef<STableViewBase>& OwnerTable);

	bool IsPoseSelected() const;
	bool IsSinglePoseSelected() const;
	bool IsCurveSelected() const;

	/** Handler for the delete poses option */
	void OnDeletePoses();

	/** Handler for rename pose option */
	void OnRenamePose();

	/** Handler for delete curves option */
	void OnDeleteCurves();

	/** Handler for pasting names from clipboard */
	void OnPastePoseNamesFromClipBoard(bool bSelectedOnly);

	/**
	* Adds a morph target override or updates the weight for an existing one
	*
	* @param Name - Name of the morph target we want to override
	* @param Weight - How much of this morph target to apply (0.0 - 1.0)
	*/
	void AddCurveOverride(const FName& Name, float Weight);

	/** Remove a named curve override */ 
	void RemoveCurveOverride(FName& Name);

	/**
	* Accessor so our rows can grab the filtertext for highlighting
	*
	*/
	FText& GetFilterText() { return FilterText; }

	/**
	* Update pose asset changes - including list of poses or names or deletions 
	*/
	void OnPoseAssetModified();

	bool ModifyName(FName OldName, FName NewName, bool bSilence = false);
	bool IsBasePose(FName PoseName) const;

private:
	void UpdateSelectedPoseWithCurrent();

	void AddPoseWithCurrent();
	void AddPoseWithReference();

	void BindCommands();
	void RestartPreviewComponent();

	/** Handler for context menus */
	TSharedPtr<SWidget> OnGetContextMenuContent();
	/** Handler for curve list context menu*/
	TSharedPtr<SWidget> OnGetContextMenuContentForCurveList() const;
	/** Called when list double-clicked */
	void OnListDoubleClick(TSharedPtr<FDisplayedPoseInfo> InItem);	
	bool IsNewPoseNameValid(FText& OutReason) const;
	/**
	* Clears and rebuilds the table, according to an optional search string
	*
	* @param SearchText - Optional search string
	*
	*/
	void CreatePoseList(const FString& SearchText = FString());
	void CreateCurveList(const FString& SearchText = FString());

	void ApplyCustomCurveOverride(UAnimInstance* AnimInstance) const;

	/** Get the anim instance we are viewing */
	UAnimInstance* GetAnimInstance() const;

	/** Pointer to the preview scene we are viewing */
	TWeakPtr<class IPersonaPreviewScene> PreviewScenePtr;

	/** Pointer to the persona toolkit we are embedded in */
	TWeakPtr<class IPersonaToolkit> PersonaToolkitPtr;

	/** Pointer to the editable skeleton we will need to modify */
	TWeakPtr<class IEditableSkeleton> EditableSkeletonPtr;

	/** Pointer to the pose asset */
	TWeakObjectPtr<UPoseAsset> PoseAssetPtr;

	/** Box to filter to a specific morph target name */
	TSharedPtr<SSearchBox>	NameFilterBox;

	/** Widget used to display the list of animation curve */
	TSharedPtr<SPoseListType> PoseListView;

	/** A list of animation curve. Used by the PoseListView. */
	TArray< TSharedPtr<FDisplayedPoseInfo> > PoseList;

	/** Widget used to display the list of animation curve */
	TSharedPtr<SCurveListType> CurveListView;

	/** A list of animation curve. Used by the PoseListView. */
	TArray< TSharedPtr<FDisplayedCurveInfo> > CurveList;

	FName NewPoseName;

	/** Current text typed into NameFilterBox */
	FText FilterText;

	/** Commands that are bound to delegates*/
	TSharedPtr<FUICommandList> UICommandList;

	TMap<FName, float> OverrideCurves;

	/** Add curve delegate */
	FOnAddCustomAnimationCurves OnAddAnimationCurveDelegate;
	FDelegateHandle OnDelegatePoseListChangedDelegateHandle;

	friend class SPoseListRow;
	friend class SCurveListRow;
};
//////////////////////////////////////////////////////////////////////////
// SPoseEditor


/** Overall animation sequence editing widget */
class SPoseEditor : public SAnimEditorBase
{
public:
	SLATE_BEGIN_ARGS( SPoseEditor )
		: _PoseAsset(NULL)
		{}

		SLATE_ARGUMENT( UPoseAsset*, PoseAsset )
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs, const TSharedRef<class IPersonaToolkit>& InPersonaToolkit, const TSharedRef<IEditableSkeleton>& InEditableSkeleton, const TSharedRef<class IPersonaPreviewScene>& InPreviewScene);

	virtual UAnimationAsset* GetEditorObject() const override { return PoseAssetObj; }

private:
	/** Pointer to the animation sequence being edited */
	UPoseAsset* PoseAssetObj;
};
