// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPoseWatchManager.h"

#include "EdMode.h"
#include "Editor.h"
#include "Editor/UnrealEdEngine.h"
#include "EditorModeManager.h"
#include "Styling/AppStyle.h"
#include "Engine/GameViewportClient.h"
#include "Engine/Selection.h"
#include "EngineUtils.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/PlatformApplicationMisc.h"
#include "IPoseWatchManagerColumn.h"
#include "Kismet2/ComponentEditorUtils.h"
#include "Layout/WidgetPath.h"
#include "Modules/ModuleManager.h"
#include "ScopedTransaction.h"
#include "Textures/SlateIcon.h"
#include "ToolMenus.h"
#include "UnrealEdGlobals.h"
#include "UObject/PackageReload.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/SOverlay.h"
#include "EditorFolderUtils.h"
#include "Animation/AnimBlueprint.h"
#include "BlueprintEditor.h"
#include "EdGraph/EdGraph.h"
#include "GraphEditAction.h"
#include "AnimationEditorUtils.h"
#include "PoseWatchManagerDefaultHierarchy.h"
#include "PoseWatchManagerDefaultMode.h"
#include "PoseWatchManagerElementTreeItem.h"
#include "PoseWatchManagerPoseWatchTreeItem.h"
#include "PoseWatchManagerColumnVisibility.h"
#include "PoseWatchManagerColumnColor.h"
#include "PoseWatchManagerColumnLabel.h"

#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Framework/Commands/GenericCommands.h"
#include "SKismetInspector.h"

#define LOCTEXT_NAMESPACE "SPoseWatchManager"

SPoseWatchManager::SPoseWatchManager()
{}

SPoseWatchManager::~SPoseWatchManager()
{
	SearchBoxFilter->OnChanged().RemoveAll(this);

	AnimationEditorUtils::OnPoseWatchesChanged().RemoveAll(this);
	FCoreUObjectDelegates::OnPackageReloaded.RemoveAll(this);
}

void SPoseWatchManager::Construct(const FArguments& InArgs, const FPoseWatchManagerInitializationOptions& InInitOptions)
{
	AnimationEditorUtils::OnPoseWatchesChanged().AddSP(this, &SPoseWatchManager::OnPoseWatchesChanged);

	Mode = MakeShared<FPoseWatchManagerDefaultMode>(this);

	BlueprintEditor = InInitOptions.BlueprintEditor.Pin().Get();
	AnimBlueprint = CastChecked<UAnimBlueprint>(BlueprintEditor->GetBlueprintObj());

	bProcessingFullRefresh = false;
	bFullRefresh = true;
	bNeedsRefresh = true;
	bNeedsColumnRefresh = true;
	bIsReentrant = false;
	bSortDirty = true;
	bSelectionDirty = true;
	bPendingFocusNextFrame = false;

	SortByColumn = FPoseWatchManagerBuiltInColumnTypes::Label();
	SortMode = EColumnSortMode::Ascending;

	FGenericCommands::Register();
	BindCommands();

	//Setup the search box
	{
		auto Delegate = PoseWatchManager::TreeItemTextFilter::FItemToStringArray::CreateSP(this, &SPoseWatchManager::PopulateSearchStrings);
		SearchBoxFilter = MakeShareable(new PoseWatchManager::TreeItemTextFilter(Delegate));
	}

	TSharedRef<SVerticalBox> VerticalBox = SNew(SVerticalBox);


	SearchBoxFilter->OnChanged().AddSP(this, &SPoseWatchManager::FullRefresh);

	HeaderRowWidget =
		SNew(SHeaderRow)
		.Visibility(EVisibility::Visible)
		.CanSelectGeneratedColumn(true);

	SetupColumns(*HeaderRowWidget);

	ChildSlot
		[
			VerticalBox
		];

	auto Toolbar = SNew(SHorizontalBox);

	Toolbar->AddSlot()
		.VAlign(VAlign_Center)
		[
			SAssignNew(FilterTextBoxWidget, SSearchBox)
			.Visibility(EVisibility::Visible)
			.HintText(LOCTEXT("FilterSearch", "Search..."))
			.ToolTipText(LOCTEXT("FilterSearchHint", "Type here to search"))
			.OnTextChanged(this, &SPoseWatchManager::OnFilterTextChanged)
		];


	Toolbar->AddSlot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		.Padding(4.f, 0.f, 0.f, 0.f)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "SimpleButton")
			.ToolTipText(LOCTEXT("CreateFolderToolTip", "Create a new folder"))
			.OnClicked(this, &SPoseWatchManager::OnCreateFolderClicked)
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(FAppStyle::Get().GetBrush("SceneOutliner.NewFolderIcon"))
			]
		];

	VerticalBox->AddSlot()
		.AutoHeight()
		.Padding(8.0f, 8.0f, 8.0f, 4.0f)
		[
			Toolbar
		];

	VerticalBox->AddSlot()
		.FillHeight(1.0)
		[
			SNew(SOverlay)
			+ SOverlay::Slot()
			.HAlign(HAlign_Center)
			[
				SNew(STextBlock)
				.Visibility(this, &SPoseWatchManager::GetEmptyLabelVisibility)
				.Text(LOCTEXT("EmptyLabel", "Empty"))
				.ColorAndOpacity(FLinearColor(0.4f, 1.0f, 0.4f))
			]

		+ SOverlay::Slot()
		[
			SNew(SBorder).BorderImage(FAppStyle::Get().GetBrush("Brushes.Recessed"))
		]

		+ SOverlay::Slot()
		[
			SAssignNew(PoseWatchManagerTreeView, SPoseWatchManagerTreeView, StaticCastSharedRef<SPoseWatchManager>(AsShared()))

			// Determined by the mode
			.SelectionMode(this, &SPoseWatchManager::GetSelectionMode)

			// Point the tree to our array of root-level items.  Whenever this changes, we'll call RequestTreeRefresh()
			.TreeItemsSource(&RootTreeItems)

			// Find out when the user selects something in the tree
			.OnSelectionChanged(this, &SPoseWatchManager::OnManagerTreeSelectionChanged)

			// Called when the user double-clicks with LMB on an item in the list
			.OnMouseButtonDoubleClick(this, &SPoseWatchManager::OnManagerTreeDoubleClick)

			// Called when an item is scrolled into view
			.OnItemScrolledIntoView(this, &SPoseWatchManager::OnManagerTreeItemScrolledIntoView)

			// Called when an item is expanded or collapsed
			.OnExpansionChanged(this, &SPoseWatchManager::OnItemExpansionChanged)

			// Called to child items for any given parent item
			.OnGetChildren(this, &SPoseWatchManager::OnGetChildrenForManagerTree)

			// Generates the actual widget for a tree item
			.OnGenerateRow(this, &SPoseWatchManager::OnGenerateRowForManagerTree)

			// Use the level viewport context menu as the right click menu for tree items
			.OnContextMenuOpening(this, &SPoseWatchManager::OnOpenContextMenu)

			// Header for the tree
			.HeaderRow(HeaderRowWidget)

			// Make it easier to see hierarchies when there are a lot of items
			.HighlightParentNodesForSelection(true)
		]
	];

	// Don't allow tool-tips over the header
	HeaderRowWidget->EnableToolTipForceField(true);

	// Populate our data set
	Populate();
}

void SPoseWatchManager::SetupColumns(SHeaderRow& HeaderRow)
{
	Columns.Empty(3);
	Columns.Add(FPoseWatchManagerColumnVisibility::GetID(), MakeShared<FPoseWatchManagerColumnVisibility>(*this));
	Columns.Add(FPoseWatchManagerColumnColor::GetID(), MakeShared<FPoseWatchManagerColumnColor>());
	Columns.Add(FPoseWatchManagerItemLabelColumn::GetID(), MakeShared<FPoseWatchManagerItemLabelColumn>(*this));

	HeaderRow.ClearColumns();

	for (auto& Pair : Columns)
	{
		FName ID = Pair.Key;
		TSharedPtr<IPoseWatchManagerColumn> Column = Pair.Value;

		if (ensure(Column.IsValid()))
		{
			auto ColumnArgs = Column->ConstructHeaderRowColumn();

			if (Column->SupportsSorting())
			{
				ColumnArgs
					.SortMode(this, &SPoseWatchManager::GetColumnSortMode, ID)
					.OnSort(this, &SPoseWatchManager::OnColumnSortModeChanged);
			}

			ColumnArgs.DefaultLabel(FText::FromName(ID));
			ColumnArgs.ShouldGenerateWidget(true);

			HeaderRow.AddColumn(ColumnArgs);
			HeaderRowWidget->SetShowGeneratedColumn(ID);
		}
	}

	bNeedsColumnRefresh = false;
}

void SPoseWatchManager::RefreshColumns()
{
	bNeedsColumnRefresh = true;
}

void SPoseWatchManager::OnItemAdded(const FObjectKey& ItemID, uint8 ActionMask)
{
	NewItemActions.Add(ItemID, ActionMask);
}

ESelectionMode::Type SPoseWatchManager::GetSelectionMode() const
{
	return ESelectionMode::Single;
}

void SPoseWatchManager::Refresh()
{
	bNeedsRefresh = true;
}

void SPoseWatchManager::FullRefresh()
{
	bDisableIntermediateSorting = true;
	bFullRefresh = true;
	Refresh();
}

void SPoseWatchManager::RefreshSelection()
{
	bSelectionDirty = true;
}

void SPoseWatchManager::Populate()
{
	// Block events while we clear out the list
	TGuardValue<bool> ReentrantGuard(bIsReentrant, true);

	bool bMadeAnySignificantChanges = false;
	if (bFullRefresh)
	{
		// Clear the selection here - RepopulateEntireTree will reconstruct it.
		PoseWatchManagerTreeView->ClearSelection();

		RepopulateEntireTree();

		bMadeAnySignificantChanges = true;
		bFullRefresh = false;
	}

	int32 Index = 0;
	while (Index < PendingOperations.Num())
	{
		auto& PendingOp = PendingOperations[Index];
		switch (PendingOp.Type)
		{
		case PoseWatchManager::FPendingTreeOperation::Added:
			bMadeAnySignificantChanges = AddItemToTree(PendingOp.Item) || bMadeAnySignificantChanges;
			break;

		default:
			check(false);
			break;
		}

		++Index;
	}

	PendingOperations.RemoveAt(0, Index);

	// Check if we need to sort because we are finished with the populating operations
	bool bFinalSort = false;
	if (PendingOperations.Num() == 0)
	{
		// When done processing a FullRefresh Scroll to First item in selection as it may have been
		// scrolled out of view by the Refresh
		if (bProcessingFullRefresh)
		{
			FPoseWatchManagerTreeItemPtr ItemToScroll = GetSelection();
			if (ItemToScroll)
			{
				ScrollItemIntoView(ItemToScroll);
			}
		}

		bProcessingFullRefresh = false;
		// We're fully refreshed now.
		NewItemActions.Empty();
		bNeedsRefresh = false;
		if (bDisableIntermediateSorting)
		{
			bDisableIntermediateSorting = false;
			bFinalSort = true;
		}
	}

	// If we are allowing intermediate sorts and met the conditions, or this is the final sort after all ops are complete
	if ((bMadeAnySignificantChanges && !bDisableIntermediateSorting) || bFinalSort)
	{
		RequestSort();
	}
}


void SPoseWatchManager::EmptyTreeItems()
{
	PendingOperations.Empty();
	TreeItemMap.Reset();
	PendingTreeItemMap.Empty();
	RootTreeItems.Empty();
}

void SPoseWatchManager::AddPendingItem(FPoseWatchManagerTreeItemPtr Item)
{
	if (Item)
	{
		PendingTreeItemMap.Add(Item->GetID(), Item);
		PendingOperations.Emplace(PoseWatchManager::FPendingTreeOperation::Added, Item.ToSharedRef());
	}
}

void SPoseWatchManager::AddPendingItemAndChildren(FPoseWatchManagerTreeItemPtr Item)
{
	if (Item)
	{
		AddPendingItem(Item);
		Refresh();
	}
}

void SPoseWatchManager::RepopulateEntireTree()
{
	EmptyTreeItems();

	// Rebuild the hierarchy
	Mode->Rebuild();

	// Create all the items which match the filters, parent-child relationships are handled when each item is actually added to the tree

	TArray<FPoseWatchManagerTreeItemPtr> Items;
	Mode->Hierarchy->CreateItems(Items);

	for (FPoseWatchManagerTreeItemPtr& Item : Items)
	{
		if (Item && Item->IsValid())
		{
			AddPendingItem(Item);

			if (FPoseWatchManagerPoseWatchTreeItem* PoseWatchItem = Item->CastTo<FPoseWatchManagerPoseWatchTreeItem>())
			{
				PoseWatchManagerTreeView->SetItemExpansion(Item, PoseWatchItem->PoseWatch->GetIsExpanded());
			}
			else if (FPoseWatchManagerFolderTreeItem* FolderItem = Item->CastTo<FPoseWatchManagerFolderTreeItem>())
			{
				PoseWatchManagerTreeView->SetItemExpansion(Item, FolderItem->PoseWatchFolder->GetIsExpanded());
			}
		}
	}

	bProcessingFullRefresh = PendingOperations.Num() > 0;

	Refresh();
}

FPoseWatchManagerTreeItemPtr SPoseWatchManager::GetTreeItem(FObjectKey ItemID, bool bIncludePending)
{
	FPoseWatchManagerTreeItemPtr Result = TreeItemMap.FindRef(ItemID);
	if (bIncludePending && !Result.IsValid())
	{
		Result = PendingTreeItemMap.FindRef(ItemID);
	}
	return Result;
}

FPoseWatchManagerTreeItemPtr SPoseWatchManager::EnsureParentForItem(FPoseWatchManagerTreeItemRef Item)
{
	FPoseWatchManagerTreeItemPtr Parent = Mode->Hierarchy->FindParent(*Item, TreeItemMap);
	if (Parent.IsValid())
	{
		return Parent;
	}

	// Try to find the parent in the pending items
	Parent = Mode->Hierarchy->FindParent(*Item, PendingTreeItemMap);
	if (Parent.IsValid())
	{
		AddUnfilteredItemToTree(Parent.ToSharedRef());
		return Parent;
	}

	return nullptr;
}

bool SPoseWatchManager::AddItemToTree(FPoseWatchManagerTreeItemRef Item)
{
	const auto ItemID = Item->GetID();

	PendingTreeItemMap.Remove(ItemID);

	// If a tree item already exists that represents the same data or if the item represents invalid data, bail
	if (TreeItemMap.Find(ItemID) || !Item->IsValid())
	{
		return false;
	}

	AddUnfilteredItemToTree(Item);

	// Check if we need to do anything with this new item
	if (uint8* ActionMask = NewItemActions.Find(ItemID))
	{
		if (*ActionMask & PoseWatchManager::ENewItemAction::Select)
		{
			PoseWatchManagerTreeView->ClearSelection();
			PoseWatchManagerTreeView->SetItemSelection(Item, true);
		}

		if (*ActionMask & PoseWatchManager::ENewItemAction::Rename)
		{
			PendingRenameItem = Item;
		}

		if (*ActionMask & (PoseWatchManager::ENewItemAction::ScrollIntoView | PoseWatchManager::ENewItemAction::Rename))
		{
			ScrollItemIntoView(Item);
		}
	}

	return true;
}

void SPoseWatchManager::AddUnfilteredItemToTree(FPoseWatchManagerTreeItemRef Item)
{
	FPoseWatchManagerTreeItemPtr Parent = EnsureParentForItem(Item);

	const FObjectKey ItemID = Item->GetID();
	check(!TreeItemMap.Contains(ItemID));

	TreeItemMap.Add(ItemID, Item);

	if (Parent.IsValid())
	{
		Parent->AddChild(Item);
	}
	else
	{
		RootTreeItems.Add(Item);
	}
}

void SPoseWatchManager::PopulateSearchStrings(const IPoseWatchManagerTreeItem& Item, TArray< FString >& OutSearchStrings) const
{
	for (const auto& Pair : Columns)
	{
		Pair.Value->PopulateSearchStrings(Item, OutSearchStrings);
	}
}

TSharedPtr<SWidget> SPoseWatchManager::OnOpenContextMenu()
{
	FMenuBuilder MenuBuilder(true, CommandList);

	MenuBuilder.BeginSection("Hierarchy", LOCTEXT("HierarchyMenuHeader", "Hierarchy"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("CreateFolder", "Create Folder"),
			LOCTEXT("CreateFolderDescription", "Create a new folder"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "SceneOutliner.NewFolderIcon"),
			FUIAction(FExecuteAction::CreateSP(this, &SPoseWatchManager::CreateFolder))
		);
	}
	MenuBuilder.EndSection();

	if (PoseWatchManagerTreeView.Get()->GetNumItemsSelected() > 0)
	{
		MenuBuilder.BeginSection("Edit", LOCTEXT("EditMenuHeader", "Edit"));
		{
			MenuBuilder.AddMenuEntry(FGenericCommands::Get().Delete);
			MenuBuilder.AddMenuEntry(FGenericCommands::Get().Rename);
		}
		MenuBuilder.EndSection();
	}

	return MenuBuilder.MakeWidget();
}

void SPoseWatchManager::Rename_Execute()
{
	FPoseWatchManagerTreeItemPtr ItemToRename = GetSelection();
	if (ItemToRename && ItemToRename.IsValid())
	{
		PendingRenameItem = ItemToRename->AsShared();
		ScrollItemIntoView(ItemToRename);
	}
}

void SPoseWatchManager::Delete_Execute()
{
	if (PoseWatchManagerTreeView.Get()->GetNumItemsSelected() != 1)
	{
		return;
	}
	FPoseWatchManagerTreeItemPtr SelectedItem = PoseWatchManagerTreeView.Get()->GetSelectedItems()[0];
	SelectedItem->OnRemoved();
}

void SPoseWatchManager::SetColumnVisibility(FName ColumnId, bool bIsVisible)
{
	if (Columns.Contains(ColumnId))
	{
		HeaderRowWidget->SetShowGeneratedColumn(ColumnId, bIsVisible);
	}
}

void SPoseWatchManager::SetSelection(const TFunctionRef<bool(IPoseWatchManagerTreeItem&)> Selector)
{
	TArray<FPoseWatchManagerTreeItemPtr> ItemsToAdd;
	for (const auto& Pair : TreeItemMap)
	{
		FPoseWatchManagerTreeItemPtr ItemPtr = Pair.Value;
		if (IPoseWatchManagerTreeItem* Item = ItemPtr.Get())
		{
			if (Selector(*Item))
			{
				ItemsToAdd.Add(ItemPtr);
			}
		}
	}

	SetItemSelection(ItemsToAdd, true);
}

void SPoseWatchManager::SetItemSelection(const TArray<FPoseWatchManagerTreeItemPtr>& InItems, bool bSelected, ESelectInfo::Type SelectInfo)
{
	PoseWatchManagerTreeView->ClearSelection();
	PoseWatchManagerTreeView->SetItemSelection(InItems, bSelected, SelectInfo);
}

void SPoseWatchManager::SetItemSelection(const FPoseWatchManagerTreeItemPtr& InItem, bool bSelected, ESelectInfo::Type SelectInfo)
{
	PoseWatchManagerTreeView->ClearSelection();
	PoseWatchManagerTreeView->SetItemSelection(InItem, bSelected, SelectInfo);
}

void SPoseWatchManager::ClearSelection()
{
	if (!bIsReentrant)
	{
		PoseWatchManagerTreeView->ClearSelection();
	}
}

void SPoseWatchManager::OnPoseWatchesChanged(UAnimBlueprint* InAnimBlueprint, UEdGraphNode* InNode)
{
	for (UPoseWatch* SomePoseWatch : AnimBlueprint->PoseWatches)
	{
		if (SomePoseWatch->Node == InNode)
		{
			FPoseWatchManagerHierarchyChangedData EventData;
			EventData.Type = FPoseWatchManagerHierarchyChangedData::Added;
			EventData.Items.Add(Mode->PoseWatchManager->CreateItemFor<FPoseWatchManagerPoseWatchTreeItem>(SomePoseWatch));
			EventData.ItemActions = PoseWatchManager::ENewItemAction::Select | PoseWatchManager::ENewItemAction::Rename;
			OnHierarchyChangedEvent(EventData);
			return;
		}
	}

	// A pose watch for this node was not found, it must've been removed
	FullRefresh();
}

FReply SPoseWatchManager::OnCreateFolderClicked()
{
	CreateFolder();
	return FReply::Handled();
}

void SPoseWatchManager::CreateFolder()
{
	check(AnimBlueprint);
	UPoseWatchFolder* NewPoseWatchFolder = NewObject<UPoseWatchFolder>(AnimBlueprint);

	if (PoseWatchManagerTreeView.Get()->GetNumItemsSelected() == 1)
	{
		// Create the folder at the same level as the selected item
		FPoseWatchManagerTreeItemPtr SelectedItem = PoseWatchManagerTreeView.Get()->GetSelectedItems()[0];
		if (FPoseWatchManagerFolderTreeItem* SelectedFolderItem = SelectedItem->CastTo<FPoseWatchManagerFolderTreeItem>())
		{
			NewPoseWatchFolder->SetParent(SelectedFolderItem->PoseWatchFolder.Get(), true /* bForce */);
		}
		else if (FPoseWatchManagerPoseWatchTreeItem* SelectedPoseWatchItem = SelectedItem->CastTo<FPoseWatchManagerPoseWatchTreeItem>())
		{
			NewPoseWatchFolder->SetParent(SelectedPoseWatchItem->PoseWatch.Get()->GetParent(), true /* bForce */);
		}
	}

	NewPoseWatchFolder->SetUniqueDefaultLabel();
	AnimBlueprint->PoseWatchFolders.Add(NewPoseWatchFolder);

	FPoseWatchManagerHierarchyChangedData EventData;
	EventData.Type = FPoseWatchManagerHierarchyChangedData::Added;
	EventData.Items.Add(Mode->PoseWatchManager->CreateItemFor<FPoseWatchManagerFolderTreeItem>(NewPoseWatchFolder));
	EventData.ItemActions = PoseWatchManager::ENewItemAction::Select | PoseWatchManager::ENewItemAction::Rename;
	OnHierarchyChangedEvent(EventData);
}

void SPoseWatchManager::ScrollItemIntoView(const FPoseWatchManagerTreeItemPtr& Item)
{
	auto Parent = Item->GetParent();
	while (Parent.IsValid())
	{
		PoseWatchManagerTreeView->SetItemExpansion(Parent->AsShared(), true);
		Parent = Parent->GetParent();
	}
	PoseWatchManagerTreeView->RequestScrollIntoView(Item);
}

TSharedRef< ITableRow > SPoseWatchManager::OnGenerateRowForManagerTree(FPoseWatchManagerTreeItemPtr Item, const TSharedRef< STableViewBase >& OwnerTable)
{
	return SNew(SPoseWatchManagerTreeRow, PoseWatchManagerTreeView.ToSharedRef(), SharedThis(this)).Item(Item);
}

void SPoseWatchManager::OnGetChildrenForManagerTree(FPoseWatchManagerTreeItemPtr InParent, TArray< FPoseWatchManagerTreeItemPtr >& OutChildren)
{
	for (auto& WeakChild : InParent->GetChildren())
	{
		auto Child = WeakChild.Pin();
		// Should never have bogus entries in this list
		check(Child.IsValid());
		OutChildren.Add(Child);
	}

	// If the item needs it's children sorting, do that now
	if (OutChildren.Num())
	{
		// Sort the children we returned
		SortItems(OutChildren);

		// Empty out the children and repopulate them in the correct order
		InParent->Children.Empty();
		for (auto& Child : OutChildren)
		{
			InParent->Children.Emplace(Child);
		}
	}
}

void SPoseWatchManager::OnManagerTreeSelectionChanged(FPoseWatchManagerTreeItemPtr TreeItem, ESelectInfo::Type SelectInfo)
{
	if (SelectInfo == ESelectInfo::Direct)
	{
		return;
	}

	if (!bIsReentrant)
	{
		TGuardValue<bool> ReentrantGuard(bIsReentrant, true);

		BlueprintEditor->GetInspector()->ShowDetailsForSingleObject(nullptr);
		if (TreeItem.IsValid())
		{
			if (const FPoseWatchManagerElementTreeItem* ElementItem = TreeItem->CastTo<FPoseWatchManagerElementTreeItem>())
			{
				BlueprintEditor->GetInspector()->ShowDetailsForSingleObject(ElementItem->PoseWatchElement.Get());
			}
		}
		OnItemSelectionChanged.Broadcast(TreeItem, SelectInfo);
	}
}

void SPoseWatchManager::OnManagerTreeItemScrolledIntoView(FPoseWatchManagerTreeItemPtr TreeItem, const TSharedPtr<ITableRow>& Widget)
{
	if (TreeItem == PendingRenameItem.Pin())
	{
		PendingRenameItem = nullptr;
		TreeItem->RenameRequestEvent.ExecuteIfBound();
	}
}

void SPoseWatchManager::OnItemExpansionChanged(FPoseWatchManagerTreeItemPtr TreeItem, bool bIsExpanded) const
{
	check(TreeItem.Get()->IsA<FPoseWatchManagerFolderTreeItem>() || TreeItem.Get()->IsA<FPoseWatchManagerPoseWatchTreeItem>());
	TreeItem->SetIsExpanded(bIsExpanded);
}

void SPoseWatchManager::OnHierarchyChangedEvent(FPoseWatchManagerHierarchyChangedData Event)
{
	if (Event.Type == FPoseWatchManagerHierarchyChangedData::Added)
	{
		for (const auto& TreeItemPtr : Event.Items)
		{
			if (TreeItemPtr.IsValid() && !TreeItemMap.Find(TreeItemPtr->GetID()))
			{
				AddPendingItemAndChildren(TreeItemPtr);
				if (Event.ItemActions)
				{
					NewItemActions.Add(TreeItemPtr->GetID(), Event.ItemActions);
				}
			}
		}
		FullRefresh();
	}
	else if (Event.Type == FPoseWatchManagerHierarchyChangedData::FullRefresh)
	{
		FullRefresh();
	}
}

void SPoseWatchManager::OnFilterTextChanged(const FText& InFilterText)
{
	SearchBoxFilter->SetRawFilterText(InFilterText);
	FilterTextBoxWidget->SetError(SearchBoxFilter->GetFilterErrorText());
}

EVisibility SPoseWatchManager::GetFilterStatusVisibility() const
{
	return IsTextFilterActive() ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SPoseWatchManager::GetEmptyLabelVisibility() const
{
	return (IsTextFilterActive() || RootTreeItems.Num() > 0) ? EVisibility::Collapsed : EVisibility::Visible;
}

bool SPoseWatchManager::IsTextFilterActive() const
{
	return FilterTextBoxWidget->GetText().ToString().Len() > 0;
}

const FSlateBrush* SPoseWatchManager::GetFilterButtonGlyph() const
{
	if (IsTextFilterActive())
	{
		return FAppStyle::GetBrush(TEXT("SceneOutliner.FilterCancel"));
	}
	else
	{
		return FAppStyle::GetBrush(TEXT("SceneOutliner.FilterSearch"));
	}
}

FString SPoseWatchManager::GetFilterButtonToolTip() const
{
	return IsTextFilterActive() ? LOCTEXT("ClearSearchFilter", "Clear search filter").ToString() : LOCTEXT("StartSearching", "Search").ToString();

}

TAttribute<FText> SPoseWatchManager::GetFilterHighlightText() const
{
	return TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateStatic([](TWeakPtr<PoseWatchManager::TreeItemTextFilter> Filter) {
		auto FilterPtr = Filter.Pin();
		return FilterPtr.IsValid() ? FilterPtr->GetRawFilterText() : FText();
		}, TWeakPtr<PoseWatchManager::TreeItemTextFilter>(SearchBoxFilter)));
}

void SPoseWatchManager::SetKeyboardFocus()
{
	if (SupportsKeyboardFocus())
	{
		FWidgetPath PoseWatchManagerTreeViewWidgetPath;
		// NOTE: Careful, GeneratePathToWidget can be reentrant in that it can call visibility delegates and such
		FSlateApplication::Get().GeneratePathToWidgetUnchecked(PoseWatchManagerTreeView.ToSharedRef(), PoseWatchManagerTreeViewWidgetPath);
		FSlateApplication::Get().SetKeyboardFocus(PoseWatchManagerTreeViewWidgetPath, EFocusCause::SetDirectly);
	}
}

bool SPoseWatchManager::SupportsKeyboardFocus() const
{
	return true;
}

FReply SPoseWatchManager::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	return CommandList->ProcessCommandBindings(InKeyEvent) ? FReply::Handled() : FReply::Unhandled();
}

void SPoseWatchManager::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (bPendingFocusNextFrame && FilterTextBoxWidget->GetVisibility() == EVisibility::Visible)
	{
		FWidgetPath WidgetToFocusPath;
		FSlateApplication::Get().GeneratePathToWidgetUnchecked(FilterTextBoxWidget.ToSharedRef(), WidgetToFocusPath);
		FSlateApplication::Get().SetKeyboardFocus(WidgetToFocusPath, EFocusCause::SetDirectly);
		bPendingFocusNextFrame = false;
	}

	if (bNeedsColumnRefresh)
	{
		SetupColumns(*HeaderRowWidget);
	}

	if (bNeedsRefresh)
	{
		if (!bIsReentrant)
		{
			Populate();
		}
	}

	if (bSortDirty)
	{
		SortItems(RootTreeItems);
		PoseWatchManagerTreeView->RequestTreeRefresh();
		bSortDirty = false;
	}

	if (bSelectionDirty)
	{
		//Mode->SynchronizeSelection();
		bSelectionDirty = false;
	}
}

EColumnSortMode::Type SPoseWatchManager::GetColumnSortMode(const FName ColumnId) const
{
	if (SortByColumn == ColumnId)
	{
		auto Column = Columns.FindRef(ColumnId);
		if (Column.IsValid() && Column->SupportsSorting())
		{
			return SortMode;
		}
	}

	return EColumnSortMode::None;
}

void SPoseWatchManager::OnColumnSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode)
{
	auto Column = Columns.FindRef(ColumnId);
	if (!Column.IsValid() || !Column->SupportsSorting())
	{
		return;
	}

	SortByColumn = ColumnId;
	SortMode = InSortMode;

	RequestSort();
}

void SPoseWatchManager::RequestSort()
{
	bSortDirty = true;
}

void SPoseWatchManager::SortItems(TArray<FPoseWatchManagerTreeItemPtr>& Items) const
{
	auto Column = Columns.FindRef(SortByColumn);
	if (Column.IsValid())
	{
		Column->SortItems(Items, SortMode);
	}
}

void SPoseWatchManager::SetItemExpansionRecursive(FPoseWatchManagerTreeItemPtr Model, bool bInExpansionState)
{
	if (Model.IsValid())
	{
		PoseWatchManagerTreeView->SetItemExpansion(Model, bInExpansionState);
		for (auto& Child : Model->Children)
		{
			if (Child.IsValid())
			{
				SetItemExpansionRecursive(Child.Pin(), bInExpansionState);
			}
		}
	}
}

TSharedPtr<FDragDropOperation> SPoseWatchManager::CreateDragDropOperation(const TArray<FPoseWatchManagerTreeItemPtr>& InTreeItems) const
{
	return Mode->CreateDragDropOperation(InTreeItems);
}

/** Parse a drag drop operation into a payload */
bool SPoseWatchManager::ParseDragDrop(FPoseWatchManagerDragDropPayload& OutPayload, const FDragDropOperation& Operation) const
{
	return Mode->ParseDragDrop(OutPayload, Operation);
}

/** Validate a drag drop operation on a drop target */
FPoseWatchManagerDragValidationInfo SPoseWatchManager::ValidateDrop(const IPoseWatchManagerTreeItem& DropTarget, const FPoseWatchManagerDragDropPayload& Payload) const
{
	return Mode->ValidateDrop(DropTarget, Payload);
}

/** Called when a payload is dropped onto a target */
void SPoseWatchManager::OnDropPayload(IPoseWatchManagerTreeItem& DropTarget, const FPoseWatchManagerDragDropPayload& Payload, const FPoseWatchManagerDragValidationInfo& ValidationInfo) const
{
	return Mode->OnDrop(DropTarget, Payload, ValidationInfo);
}

/** Called when a payload is dragged over an item */
FReply SPoseWatchManager::OnDragOverItem(const FDragDropEvent& Event, const IPoseWatchManagerTreeItem& Item) const
{
	return Mode->OnDragOverItem(Event, Item);
}

FPoseWatchManagerTreeItemPtr SPoseWatchManager::FindParent(const IPoseWatchManagerTreeItem& InItem) const
{
	FPoseWatchManagerTreeItemPtr Parent = Mode->Hierarchy->FindParent(InItem, TreeItemMap);
	if (!Parent.IsValid())
	{
		Parent = Mode->Hierarchy->FindParent(InItem, PendingTreeItemMap);
	}
	return Parent;
}

uint32 SPoseWatchManager::GetTypeSortPriority(const IPoseWatchManagerTreeItem& Item) const
{
	return Mode->GetTypeSortPriority(Item);
}

void SPoseWatchManager::OnManagerTreeDoubleClick(FPoseWatchManagerTreeItemPtr TreeItem)
{
	UPoseWatch* PoseWatch = nullptr;

	if (FPoseWatchManagerPoseWatchTreeItem* PoseWatchTreeItem = TreeItem->CastTo<FPoseWatchManagerPoseWatchTreeItem>())
	{
		PoseWatch = PoseWatchTreeItem->PoseWatch.Get();
	}
	else if (FPoseWatchManagerElementTreeItem* ElementTreeItem = TreeItem->CastTo<FPoseWatchManagerElementTreeItem>())
	{
		PoseWatch = ElementTreeItem->PoseWatchElement->GetParent();
	}

	if (PoseWatch)
	{
		BlueprintEditor->JumpToHyperlink(PoseWatch->Node.Get(), false);
	}
}

void SPoseWatchManager::BindCommands()
{
	// This should not be called twice on the same instance
	check(!CommandList.IsValid());
	CommandList = MakeShareable(new FUICommandList);
		
	CommandList->MapAction(FGenericCommands::Get().Delete, FExecuteAction::CreateSP(this, &SPoseWatchManager::Delete_Execute));
	CommandList->MapAction(FGenericCommands::Get().Rename, FExecuteAction::CreateSP(this, &SPoseWatchManager::Rename_Execute));
}

#undef LOCTEXT_NAMESPACE
