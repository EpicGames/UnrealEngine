// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "EditorUndoClient.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/Views/ITypedTableView.h"
#include "Input/Reply.h"
#include "Layout/Visibility.h"
#include "Misc/Attribute.h"
#include "Misc/TextFilter.h"
#include "SlateFwd.h"
#include "Styling/SlateColor.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"
#include "Delegates/DelegateCombinations.h"
#include "Framework/Views/ITypedTableView.h"
#include "Types/SlateEnums.h"

#include "BlueprintEditor.h"
#include "Engine/PoseWatch.h"
#include "PoseWatchManagerFolderTreeItem.h"


#include "IPoseWatchManager.h"
#include "PoseWatchManagerFwd.h"

#include "SPoseWatchManagerTreeView.h"
#include "PoseWatchManagerPublicTypes.h"
#include "PoseWatchManagerStandaloneTypes.h"

#include "PoseWatchManagerDragDrop.h"


class FMenuBuilder;
class UToolMenu;
class IPoseWatchManagerColumn;
class FPoseWatchManagerDefaultHierarchy;
class SComboButton;

template<typename ItemType> class STreeView;


namespace PoseWatchManager
{
	DECLARE_EVENT_OneParam(SPoseWatchManager, FTreeItemPtrEvent, FPoseWatchManagerTreeItemPtr);

	DECLARE_EVENT_TwoParams(SPoseWatchManager, FOnItemSelectionChanged, FPoseWatchManagerTreeItemPtr, ESelectInfo::Type);

	typedef TTextFilter<const IPoseWatchManagerTreeItem&> TreeItemTextFilter;

	/** Structure that defines an operation that should be applied to the tree */
	struct FPendingTreeOperation
	{
		enum EType { Added, Removed, Moved };
		FPendingTreeOperation(EType InType, TSharedRef<IPoseWatchManagerTreeItem> InItem) : Type(InType), Item(InItem) { }

		/** The type of operation that is to be applied */
		EType Type;

		/** The tree item to which this operation relates */
		FPoseWatchManagerTreeItemRef Item;
	};

	/** Set of actions to apply to new tree items */
	namespace ENewItemAction
	{
		enum Type
		{
			/** Do nothing when it is created */
			None = 0,
			/** Select the item when it is created */
			Select = 1 << 0,
			/** Scroll the item into view when it is created */
			ScrollIntoView = 1 << 1,
			/** Interactively rename the item when it is created (implies the above) */
			Rename = 1 << 2,
		};
	}
}

class SPoseWatchManager : public IPoseWatchManager, public FGCObject
{
public:
	SLATE_BEGIN_ARGS(SPoseWatchManager)
	{}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const FPoseWatchManagerInitializationOptions& InitOptions);

	SPoseWatchManager();
	virtual ~SPoseWatchManager();

	// SWidget interface 
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	virtual bool SupportsKeyboardFocus() const override;
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

	/** Sends a requests to refresh the next chance it gets */
	virtual void Refresh() override;

	void RefreshSelection();

	/** Get a const reference to the actual tree hierarchy */
	virtual const STreeView<FPoseWatchManagerTreeItemPtr>& GetTree() const override
	{
		return *PoseWatchManagerTreeView;
	}

	virtual const TSharedPtr<SPoseWatchManagerTreeView>& GetTreeView() const
	{
		return PoseWatchManagerTreeView;
	}

	/** @return Returns a string to use for highlighting results in the list */
	virtual TAttribute<FText> GetFilterHighlightText() const;

	/** Set the keyboard focus to the manager */
	virtual void SetKeyboardFocus() override;

	void SetColumnVisibility(FName ColumnId, bool bIsVisible);

	/** Returns the current sort mode of the specified column */
	virtual EColumnSortMode::Type GetColumnSortMode(const FName ColumnId) const;

	/** Request that the tree be sorted at a convenient time */
	virtual void RequestSort();

	/** Requests a full refresh, which will clear the entire tree and rebuild it from scratch */
	virtual void FullRefresh() override;

public:
	/** Event to react to a user double click on a item */
	PoseWatchManager::FTreeItemPtrEvent& GetDoubleClickEvent() { return OnDoubleClickOnTreeEvent; }

	PoseWatchManager::FOnItemSelectionChanged& GetOnItemSelectionChanged() { return OnItemSelectionChanged; }

	/** Set the item selection of the manager based on a selector function. Any items which return true will be added */
	virtual void SetSelection(const TFunctionRef<bool(IPoseWatchManagerTreeItem&)> Selector) override;

	/** Set the selection status of a set of items in the scene manager */
	void SetItemSelection(const TArray<FPoseWatchManagerTreeItemPtr>& InItems, bool bSelected, ESelectInfo::Type SelectInfo = ESelectInfo::Direct);

	/** Set the selection status of a single item in the scene manager */
	void SetItemSelection(const FPoseWatchManagerTreeItemPtr& InItem, bool bSelected, ESelectInfo::Type SelectInfo = ESelectInfo::Direct);


	virtual FPoseWatchManagerTreeItemPtr GetSelection() const { return PoseWatchManagerTreeView->GetSelectedItems().Num() == 1 ? PoseWatchManagerTreeView->GetSelectedItems()[0] : nullptr; }

	FPoseWatchManagerTreeItemPtr FindParent(const IPoseWatchManagerTreeItem& InItem) const;

	/** Deselect all selected items */
	void ClearSelection();

	/** Sets the next item to rename */
	void SetPendingRenameItem(const FPoseWatchManagerTreeItemPtr& InItem) { PendingRenameItem = InItem; Refresh(); }

	/** Retrieve an IPoseWatchManagerTreeItem by its ID if it exists in the tree */
	FPoseWatchManagerTreeItemPtr GetTreeItem(FObjectKey, bool bIncludePending = false);

	/** Create a drag drop operation */
	TSharedPtr<FDragDropOperation> CreateDragDropOperation(const TArray<FPoseWatchManagerTreeItemPtr>& InTreeItems) const;

	/** Parse a drag drop operation into a payload */
	bool ParseDragDrop(FPoseWatchManagerDragDropPayload& OutPayload, const FDragDropOperation& Operation) const;

	/** Validate a drag drop operation on a drop target */
	FPoseWatchManagerDragValidationInfo ValidateDrop(const IPoseWatchManagerTreeItem& DropTarget, const FPoseWatchManagerDragDropPayload& Payload) const;

	/** Called when a payload is dropped onto a target */
	void OnDropPayload(IPoseWatchManagerTreeItem& DropTarget, const FPoseWatchManagerDragDropPayload& Payload, const FPoseWatchManagerDragValidationInfo& ValidationInfo) const;

	/** Called when a payload is dragged over an item */
	FReply OnDragOverItem(const FDragDropEvent& Event, const IPoseWatchManagerTreeItem& Item) const;

	virtual uint32 GetTypeSortPriority(const IPoseWatchManagerTreeItem& Item) const;


	/** Used to test if manager related selection changes have already been handled */
	bool GetIsReentrant() const { return bIsReentrant; }

private:
	/** Empty all the tree item containers maintained by this manager */
	void EmptyTreeItems();

	/** Apply incremental changes to, or a complete repopulation of the tree  */
	void Populate();

	/** Repopulates the entire tree */
	void RepopulateEntireTree();

	/** Adds a single new item to the pending map and creates an add operation for it */
	void AddPendingItem(FPoseWatchManagerTreeItemPtr Item);

	/** Adds a new item and all of its children to the pending items. */
	void AddPendingItemAndChildren(FPoseWatchManagerTreeItemPtr Item);

	/** Attempts to add a pending item to the current tree. Will add any parents if required. */
	bool AddItemToTree(FPoseWatchManagerTreeItemRef InItem);

	/** Add an item to the tree, even if it doesn't match the filter terms. Used to add parent's that would otherwise be filtered out */
	void AddUnfilteredItemToTree(FPoseWatchManagerTreeItemRef Item);

	/** Ensure that the specified item's parent is added to the tree, if applicable */
	FPoseWatchManagerTreeItemPtr EnsureParentForItem(FPoseWatchManagerTreeItemRef Item);

public:
	// Test the filters using stack-allocated data to prevent unnecessary heap allocations
	template <typename TreeItemType, typename TreeItemData>
	FPoseWatchManagerTreeItemPtr CreateItemFor(const TreeItemData& Data, bool bForce = false)
	{
		const TreeItemType Temporary(Data);

		if (bForce || SearchBoxFilter->PassesFilter(Temporary))
		{
			FPoseWatchManagerTreeItemPtr Result = MakeShareable(new TreeItemType(Data));
			return Result;
		}

		return nullptr;
	}

	/** Instruct the manager to perform an action on the specified item when it is created */
	void OnItemAdded(const FObjectKey& ItemID, uint8 ActionMask);

	void OnPoseWatchesChanged(UAnimBlueprint* InAnimBlueprint, UEdGraphNode* InNode);

	virtual void Rename_Execute() override;

	virtual void Delete_Execute();

	/** Get the columns to be displayed in this manager */
	const TMap<FName, TSharedPtr<IPoseWatchManagerColumn>>& GetColumns() const
	{
		return Columns;
	}

	/** @return	Returns true if the text filter is currently active */
	bool IsTextFilterActive() const;

	bool PassesTextFilter(const FPoseWatchManagerTreeItemPtr& Item) const
	{
		return SearchBoxFilter->PassesFilter(*Item);
	}

	bool HasSelectorFocus(FPoseWatchManagerTreeItemPtr Item) const
	{
		return PoseWatchManagerTreeView->Private_HasSelectorFocus(Item);
	}

private:
	/** Map of columns that are shown on this manager. */
	TMap<FName, TSharedPtr<IPoseWatchManagerColumn>> Columns;

	/** Set up the columns required for this manager */
	void SetupColumns(SHeaderRow& HeaderRow);

	/** Refresh the scene manager for when a column was added or removed */
	void RefreshColumns();

	/** Populates OutSearchStrings with the strings associated with TreeItem that should be used in searching */
	void PopulateSearchStrings(const IPoseWatchManagerTreeItem& Item, TArray< FString >& OutSearchStrings) const;

public:
	/** Scroll the specified item into view */
	void ScrollItemIntoView(const FPoseWatchManagerTreeItemPtr& Item);

private:
	/** Called by STreeView to generate a table row for the specified item */
	TSharedRef< ITableRow > OnGenerateRowForManagerTree(FPoseWatchManagerTreeItemPtr Item, const TSharedRef< STableViewBase >& OwnerTable);

	/** Called by STreeView to get child items for the specified parent item */
	void OnGetChildrenForManagerTree(FPoseWatchManagerTreeItemPtr InParent, TArray< FPoseWatchManagerTreeItemPtr >& OutChildren);

	/** Called by STreeView when the tree's selection has changed */
	void OnManagerTreeSelectionChanged(FPoseWatchManagerTreeItemPtr TreeItem, ESelectInfo::Type SelectInfo);

	/** Called by STreeView when an item is scrolled into view */
	void OnManagerTreeItemScrolledIntoView(FPoseWatchManagerTreeItemPtr TreeItem, const TSharedPtr<ITableRow>& Widget);

	void OnManagerTreeDoubleClick(FPoseWatchManagerTreeItemPtr TreeItem);

	/** Called when an item in the tree has been collapsed or expanded */
	void OnItemExpansionChanged(FPoseWatchManagerTreeItemPtr TreeItem, bool bIsExpanded) const;

private:
	/** Event required to keep the pose watch manager up to date */
	void OnHierarchyChangedEvent(FPoseWatchManagerHierarchyChangedData Event);

private:
	/** Called by the editable text control when the filter text is changed by the user */
	void OnFilterTextChanged(const FText& InFilterText);

	/** Called by the filter button to get the image to display in the button */
	const FSlateBrush* GetFilterButtonGlyph() const;

	/** @return	The filter button tool-tip text */
	FString GetFilterButtonToolTip() const;

	/** @return	Returns whether the filter status line should be drawn */
	EVisibility GetFilterStatusVisibility() const;

	/**	Returns the current visibility of the Empty label */
	EVisibility GetEmptyLabelVisibility() const;

	/** Returns the selection mode*/
	ESelectionMode::Type GetSelectionMode() const;

private:
	/** Called when the user right clicks in the tree view */
	TSharedPtr<SWidget> OnOpenContextMenu();

	/** Called when the user has clicked the button to add a new folder */
	FReply OnCreateFolderClicked();

	/** Requests a new folder be created, confirmation received with OnHierarchyChangedEvent */
	void CreateFolder();

	/** Binds our UI commands to delegates. */
	void BindCommands();

private:
	/** Context menu opening delegate provided by the client */
	FOnContextMenuOpening OnContextMenuOpening;

	/** List of pending operations to be applied to the tree */
	TArray<PoseWatchManager::FPendingTreeOperation> PendingOperations;

	/** Map of actions to apply to new tree items */
	TMap<FObjectKey, uint8> NewItemActions;

	/** Our tree view */
	TSharedPtr< SPoseWatchManagerTreeView > PoseWatchManagerTreeView;

	/** A map of all items we have in the tree */
	FPoseWatchManagerTreeItemMap TreeItemMap;

	/** Pending tree items that are yet to be added the tree */
	FPoseWatchManagerTreeItemMap PendingTreeItemMap;

	/** Root level tree items */
	TArray<FPoseWatchManagerTreeItemPtr> RootTreeItems;

	TSharedPtr<FUICommandList> CommandList;
private:
	/** true if the manager  needs to be repopulated at the next appropriate opportunity */
	uint8 bNeedsRefresh : 1;

	/** Processing a full refresh until pending items are processed */
	uint8 bProcessingFullRefresh : 1;

	/** true if should do a full refresh */
	uint8 bFullRefresh : 1;

	/** true if should refresh selection */
	uint8 bSelectionDirty : 1;

	uint8 bNeedsColumnRefresh : 1;

	/** Reentrancy guard */
	bool bIsReentrant;

	/* Widget containing the filtering text box */
	TSharedPtr< SSearchBox > FilterTextBoxWidget;

	/** The header row of the manager */
	TSharedPtr< SHeaderRow > HeaderRowWidget;

	/** The TextFilter attached to the SearchBox widget */
	TSharedPtr< PoseWatchManager::TreeItemTextFilter > SearchBoxFilter;

	/** True if the search box will take keyboard focus next frame */
	bool bPendingFocusNextFrame;

	/** The tree item that is currently pending a rename */
	TWeakPtr<IPoseWatchManagerTreeItem> PendingRenameItem;

	TUniquePtr<FPoseWatchManagerDefaultHierarchy> Hierarchy;

	PoseWatchManager::FTreeItemPtrEvent OnDoubleClickOnTreeEvent;

	PoseWatchManager::FOnItemSelectionChanged OnItemSelectionChanged;

private:
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override {};
	virtual FString GetReferencerName() const override
	{
		return TEXT("PoseWatchManager::SPoseWatchManager");
	}

	bool bDisableIntermediateSorting;

	/** true if currently needs to be sorted */
	bool bSortDirty;

	/** Specify which column to sort with */
	FName SortByColumn;

	/** Currently selected sorting mode */
	EColumnSortMode::Type SortMode;

	FBlueprintEditor* BlueprintEditor;

	/** Handles column sorting mode change */
	void OnColumnSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode);

	/** Sort the specified array of items based on the current sort column */
	void SortItems(TArray<FPoseWatchManagerTreeItemPtr>& Items) const;

	/** Handler for recursively expanding/collapsing items */
	void SetItemExpansionRecursive(FPoseWatchManagerTreeItemPtr Model, bool bInExpansionState);
};
