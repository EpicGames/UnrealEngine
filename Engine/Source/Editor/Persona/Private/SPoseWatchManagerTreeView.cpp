// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPoseWatchManagerTreeView.h"
#include "Framework/Application/SlateApplication.h"
#include "Styling/AppStyle.h"
#include "IPoseWatchManagerColumn.h"
#include "PoseWatchManagerPublicTypes.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "PoseWatchManagerDragDrop.h"
#include "PoseWatchManagerStandaloneTypes.h"
#include "SPoseWatchManager.h"

#include "PoseWatchManagerFolderTreeItem.h"

#define LOCTEXT_NAMESPACE "SPoseWatchManager"

static void UpdateOperationDecorator(const FDragDropEvent& Event, const FPoseWatchManagerDragValidationInfo& ValidationInfo)
{
	const FSlateBrush* Icon = ValidationInfo.IsValid() ? FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK")) : FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error"));

	FDragDropOperation* Operation = Event.GetOperation().Get();

	if (Operation && Operation->IsOfType<FPoseWatchManagerDragDropOp>())
	{
		FPoseWatchManagerDragDropOp* PoseWatchDecoratedOp = static_cast<FPoseWatchManagerDragDropOp*>(Operation);
		PoseWatchDecoratedOp->SetTooltip(ValidationInfo.ValidationText, Icon);
	}
}

static void ResetOperationDecorator(const FDragDropEvent& Event)
{
	FDragDropOperation* Operation = Event.GetOperation().Get();
	if (Operation)
	{
		if (Operation->IsOfType<FPoseWatchManagerDragDropOp>())
		{
			static_cast<FPoseWatchManagerDragDropOp*>(Operation)->ResetTooltip();
		}
	}
}

static FReply HandleOnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, TWeakPtr<SPoseWatchManagerTreeView> Table)
{
	TSharedPtr<SPoseWatchManagerTreeView> TablePtr = Table.Pin();
	if (TablePtr.IsValid() && MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
	{
		TSharedPtr<FDragDropOperation> Operation = TablePtr->GetPoseWatchManagerPtr().Pin()->CreateDragDropOperation(TablePtr->GetSelectedItems());

		if (Operation.IsValid())
		{
			return FReply::Handled().BeginDragDrop(Operation.ToSharedRef());
		}
	}

	return FReply::Unhandled();
}

FReply HandleDrop(TSharedPtr<SPoseWatchManager> PoseWatchManagerPtr, const FDragDropEvent& DragDropEvent, IPoseWatchManagerTreeItem& DropTarget, FPoseWatchManagerDragValidationInfo& ValidationInfo, bool bApplyDrop = false)
{
	if (!PoseWatchManagerPtr.IsValid())
	{
		return FReply::Unhandled();
	}

	FPoseWatchManagerDragDropPayload DraggedObjects(*DragDropEvent.GetOperation());
	// Validate now to make sure we don't doing anything we shouldn't
	if (!PoseWatchManagerPtr->ParseDragDrop(DraggedObjects, *DragDropEvent.GetOperation()))
	{
		return FReply::Unhandled();
	}

	ValidationInfo = PoseWatchManagerPtr->ValidateDrop(StaticCast<IPoseWatchManagerTreeItem&>(DropTarget), DraggedObjects);

	if (!ValidationInfo.IsValid())
	{
		// Return handled here to stop anything else trying to handle it - the operation is invalid as far as we're concerned
		return FReply::Handled();
	}

	if (bApplyDrop)
	{
		PoseWatchManagerPtr->OnDropPayload(DropTarget, DraggedObjects, ValidationInfo);
	}

	return FReply::Handled();
}

FReply HandleDropFromWeak(TWeakPtr<SPoseWatchManager> PoseWatchManagerWeak, const FDragDropEvent& DragDropEvent, IPoseWatchManagerTreeItem& DropTarget, FPoseWatchManagerDragValidationInfo& ValidationInfo, bool bApplyDrop = false)
{
	return HandleDrop(PoseWatchManagerWeak.Pin(), DragDropEvent, DropTarget, ValidationInfo, bApplyDrop);
}

void SPoseWatchManagerTreeView::Construct(const SPoseWatchManagerTreeView::FArguments& InArgs, TSharedRef<SPoseWatchManager> Owner)
{
	PoseWatchManagerWeak = Owner;
	STreeView::Construct(InArgs);
}

void SPoseWatchManagerTreeView::FlashHighlightOnItem(FPoseWatchManagerTreeItemPtr FlashHighlightOnItem)
{
	TSharedPtr< SPoseWatchManagerTreeRow > RowWidget = StaticCastSharedPtr< SPoseWatchManagerTreeRow >(WidgetGenerator.GetWidgetForItem(FlashHighlightOnItem));
	if (RowWidget.IsValid())
	{
		RowWidget->FlashHighlight();
	}
}

FReply SPoseWatchManagerTreeView::OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	FPoseWatchManagerDragValidationInfo ValidationInfo = FPoseWatchManagerDragValidationInfo::Invalid();

	FPoseWatchManagerFolderTreeItem DropTarget(nullptr);

	FReply Reply = HandleDropFromWeak(PoseWatchManagerWeak, DragDropEvent, DropTarget, ValidationInfo);
	if (Reply.IsEventHandled())
	{
		UpdateOperationDecorator(DragDropEvent, ValidationInfo);
	}

	return Reply;
}

void SPoseWatchManagerTreeView::OnDragLeave(const FDragDropEvent& DragDropEvent)
{
	ResetOperationDecorator(DragDropEvent);
}

FReply SPoseWatchManagerTreeView::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	FPoseWatchManagerDragValidationInfo ValidationInfo = FPoseWatchManagerDragValidationInfo::Invalid();

	// Dropping to folder nullptr will assign the payload to the root of the tree
	FPoseWatchManagerFolderTreeItem DropTarget(nullptr);

	FReply Reply = HandleDropFromWeak(PoseWatchManagerWeak, DragDropEvent, DropTarget, ValidationInfo, true);
	//PoseWatchManagerPtr.Get()->FullRefresh();
	PoseWatchManagerWeak.Pin()->FullRefresh();

	return Reply;
}

FReply SPoseWatchManagerTreeRow::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	auto ItemPtr = Item.Pin();
	auto PoseWatchManagerPtr = PoseWatchManagerWeak.Pin();
	if (ItemPtr.IsValid() && PoseWatchManagerPtr.IsValid())
	{
		FPoseWatchManagerDragValidationInfo ValidationInfo = FPoseWatchManagerDragValidationInfo::Invalid();
		FReply Reply = HandleDrop(PoseWatchManagerPtr, DragDropEvent, *ItemPtr, ValidationInfo, true);
		PoseWatchManagerPtr.Get()->FullRefresh();
		return Reply;
	}

	return FReply::Unhandled();
}

void SPoseWatchManagerTreeRow::OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	FPoseWatchManagerTreeItemPtr ItemPtr = Item.Pin();
	TSharedPtr<SPoseWatchManager> PoseWatchManagerPtr = PoseWatchManagerWeak.Pin();
	if (ItemPtr.IsValid() && PoseWatchManagerPtr.IsValid())
	{
		FPoseWatchManagerDragValidationInfo ValidationInfo = FPoseWatchManagerDragValidationInfo::Invalid();

		FReply Reply = HandleDrop(PoseWatchManagerPtr, DragDropEvent, *ItemPtr, ValidationInfo, false);
		if (Reply.IsEventHandled()) 
		{
			UpdateOperationDecorator(DragDropEvent, ValidationInfo);
		}
	}
}

void SPoseWatchManagerTreeRow::OnDragLeave(const FDragDropEvent& DragDropEvent)
{
	ResetOperationDecorator(DragDropEvent);
}

FReply SPoseWatchManagerTreeRow::OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<SPoseWatchManager> PoseWatchManagerPtr = PoseWatchManagerWeak.Pin();
	if (SPoseWatchManager* PoseWatchManager = PoseWatchManagerPtr.Get())
	{
		if (const FPoseWatchManagerTreeItemPtr ItemPtr = Item.Pin())
		{
			return PoseWatchManager->OnDragOverItem(DragDropEvent, *ItemPtr.Get());
		}
		return FReply::Unhandled();
	}

	return FReply::Handled();
}

FReply SPoseWatchManagerTreeRow::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	FPoseWatchManagerTreeItemPtr ItemPtr = Item.Pin();
	if (ItemPtr.IsValid())
	{
		if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
		{
			FReply Reply = SMultiColumnTableRow<FPoseWatchManagerTreeItemPtr>::OnMouseButtonDown(MyGeometry, MouseEvent);
			return Reply.DetectDrag(SharedThis(this), EKeys::LeftMouseButton);
		}
	}

	return FReply::Handled();
}

FReply SPoseWatchManagerTreeRow::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	FPoseWatchManagerTreeItemPtr ItemPtr = Item.Pin();
	// We don't to change the selection when it is a left click since this was handle in the on mouse down
	if (ItemPtr.IsValid())
	{
		return SMultiColumnTableRow<FPoseWatchManagerTreeItemPtr>::OnMouseButtonUp(MyGeometry, MouseEvent);
	}

	return FReply::Handled();
}

TSharedRef<SWidget> SPoseWatchManagerTreeRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	FPoseWatchManagerTreeItemPtr ItemPtr = Item.Pin();
	if (!ItemPtr.IsValid())
	{
		return SNullWidget::NullWidget;
	}

	// Create the widget for this item
	TSharedRef<SWidget> NewItemWidget = SNullWidget::NullWidget;

	check(PoseWatchManagerWeak.IsValid());
	auto Column = PoseWatchManagerWeak.Pin()->GetColumns().FindRef(ColumnName);
	if (Column.IsValid())
	{
		NewItemWidget = Column->ConstructRowWidget(ItemPtr.ToSharedRef(), *this);
	}

	if (ColumnName == FPoseWatchManagerBuiltInColumnTypes::Label())
	{
		// The first column gets the tree expansion arrow for this row
		return SNew(SBox)
			.MinDesiredHeight(FPoseWatchManagerDefaultTreeItemMetrics::RowHeight())
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(6.f, 0.f, 0.f, 0.f)
				[
					SNew(SExpanderArrow, SharedThis(this)).IndentAmount(12)
				]

				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				[
					NewItemWidget
				]
			];
	}
	else
	{
		// No expansion arrow needed
		return NewItemWidget;
	}
}

void SPoseWatchManagerTreeRow::Construct(const FArguments& InArgs, const TSharedRef<SPoseWatchManagerTreeView>& PoseWatchManagerTreeView, TSharedRef<SPoseWatchManager> PoseWatchManager)
{
	Item = InArgs._Item->AsShared();
	PoseWatchManagerWeak = PoseWatchManager;
	LastHighlightInteractionTime = 0.0;

	auto Args = FSuperRowType::FArguments()
		.Style(&FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("SceneOutliner.TableViewRow"));


	Args.OnDragDetected_Static(HandleOnDragDetected, TWeakPtr<SPoseWatchManagerTreeView>(PoseWatchManagerTreeView));

	SMultiColumnTableRow<FPoseWatchManagerTreeItemPtr>::Construct(Args, PoseWatchManagerTreeView);
}

const float SPoseWatchManagerTreeRow::HighlightRectLeftOffset = 0.0f;
const float SPoseWatchManagerTreeRow::HighlightRectRightOffset = 0.0f;
const float SPoseWatchManagerTreeRow::HighlightTargetSpringConstant = 25.0f;
const float SPoseWatchManagerTreeRow::HighlightTargetEffectDuration = 0.5f;
const float SPoseWatchManagerTreeRow::HighlightTargetOpacity = 0.8f;
const float SPoseWatchManagerTreeRow::LabelChangedAnimOffsetPercent = 0.2f;

void SPoseWatchManagerTreeRow::FlashHighlight()
{
	LastHighlightInteractionTime = FSlateApplication::Get().GetCurrentTime();
}

void SPoseWatchManagerTreeRow::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	// Call parent implementation.
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	// We'll draw with the 'focused' look if we're either focused or we have a context menu summoned
	const bool bShouldAppearFocused = HasKeyboardFocus();

	// Update highlight 'target' effect
	{
		const float HighlightLeftX = HighlightRectLeftOffset;
		const float HighlightRightX = HighlightRectRightOffset + static_cast<float>(AllottedGeometry.GetLocalSize().X);

		HighlightTargetLeftSpring.SetTarget(HighlightLeftX);
		HighlightTargetRightSpring.SetTarget(HighlightRightX);

		const float TimeSinceHighlightInteraction = static_cast<float>(InCurrentTime - LastHighlightInteractionTime);
		if (TimeSinceHighlightInteraction <= HighlightTargetEffectDuration || bShouldAppearFocused)
		{
			HighlightTargetLeftSpring.Tick(InDeltaTime);
			HighlightTargetRightSpring.Tick(InDeltaTime);
		}
	}
}

int32 SPoseWatchManagerTreeRow::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	int32 StartLayer = SMultiColumnTableRow::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);

	const int32 TextLayer = 1;

	// See if a disabled effect should be used
	bool bEnabled = ShouldBeEnabled(bParentEnabled);
	ESlateDrawEffect DrawEffects = (bEnabled) ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

	const double CurrentTime = FSlateApplication::Get().GetCurrentTime();

	// We'll draw with the 'focused' look if we're either focused or we have a context menu summoned
	const bool bShouldAppearFocused = HasKeyboardFocus();

	// Draw highlight targeting effect
	const float TimeSinceHighlightInteraction = (float)(CurrentTime - LastHighlightInteractionTime);
	if (TimeSinceHighlightInteraction <= HighlightTargetEffectDuration)
	{

		// Compute animation progress
		float EffectAlpha = FMath::Clamp(TimeSinceHighlightInteraction / HighlightTargetEffectDuration, 0.0f, 1.0f);
		EffectAlpha = 1.0f - EffectAlpha * EffectAlpha;  // Inverse square falloff (looks nicer!)

		// Apply extra opacity falloff when dehighlighting
		float EffectOpacity = EffectAlpha;

		// Figure out a universally visible highlight color.
		FLinearColor HighlightTargetColorAndOpacity = ((FLinearColor::White - GetColorAndOpacity()) * 0.5f + FLinearColor(+0.4f, +0.1f, -0.2f)) * InWidgetStyle.GetColorAndOpacityTint();
		HighlightTargetColorAndOpacity.A = HighlightTargetOpacity * EffectOpacity * 255.0f;

		// Compute the bounds offset of the highlight target from where the highlight target spring
		// extents currently lie.  This is used to "grow" or "shrink" the highlight as needed.
		const float LabelChangedAnimOffset = LabelChangedAnimOffsetPercent * static_cast<float>(AllottedGeometry.GetLocalSize().Y);

		// Choose an offset amount depending on whether we're highlighting, or clearing highlight
		const float EffectOffset = EffectAlpha * LabelChangedAnimOffset;

		const float HighlightLeftX = HighlightTargetLeftSpring.GetPosition() - EffectOffset;
		const float HighlightRightX = HighlightTargetRightSpring.GetPosition() + EffectOffset;
		const float HighlightTopY = 0.0f - LabelChangedAnimOffset;
		const float HighlightBottomY = static_cast<float>(AllottedGeometry.GetLocalSize().Y) + EffectOffset;

		const FVector2D DrawPosition = FVector2D(HighlightLeftX, HighlightTopY);
		const FVector2D DrawSize = FVector2D(HighlightRightX - HighlightLeftX, HighlightBottomY - HighlightTopY);

		const FSlateBrush* StyleInfo = FAppStyle::GetBrush("SceneOutliner.ChangedItemHighlight");

		// NOTE: We rely on scissor clipping for the highlight rectangle
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId + TextLayer,
			AllottedGeometry.ToPaintGeometry(DrawSize, FSlateLayoutTransform(DrawPosition)),	// Position, Size, Scale
			StyleInfo,													// Style
			DrawEffects,												// Effects to use
			HighlightTargetColorAndOpacity);							// Color
	}

	return FMath::Max(StartLayer, LayerId + TextLayer);
}

#undef LOCTEXT_NAMESPACE
