// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "PoseWatchManagerFwd.h"
#include "PoseWatchManagerStandaloneTypes.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STreeView.h"
#include "Animation/SlateSprings.h"

class SPoseWatchManager;

class SPoseWatchManagerTreeView : public STreeView< FPoseWatchManagerTreeItemPtr >
{
public:
	void Construct(const FArguments& InArgs, TSharedRef<SPoseWatchManager> Owner);

	void FlashHighlightOnItem(FPoseWatchManagerTreeItemPtr FlashHighlightOnItem);

	const TWeakPtr<SPoseWatchManager>& GetPoseWatchManagerPtr() { return PoseWatchManagerWeak; }

	

protected:
	virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;

	virtual void OnDragLeave(const FDragDropEvent& DragDropEvent) override;

	virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;

	TWeakPtr<SPoseWatchManager> PoseWatchManagerWeak;
};

class SPoseWatchManagerTreeRow
	: public SMultiColumnTableRow< FPoseWatchManagerTreeItemPtr >
{
public:
	SLATE_BEGIN_ARGS(SPoseWatchManagerTreeRow) {}
		SLATE_ARGUMENT(FPoseWatchManagerTreeItemPtr, Item)
	SLATE_END_ARGS()

	/** Construct function for this widget */
	void Construct(const FArguments& InArgs, const TSharedRef<SPoseWatchManagerTreeView>& PoseWatchManagerTreeView, TSharedRef<SPoseWatchManager> PoseWatchManager);

	/** Overridden from SMultiColumnTableRow.  Generates a widget for this column of the tree row. */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

	void FlashHighlight();

protected:
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;

	virtual void OnDragLeave(const FDragDropEvent& DragDropEvent) override;

	virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;

	virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;

private:

	/** Weak reference to the outliner widget that owns our list */
	TWeakPtr< SPoseWatchManager > PoseWatchManagerWeak;

	/** The item associated with this row of data */
	TWeakPtr<IPoseWatchManagerTreeItem> Item;

protected:

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;

private:
	/** How many pixels to extend the highlight rectangle's left side horizontally */
	static const float HighlightRectLeftOffset;

	/** How many pixels to extend the highlight rectangle's right side horizontally */
	static const float HighlightRectRightOffset;

	/** How quickly the highlight 'targeting' rectangle will slide around.  Larger is faster. */
	static const float HighlightTargetSpringConstant;

	/** Duration of animation highlight target effects */
	static const float HighlightTargetEffectDuration;

	/** Opacity of the highlight target effect overlay */
	static const float HighlightTargetOpacity;

	/** How large the highlight target effect will be when highlighting, as a scalar percentage of font height */
	static const float LabelChangedAnimOffsetPercent;

	/** Highlight "targeting" visual effect left position */
	FFloatSpring1D HighlightTargetLeftSpring;

	/** Highlight "targeting" visual effect right position */
	FFloatSpring1D HighlightTargetRightSpring;

	/** Last time that the user had a major interaction with the highlight */
	double LastHighlightInteractionTime;
};

