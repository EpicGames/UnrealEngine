// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPoseWatchPicker.h"

#include "AnimationEditorUtils.h"
#include "Animation/AnimBlueprintGeneratedClass.h"
#include "Engine/PoseWatch.h"
#include "Widgets/Colors/SColorBlock.h"

#define LOCTEXT_NAMESPACE "SPoseWatchPicker"

void SPoseWatchPicker::Construct(const FArguments& InArgs)
{
	AnimBlueprintAttribute = InArgs._AnimBlueprintGeneratedClass;

	PoseWatchComboBox = SNew(SComboBox<TSharedPtr<TWeakObjectPtr<UPoseWatchPoseElement>>>)
		.ToolTipText_Lambda([this, DefaultDisplayText = InArgs._DefaultEntryDisplayText]()
		{
			return FText::Format(LOCTEXT("PoseWatchTooltipFormat", "Previewing '{0}'"),  SelectedPoseWatch.Get() != nullptr && SelectedPoseWatch->GetParent() != nullptr ? SelectedPoseWatch->GetParent()->GetLabel() : DefaultDisplayText);
		})
		.OptionsSource(&CachedPoseWatches)
		.OnGenerateWidget_Lambda([this, DefaultDisplayText = InArgs._DefaultEntryDisplayText](const TSharedPtr<TWeakObjectPtr<UPoseWatchPoseElement>>& InElement)
		{			
			return
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(2.0f, 0.0f)
				[
					SNew(SColorBlock)
					.Visibility_Lambda([InElement]()
					{
						TWeakObjectPtr<UPoseWatchPoseElement> PoseWatchPoseElement = *InElement;
						return PoseWatchPoseElement.Get() != nullptr ? EVisibility::Visible : EVisibility::Collapsed;
					})
					.AlphaBackgroundBrush(FAppStyle::Get().GetBrush("ColorPicker.RoundedAlphaBackground"))
					.ShowBackgroundForAlpha(true)
					.AlphaDisplayMode(EColorBlockAlphaDisplayMode::Ignore)
					.Size(FVector2D(16.0f, 16.0f))
					.CornerRadius(FVector4(4.0f, 4.0f, 4.0f, 4.0f))	
					.Color_Lambda([InElement]()
					{
						TWeakObjectPtr<UPoseWatchPoseElement> PoseWatchPoseElement = *InElement;
						if(PoseWatchPoseElement.Get())
						{
							return FLinearColor(PoseWatchPoseElement->GetColor());
						}
						return FLinearColor::Gray;
					})
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(2.0f, 0.0f)
				[
					SNew(STextBlock)
					.Text_Lambda([InElement, DefaultDisplayText]()
					{
						TWeakObjectPtr<UPoseWatchPoseElement> PoseWatchPoseElement = *InElement;
						return PoseWatchPoseElement.Get() != nullptr && PoseWatchPoseElement->GetParent() != nullptr ? PoseWatchPoseElement->GetParent()->GetLabel() : DefaultDisplayText;
					})
				];
		})
		.OnSelectionChanged_Lambda([this](const TSharedPtr<TWeakObjectPtr<UPoseWatchPoseElement>>& InElement, ESelectInfo::Type InSelectionType)
		{
			if(!InElement.IsValid())
			{
				return;
			}

			TWeakObjectPtr<UPoseWatchPoseElement> PoseWatchPoseElement = *InElement;			
			if(PoseWatchPoseElement.Get() == nullptr)
			{
				SelectedPoseWatch = nullptr;
				return;
			}

			if(const UAnimBlueprintGeneratedClass* AnimClass = AnimBlueprintAttribute.Get())
			{
				// We have to grab our pose watches from the root class as no pose watches can be set on child anim BPs
				if(const UAnimBlueprintGeneratedClass* RootClass = Cast<UAnimBlueprintGeneratedClass>(AnimClass->GetRootClass()))
				{
					for(const FAnimNodePoseWatch& AnimNodePoseWatch : RootClass->AnimBlueprintDebugData.AnimNodePoseWatch)
					{
						if(AnimNodePoseWatch.PoseWatchPoseElement && PoseWatchPoseElement.Get() == AnimNodePoseWatch.PoseWatchPoseElement)
						{
							SelectedPoseWatch = AnimNodePoseWatch.PoseWatchPoseElement;
							break;
						}
					}
				}
			}
		})
		.Content()
		[
			SNew(SHorizontalBox)
			.Clipping(EWidgetClipping::OnDemand)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(2.0f, 0.0f)
			[
				SNew(SColorBlock)
				.Visibility_Lambda([this]()
				{
					return SelectedPoseWatch.Get() != nullptr ? EVisibility::Visible : EVisibility::Collapsed;
				})
				.AlphaBackgroundBrush(FAppStyle::Get().GetBrush("ColorPicker.RoundedAlphaBackground"))
				.ShowBackgroundForAlpha(true)
				.AlphaDisplayMode(EColorBlockAlphaDisplayMode::Ignore)
				.Size(FVector2D(16.0f, 16.0f))
				.CornerRadius(FVector4(4.0f, 4.0f, 4.0f, 4.0f))	
				.Color_Lambda([this]()
				{
					if(UPoseWatchPoseElement* Element = SelectedPoseWatch.Get())
					{
						return FLinearColor(Element->GetColor());
					}
					return FLinearColor::Gray;
				})
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(2.0f, 0.0f)
			[
				SNew(STextBlock)
				.Text_Lambda([this, DefaultDisplayText = InArgs._DefaultEntryDisplayText]()
				{
					return SelectedPoseWatch.Get() != nullptr && SelectedPoseWatch->GetParent() != nullptr ? SelectedPoseWatch->GetParent()->GetLabel() : DefaultDisplayText;
				})
			]
		];
	
	ChildSlot
	[
		PoseWatchComboBox.ToSharedRef()
	];

	AnimationEditorUtils::OnPoseWatchesChanged().AddSP(this, &SPoseWatchPicker::OnPoseWatchesChanged);
	RebuildPoseWatches();
}

UPoseWatchPoseElement* SPoseWatchPicker::GetCurrentPoseWatch() const
{
	return SelectedPoseWatch.Get();
}

void SPoseWatchPicker::OnPoseWatchesChanged(UAnimBlueprint* InAnimBlueprint, UEdGraphNode*)
{
	if(const UAnimBlueprintGeneratedClass* TargetClass = AnimBlueprintAttribute.Get())
	{
		if(TargetClass->IsChildOf(InAnimBlueprint->GeneratedClass))
		{
			RegisterActiveTimer(0.0f, FWidgetActiveTimerDelegate::CreateLambda([this](double /*InCurrentTime*/, float /*InDeltaTime*/)
			{
				RebuildPoseWatches();
				return EActiveTimerReturnType::Stop;
			}));
		}
	}
}

void SPoseWatchPicker::RebuildPoseWatches()
{
	CachedPoseWatches.Empty();

	if(const UAnimBlueprintGeneratedClass* TargetClass = AnimBlueprintAttribute.Get())
	{
		CachedPoseWatches.Add(MakeShared<TWeakObjectPtr<UPoseWatchPoseElement>>());
		// We have to grab our pose watches from the root class as no pose watches can be set on child anim BPs
		if(const UAnimBlueprintGeneratedClass* RootClass = Cast<UAnimBlueprintGeneratedClass>(TargetClass->GetRootClass()))
		{
			for(const FAnimNodePoseWatch& AnimNodePoseWatch : RootClass->AnimBlueprintDebugData.AnimNodePoseWatch)
			{
				if(AnimNodePoseWatch.PoseWatchPoseElement)
				{						
					CachedPoseWatches.Add(MakeShared<TWeakObjectPtr<UPoseWatchPoseElement>>(AnimNodePoseWatch.PoseWatchPoseElement));
				}
			}
		}

		PoseWatchComboBox->RefreshOptions();
	}
	
}

#undef LOCTEXT_NAMESPACE // "SPoseWatchPicker"