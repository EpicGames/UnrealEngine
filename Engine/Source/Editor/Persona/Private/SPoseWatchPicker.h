// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Animation/AnimBlueprint.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Input/SComboBox.h"

class UAnimBlueprint;
class UPoseWatchPoseElement;

class SPoseWatchPicker : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SPoseWatchPicker) : _AnimBlueprintGeneratedClass(nullptr) {}
		SLATE_ATTRIBUTE(const UAnimBlueprintGeneratedClass*, AnimBlueprintGeneratedClass)
		SLATE_ARGUMENT(FText, DefaultEntryDisplayText)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	UPoseWatchPoseElement* GetCurrentPoseWatch() const;

protected:
	void OnPoseWatchesChanged(UAnimBlueprint* InAnimBlueprint, UEdGraphNode* /*InNode*/);
	void RebuildPoseWatches();

protected:
	TSharedPtr<SComboBox<TSharedPtr<TWeakObjectPtr<UPoseWatchPoseElement>>>> PoseWatchComboBox;
	TArray<TSharedPtr<TWeakObjectPtr<UPoseWatchPoseElement>>> CachedPoseWatches;
	TAttribute<const UAnimBlueprintGeneratedClass*> AnimBlueprintAttribute;
	TWeakObjectPtr<UPoseWatchPoseElement> SelectedPoseWatch = nullptr;
};