// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRenameMorphTargetDialog.h"

#include "Animation/MorphTarget.h"
#include "Engine/SkinnedAssetCommon.h"
#include "Framework/Application/SlateApplication.h"
#include "SkeletalMeshAttributes.h"
#include "SkeletalMeshTypes.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SWindow.h"

#define LOCTEXT_NAMESPACE "RenameMorphTarget"

void SRenameMorphTargetDialog::Construct(const FArguments& InArgs)
{
	check(InArgs._SkeletalMesh)
	check(InArgs._MorphTarget)

	SkeletalMesh = InArgs._SkeletalMesh;
	MorphTarget = InArgs._MorphTarget;

	ChildSlot
	[
		SNew(SBox)
		.Padding(InArgs._Padding)
		[
			SNew(SGridPanel)
			.FillColumn(1, 1.0)

			// Current name display
			+ SGridPanel::Slot(0, 0)
			.Padding(2)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			[
				SNew(STextBlock)
				.Font(FAppStyle::GetFontStyle( TEXT("PropertyWindow.NormalFont")))
				.Text(LOCTEXT("CurrentName", "Current Name:"))
			]

			+ SGridPanel::Slot(1, 0)
			.Padding(2)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Fill)
			[
				SNew(STextBlock)
				.Font(FAppStyle::GetFontStyle( TEXT("PropertyWindow.NormalFont")))
				.Text(FText::FromName(MorphTarget->GetFName()))
			]
			
			// New name controls
			+ SGridPanel::Slot(0, 1)
			.Padding(2)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			[
				SNew(STextBlock )
				.Font(FAppStyle::GetFontStyle( TEXT("PropertyWindow.NormalFont")))
				.Text(LOCTEXT("NewName", "New Name:"))
			]

			+ SGridPanel::Slot(1, 1)
			.Padding(2)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Fill)
			[
				SAssignNew(NewMorphTargetNameTextBox, SEditableTextBox)
				.Font(FAppStyle::GetFontStyle( TEXT("PropertyWindow.NormalFont")))
				.Text(FText::FromName(MorphTarget->GetFName()))
				.MaximumLength(NAME_SIZE-1)
				.OnVerifyTextChanged(this, &SRenameMorphTargetDialog::OnVerifyNameChanged)
				.OnTextCommitted(this, &SRenameMorphTargetDialog::OnRenameTextCommitted)
			]

			// Dialog controls
			+ SGridPanel::Slot(0, 2)
			.ColumnSpan(2)
			.HAlign(HAlign_Right)
			.Padding(FMargin(0, 16))
			[
				SNew(SHorizontalBox)

				// Rename
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0, 0, 8.0f, 0)
				[
					SNew(SButton)
					.IsFocusable(false)
					.OnClicked(this, &SRenameMorphTargetDialog::OnRenameClicked)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("RenameMorphTargetButtonText", "Rename"))
					]
				]

				// Cancel
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SButton)
					.IsFocusable(false)
					.OnClicked(this, & SRenameMorphTargetDialog::OnCancelClicked)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("CancelRenameButtonText", "Cancel"))
					]
				]
			]
		]
	];
}

bool SRenameMorphTargetDialog::OnVerifyNameChanged(const FText& InText, FText& OutErrorMessage)
{
	const FString OriginalMorphTargetName = MorphTarget->GetName();
	if (InText.ToString().Equals(OriginalMorphTargetName))
	{
		return true;
	}

	if (!MorphTarget->Rename(*(InText.ToString()), nullptr, REN_Test))
	{
		OutErrorMessage = LOCTEXT("VerifyBadName", "Bad name");
		return false;
	}
	return true;
}

void SRenameMorphTargetDialog::RenameAndClose()
{
	const FString MorphTargetToRename = MorphTarget->GetName();
	const FString NewMorphTargetName = NewMorphTargetNameTextBox->GetText().ToString();
	if (NewMorphTargetName.Equals(MorphTargetToRename))
	{
		CloseContainingWindow();
		return;
	}
	if (MorphTarget->Rename(*NewMorphTargetName, nullptr, REN_Test))
	{
		if (MorphTarget->Rename(*NewMorphTargetName, nullptr))
		{
			for (int32 LodIndex = 0; LodIndex < SkeletalMesh->GetLODNum(); ++LodIndex)
			{
				//The rename is successful, we now update the source model so it build the skeletal mesh with the correct name
				if (FMeshDescription* MeshDescription = SkeletalMesh->GetMeshDescription(LodIndex))
				{
					FSkeletalMeshAttributes MeshDescriptionAttribute(*MeshDescription);
					FName OriginalMorphAttributeName = *MorphTargetToRename;
					FName NewMorphAttributeName = *NewMorphTargetName;
					bool bHasVertexMorphPositionDelta = MeshDescriptionAttribute.HasMorphTargetPositionsAttribute(OriginalMorphAttributeName);
					bool bHasMorphTargetNormalAttribute = MeshDescriptionAttribute.HasMorphTargetNormalsAttribute(OriginalMorphAttributeName);
					if (bHasVertexMorphPositionDelta)
					{
						bool bMorphTargetAttributeExist = MeshDescriptionAttribute.HasMorphTargetPositionsAttribute(NewMorphAttributeName);
						if (!bMorphTargetAttributeExist && bHasVertexMorphPositionDelta)
						{
							FScopedSkeletalMeshPostEditChange PostEditChangeScope(SkeletalMesh);
							MeshDescriptionAttribute.RegisterMorphTargetAttribute(NewMorphAttributeName, bHasMorphTargetNormalAttribute);
							TVertexAttributesRef<FVector3f> OriginalVertexMorphPositionDelta = MeshDescriptionAttribute.GetVertexMorphPositionDelta(OriginalMorphAttributeName);
							TVertexAttributesRef<FVector3f> RenamedVertexMorphPositionDelta = MeshDescriptionAttribute.GetVertexMorphPositionDelta(NewMorphAttributeName);
							RenamedVertexMorphPositionDelta.Copy(OriginalVertexMorphPositionDelta);
							if (bHasMorphTargetNormalAttribute)
							{
								TVertexInstanceAttributesRef<FVector3f> OriginalVertexInstanceMorphNormalDelta = MeshDescriptionAttribute.GetVertexInstanceMorphNormalDelta(OriginalMorphAttributeName);
								TVertexInstanceAttributesRef<FVector3f> RenamedVertexInstanceMorphNormalDelta = MeshDescriptionAttribute.GetVertexInstanceMorphNormalDelta(NewMorphAttributeName);
								RenamedVertexInstanceMorphNormalDelta.Copy(OriginalVertexInstanceMorphNormalDelta);
							}
							MeshDescriptionAttribute.UnregisterMorphTargetAttribute(OriginalMorphAttributeName);
							SkeletalMesh->CommitMeshDescription(0);
							FSkeletalMeshLODInfo* LodInfo = SkeletalMesh->GetLODInfo(LodIndex);
							if (ensure(LodInfo))
							{
								if (FMorphTargetImportedSourceFileInfo* MorphTargetImportedSourceFileInfo = LodInfo->ImportedMorphTargetSourceFilename.Find(MorphTargetToRename))
								{
									const FString OriginalSourceFile = MorphTargetImportedSourceFileInfo->GetSourceFilename();
									const bool bIsGeneratedByEngine = MorphTargetImportedSourceFileInfo->IsGeneratedByEngine();
									LodInfo->ImportedMorphTargetSourceFilename.Remove(MorphTargetToRename);
									//Add the mapping to the lod info so we can re-import the morph target
									FMorphTargetImportedSourceFileInfo& NewData = LodInfo->ImportedMorphTargetSourceFilename.FindOrAdd(NewMorphTargetName);
									NewData.SetSourceFilename(OriginalSourceFile);
									NewData.SetGeneratedByEngine(bIsGeneratedByEngine);
								}
							}
						}
					}
				}
			}
		}
	}
	CloseContainingWindow();
}

void SRenameMorphTargetDialog::OnRenameTextCommitted(const FText& InText, ETextCommit::Type InCommitType)
{
	if (InCommitType == ETextCommit::OnEnter)
	{
		RenameAndClose();
	}
}

FReply SRenameMorphTargetDialog::OnRenameClicked()
{
	RenameAndClose();

	return FReply::Handled();
}

FReply SRenameMorphTargetDialog::OnCancelClicked()
{
	CloseContainingWindow();

	return FReply::Handled();
}

void SRenameMorphTargetDialog::CloseContainingWindow()
{
	TSharedPtr<SWindow> CurrentWindow = FSlateApplication::Get().FindWidgetWindow( AsShared() );

	if (CurrentWindow.IsValid())
	{
		CurrentWindow->RequestDestroyWindow();
	}
}

#undef LOCTEXT_NAMESPACE
