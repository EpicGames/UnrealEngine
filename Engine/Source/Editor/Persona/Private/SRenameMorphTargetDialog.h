// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/MorphTarget.h"
#include "Engine/SkeletalMesh.h"
#include "UObject/ObjectPtr.h"
#include "Widgets/SCompoundWidget.h"

class SEditableTextBox;
namespace ETextCommit { enum Type : int; }
class UMorphTarget;

class SRenameMorphTargetDialog : public SCompoundWidget
{
public:


	SLATE_BEGIN_ARGS(SRenameMorphTargetDialog)
		: _SkeletalMesh(nullptr)
		, _MorphTarget(nullptr)
		, _Padding(FMargin(15))
		{}
		SLATE_ARGUMENT(TObjectPtr<USkeletalMesh>, SkeletalMesh)
		SLATE_ARGUMENT(TObjectPtr<UMorphTarget>, MorphTarget)
		SLATE_ARGUMENT(FMargin, Padding)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:

	/** Renames the morph target based on dialog parameters */
	FReply OnRenameClicked();

	/** Callback for when Cancel is clicked */
	FReply OnCancelClicked();

	/** Callback to verify the rename is good */
	bool OnVerifyNameChanged(const FText& InText, FText& OutErrorMessage);

	/** Renames the morph target and attempts to close the active window */
	void RenameAndClose();

	/** Attempts to rename the morph target if enter is pressed while editing the morph target name */
	void OnRenameTextCommitted(const FText& InText, ETextCommit::Type InCommitType);

	/** Closes the window that contains this widget */
	void CloseContainingWindow();

private:
	TObjectPtr<USkeletalMesh> SkeletalMesh = nullptr;

	TObjectPtr<UMorphTarget> MorphTarget = nullptr;

	TSharedPtr<SEditableTextBox> NewMorphTargetNameTextBox;
};
