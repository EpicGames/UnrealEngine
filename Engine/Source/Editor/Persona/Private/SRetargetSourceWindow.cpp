// Copyright Epic Games, Inc. All Rights Reserved.


#include "SRetargetSourceWindow.h"
#include "Modules/ModuleManager.h"
#include "Framework/Commands/UIAction.h"
#include "Textures/SlateIcon.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "AssetRegistry/AssetData.h"
#include "Styling/AppStyle.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Views/SListView.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "AssetNotifications.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Framework/Notifications/NotificationManager.h"
#include "IEditableSkeleton.h"
#include "AnimationRuntime.h"

#define LOCTEXT_NAMESPACE "SRetargetSourceWindow"

static const FName ColumnId_RetargetSourceNameLabel( "Retarget Source Name" );
static const FName ColumnID_BaseReferenceMeshLabel( "Reference Mesh" );

DECLARE_DELEGATE_TwoParams( FOnRenameCommit, const FName& /*OldName*/,  const FString& /*NewName*/ )
DECLARE_DELEGATE_RetVal_ThreeParams( bool, FOnVerifyRenameCommit, const FName& /*OldName*/, const FString& /*NewName*/, FText& /*OutErrorMessage*/)

//////////////////////////////////////////////////////////////////////////
// SRetargetSourceListRow

typedef TSharedPtr< FDisplayedRetargetSourceInfo > FDisplayedRetargetSourceInfoPtr;

class SRetargetSourceListRow
	: public SMultiColumnTableRow< FDisplayedRetargetSourceInfoPtr >
{
public:

	SLATE_BEGIN_ARGS( SRetargetSourceListRow ) {}

	/** The item for this row **/
	SLATE_ARGUMENT( FDisplayedRetargetSourceInfoPtr, Item )

	/* The SRetargetSourceWindow that handles all retarget sources */
	SLATE_ARGUMENT( class SRetargetSourceWindow*, RetargetSourceWindow )

	/* Widget used to display the list of retarget sources*/
	SLATE_ARGUMENT( TSharedPtr<SRetargetSourceListType>, RetargetSourceListView )

	/** Delegate for when an asset name has been entered for an item that is in a rename state */
	SLATE_EVENT( FOnRenameCommit, OnRenameCommit )

	/** Delegate for when an asset name has been entered for an item to verify the name before commit */
	SLATE_EVENT( FOnVerifyRenameCommit, OnVerifyRenameCommit )

	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTableView );

	/** Overridden from SMultiColumnTableRow.  Generates a widget for this column of the tree row. */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn( const FName& ColumnName ) override;

private:

	/* Returns the reference mesh this pose is based on	*/
	FString GetReferenceMeshName() { return Item->ReferenceMesh->GetPathName(); }

	/* The SRetargetSourceWindow that handles all retarget sources*/
	SRetargetSourceWindow* RetargetSourceWindow;

	/** Widget used to display the list of retarget sources*/
	TSharedPtr<SRetargetSourceListType> RetargetSourceListView;

	/** The name and weight of the retarget source*/
	FDisplayedRetargetSourceInfoPtr	Item;

	/** Delegate for when an asset name has been entered for an item that is in a rename state */
	FOnRenameCommit OnRenameCommit;

	/** Delegate for when an asset name has been entered for an item to verify the name before commit */
	FOnVerifyRenameCommit OnVerifyRenameCommit;

	/** Handles committing a name change */
	virtual void HandleNameCommitted( const FText& NewText, ETextCommit::Type CommitInfo );

	/** Handles verifying a name change */
	virtual bool HandleVerifyNameChanged( const FText& NewText, FText& OutErrorMessage );
};

void SRetargetSourceListRow::Construct( const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView )
{
	Item = InArgs._Item;
	RetargetSourceWindow = InArgs._RetargetSourceWindow;
	RetargetSourceListView = InArgs._RetargetSourceListView;
	OnRenameCommit = InArgs._OnRenameCommit;
	OnVerifyRenameCommit = InArgs._OnVerifyRenameCommit;

	check( Item.IsValid() );

	SMultiColumnTableRow< FDisplayedRetargetSourceInfoPtr >::Construct( FSuperRowType::FArguments(), InOwnerTableView );
}

TSharedRef< SWidget > SRetargetSourceListRow::GenerateWidgetForColumn( const FName& ColumnName )
{
	if ( ColumnName == ColumnId_RetargetSourceNameLabel )
	{
		TSharedPtr< SInlineEditableTextBlock > InlineWidget;
		TSharedRef< SWidget > NewWidget = 
			SNew( SVerticalBox )

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding( 0.0f, 4.0f )
			.VAlign( VAlign_Center )
			[
				SAssignNew(InlineWidget, SInlineEditableTextBlock)
				.Text( FText::FromString(Item->GetDisplayName()) )
				.OnTextCommitted(this, &SRetargetSourceListRow::HandleNameCommitted)
				.OnVerifyTextChanged(this, &SRetargetSourceListRow::HandleVerifyNameChanged)
				.HighlightText( RetargetSourceWindow->GetFilterText() )
				.IsReadOnly(false)
				.IsSelected(this, &SMultiColumnTableRow< FDisplayedRetargetSourceInfoPtr >::IsSelectedExclusively)
			];

		Item->OnEnterEditingMode.BindSP( InlineWidget.Get(), &SInlineEditableTextBlock::EnterEditingMode);

		return NewWidget;
	}
	else
	{
		// Encase the SSpinbox in an SVertical box so we can apply padding. Setting ItemHeight on the containing SListView has no effect :-(
		return
			SNew( SVerticalBox )

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding( 0.0f, 1.0f )
			.VAlign( VAlign_Center )
			[
				SNew( STextBlock )
				.Text( FText::FromString(Item->GetReferenceMeshName()) )
				.HighlightText( RetargetSourceWindow->GetFilterText() )
			];
	}
}

/** Handles committing a name change */
void SRetargetSourceListRow::HandleNameCommitted( const FText& NewText, ETextCommit::Type CommitInfo )
{
	OnRenameCommit.ExecuteIfBound(Item->Name, NewText.ToString());
}

/** Handles verifying a name change */
bool SRetargetSourceListRow::HandleVerifyNameChanged( const FText& NewText, FText& OutErrorMessage )
{
	return !OnVerifyRenameCommit.IsBound() || OnVerifyRenameCommit.Execute(Item->Name, NewText.ToString(), OutErrorMessage);
}
//////////////////////////////////////////////////////////////////////////
// SRetargetSourceWindow

void SRetargetSourceWindow::Construct(const FArguments& InArgs, const TSharedRef<IEditableSkeleton>& InEditableSkeleton, FSimpleMulticastDelegate& InOnPostUndo)
{
	EditableSkeletonPtr = InEditableSkeleton;

	InOnPostUndo.Add(FSimpleDelegate::CreateSP( this, &SRetargetSourceWindow::PostUndo ) );
	
	ChildSlot
	[
		SNew( SVerticalBox )
		
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew( SHorizontalBox )

			+SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			.AutoWidth()
			.Padding(2, 0)
			[
				SNew(SButton)
				.OnClicked(FOnClicked::CreateSP(this, &SRetargetSourceWindow::OnAddRetargetSourceButtonClicked))
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Text(LOCTEXT("AddRetargetSourceButton_Label", "Add New"))
				.ToolTipText(LOCTEXT("AddRetargetSourceButton_ToolTip", "Select a Skeletal Mesh asset to become a new retarget source for this Skeleton asset.\n\nRetarget Sources indicate what proportions a sequence was authored with so that animation is correctly retargeted to other proportions.\n\nThese become 'Retarget Source' options on sequences.\n\nRetarget Sources are only needed when an animation sequence is authored on a skeletal mesh with proportions that are different than the default skeleton asset."))
			]

			+SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			.AutoWidth()
			.Padding(2, 0)
			[
				SNew(SButton)
				.OnClicked(FOnClicked::CreateSP(this, &SRetargetSourceWindow::OnUpdateAllRetargetSourceButtonClicked))
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Text(LOCTEXT("UpdateAllRetargetSourceButton_Label", "Update All"))
				.ToolTipText(LOCTEXT("UpdateAllRetargetSourceButton_ToolTip", "Use this to update all retarget source poses with latest mesh proportions. If you want to update individually, use the context menu."))
			]
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0,2)
		[
			SNew(SHorizontalBox)
			// Filter entry
			+SHorizontalBox::Slot()
			.FillWidth( 1 )
			[
				SAssignNew( NameFilterBox, SSearchBox )
				.SelectAllTextWhenFocused( true )
				.OnTextChanged( this, &SRetargetSourceWindow::OnFilterTextChanged )
				.OnTextCommitted( this, &SRetargetSourceWindow::OnFilterTextCommitted )
			]
		]

		+ SVerticalBox::Slot()
		.FillHeight( 1.0f )		// This is required to make the scrollbar work, as content overflows Slate containers by default
		[
			SAssignNew( RetargetSourceListView, SRetargetSourceListType )
			.ListItemsSource( &RetargetSourceList )
			.OnGenerateRow( this, &SRetargetSourceWindow::GenerateRetargetSourceRow )
			.OnContextMenuOpening( this, &SRetargetSourceWindow::OnGetContextMenuContent )
			.HeaderRow
			(
				SNew( SHeaderRow )
				+ SHeaderRow::Column( ColumnId_RetargetSourceNameLabel )
				.DefaultLabel( LOCTEXT( "RetargetSourceNameLabel", "Retarget Source Name" ) )

				+ SHeaderRow::Column( ColumnID_BaseReferenceMeshLabel )
				.DefaultLabel( LOCTEXT( "RetargetSourceWeightLabel", "Source Mesh" ) )
			)
		]
	];

	CreateRetargetSourceList();
}

void SRetargetSourceWindow::OnFilterTextChanged( const FText& SearchText )
{
	FilterText = SearchText;

	CreateRetargetSourceList( SearchText.ToString() );
}

void SRetargetSourceWindow::OnFilterTextCommitted( const FText& SearchText, ETextCommit::Type CommitInfo )
{
	// Just do the same as if the user typed in the box
	OnFilterTextChanged( SearchText );
}

TSharedRef<ITableRow> SRetargetSourceWindow::GenerateRetargetSourceRow(TSharedPtr<FDisplayedRetargetSourceInfo> InInfo, const TSharedRef<STableViewBase>& OwnerTable)
{
	check( InInfo.IsValid() );

	return
		SNew( SRetargetSourceListRow, OwnerTable )
		.Item( InInfo )
		.RetargetSourceWindow( this )
		.RetargetSourceListView( RetargetSourceListView )
		.OnRenameCommit(this, &SRetargetSourceWindow::OnRenameCommit)
		.OnVerifyRenameCommit(this, &SRetargetSourceWindow::OnVerifyRenameCommit);
}


void SRetargetSourceWindow::OnRenameCommit( const FName& InOldName, const FString& InNewName )
{
	FString NewNameString = InNewName;
	if (InOldName != FName(*NewNameString.TrimStartAndEnd()))
	{
		FName NewName = *InNewName;
		EditableSkeletonPtr.Pin()->RenameRetargetSource(InOldName, NewName);

		FAssetNotifications::SkeletonNeedsToBeSaved(&EditableSkeletonPtr.Pin()->GetSkeleton());
		CreateRetargetSourceList( NameFilterBox->GetText().ToString() );
	}
}

bool SRetargetSourceWindow::OnVerifyRenameCommit( const FName& OldName, const FString& NewName, FText& OutErrorMessage)
{
	// if same reject
	FString NewString = NewName;

	if (OldName == FName(*NewString.TrimStartAndEnd()))
	{
		OutErrorMessage = FText::Format (LOCTEXT("RetargetSourceWindowNameSame", "{0} Nothing modified"), FText::FromString(OldName.ToString()) );
		return false;
	}

	const USkeleton& Skeleton = EditableSkeletonPtr.Pin()->GetSkeleton();
	const FReferencePose* Pose = Skeleton.AnimRetargetSources.Find(OldName);
	if (!Pose)
	{
		OutErrorMessage = FText::Format (LOCTEXT("RetargetSourceWindowNameNotFound", "{0} Not found"), FText::FromString(OldName.ToString()) );
		return false;
	}

	Pose = Skeleton.AnimRetargetSources.Find(FName(*NewName));
	if (Pose)
	{
		OutErrorMessage = FText::Format (LOCTEXT("RetargetSourceWindowNameDuplicated", "{0} already exists"), FText::FromString(NewName) );
		return false;
	}

	return true;
}

TSharedPtr<SWidget> SRetargetSourceWindow::OnGetContextMenuContent() const
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder( bShouldCloseWindowAfterMenuSelection, NULL);

	MenuBuilder.BeginSection("RetargetSourceAction", LOCTEXT( "New", "New" ) );
	{
		FUIAction Action = FUIAction( FExecuteAction::CreateSP( const_cast<SRetargetSourceWindow*>(this), &SRetargetSourceWindow::OnAddRetargetSource ) );
		const FText Label = LOCTEXT("AddRetargetSourceActionLabel", "Add...");
		const FText ToolTipText = LOCTEXT("AddRetargetSourceActionTooltip", "Add new retarget source.");
		MenuBuilder.AddMenuEntry( Label, ToolTipText, FSlateIcon(), Action);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("RetargetSourceAction", LOCTEXT( "Selected", "Selected Item Actions" ) );
	{
		FUIAction Action = FUIAction( FExecuteAction::CreateSP( const_cast<SRetargetSourceWindow*>(this), &SRetargetSourceWindow::OnRenameRetargetSource ), 
			FCanExecuteAction::CreateSP( this, &SRetargetSourceWindow::CanPerformRename ) );
		const FText Label = LOCTEXT("RenameRetargetSourceActionLabel", "Rename");
		const FText ToolTipText = LOCTEXT("RenameRetargetSourceActionTooltip", "Rename the selected retarget source.");
		MenuBuilder.AddMenuEntry( Label, ToolTipText, FSlateIcon(), Action);
	}
	{
		FUIAction Action = FUIAction( FExecuteAction::CreateSP( const_cast<SRetargetSourceWindow*>(this), &SRetargetSourceWindow::OnDeleteRetargetSource ), 
			FCanExecuteAction::CreateSP( this, &SRetargetSourceWindow::CanPerformDelete ) );
		const FText Label = LOCTEXT("DeleteRetargetSourceActionLabel", "Delete");
		const FText ToolTipText = LOCTEXT("DeleteRetargetSourceActionTooltip", "Deletes the selected retarget sources.");
		MenuBuilder.AddMenuEntry( Label, ToolTipText, FSlateIcon(), Action);
	}
	{
		FUIAction Action = FUIAction( FExecuteAction::CreateSP( const_cast<SRetargetSourceWindow*>(this), &SRetargetSourceWindow::OnRefreshRetargetSource, false ), 
			FCanExecuteAction::CreateSP( this, &SRetargetSourceWindow::CanPerformRefresh ) );
		const FText Label = LOCTEXT("RefreshRetargetSourceActionLabel", "Update");
		const FText ToolTipText = LOCTEXT("RefreshRetargetSourceActionTooltip", "Updates the selected retarget sources from source mesh.");
		MenuBuilder.AddMenuEntry( Label, ToolTipText, FSlateIcon(), Action);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

/** @TODO: FIXME: Item to rename. Only valid for adding and this is so hacky, I know**/
//TSharedPtr<FDisplayedBasePoseInfo> ItemToRename;

void SRetargetSourceWindow::CreateRetargetSourceList( const FString& SearchText, const FName  NewName )
{
	RetargetSourceList.Empty();
	bool bDoFiltering = !SearchText.IsEmpty();

	const USkeleton& Skeleton = EditableSkeletonPtr.Pin()->GetSkeleton();
	for (auto Iter=Skeleton.AnimRetargetSources.CreateConstIterator(); Iter; ++Iter)
	{
		const FName& Name = Iter.Key();
		const FReferencePose& RefPose = Iter.Value();
		const FString SourceMeshName = RefPose.SourceReferenceMesh.ToString();

		if ( bDoFiltering )
		{
			if (!Name.ToString().Contains( SearchText ) && !SourceMeshName.Contains(SearchText))
			{
				continue; // Skip items that don't match our filter
			}
		}

		USkeletalMesh* LoadedMesh = RefPose.SourceReferenceMesh.Get();
		bool bDirty = false;
		if (LoadedMesh)
		{
			TArray<FTransform> TransformArray;
			FAnimationRuntime::MakeSkeletonRefPoseFromMesh(LoadedMesh, &Skeleton, TransformArray);
			bDirty = (RefPose.ReferencePose.Num() != TransformArray.Num()) || (FMemory::Memcmp(TransformArray.GetData(), RefPose.ReferencePose.GetData(), RefPose.ReferencePose.GetAllocatedSize()) != 0);
		}

		TSharedRef<FDisplayedRetargetSourceInfo> Info = FDisplayedRetargetSourceInfo::Make( Name, RefPose.SourceReferenceMesh, bDirty);

		if (Name == NewName)
		{
			ItemToRename = Info;
		}

		RetargetSourceList.Add( Info );
	}

	RetargetSourceListView->RequestListRefresh();
}

int32 SRetargetSourceWindow::OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	const int32 Result = SCompoundWidget::OnPaint( Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled );

	// I need to do this after first paint	
	if (ItemToRename.IsValid())
	{
		ItemToRename->RequestRename();
		ItemToRename = NULL;
	}

	return Result;
}

void SRetargetSourceWindow::AddRetargetSource( const FName Name, USkeletalMesh * ReferenceMesh  )
{
	EditableSkeletonPtr.Pin()->AddRetargetSource(Name, ReferenceMesh);

	FAssetNotifications::SkeletonNeedsToBeSaved(&EditableSkeletonPtr.Pin()->GetSkeleton());

	// clear search filter
	NameFilterBox->SetText(FText::GetEmpty());
	CreateRetargetSourceList( NameFilterBox->GetText().ToString(), Name );
}

void SRetargetSourceWindow::OnAddRetargetSource()
{
	// show list of skeletalmeshes that they can choose from
	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

	FAssetPickerConfig AssetPickerConfig;
	AssetPickerConfig.Filter.ClassPaths.Add(USkeletalMesh::StaticClass()->GetClassPathName());
	AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateSP(this, &SRetargetSourceWindow::OnAssetSelectedFromMeshPicker);
	AssetPickerConfig.bAllowNullSelection = false;
	AssetPickerConfig.InitialAssetViewType = EAssetViewType::Tile;

	const USkeleton& Skeleton = EditableSkeletonPtr.Pin()->GetSkeleton();

	AssetPickerConfig.OnShouldFilterAsset = FOnShouldFilterAsset::CreateLambda([&Skeleton](const FAssetData& InAssetData)
	{
		if(Skeleton.IsCompatibleForEditor(InAssetData))
		{
			return false;
		}
		return true;
	});

	TSharedRef<SWidget> Widget = SNew(SBox)
		.WidthOverride(384.f)
		.HeightOverride(768.f)
		[
			SNew(SBorder)
			.BorderBackgroundColor(FLinearColor(0.25f, 0.25f, 0.25f, 1.f))
			.Padding( 2.f )
			[
				SNew(SBorder)
				.BorderImage( FAppStyle::GetBrush("ToolPanel.GroupBorder") )
				.Padding( 8.f )
				[
					ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
				]
			]
		];

	FSlateApplication::Get().PushMenu(
		AsShared(),
		FWidgetPath(),
		Widget,
		FSlateApplication::Get().GetCursorPos(),
		FPopupTransitionEffect( FPopupTransitionEffect::TopMenu )
		);
}

void SRetargetSourceWindow::OnAssetSelectedFromMeshPicker(const FAssetData& AssetData)
{
	// make sure you don't have any more retarget sources from the same mesh
	const USkeleton& Skeleton = EditableSkeletonPtr.Pin()->GetSkeleton();
	const FString AssetFullPath = AssetData.ToSoftObjectPath().ToString();
	for (auto Iter=Skeleton.AnimRetargetSources.CreateConstIterator(); Iter; ++Iter)
	{
		const FName& Name = Iter.Key();
		const FReferencePose& RefPose = Iter.Value();

		if (RefPose.SourceReferenceMesh.ToString() == AssetFullPath)
		{
			// make ask users if they'd like to create new source when there is existing source. 
			// they could update existing source

			// redundant source exists
			FFormatNamedArguments Args;
			Args.Add(TEXT("SkeletalMeshName"), FText::FromString(AssetFullPath));
			Args.Add(TEXT("ExistingSourceName"), FText::FromName(RefPose.PoseName));
			FNotificationInfo Info(FText::Format(LOCTEXT("RetargetSourceAlreadyExists", "Retarget Source for {SkeletalMeshName} already exists : {ExistingSourceName}"), Args));
			Info.ExpireDuration = 5.0f;
			Info.bUseLargeFont = false;
			TSharedPtr<SNotificationItem> Notification = FSlateNotificationManager::Get().AddNotification(Info);
			if (Notification.IsValid())
			{
				Notification->SetCompletionState(SNotificationItem::CS_Fail);
			}

			FSlateApplication::Get().DismissAllMenus();
			return;
		}
	}

	USkeletalMesh * SelectedMesh = CastChecked<USkeletalMesh>(AssetData.GetAsset());
	// give temporary name, and make it editable first time
	AddRetargetSource(FName(*SelectedMesh->GetName()), SelectedMesh);
	FSlateApplication::Get().DismissAllMenus();
}

bool SRetargetSourceWindow::CanPerformDelete() const
{
	TArray< TSharedPtr< FDisplayedRetargetSourceInfo > > SelectedRows = RetargetSourceListView->GetSelectedItems();
	return SelectedRows.Num() > 0;
}

void SRetargetSourceWindow::OnDeleteRetargetSource()
{
	TArray<FName> SelectedNames;
	TArray< TSharedPtr< FDisplayedRetargetSourceInfo > > SelectedRows = RetargetSourceListView->GetSelectedItems();
	for (int RowIndex = 0; RowIndex < SelectedRows.Num(); ++RowIndex)
	{
		SelectedNames.Add(SelectedRows[RowIndex]->Name);
	}

	EditableSkeletonPtr.Pin()->DeleteRetargetSources(SelectedNames);

	FAssetNotifications::SkeletonNeedsToBeSaved(&EditableSkeletonPtr.Pin()->GetSkeleton());

	CreateRetargetSourceList( NameFilterBox->GetText().ToString() );
}

bool SRetargetSourceWindow::CanPerformRename() const
{
	TArray< TSharedPtr< FDisplayedRetargetSourceInfo > > SelectedRows = RetargetSourceListView->GetSelectedItems();
	return SelectedRows.Num() == 1;
}

void SRetargetSourceWindow::OnRenameRetargetSource()
{
	TArray< TSharedPtr< FDisplayedRetargetSourceInfo > > SelectedRows = RetargetSourceListView->GetSelectedItems();

	if (ensure (SelectedRows.Num() == 1))
	{
		int32 RowIndex = 0;
		const USkeleton& Skeleton = EditableSkeletonPtr.Pin()->GetSkeleton();
		const FReferencePose* PoseFound = Skeleton.AnimRetargetSources.Find(SelectedRows[RowIndex]->Name);
		if(PoseFound)
		{
			// we used to verify if there is any animation referencing and warn them, but that doesn't really help
			// because you can rename by just double click as well, and it slows process, so removed it
			// request rename
			SelectedRows[RowIndex]->RequestRename();
		}
	}
}

bool SRetargetSourceWindow::CanPerformRefresh() const
{
	TArray< TSharedPtr< FDisplayedRetargetSourceInfo > > SelectedRows = RetargetSourceListView->GetSelectedItems();
	return SelectedRows.Num() > 0;
}

void SRetargetSourceWindow::OnRefreshRetargetSource(bool bAll)
{
	TArray<FName> SelectedNames;
	if (bAll)
	{
		for (int RowIndex = 0; RowIndex < RetargetSourceList.Num(); ++RowIndex)
		{
			SelectedNames.Add(RetargetSourceList[RowIndex]->Name);
		}
	}
	else
	{
		TArray< TSharedPtr< FDisplayedRetargetSourceInfo > > SelectedRows = RetargetSourceListView->GetSelectedItems();
		for (int RowIndex = 0; RowIndex < SelectedRows.Num(); ++RowIndex)
		{
			SelectedNames.Add(SelectedRows[RowIndex]->Name);
		}
	}

	EditableSkeletonPtr.Pin()->RefreshRetargetSources(SelectedNames);

	FAssetNotifications::SkeletonNeedsToBeSaved(&EditableSkeletonPtr.Pin()->GetSkeleton());
}

void SRetargetSourceWindow::PostUndo()
{
	CreateRetargetSourceList();
}

FReply SRetargetSourceWindow::OnAddRetargetSourceButtonClicked()
{
	OnAddRetargetSource();
	return FReply::Handled();
}

FReply SRetargetSourceWindow::OnUpdateAllRetargetSourceButtonClicked()
{
	OnRefreshRetargetSource(true);
	return FReply::Handled();
}

void SCompatibleSkeletons::Construct(
	const FArguments& InArgs,
	const TSharedRef<IEditableSkeleton>& InEditableSkeleton,
	FSimpleMulticastDelegate& InOnPostUndo)
{
	EditableSkeletonPtr = InEditableSkeleton;
	UpdateCompatibleSkeletonAssets(InEditableSkeleton->GetSkeleton());

	ChildSlot
	[
		SNew( SVerticalBox )
		
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew( SHorizontalBox )

			+SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			.AutoWidth()
			.Padding(2, 0)
			[
				SNew(SButton)
				.OnClicked(FOnClicked::CreateSP(this, &SCompatibleSkeletons::OnAddSkeletonClicked))
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Text(LOCTEXT("AddCompatibleSkeletonButton_Label", "Add Skeleton"))
				.ToolTipText(LOCTEXT("AddCompatibleSkeletonButton_ToolTip", "When Skeleton assets share an identical hierarchy, bone names and orientations they can be added to the Compatible Skeletons list. Animation assets authored on Compatible Skeletons can then be used in this Skeleton's Animation Blueprints."))
			]

			+SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			.AutoWidth()
			.Padding(2, 0)
			[
				SNew(SButton)
				.OnClicked(FOnClicked::CreateSP(this, &SCompatibleSkeletons::OnRemoveSkeletonClicked))
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Text(LOCTEXT("RemoveCompatibleSkeletonButton_Label", "Remove Selected"))
				.ToolTipText(LOCTEXT("RemoveCompatibleSkeletonButton_ToolTip", "Remove the selected skeleton assets from the list of Compatible Skeletons."))
			]
		]

		+ SVerticalBox::Slot()
		.FillHeight( 1.0f )
		.Padding(2.0f)
		[
			SAssignNew(CompatibleSkeletonListView, SListView<TSharedRef<FSoftObjectPath>>)
			.ListItemsSource(&CompatibleSkeletonAssets)
			.OnGenerateRow(this, &SCompatibleSkeletons::GenerateRowForItem)
		]
	];
}

void SCompatibleSkeletons::UpdateCompatibleSkeletonAssets(const USkeleton& Skeleton)
{
	CompatibleSkeletonAssets.Empty();

	const TArray<TSoftObjectPtr<USkeleton>>& CompatibleSkeletons = Skeleton.GetCompatibleSkeletons();
	for (const TSoftObjectPtr<USkeleton>& SoftCompatibleSkeleton : CompatibleSkeletons)
	{
		TSharedRef<FSoftObjectPath> AssetPath = MakeShareable(new FSoftObjectPath(SoftCompatibleSkeleton.ToSoftObjectPath()));
		CompatibleSkeletonAssets.Add(AssetPath);
	}
}

FReply SCompatibleSkeletons::OnAddSkeletonClicked()
{
	// show list of skeletalmeshes that they can choose from
	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

	FAssetPickerConfig AssetPickerConfig;
	AssetPickerConfig.Filter.ClassPaths.Add(USkeleton::StaticClass()->GetClassPathName());
	AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateSP(this, &SCompatibleSkeletons::OnAssetSelectedFromSkeletonPicker);
	AssetPickerConfig.bAllowNullSelection = false;
	AssetPickerConfig.InitialAssetViewType = EAssetViewType::Tile;

	const TSharedRef<SWidget> Widget = SNew(SBox)
		.WidthOverride(384.f)
		.HeightOverride(768.f)
		[
			SNew(SBorder)
			.BorderBackgroundColor(FLinearColor(0.25f, 0.25f, 0.25f, 1.f))
			.Padding( 2.f )
			[
				SNew(SBorder)
				.BorderImage( FAppStyle::GetBrush("ToolPanel.GroupBorder") )
				.Padding( 8.f )
				[
					ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
				]
			]
		];

	FSlateApplication::Get().PushMenu(
		AsShared(),
		FWidgetPath(),
		Widget,
		FSlateApplication::Get().GetCursorPos(),
		FPopupTransitionEffect( FPopupTransitionEffect::TopMenu )
		);
	
	return FReply::Handled();
}

FReply SCompatibleSkeletons::OnRemoveSkeletonClicked()
{
	TArray<TSharedRef<FSoftObjectPath>> SelectedAssets = CompatibleSkeletonListView->GetSelectedItems();
	for (TSharedRef<FSoftObjectPath>& SkeletonAssetPath : SelectedAssets)
	{
		USkeleton* SkeletonToRemove = Cast<USkeleton>(SkeletonAssetPath.Get().TryLoad());
		EditableSkeletonPtr.Pin()->RemoveCompatibleSkeleton(SkeletonToRemove);
	}

	UpdateCompatibleSkeletonAssets(EditableSkeletonPtr.Pin()->GetSkeleton());
	CompatibleSkeletonListView->RequestListRefresh();

	FAssetNotifications::SkeletonNeedsToBeSaved(&EditableSkeletonPtr.Pin()->GetSkeleton());
	
	return FReply::Handled();
}

TSharedRef<ITableRow> SCompatibleSkeletons::GenerateRowForItem(TSharedRef<FSoftObjectPath> Item, const TSharedRef<STableViewBase>& OwnerTable) const
{
	return SNew(STableRow<TSharedPtr<USkeleton>>, OwnerTable)
		.Content()
		[
			SNew(STextBlock).Text(FText::FromString(Item.Get().GetAssetName()))
		];
}

void SCompatibleSkeletons::OnAssetSelectedFromSkeletonPicker(const FAssetData& AssetData)
{
	// make sure we haven't already added this asset as a compatible skeleton
	const USkeleton& Skeleton = EditableSkeletonPtr.Pin()->GetSkeleton();
	const FString AssetFullPath = AssetData.ToSoftObjectPath().ToString();
	const TArray<TSoftObjectPtr<USkeleton>>& CompatibleSkeletons = Skeleton.GetCompatibleSkeletons();
	for (const TSoftObjectPtr<USkeleton>& CompatibleSkeleton : CompatibleSkeletons)
	{
		if (CompatibleSkeleton.ToSoftObjectPath() == AssetData.ToSoftObjectPath())
		{
			FSlateApplication::Get().DismissAllMenus();
			return;
		}
	}

	const USkeleton* CompatibleSkeleton = CastChecked<USkeleton>(AssetData.GetAsset());
	EditableSkeletonPtr.Pin()->AddCompatibleSkeleton(CompatibleSkeleton);
	FAssetNotifications::SkeletonNeedsToBeSaved(&EditableSkeletonPtr.Pin()->GetSkeleton());
	FSlateApplication::Get().DismissAllMenus();

	UpdateCompatibleSkeletonAssets(EditableSkeletonPtr.Pin()->GetSkeleton());
	CompatibleSkeletonListView->RequestListRefresh();
}

#undef LOCTEXT_NAMESPACE

