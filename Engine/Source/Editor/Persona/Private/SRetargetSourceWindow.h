// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Widgets/SCompoundWidget.h"
#include "Engine/SkeletalMesh.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"

struct FAssetData;
class IEditableSkeleton;
template <typename ItemType> class SListView;

//////////////////////////////////////////////////////////////////////////
// FDisplayedRetargetSourceInfo

class  FDisplayedRetargetSourceInfo
{
public:
	FName Name;
	TSoftObjectPtr<USkeletalMesh> ReferenceMesh;

	/** Static function for creating a new item, but ensures that you can only have a TSharedRef to one */
	static TSharedRef<FDisplayedRetargetSourceInfo> Make(const FName& SourcePose, TSoftObjectPtr<USkeletalMesh> InReferenceMesh, bool bInDirtyFlag)
	{
		return MakeShareable(new FDisplayedRetargetSourceInfo(SourcePose, InReferenceMesh, bInDirtyFlag));
	}

	FString GetDisplayName()
	{
		TArray<FStringFormatArg> Args;
		Args.Add(FStringFormatArg(Name.ToString()));
		Args.Add(FStringFormatArg(bDirtyFlag ? TEXT(" [out of date]") : TEXT("")));

		return FString::Format(TEXT("{0}{1}"), Args);
	}

	FString GetReferenceMeshName()
	{
		TArray<FStringFormatArg> Args;
		Args.Add(FStringFormatArg((!ReferenceMesh.IsNull()) ? ReferenceMesh.ToString() : TEXT("None")));

		return FString::Format(TEXT("{0}"), Args);
	}

	/** Requests a rename on the socket item */
	void RequestRename()
	{
		OnEnterEditingMode.ExecuteIfBound();
	}

	/** Delegate for when the context menu requests a rename */
	DECLARE_DELEGATE(FOnEnterEditingMode);
	FOnEnterEditingMode OnEnterEditingMode;

protected:
	/** Hidden constructor, always use Make above */
	FDisplayedRetargetSourceInfo(const FName& InSource, TSoftObjectPtr<USkeletalMesh> InRefMesh, bool InDirtyFlag)
		: Name( InSource )
		, ReferenceMesh( InRefMesh )
		, bRenamePending( false )
		, bDirtyFlag(InDirtyFlag)
	{}

	/** Hidden constructor, always use Make above */
	FDisplayedRetargetSourceInfo() {}

	bool bRenamePending;
	bool bDirtyFlag;
};

typedef SListView< TSharedPtr<FDisplayedRetargetSourceInfo> > SRetargetSourceListType;

//////////////////////////////////////////////////////////////////////////
// SRetargetSourceWindow

class SRetargetSourceWindow : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SRetargetSourceWindow )
	{}

	SLATE_END_ARGS()

	/**
	* Slate construction function
	*
	* @param InArgs - Arguments passed from Slate
	*
	*/
	void Construct( const FArguments& InArgs, const TSharedRef<class IEditableSkeleton>& InEditableSkeleton, FSimpleMulticastDelegate& InOnPostUndo);

	/**
	* Filters the SListView when the user changes the search text box (NameFilterBox)
	*
	* @param SearchText - The text the user has typed
	*
	*/
	void OnFilterTextChanged( const FText& SearchText );

	/**
	* Filters the SListView when the user hits enter or clears the search box
	* Simply calls OnFilterTextChanged
	*
	* @param SearchText - The text the user has typed
	* @param CommitInfo - Not used
	*
	*/
	void OnFilterTextCommitted( const FText& SearchText, ETextCommit::Type CommitInfo );

	/**
	* Create a widget for an entry in the tree from an info
	*
	* @param InInfo - Shared pointer to the morph target we're generating a row for
	* @param OwnerTable - The table that owns this row
	*
	* @return A new Slate widget, containing the UI for this row
	*/
	TSharedRef<ITableRow> GenerateRetargetSourceRow(TSharedPtr<FDisplayedRetargetSourceInfo> InInfo, const TSharedRef<STableViewBase>& OwnerTable);

	/**
	* Handler for adding new retarget source. It displays the asset picker
	*/
	void OnAddRetargetSource();

	/**
	 * Callback for asset picker
	 */
	void OnAssetSelectedFromMeshPicker(const FAssetData& AssetData);

	/**
	* Adds a new retarget source with given name
	*
	* @param Name - Name of retarget source
	* @param ReferenceMesh	- Reference Mesh this pose is based on
	*
	*/
	void AddRetargetSource( const FName Name, USkeletalMesh * ReferenceMesh  );

	/**
	* Return true if it can delete
	*
	*/
	bool CanPerformDelete() const;

	/**
	* Handler for the delete of retarget source
	*/
	void OnDeleteRetargetSource();

	/**
	* Return true if it can perform rename
	*
	*/
	bool CanPerformRename() const;

	/**
	* Handler for the rename of retarget source
	*/
	void OnRenameRetargetSource();

	/**
	* Return true if it can delete
	*
	*/
	bool CanPerformRefresh() const;

	/**
	* Handler for the delete of retarget source
	*/
	void OnRefreshRetargetSource(bool bAll = false);

	/**
	* Accessor so our rows can grab the filtertext for highlighting
	*
	*/
	FText& GetFilterText() { return FilterText; }

	// SWidget interface
	virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;
	// end of SWidget interface
private:

	/** Handler for context menus */
	TSharedPtr<SWidget> OnGetContextMenuContent() const;

	/**
	* Clears and rebuilds the table, according to an optional search string
	*
	* @param SearchText - Optional search string
	*
	*/
	void CreateRetargetSourceList( const FString& SearchText = FString(""), const FName  NewName = NAME_None );

	/** The editable skeleton */
	TWeakPtr<class IEditableSkeleton> EditableSkeletonPtr;

	/** Box to filter to a specific morph target name */
	TSharedPtr<SSearchBox>	NameFilterBox;

	/** Widget used to display the list of retarget sources */
	TSharedPtr<SRetargetSourceListType> RetargetSourceListView;

	/** A list of retarget sources. Used by the RetargetSourceListView. */
	TArray< TSharedPtr<FDisplayedRetargetSourceInfo> > RetargetSourceList;

	/** Current text typed into NameFilterBox */
	FText FilterText;

	/** Item to rename. Only valid for adding **/
	mutable TSharedPtr<FDisplayedRetargetSourceInfo> ItemToRename;

	/** Rename committed. Called by Inline Widget **/
	void OnRenameCommit(const FName& InOldName, const FString& InNewName);
	/** Verify Rename is legit or not **/
	bool OnVerifyRenameCommit( const FName& OldName, const FString& NewName, FText& OutErrorMessage);

	/** Delegate for undo/redo transaction **/
	void PostUndo();

	/** Button handler **/
	FReply OnAddRetargetSourceButtonClicked();
	FReply OnUpdateAllRetargetSourceButtonClicked();
};

class SCompatibleSkeletons : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SCompatibleSkeletons ){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<class IEditableSkeleton>& InEditableSkeleton, FSimpleMulticastDelegate& InOnPostUndo);

	FReply OnAddSkeletonClicked();

	FReply OnRemoveSkeletonClicked();

	TSharedRef<ITableRow> GenerateRowForItem(TSharedRef<FSoftObjectPath> Item, const TSharedRef<STableViewBase>& OwnerTable) const;

	void OnAssetSelectedFromSkeletonPicker(const FAssetData& AssetData);

private:
	void UpdateCompatibleSkeletonAssets(const USkeleton& Skeleton);
	
	TArray<TSharedRef<FSoftObjectPath>> CompatibleSkeletonAssets;

	TSharedPtr<SListView<TSharedRef<FSoftObjectPath>>> CompatibleSkeletonListView;
	
	TWeakPtr<IEditableSkeleton> EditableSkeletonPtr;
};
