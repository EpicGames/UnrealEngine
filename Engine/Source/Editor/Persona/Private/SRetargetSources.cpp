// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRetargetSources.h"
#include "AssetRegistry/AssetData.h"
#include "Misc/MessageDialog.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Styling/AppStyle.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "Widgets/SToolTip.h"
#include "IDocumentation.h"
#include "ScopedTransaction.h"
#include "SRetargetSourceWindow.h"
#include "AnimPreviewInstance.h"
#include "IEditableSkeleton.h"
#include "PropertyCustomizationHelpers.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Animation/Skeleton.h"


#define LOCTEXT_NAMESPACE "SRetargetSources"

void SRetargetSources::Construct(
	const FArguments& InArgs,
	const TSharedRef<IEditableSkeleton>& InEditableSkeleton,
	FSimpleMulticastDelegate& InOnPostUndo)
{
	const FText SourceRetargetModesToolTip = LOCTEXT("SourceRetargetModesCheckBoxTooltip", 
		"Should we use the per bone translational retarget mode from the source (compatible) skeleton's instead of from this skeleton? On default this is disabled. "
		"Enabling this would allow you to have one shared set of animations. You would configure the retarget settings on the animation skeleton. "
		"Then every character that plays animations from this source skeleton will use the translational retarget settings from the source skeleton, which saves you from "
		"having to configure the retarget modes for every bone in every character as they can be setup just once now on the animation skeleton.");

	EditableSkeletonWeakPtr = InEditableSkeleton;

	const FString DocLink = TEXT("Shared/Editors/Persona");
	ChildSlot
	[
		SNew (SVerticalBox)
		
		+ SVerticalBox::Slot()
		.Padding(5, 5)
		.AutoHeight()
		[
			SNew(STextBlock)
			.TextStyle(FAppStyle::Get(), "Persona.RetargetManager.ImportantText")
			.Text(LOCTEXT("RetargetSource_Title", "Manage Retarget Sources"))
		]

		+ SVerticalBox::Slot()
		.Padding(5, 5)
		.FillHeight(0.5)
		[
			// construct retarget source UI
			SNew(SRetargetSourceWindow, InEditableSkeleton, InOnPostUndo)
		]

		+SVerticalBox::Slot()
		.Padding(5, 5)
		.AutoHeight()
		[
			SNew(SSeparator)
			.Orientation(Orient_Horizontal)
		]

		+ SVerticalBox::Slot()
		.Padding(5, 5)
		.AutoHeight()
		[
			SNew(STextBlock)
			.TextStyle(FAppStyle::Get(), "Persona.RetargetManager.ImportantText")
			.Text(LOCTEXT("CompatibleSkeletons_Title", "Manage Compatible Skeletons"))
		]

		+SVerticalBox::Slot()
		.AutoHeight()
		.VAlign(EVerticalAlignment::VAlign_Center)
		.Padding(5, 5)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(EVerticalAlignment::VAlign_Center)
			[
				SNew(SCheckBox)
				.IsChecked_Lambda([this]() -> ECheckBoxState
				{
					if (EditableSkeletonWeakPtr.IsValid())
					{
						return EditableSkeletonWeakPtr.Pin()->GetSkeleton().GetUseRetargetModesFromCompatibleSkeleton() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
					}
					return ECheckBoxState::Undetermined;
				})
				.OnCheckStateChanged_Lambda([this](ECheckBoxState NewState)
				{
					if (EditableSkeletonWeakPtr.IsValid())
					{
						const bool bIsChecked = (NewState == ECheckBoxState::Checked);
						USkeleton& Skeleton = const_cast<USkeleton&>(EditableSkeletonWeakPtr.Pin()->GetSkeleton());
						Skeleton.SetUseRetargetModesFromCompatibleSkeleton(bIsChecked);
						Skeleton.Modify();
					}
				})
				.ToolTipText(SourceRetargetModesToolTip)
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(5, 0)
			.VAlign(EVerticalAlignment::VAlign_Center)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("UseFromSource_Text", "Inherit Translation Retargeting"))
				.ToolTipText(SourceRetargetModesToolTip)
			]
		]		

		+ SVerticalBox::Slot()
		.Padding(5, 5)
		.FillHeight(0.5)
		[
			// construct compatible skeletons UI
			SNew(SCompatibleSkeletons, InEditableSkeleton, InOnPostUndo)
		]
	];
}

#undef LOCTEXT_NAMESPACE
