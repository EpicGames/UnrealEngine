// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Widgets/SCompoundWidget.h"
#include "IPersonaPreviewScene.h"

class IEditableSkeleton;
struct FAssetData;

class SRetargetSources : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SRetargetSources )
	{}

	SLATE_END_ARGS()
	
	void Construct(
		const FArguments& InArgs,
		const TSharedRef<class IEditableSkeleton>& InEditableSkeleton,
		FSimpleMulticastDelegate& InOnPostUndo);

private:
	TWeakPtr<class IEditableSkeleton> EditableSkeletonWeakPtr;
};
