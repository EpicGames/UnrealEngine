// Copyright Epic Games, Inc. All Rights Reserved.


#include "SSequenceEditor.h"
#include "Animation/AnimSequence.h"

#include "SAnimNotifyPanel.h"
#include "AnimPreviewInstance.h"
#include "Editor.h"
#include "AnimTimeline/AnimModel_AnimSequenceBase.h"
#include "AnimTimeline/SAnimTimeline.h"

#define LOCTEXT_NAMESPACE "AnimSequenceEditor"

//////////////////////////////////////////////////////////////////////////
// SSequenceEditor

void SSequenceEditor::Construct(const FArguments& InArgs, TSharedRef<class IPersonaPreviewScene> InPreviewScene, TSharedRef<class IEditableSkeleton> InEditableSkeleton, const TSharedRef<FUICommandList>& InCommandList)
{
	SequenceObj = InArgs._Sequence;
	check(SequenceObj);
	PreviewScenePtr = InPreviewScene;

	AnimModel = MakeShared<FAnimModel_AnimSequenceBase>(InPreviewScene, InEditableSkeleton, InCommandList, SequenceObj);

	AnimModel->OnEditCurves = FOnEditCurves::CreateLambda([this, InOnEditCurves = InArgs._OnEditCurves](UAnimSequenceBase* InAnimSequence, const TArray<IAnimationEditor::FCurveEditInfo>& InCurveInfo, const TSharedPtr<ITimeSliderController>& InExternalTimeSliderController)
	{
		InOnEditCurves.ExecuteIfBound(InAnimSequence, InCurveInfo, TimelineWidget->GetTimeSliderController());
	});

	AnimModel->OnSelectObjects = FOnObjectsSelected::CreateSP(this, &SAnimEditorBase::OnSelectionChanged);
	AnimModel->OnInvokeTab = InArgs._OnInvokeTab;
	AnimModel->Initialize();

	SAnimEditorBase::Construct( SAnimEditorBase::FArguments()
		.OnObjectsSelected(InArgs._OnObjectsSelected)
		.AnimModel(AnimModel), 
		InPreviewScene);

	if(GEditor)
	{
		GEditor->RegisterForUndo(this);
	}
}

SSequenceEditor::~SSequenceEditor()
{
	if(GEditor)
	{
		GEditor->UnregisterForUndo(this);
	}
}

void SSequenceEditor::PostUndo( bool bSuccess )
{
	PostUndoRedo();
}

void SSequenceEditor::PostRedo( bool bSuccess )
{
	PostUndoRedo();
}

void SSequenceEditor::PostUndoRedo()
{
	GetPreviewScene()->SetPreviewAnimationAsset(SequenceObj);

	AnimModel->RefreshTracks();
}

#undef LOCTEXT_NAMESPACE
