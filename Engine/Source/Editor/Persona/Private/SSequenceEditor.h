// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "IPersonaPreviewScene.h"
#include "SAnimationScrubPanel.h"
#include "SAnimEditorBase.h"
#include "EditorUndoClient.h"
#include "Animation/AnimSequence.h"

class SAnimNotifyPanel;
class FAnimModel_AnimSequenceBase;

//////////////////////////////////////////////////////////////////////////
// SSequenceEditor

/** Overall animation sequence editing widget */
class SSequenceEditor : public SAnimEditorBase, public FEditorUndoClient
{
public:
	SLATE_BEGIN_ARGS( SSequenceEditor )
		: _Sequence(NULL)
		{}

		SLATE_ARGUMENT(UAnimSequenceBase*, Sequence)
		SLATE_EVENT(FOnObjectsSelected, OnObjectsSelected)
		SLATE_EVENT(FOnInvokeTab, OnInvokeTab)
		SLATE_EVENT(FOnEditCurves, OnEditCurves)

	SLATE_END_ARGS()

private:
	TWeakPtr<class IPersonaPreviewScene> PreviewScenePtr;
	TSharedPtr<FAnimModel_AnimSequenceBase> AnimModel;
public:
	void Construct(const FArguments& InArgs, TSharedRef<class IPersonaPreviewScene> InPreviewScene, TSharedRef<class IEditableSkeleton> InEditableSkeleton, const TSharedRef<FUICommandList>& InCommandList);

	~SSequenceEditor();

	virtual UAnimationAsset* GetEditorObject() const override { return SequenceObj; }

private:
	/** Pointer to the animation sequence being edited */
	UAnimSequenceBase* SequenceObj;

	/** FEditorUndoClient interface */
	virtual void PostUndo( bool bSuccess ) override;
	virtual void PostRedo( bool bSuccess ) override;

	/** Post undo **/
	void PostUndoRedo();
};
