// Copyright Epic Games, Inc. All Rights Reserved.


#include "SSkeletonAnimNotifies.h"

#include "AnimAssetFindReplaceNotifies.h"
#include "AnimAssetFindReplaceSyncMarkers.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "AssetRegistry/AssetData.h"
#include "Animation/AnimSequenceBase.h"
#include "Styling/AppStyle.h"
#include "Preferences/PersonaOptions.h"
#include "Animation/EditorSkeletonNotifyObj.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "ScopedTransaction.h"
#include "IEditableSkeleton.h"
#include "TabSpawners.h"
#include "Editor.h"
#include "IAnimationEditor.h"
#include "IAnimationBlueprintEditor.h"
#include "IAnimationSequenceBrowser.h"
#include "ISkeletonEditor.h"
#include "SAnimAssetFindReplace.h"
#include "Filters/GenericFilter.h"
#include "Filters/SBasicFilterBar.h"
#include "Widgets/Docking/SDockTab.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "String/ParseTokens.h"

#define LOCTEXT_NAMESPACE "SkeletonAnimNotifies"

typedef TSharedPtr< FDisplayedAnimNotifyInfo > FDisplayedAnimNotifyInfoPtr;

class FSkeletonAnimNotifiesFilter : public FGenericFilter<EAnimNotifyFilterFlags>
{
public:
	FSkeletonAnimNotifiesFilter(EAnimNotifyFilterFlags InFlags, const FString& InName, const FText& InDisplayName, const FText& InToolTipText, FLinearColor InColor, TSharedPtr<FFilterCategory> InCategory)
		: FGenericFilter<EAnimNotifyFilterFlags>(InCategory, InName, InDisplayName, FGenericFilter<EAnimNotifyFilterFlags>::FOnItemFiltered())
		, Flags(InFlags)
	{
		ToolTip = InToolTipText;
		Color = InColor;
	}

	bool IsActive() const
	{
		return bIsActive;
	}

	EAnimNotifyFilterFlags GetFlags() const
	{
		return Flags;
	}

private:
	// FFilterBase interface
	virtual void ActiveStateChanged(bool bActive) override
	{
		FGenericFilter<EAnimNotifyFilterFlags>::ActiveStateChanged(bActive);

		bIsActive = bActive;
	}

	virtual bool PassesFilter(EAnimNotifyFilterFlags InItem) const override
	{
		return EnumHasAnyFlags(InItem, Flags);
	}
	
private:
	EAnimNotifyFilterFlags Flags;
	bool bIsActive = false;
};

/////////////////////////////////////////////////////
// FSkeletonAnimNotifiesSummoner

FSkeletonAnimNotifiesSummoner::FSkeletonAnimNotifiesSummoner(TSharedPtr<class FAssetEditorToolkit> InHostingApp, const TSharedRef<class IEditableSkeleton>& InEditableSkeleton, FOnObjectsSelected InOnObjectsSelected)
	: FWorkflowTabFactory(FPersonaTabs::SkeletonAnimNotifiesID, InHostingApp)
	, EditableSkeleton(InEditableSkeleton)
	, OnObjectsSelected(InOnObjectsSelected)
{
	TabLabel = LOCTEXT("SkeletonAnimNotifiesTabTitle", "Animation Notifies");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Persona.AnimNotifyWindow");

	EnableTabPadding();
	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("SkeletonAnimNotifiesMenu", "Animation Notifies");
	ViewMenuTooltip = LOCTEXT("SkeletonAnimNotifies_ToolTip", "Shows the notify and sync marker list");
}

TSharedRef<SWidget> FSkeletonAnimNotifiesSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	return SNew(SSkeletonAnimNotifies, HostingApp.Pin())
		.OnObjectsSelected(OnObjectsSelected)
		.EditableSkeleton(EditableSkeleton.Pin());
}

/////////////////////////////////////////////////////
// SSkeletonAnimNotifies

void SSkeletonAnimNotifies::Construct(const FArguments& InArgs, const TSharedPtr<class FAssetEditorToolkit>& InHostingApp)
{
	bool bNotifiesAllowed = GetDefault<UPersonaOptions>()->bExposeNotifiesUICommands;

	OnObjectsSelected = InArgs._OnObjectsSelected;
	OnItemSelected = InArgs._OnItemSelected;
	bIsPicker = InArgs._IsPicker;
	bShowSyncMarkers = InArgs._ShowSyncMarkers;
	bShowOtherAssets = InArgs._ShowOtherAssets;
	bShowCompatibleSkeletonAssets = InArgs._ShowCompatibleSkeletonAssets;
	bShowNotifies = InArgs._ShowNotifies && bNotifiesAllowed;
	EditableSkeleton = InArgs._EditableSkeleton;

	WeakHostingApp = InHostingApp;

	if (EditableSkeleton.IsValid())
	{
		EditableSkeleton->RegisterOnNotifiesChanged(FSimpleDelegate::CreateSP(this, &SSkeletonAnimNotifies::OnNotifiesChanged));
	}

	if(GEditor)
	{
		GEditor->RegisterForUndo(this);
	}

	FOnContextMenuOpening OnContextMenuOpening = (!bIsPicker && EditableSkeleton.IsValid()) ? FOnContextMenuOpening::CreateSP(this, &SSkeletonAnimNotifies::OnGetContextMenuContent) : FOnContextMenuOpening();

	NameFilterBox = SNew( SSearchBox )
		.SelectAllTextWhenFocused( true )
		.OnTextChanged( this, &SSkeletonAnimNotifies::OnFilterTextChanged )
		.OnTextCommitted( this, &SSkeletonAnimNotifies::OnFilterTextCommitted );

	NotifiesListView = SNew( SAnimNotifyListType )
		.ListItemsSource( &NotifyList )
		.OnGenerateRow( this, &SSkeletonAnimNotifies::GenerateNotifyRow )
		.OnContextMenuOpening( OnContextMenuOpening )
		.OnSelectionChanged( this, &SSkeletonAnimNotifies::OnNotifySelectionChanged )
		.OnItemScrolledIntoView( this, &SSkeletonAnimNotifies::OnItemScrolledIntoView );
	
	CurrentFilterFlags = EAnimNotifyFilterFlags::None;

	TSharedPtr<FFilterCategory> FilterCategory = MakeShared<FFilterCategory>(LOCTEXT("AnimNotifyFiltersLabel", "Anim Notify Filters"), LOCTEXT("AnimNotifyFiltersLabelToolTip", "Filter what kind of notifies and sync markers can be displayed."));

	const bool bSingleType = (bShowNotifies ^ bShowSyncMarkers);

	{
		TGuardValue<bool> SuspendRefreshFilter(bAllowRefreshFilter, false);
		
		TSharedRef<SBasicFilterBar<EAnimNotifyFilterFlags>> FilterBar = SNew(SBasicFilterBar<EAnimNotifyFilterFlags>)
		.CustomFilters(Filters)
		.bPinAllFrontendFilters(true)
		.UseSectionsForCategories(true)
		.OnFilterChanged_Lambda([this]()
		{
			for(const TSharedRef<FFilterBase<EAnimNotifyFilterFlags>>& Filter : Filters)
			{
				TSharedRef<FSkeletonAnimNotifiesFilter> AnimNotifiesFilter = StaticCastSharedRef<FSkeletonAnimNotifiesFilter>(Filter);
				if(AnimNotifiesFilter->IsActive())
				{
					CurrentFilterFlags |= AnimNotifiesFilter->GetFlags();
				}
				else
				{
					CurrentFilterFlags &= ~AnimNotifiesFilter->GetFlags();
				}
			}

			RefreshNotifiesListWithFilter();
		});
		
		if (EditableSkeleton.IsValid())
		{
			CurrentFilterFlags |= EAnimNotifyFilterFlags::CurrentSkeleton;

			TSharedRef<FFilterBase<EAnimNotifyFilterFlags>> Filter = Filters.Add_GetRef(MakeShared<FSkeletonAnimNotifiesFilter>(
				EAnimNotifyFilterFlags::CurrentSkeleton,
				"CurrentSkeleton",
				LOCTEXT("ShowSkeletonItemsLabel", "Skeleton"),
				LOCTEXT("ShowSkeletonItemsTooltip", "Show items for the current skeleton"),
				FLinearColor::Blue.Desaturate(0.25f),
				FilterCategory
			));

			FilterBar->AddFilter(Filter);
			Filter->SetActive(true);
		}

		if (bShowNotifies)
		{
			CurrentFilterFlags |= EAnimNotifyFilterFlags::Notifies;
		}

		if (!bSingleType && bShowNotifies)
		{
			TSharedRef<FFilterBase<EAnimNotifyFilterFlags>> Filter = Filters.Add_GetRef(MakeShared<FSkeletonAnimNotifiesFilter>(
				EAnimNotifyFilterFlags::Notifies,
				"Notifies",
				LOCTEXT("ShowNotifiesLabel", "Notifies"),
				LOCTEXT("ShowNotifiesTooltip", "Show notifies"),
				FLinearColor::Red.Desaturate(0.5f),
				FilterCategory
			));

			FilterBar->AddFilter(Filter);
			Filter->SetActive(true);
		}

		if (bShowSyncMarkers)
		{
			CurrentFilterFlags |= EAnimNotifyFilterFlags::SyncMarkers;
		}

		if (!bSingleType && bShowSyncMarkers)
		{
			TSharedRef<FFilterBase<EAnimNotifyFilterFlags>> Filter = Filters.Add_GetRef(MakeShared<FSkeletonAnimNotifiesFilter>(
				EAnimNotifyFilterFlags::SyncMarkers,
				"SyncMarkers",
				LOCTEXT("ShowSyncMarkersLabel", "Sync Markers"),
				LOCTEXT("ShowSyncMarkersTooltip", "Show sync markers"),
				FLinearColor::Green.Desaturate(0.5f),
				FilterCategory
			));

			FilterBar->AddFilter(Filter);
			Filter->SetActive(true);
		}

		if (EditableSkeleton.IsValid())
		{
			CurrentFilterFlags |= bShowOtherAssets ? EAnimNotifyFilterFlags::OtherAssets : EAnimNotifyFilterFlags::None;

			TSharedRef<FFilterBase<EAnimNotifyFilterFlags>> Filter = Filters.Add_GetRef(MakeShared<FSkeletonAnimNotifiesFilter>(
				EAnimNotifyFilterFlags::OtherAssets,
				"OtherAssets",
				LOCTEXT("ShowOtherAssetsLabel", "Other Assets"),
				LOCTEXT("ShowOtherAssetsTooltip", "Show items that are present on other assets"),
				FLinearColor::Yellow.Desaturate(0.5f),
				FilterCategory
			));

			FilterBar->AddFilter(Filter);
			Filter->SetActive(bShowOtherAssets);
		}
		else
		{
			CurrentFilterFlags |= EAnimNotifyFilterFlags::OtherAssets;
		}

		if (EditableSkeleton.IsValid())
		{
			CurrentFilterFlags |= bShowCompatibleSkeletonAssets ? EAnimNotifyFilterFlags::CompatibleAssets : EAnimNotifyFilterFlags::None;

			TSharedRef<FFilterBase<EAnimNotifyFilterFlags>> Filter = Filters.Add_GetRef(MakeShared<FSkeletonAnimNotifiesFilter>(
				EAnimNotifyFilterFlags::CompatibleAssets,
				"CompatibleAssets",
				LOCTEXT("ShowCompatibleAssetsLabel", "Compatible"),
				LOCTEXT("ShowCompatibleAssetsTooltip", "Show items that are present on other assets that are compatible with the current skeleton"),
				FLinearColor::Blue.Desaturate(0.5f),
				FilterCategory
			));

			FilterBar->AddFilter(Filter);
			Filter->SetActive(bShowCompatibleSkeletonAssets);
		}
		else
		{
			CurrentFilterFlags |= EAnimNotifyFilterFlags::CompatibleAssets;
		}

		TSharedRef<SWidget> AddFilterButton = SBasicFilterBar<EAnimNotifyFilterFlags>::MakeAddFilterButton(FilterBar);
		
		ChildSlot
		[
			SNew( SVerticalBox )

			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding( FMargin( 0.0f, 0.0f, 0.0f, 4.0f ) )
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2.0f,0.0f)
				[
					AddFilterButton
				]

				+SHorizontalBox::Slot()
				.FillWidth(1.0f)
				[
					NameFilterBox.ToSharedRef()
				]
			]

			+SVerticalBox::Slot()
			.AutoHeight()
			[
				FilterBar
			]

			+SVerticalBox::Slot()
			.FillHeight( 1.0f )		// This is required to make the scrollbar work, as content overflows Slate containers by default
			[
				NotifiesListView.ToSharedRef()
			]
		];
	}

	RefreshNotifiesListWithFilter();
}

SSkeletonAnimNotifies::~SSkeletonAnimNotifies()
{
	if(GEditor)
	{
		GEditor->UnregisterForUndo(this);
	}
}

void SSkeletonAnimNotifies::OnNotifiesChanged()
{
	RefreshNotifiesListWithFilter();
}

void SSkeletonAnimNotifies::OnFilterTextChanged( const FText& SearchText )
{
	FilterText = SearchText;

	RefreshNotifiesListWithFilter();
}

void SSkeletonAnimNotifies::OnFilterTextCommitted( const FText& SearchText, ETextCommit::Type CommitInfo )
{
	// Just do the same as if the user typed in the box
	OnFilterTextChanged( SearchText );
}

TSharedRef<ITableRow> SSkeletonAnimNotifies::GenerateNotifyRow(TSharedPtr<FDisplayedAnimNotifyInfo> InInfo, const TSharedRef<STableViewBase>& OwnerTable)
{
	check( InInfo.IsValid() );

	FTextBuilder TooltipBuilder;
	if (InInfo->bIsSyncMarker)
	{
		TooltipBuilder.AppendLineFormat(LOCTEXT("SyncMarkerTooltip", "Sync Marker '{0}'"), FText::FromName(InInfo->Name));
	}
	else
	{
		TooltipBuilder.AppendLineFormat(LOCTEXT("NotifyTooltip", "Notify '{0}'"), FText::FromName(InInfo->Name));
	}

	const USkeleton* CurrentSkeleton = EditableSkeleton.IsValid() ? &EditableSkeleton->GetSkeleton() : nullptr;
	if(CurrentSkeleton)
	{
		if (EnumHasAnyFlags(InInfo->ItemFlags, EAnimNotifyFilterFlags::CurrentSkeleton))
		{
			TooltipBuilder.AppendLine(LOCTEXT("CurrentSkeleton", "Item is on the current skeleton"));
		}

		if (EnumHasAnyFlags(InInfo->ItemFlags, EAnimNotifyFilterFlags::CompatibleAssets))
		{
			TooltipBuilder.AppendLine(LOCTEXT("CompatibleAsset", "Item is on a compatible asset"));
		}

		if (EnumHasAnyFlags(InInfo->ItemFlags, EAnimNotifyFilterFlags::OtherAssets))
		{
			TooltipBuilder.AppendLine(LOCTEXT("AnotherAsset", "Item is on another asset"));
		}
	}

	return
		SNew( STableRow<TSharedPtr<FDisplayedAnimNotifyInfo>>, OwnerTable )
		.ToolTipText(TooltipBuilder.ToText())
		[
			SNew( SHorizontalBox )
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding( 4.0f, 4.0f )
			.VAlign( VAlign_Center )
			[
				SNew(SImage)
				.Image(InInfo->bIsSyncMarker ? FAppStyle::Get().GetBrush("AnimNotifyEditor.AnimSyncMarker") : FAppStyle::Get().GetBrush("AnimNotifyEditor.AnimNotify"))
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding( 4.0f, 4.0f )
			.VAlign( VAlign_Center )
			[
				SAssignNew(InInfo->InlineEditableText, SInlineEditableTextBlock)
				.Text(FText::FromName(InInfo->Name))
				.Font(FAppStyle::Get().GetFontStyle("SmallFont"))
				.ColorAndOpacity((CurrentSkeleton == nullptr || EnumHasAnyFlags(InInfo->ItemFlags, EAnimNotifyFilterFlags::CurrentSkeleton)) ? FSlateColor::UseForeground() : FSlateColor::UseSubduedForeground())
				.OnVerifyTextChanged(this, &SSkeletonAnimNotifies::OnVerifyNotifyNameCommit, InInfo)
				.OnTextCommitted(this, &SSkeletonAnimNotifies::OnNotifyNameCommitted, InInfo)
				.IsSelected(this, &SSkeletonAnimNotifies::IsSelected)
				.HighlightText_Lambda([this](){ return FilterText; })
				.IsReadOnly(bIsPicker || !EditableSkeleton.IsValid())
			]
		];
}

TSharedPtr<SWidget> SSkeletonAnimNotifies::OnGetContextMenuContent() const
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder( bShouldCloseWindowAfterMenuSelection, NULL);

	bool bNotifiesAllowed = GetDefault<UPersonaOptions>()->bExposeNotifiesUICommands;

	if (bNotifiesAllowed)
	{
		MenuBuilder.BeginSection("AnimItemAction", LOCTEXT("ItemActions", "New Item"));
		{
			FUIAction Action = FUIAction(FExecuteAction::CreateSP(const_cast<SSkeletonAnimNotifies*>(this), &SSkeletonAnimNotifies::OnAddItem, false));
			const FText Label = LOCTEXT("NewAnimNotifyButtonLabel", "New Notify...");
			const FText ToolTipText = LOCTEXT("NewAnimNotifyButtonTooltip", "Creates a new anim notify.");
			MenuBuilder.AddMenuEntry(Label, ToolTipText, FSlateIcon(), Action);
		}
	}

	{
		FUIAction Action = FUIAction(FExecuteAction::CreateSP(const_cast<SSkeletonAnimNotifies*>(this), &SSkeletonAnimNotifies::OnAddItem, true));
		const FText Label = LOCTEXT("NewSyncMarkerButtonLabel", "New Sync Marker...");
		const FText ToolTipText = LOCTEXT("NewSyncMarkerButtonTooltip", "Creates a new sync marker.");
		MenuBuilder.AddMenuEntry(Label, ToolTipText, FSlateIcon(), Action);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("AnimNotifyAction", LOCTEXT("SelectedItemActions", "Selected Item Actions"));
	{
		{
			FUIAction Action = FUIAction(FExecuteAction::CreateSP(const_cast<SSkeletonAnimNotifies*>(this), &SSkeletonAnimNotifies::OnRenameItem),
				FCanExecuteAction::CreateSP(this, &SSkeletonAnimNotifies::CanPerformRename));
			const FText Label = LOCTEXT("RenameAnimNotifyButtonLabel", "Rename");
			const FText ToolTipText = LOCTEXT("RenameAnimNotifyButtonTooltip", "Renames the selected item.");
			MenuBuilder.AddMenuEntry(Label, ToolTipText, FSlateIcon(), Action);
		}

		{
			FUIAction Action = FUIAction(FExecuteAction::CreateSP(const_cast<SSkeletonAnimNotifies*>(this), &SSkeletonAnimNotifies::OnDeleteItems),
				FCanExecuteAction::CreateSP(this, &SSkeletonAnimNotifies::CanPerformDelete));
			const FText Label = LOCTEXT("DeleteAnimNotifyButtonLabel", "Delete");
			const FText ToolTipText = LOCTEXT("DeleteAnimNotifyButtonTooltip", "Deletes the selected items.");
			MenuBuilder.AddMenuEntry(Label, ToolTipText, FSlateIcon(), Action);
		}

		if(WeakHostingApp.IsValid() && NotifiesListView->GetNumItemsSelected() == 1)
		{
			TSharedPtr<FAssetEditorToolkit> HostingApp = WeakHostingApp.Pin();
			bool bFindReplaceAvailable = HostingApp->GetTabManager()->GetTabPermissionList()->PassesFilter(FPersonaTabs::FindReplaceID);
			if (bFindReplaceAvailable)
			{
				FUIAction Action = FUIAction(FExecuteAction::CreateSP(const_cast<SSkeletonAnimNotifies*>(this), &SSkeletonAnimNotifies::OnFindReferences),
					FCanExecuteAction::CreateSP(this, &SSkeletonAnimNotifies::CanPerformFindReferences));
				const FText Label = LOCTEXT("FindNotifyReferences", "Find/Replace References...");
				const FText ToolTipText = LOCTEXT("FindNotifyReferencesTooltip", "Find, replace and remove references to this item in the find/replace tab");
				MenuBuilder.AddMenuEntry(Label, ToolTipText, FSlateIcon(), Action);
			}
		}
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void SSkeletonAnimNotifies::OnNotifySelectionChanged(TSharedPtr<FDisplayedAnimNotifyInfo> Selection, ESelectInfo::Type SelectInfo)
{
	if(Selection.IsValid())
	{
		if (!Selection->bIsSyncMarker)
		{
			ShowNotifyInDetailsView(Selection->Name);
		}

		OnItemSelected.ExecuteIfBound(Selection->Name);
	}
}

bool SSkeletonAnimNotifies::CanPerformDelete() const
{
	if (NotifiesListView->GetNumItemsSelected() > 0)
	{
		for (const TSharedPtr<FDisplayedAnimNotifyInfo>& Item : NotifiesListView->GetSelectedItems())
		{
			// We can delete only if an item was from 'this' skeleton
			if (EnumHasAnyFlags(Item->ItemFlags, EAnimNotifyFilterFlags::CurrentSkeleton))
			{
				return true;
			}
		}
	}

	return false;
}

bool SSkeletonAnimNotifies::CanPerformRename() const
{
	if (NotifiesListView->GetNumItemsSelected() == 1)
	{
		// We can rename only if the item was from 'this' skeleton
		return EnumHasAnyFlags(NotifiesListView->GetSelectedItems()[0]->ItemFlags, EAnimNotifyFilterFlags::CurrentSkeleton);
	}

	return false;
}

bool SSkeletonAnimNotifies::CanPerformFindReferences() const
{
	return NotifiesListView->GetNumItemsSelected() == 1;
}

void SSkeletonAnimNotifies::OnAddItem(bool bIsSyncMarker)
{
	// Find a unique name for this notify
	const TCHAR* BaseNotifyString = bIsSyncMarker ? TEXT("NewSyncMarker") : TEXT("NewNotify");
	FString NewNotifyString = BaseNotifyString;
	int32 NumericExtension = 0;

	auto NameAndTypeMatches = [&NewNotifyString, bIsSyncMarker](const TSharedPtr<FDisplayedAnimNotifyInfo>& InNotify)
	{
		return InNotify->Name.ToString() == NewNotifyString && InNotify->bIsSyncMarker == bIsSyncMarker;
	};

	while(NotifyList.ContainsByPredicate(NameAndTypeMatches))
	{
		NewNotifyString = FString::Printf(TEXT("%s_%d"), BaseNotifyString, NumericExtension);
		NumericExtension++;
	}

	// Add an item. The subsequent rename will commit the item.
	TSharedPtr<FDisplayedAnimNotifyInfo> NewItem = FDisplayedAnimNotifyInfo::Make(*NewNotifyString, bIsSyncMarker, EAnimNotifyFilterFlags::CurrentSkeleton);
	NewItem->bIsNew = true;
	NotifyList.Add(NewItem);

	NotifiesListView->ClearSelection();
	NotifiesListView->RequestListRefresh();
	NotifiesListView->RequestScrollIntoView(NewItem);
}

void SSkeletonAnimNotifies::OnItemScrolledIntoView(TSharedPtr<FDisplayedAnimNotifyInfo> InItem, const TSharedPtr<ITableRow>& InTableRow)
{
	if(InItem.IsValid() && InItem->InlineEditableText.IsValid() && InItem->bIsNew)
	{
		InItem->InlineEditableText->EnterEditingMode();
	}
}

void SSkeletonAnimNotifies::OnDeleteItems()
{
	if (EditableSkeleton.IsValid())
	{
		TArray<TSharedPtr<FDisplayedAnimNotifyInfo>> SelectedRows = NotifiesListView->GetSelectedItems();

		TArray<FName> SelectedSyncMarkerNames;
		TArray<FName> SelectedNotifyNames;

		for (TSharedPtr<FDisplayedAnimNotifyInfo> Selection : SelectedRows)
		{
			if (EnumHasAnyFlags(Selection->ItemFlags, EAnimNotifyFilterFlags::CurrentSkeleton))
			{
				if (Selection->bIsSyncMarker)
				{
					SelectedSyncMarkerNames.Add(Selection->Name);
				}
				else
				{
					SelectedNotifyNames.Add(Selection->Name);
				}
			}
		}

		if (SelectedSyncMarkerNames.Num())
		{
			EditableSkeleton->DeleteSyncMarkers(SelectedSyncMarkerNames, false);
		}

		if (SelectedNotifyNames.Num())
		{
			EditableSkeleton->DeleteAnimNotifies(SelectedNotifyNames, false);
		}
	
		RefreshNotifiesListWithFilter();
	}
}

void SSkeletonAnimNotifies::OnRenameItem()
{
	TArray< TSharedPtr< FDisplayedAnimNotifyInfo > > SelectedRows = NotifiesListView->GetSelectedItems();

	check(SelectedRows.Num() == 1); // Should be guaranteed by CanPerformRename
	if (EnumHasAnyFlags(SelectedRows[0]->ItemFlags, EAnimNotifyFilterFlags::CurrentSkeleton))
	{
		SelectedRows[0]->InlineEditableText->EnterEditingMode();
	}
}

bool SSkeletonAnimNotifies::OnVerifyNotifyNameCommit( const FText& NewName, FText& OutErrorMessage, TSharedPtr<FDisplayedAnimNotifyInfo> Item )
{
	if (EditableSkeleton.IsValid())
	{
		bool bValid(true);

		if (NewName.IsEmpty())
		{
			OutErrorMessage = LOCTEXT("NameMissing_Error", "You must provide a name.");
			bValid = false;
		}

		FName NotifyName(*NewName.ToString());
		if (NotifyName != Item->Name || Item->bIsNew)
		{
			if (Item->bIsSyncMarker)
			{
				if (EditableSkeleton->GetSkeleton().GetExistingMarkerNames().Contains(NotifyName))
				{
					OutErrorMessage = FText::Format(LOCTEXT("AlreadyInUseMessage", "'{0}' is already in use."), NewName);
					bValid = false;
				}
			}
			else
			{
				if (EditableSkeleton->GetSkeleton().AnimationNotifies.Contains(NotifyName))
				{
					OutErrorMessage = FText::Format(LOCTEXT("AlreadyInUseMessage", "'{0}' is already in use."), NewName);
					bValid = false;
				}
			}
		}

		return bValid;
	}

	return false;
}

void SSkeletonAnimNotifies::OnNotifyNameCommitted( const FText& NewName, ETextCommit::Type, TSharedPtr<FDisplayedAnimNotifyInfo> Item )
{
	if (EditableSkeleton.IsValid())
	{
		FName NewFName = FName(*NewName.ToString());
		if (Item->bIsNew)
		{
			if (Item->bIsSyncMarker)
			{
				EditableSkeleton->AddSyncMarker(NewFName);
			}
			else
			{
				EditableSkeleton->AddNotify(NewFName);
			}
			Item->bIsNew = false;
		}
		else
		{
			if (NewFName != Item->Name)
			{
				if (Item->bIsSyncMarker)
				{
					int32 NumAnimationsModified = EditableSkeleton->RenameSyncMarker(FName(*NewName.ToString()), Item->Name, false);
					if (NumAnimationsModified > 0)
					{
						FFormatNamedArguments Args;
						Args.Add(TEXT("NumAnimationsModified"), NumAnimationsModified);
						FNotificationInfo Info(FText::Format(LOCTEXT("SyncMarkersRenamed", "{NumAnimationsModified} animation(s) modified to rename sync marker"), Args));

						Info.bUseLargeFont = false;
						Info.ExpireDuration = 5.0f;

						NotifyUser(Info);
					}
				}
				else
				{
					int32 NumAnimationsModified = EditableSkeleton->RenameNotify(FName(*NewName.ToString()), Item->Name, false);
					if (NumAnimationsModified > 0)
					{
						FFormatNamedArguments Args;
						Args.Add(TEXT("NumAnimationsModified"), NumAnimationsModified);
						FNotificationInfo Info(FText::Format(LOCTEXT("AnimNotifiesRenamed", "{NumAnimationsModified} animation(s) modified to rename notification"), Args));

						Info.bUseLargeFont = false;
						Info.ExpireDuration = 5.0f;

						NotifyUser(Info);
					}
				}
			
				RefreshNotifiesListWithFilter();
			}
		}
	}
}

void SSkeletonAnimNotifies::RefreshNotifiesListWithFilter()
{
	if(bAllowRefreshFilter)
	{
		CreateNotifiesList();
		FilterNotifiesList(NameFilterBox->GetText().ToString());
	}
}

void SSkeletonAnimNotifies::FilterNotifiesList(const FString& InSearchText)
{
	FilteredNotifyList.Empty();

	for (const TSharedPtr<FDisplayedAnimNotifyInfo>& Item : NotifyList)
	{
		if (!InSearchText.IsEmpty())
		{
			if (Item->Name.ToString().Contains(InSearchText))
			{
				FilteredNotifyList.Add(Item);
			}
		}
		else
		{
			FilteredNotifyList.Add(Item);
		}
	}

	NotifiesListView->RequestListRefresh();
}

void SSkeletonAnimNotifies::CreateNotifiesList()
{
	const USkeleton* CurrentSkeleton = EditableSkeleton.IsValid() ? &EditableSkeleton->GetSkeleton() : nullptr;
	FAssetData CurrentSkeletonAsset;
	if(CurrentSkeleton)
	{
		CurrentSkeletonAsset = FAssetData(CurrentSkeleton);
	}

	NotifyList.Empty();
	FilteredNotifyList.Empty();

	// Query the asset registry for our notifies and sync markers
	const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");

	FARFilter Filter;
	Filter.bRecursiveClasses = true;
	Filter.ClassPaths.Append({ UAnimSequenceBase::StaticClass()->GetClassPathName(), USkeleton::StaticClass()->GetClassPathName() } );

	FPersonaModule& PersonaModule = FModuleManager::GetModuleChecked<FPersonaModule>("Persona");
	Filter.ClassPaths.Append(PersonaModule.GetAllNotifyHostAssetClassPaths());
	
	Filter.TagsAndValues.Add(USkeleton::AnimNotifyTag);
	Filter.TagsAndValues.Add(USkeleton::AnimSyncMarkerTag);

	TArray<FAssetData> FoundAssetData;
	AssetRegistryModule.Get().GetAssets(Filter, FoundAssetData);

	// Build set of unique names
	TMap<FName, EAnimNotifyFilterFlags> NotifyNames;
	TMap<FName, EAnimNotifyFilterFlags> SyncMarkerNames;

	for (const FAssetData& AssetData : FoundAssetData)
	{
		EAnimNotifyFilterFlags AssetFlags = EAnimNotifyFilterFlags::None;

		auto HasNotifies = [&AssetData]()
		{
			FAssetTagValueRef TagRef = AssetData.TagsAndValues.FindTag(USkeleton::AnimNotifyTag);
			return TagRef.IsSet() && !TagRef.Equals(USkeleton::AnimNotifyTagDelimiter);
		};

		auto HasSyncMarkers = [&AssetData]()
		{
			FAssetTagValueRef TagRef = AssetData.TagsAndValues.FindTag(USkeleton::AnimSyncMarkerTag);
			return TagRef.IsSet() && !TagRef.Equals(USkeleton::AnimSyncMarkerTagDelimiter);
		};
		
		bool bHasAssetRegistryData =
			(EnumHasAnyFlags(CurrentFilterFlags, EAnimNotifyFilterFlags::Notifies) && HasNotifies()) ||
			(EnumHasAnyFlags(CurrentFilterFlags, EAnimNotifyFilterFlags::SyncMarkers) && HasSyncMarkers());

		if(!bHasAssetRegistryData)
		{
			continue;
		}

		if (AssetData.GetClass() != USkeleton::StaticClass())
		{
			if (CurrentSkeleton && CurrentSkeleton->IsCompatibleForEditor(AssetData))
			{
				AssetFlags |= EAnimNotifyFilterFlags::CompatibleAssets;
			}
			else
			{
				AssetFlags |= EAnimNotifyFilterFlags::OtherAssets;
			}
		}
		else
		{
			if (AssetData == CurrentSkeletonAsset)
			{
				AssetFlags |= EAnimNotifyFilterFlags::CurrentSkeleton;
			}
			else if (CurrentSkeleton && CurrentSkeleton->IsCompatibleForEditor(AssetData))
			{
				AssetFlags |= EAnimNotifyFilterFlags::CompatibleAssets;
			}
		}

		if(!EnumHasAnyFlags(CurrentFilterFlags, AssetFlags))
		{
			continue;
		}

		if (EnumHasAnyFlags(CurrentFilterFlags, EAnimNotifyFilterFlags::Notifies))
		{
			const FString TagValue = AssetData.GetTagValueRef<FString>(USkeleton::AnimNotifyTag);
			if (!TagValue.IsEmpty())
			{
				UE::String::ParseTokens(TagValue, USkeleton::AnimNotifyTagDelimiter, 
				 [&NotifyNames, AssetFlags](FStringView InToken)
				{
					EAnimNotifyFilterFlags& ItemFlags = NotifyNames.FindOrAdd(FName(InToken));
					ItemFlags |= AssetFlags;
				}, UE::String::EParseTokensOptions::SkipEmpty);
			}
		}

		if (EnumHasAnyFlags(CurrentFilterFlags, EAnimNotifyFilterFlags::SyncMarkers))
		{
			const FString TagValue = AssetData.GetTagValueRef<FString>(USkeleton::AnimSyncMarkerTag);
			if (!TagValue.IsEmpty())
			{
				UE::String::ParseTokens(TagValue, USkeleton::AnimSyncMarkerTagDelimiter, 
				[&SyncMarkerNames, AssetFlags](FStringView InToken)
				{
					EAnimNotifyFilterFlags& ItemFlags = SyncMarkerNames.FindOrAdd(FName(InToken));
					ItemFlags |= AssetFlags;
				}, UE::String::EParseTokensOptions::SkipEmpty);
			}
		}
	}

	if(EnumHasAnyFlags(CurrentFilterFlags, EAnimNotifyFilterFlags::Notifies))
	{
		for(const TPair<FName, EAnimNotifyFilterFlags>& Item : NotifyNames)
		{
			NotifyList.Add(FDisplayedAnimNotifyInfo::Make(Item.Key, false, Item.Value));
		}
	}

	if(EnumHasAnyFlags(CurrentFilterFlags, EAnimNotifyFilterFlags::SyncMarkers))
	{
		for(const TPair<FName, EAnimNotifyFilterFlags>& Item : SyncMarkerNames)
		{
			NotifyList.Add(FDisplayedAnimNotifyInfo::Make(Item.Key, true, Item.Value));
		}
	}
}

void SSkeletonAnimNotifies::ShowNotifyInDetailsView(FName NotifyName)
{
	if(EditableSkeleton.IsValid() && OnObjectsSelected.IsBound())
	{
		ClearDetailsView();

		UEditorSkeletonNotifyObj *Obj = Cast<UEditorSkeletonNotifyObj>(ShowInDetailsView(UEditorSkeletonNotifyObj::StaticClass()));
		if(Obj != NULL)
		{
			Obj->EditableSkeleton = EditableSkeleton;
			Obj->Name = NotifyName;
		}
	}
}

UObject* SSkeletonAnimNotifies::ShowInDetailsView( UClass* EdClass )
{
	UObject *Obj = EditorObjectTracker.GetEditorObjectForClass(EdClass);

	if(Obj != NULL)
	{
		TArray<UObject*> Objects;
		Objects.Add(Obj);
		OnObjectsSelected.ExecuteIfBound(Objects);
	}
	return Obj;
}

void SSkeletonAnimNotifies::ClearDetailsView()
{
	TArray<UObject*> Objects;
	OnObjectsSelected.ExecuteIfBound(Objects);
}

void SSkeletonAnimNotifies::PostUndo( bool bSuccess )
{
	RefreshNotifiesListWithFilter();
}

void SSkeletonAnimNotifies::PostRedo( bool bSuccess )
{
	RefreshNotifiesListWithFilter();
}

void SSkeletonAnimNotifies::AddReferencedObjects( FReferenceCollector& Collector )
{
	EditorObjectTracker.AddReferencedObjects(Collector);
}

void SSkeletonAnimNotifies::NotifyUser( FNotificationInfo& NotificationInfo )
{
	TSharedPtr<SNotificationItem> Notification = FSlateNotificationManager::Get().AddNotification( NotificationInfo );
	if ( Notification.IsValid() )
	{
		Notification->SetCompletionState( SNotificationItem::CS_Fail );
	}
}

void SSkeletonAnimNotifies::OnFindReferences()
{
	TSharedPtr<FAssetEditorToolkit> HostingApp = WeakHostingApp.Pin();
	if (HostingApp.IsValid())
	{
		check(NotifiesListView->GetNumItemsSelected() == 1);
		TArray<TSharedPtr<FDisplayedAnimNotifyInfo>> SelectedItems;
		NotifiesListView->GetSelectedItems(SelectedItems);
		FName Name = SelectedItems[0]->Name;

		if(TSharedPtr<SDockTab> Tab = HostingApp->GetTabManager()->TryInvokeTab(FPersonaTabs::FindReplaceID))
		{
			TSharedRef<IAnimAssetFindReplace> FindReplaceWidget = StaticCastSharedRef<IAnimAssetFindReplace>(Tab->GetContent());
			FindReplaceWidget->SetCurrentProcessor(SelectedItems[0]->bIsSyncMarker ? UAnimAssetFindReplaceSyncMarkers::StaticClass() : UAnimAssetFindReplaceNotifies::StaticClass());
			UAnimAssetFindReplaceProcessor_StringBase* Processor = Cast<UAnimAssetFindReplaceProcessor_StringBase>(FindReplaceWidget->GetCurrentProcessor());
			Processor->SetFindString(Name.ToString());
		}
	}
}

#undef LOCTEXT_NAMESPACE
