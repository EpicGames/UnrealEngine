// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "UObject/GCObject.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "WorkflowOrientedApp/WorkflowTabFactory.h"
#include "PersonaDelegates.h"
#include "IDocumentation.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"
#include "EditorObjectsTracker.h"
#include "EditorUndoClient.h"
#include "Filters/FilterBase.h"

#define LOCTEXT_NAMESPACE "SkeletonAnimnotifies"

class IEditableSkeleton;
class SToolTip;
struct FNotificationInfo;

/////////////////////////////////////////////////////
// FSkeletonAnimNotifiesSummoner
struct FSkeletonAnimNotifiesSummoner : public FWorkflowTabFactory
{
public:
	FSkeletonAnimNotifiesSummoner(TSharedPtr<class FAssetEditorToolkit> InHostingApp, const TSharedRef<class IEditableSkeleton>& InEditableSkeleton, FOnObjectsSelected InOnObjectsSelected);

	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;

	// Create a tooltip widget for the tab
	virtual TSharedPtr<SToolTip> CreateTabToolTipWidget(const FWorkflowTabSpawnInfo& Info) const override
	{
		return  IDocumentation::Get()->CreateToolTip(LOCTEXT("AnimationNotifierTooltip", "This tab lets you modify custom animation notifies"), NULL, TEXT("Shared/Editors/Persona"), TEXT("AnimationNotifies_Window"));
	}

private:
	TWeakPtr<class IEditableSkeleton> EditableSkeleton;
	FOnObjectsSelected OnObjectsSelected;
};

// This is a flag that is used to filter UI part
enum class EAnimNotifyFilterFlags : uint8 
{
	// Show none
	None			= 0, 
	// Show notifies
	Notifies		= 0x01, 
	// Show sync markers
	SyncMarkers		= 0x02, 
	// Show the current skeleton's items
	CurrentSkeleton	= 0x04, 
	// Show other compatible asset's items
	CompatibleAssets = 0x08, 
	// Show other asset's items
	OtherAssets		= 0x10, 
};

ENUM_CLASS_FLAGS(EAnimNotifyFilterFlags);

//////////////////////////////////////////////////////////////////////////
// FDisplayedAnimNotifyInfo

class FDisplayedAnimNotifyInfo
{
public:
	FName Name;

	/** Handle to editable text block for rename */
	TSharedPtr<SInlineEditableTextBlock> InlineEditableText;

	/** Flag to say whether this is a new item we are creating */
	bool bIsNew;

	/** Identifies sync markers vs notifies */
	bool bIsSyncMarker;

	/** Flags for this item */
	EAnimNotifyFilterFlags ItemFlags;

	/** Static function for creating a new item, but ensures that you can only have a TSharedRef to one */
	static TSharedRef<FDisplayedAnimNotifyInfo> Make(const FName& NotifyName, bool bInIsSyncMarker, EAnimNotifyFilterFlags InItemFlags)
	{
		return MakeShareable(new FDisplayedAnimNotifyInfo(NotifyName, bInIsSyncMarker, InItemFlags));
	}

protected:
	/** Hidden constructor, always use Make above */
	FDisplayedAnimNotifyInfo(const FName& InNotifyName, bool bInIsSyncMarker, EAnimNotifyFilterFlags InItemFlags)
		: Name( InNotifyName )
		, bIsNew(false)
		, bIsSyncMarker(bInIsSyncMarker)
		, ItemFlags(InItemFlags)
	{}

	/** Hidden constructor, always use Make above */
	FDisplayedAnimNotifyInfo() {}
};

/** Widgets list type */
typedef SListView< TSharedPtr<FDisplayedAnimNotifyInfo> > SAnimNotifyListType;

class SSkeletonAnimNotifies : public SCompoundWidget, public FGCObject, public FEditorUndoClient
{
public:
	SLATE_BEGIN_ARGS( SSkeletonAnimNotifies )
		: _IsPicker(false)
		, _ShowSyncMarkers(true)
		, _ShowNotifies(true)
		, _ShowCompatibleSkeletonAssets(false)
		, _ShowOtherAssets(false)
	{}

	/** Delegate called to select an object in the details panel */
	SLATE_EVENT(FOnObjectsSelected, OnObjectsSelected)

	/** Delegate fired when a notify is selected */
	SLATE_EVENT(FOnNotifyPicked, OnItemSelected)

	/** Whether we should use this dialog as a picker or an editor. In picker mode we cant add, remove or rename notifies. */
	SLATE_ARGUMENT(bool, IsPicker)

	/** Whether we should show sync markers */
	SLATE_ARGUMENT_DEPRECATED(bool, IsSyncMarker, 5.3, "Please use ShowSyncMarkers or ShowNotifies instead")

	/** Whether we should show sync markers */
	SLATE_ARGUMENT(bool, ShowSyncMarkers)

	/** Whether we should show notifies */
	SLATE_ARGUMENT(bool, ShowNotifies)

	/** Whether we notifies and sync markers from assets compatible with the current skeleton */
	SLATE_ARGUMENT(bool, ShowCompatibleSkeletonAssets)

	/** Whether we notifies and sync markers from assets other than those compatible with the current skeleton */
	SLATE_ARGUMENT(bool, ShowOtherAssets)

	/** Editable skeleton - if this is set, notifies and sync markers will be added to the skeleton on creation */
	SLATE_ARGUMENT(TSharedPtr<IEditableSkeleton>, EditableSkeleton)

	SLATE_END_ARGS()
public:
	void Construct(const FArguments& InArgs, const TSharedPtr<class FAssetEditorToolkit>& InHostingApp = nullptr);

	~SSkeletonAnimNotifies();

	/**
	* Accessor so our rows can grab the filter text for highlighting
	*
	*/
	FText& GetFilterText() { return FilterText; }

	/** Creates an editor object from the given type to be used in a details panel */
	UObject* ShowInDetailsView( UClass* EdClass );

	/** Clears the detail view of whatever we displayed last */
	void ClearDetailsView();

	/** FEditorUndoClient interface */
	virtual void PostUndo( bool bSuccess ) override;
	virtual void PostRedo( bool bSuccess ) override;

	// FGCObject interface start
	virtual void AddReferencedObjects( FReferenceCollector& Collector ) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("SSkeletonAnimNotifies");
	}
	// FGCObject interface end

	/** When user attempts to commit the name of a track*/
	bool OnVerifyNotifyNameCommit( const FText& NewName, FText& OutErrorMessage, TSharedPtr<FDisplayedAnimNotifyInfo> Item );

	/** When user commits the name of a track*/
	void OnNotifyNameCommitted( const FText& NewName, ETextCommit::Type, TSharedPtr<FDisplayedAnimNotifyInfo> Item );

	/** Dummy handler to stop editable text boxes swallowing our list selected events */
	bool IsSelected(){return false;}
private:

	/** Called when the user changes the contents of the search box */
	void OnFilterTextChanged( const FText& SearchText );

	/** Called when the user changes the contents of the search box */
	void OnFilterTextCommitted( const FText& SearchText, ETextCommit::Type CommitInfo );

	/** Delegate handler for generating rows in NotifiesListView */ 
	TSharedRef<ITableRow> GenerateNotifyRow( TSharedPtr<FDisplayedAnimNotifyInfo> InInfo, const TSharedRef<STableViewBase>& OwnerTable );

	/** Delegate handler called when the user right clicks in NotifiesListView */
	TSharedPtr<SWidget> OnGetContextMenuContent() const;

	/** Delegate handler for when the user selects something in NotifiesListView */
	void OnNotifySelectionChanged( TSharedPtr<FDisplayedAnimNotifyInfo> Selection, ESelectInfo::Type SelectInfo );

	/** Delegate handler for determining whether we can show the delete menu options */
	bool CanPerformDelete() const;
	
	/** Delegate handler for determining whether we can show the find references menu option */
	bool CanPerformFindReferences() const;
	
	/** Delegate handler for deleting items */
	void OnDeleteItems();

	/** Delegate handler for determining whether we can show the rename menu options */
	bool CanPerformRename() const;

	/** Delegate handler for adding anim notifies & sync markers */
	void OnAddItem(bool bIsSyncMarker);

	/** Delegate handler for renaming items */
	void OnRenameItem();

	/** Wrapper that populates NotifiesListView using current filter text */
	void RefreshNotifiesListWithFilter();

	/** Populates NotifyList based on the project's notifies */
	void CreateNotifiesList();

	/** Populates FilteredNotifyList based on the project's notifies and the supplied filter text */
	void FilterNotifiesList(const FString& InSearchText);

	/** handler for user selecting a Notify in NotifiesListView - populates the details panel */
	void ShowNotifyInDetailsView( FName NotifyName );

	/** Utility function to display notifications to the user */
	void NotifyUser( FNotificationInfo& NotificationInfo );

	/** Handler function for when notifies are modified on the skeleton */
	void OnNotifiesChanged();

	/** Handle when an item is scrolled into view, triggers a rename for new items */
	void OnItemScrolledIntoView(TSharedPtr<FDisplayedAnimNotifyInfo> InItem, const TSharedPtr<ITableRow>& InTableRow);

	/** Handle find references from the context menu */
	void OnFindReferences();

	/** The skeleton we are currently editing */
	TSharedPtr<class IEditableSkeleton> EditableSkeleton;

	/** SSearchBox to filter the notify list */
	TSharedPtr<SSearchBox>	NameFilterBox;

	/** Widget used to display the list of notifies */
	TSharedPtr<SAnimNotifyListType> NotifiesListView;

	/** A list of all notifies and sync markers. */
	TArray< TSharedPtr<FDisplayedAnimNotifyInfo> > NotifyList;

	/** Filtered list of notifies. Used by the NotifiesListView. */
	TArray< TSharedPtr<FDisplayedAnimNotifyInfo> > FilteredNotifyList;

	/** Current text typed into NameFilterBox */
	FText FilterText;

	/** Tracks objects created for displaying in the details panel*/
	FEditorObjectTracker EditorObjectTracker;

	/** Delegate called to select an object in the details panel */
	FOnObjectsSelected OnObjectsSelected;

	/** Delegate fired when a notify is selected */
	FOnNotifyPicked OnItemSelected;

	/** The app we are embedded in */
	TWeakPtr<class FAssetEditorToolkit> WeakHostingApp;

	/** Whether we should use this dialog as a picker or an editor. In picker mode we cant add, remove or rename notifies. */
	bool bIsPicker = false;

	/** Whether we should display markers */
	bool bShowSyncMarkers = false;

	/** Whether we should display notifies */
	bool bShowNotifies = false;

	/** Whether we should display notifies or sync markers from other skeletons */
	bool bShowOtherSkeletonItems = false;

	/** Whether we notifies and sync markers from assets other than the current skeleton */
	bool bShowOtherAssets = false;
	
	/** Whether we notifies and sync markers from assets compatible with the current skeleton */
	bool bShowCompatibleSkeletonAssets = false;

	/** Whether to suspend refreshing the UI when filtering */
	bool bAllowRefreshFilter = true;

	/** All filters that can be applied to the widget's display */
	TArray<TSharedRef<FFilterBase<EAnimNotifyFilterFlags>>> Filters;

	/** Current filter flags */
	EAnimNotifyFilterFlags CurrentFilterFlags = EAnimNotifyFilterFlags::None;
};

#undef LOCTEXT_NAMESPACE
