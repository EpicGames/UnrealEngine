// Copyright Epic Games, Inc. All Rights Reserved.


#include "SSkeletonSlotNames.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Misc/ScopedSlowTask.h"
#include "Framework/MultiBox/MultiBoxDefs.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Animation/Skeleton.h"
#include "Styling/AppStyle.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Animation/AnimMontage.h"
#include "FileHelpers.h"
#include "SSlotNameReferenceWindow.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "ScopedTransaction.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Input/STextEntryPopup.h"
#include "Animation/AnimBlueprint.h"
#include "AnimGraphNode_Slot.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Interfaces/IMainFrameModule.h"
#include "IEditableSkeleton.h"
#include "TabSpawners.h"

#define LOCTEXT_NAMESPACE "SkeletonSlotNames"

static const FName ColumnId_SlotNameLabel( "SlotName" );

//////////////////////////////////////////////////////////////////////////
// SMorphTargetListRow

typedef TSharedPtr< FDisplayedSlotNameInfo > FDisplayedSlotNameInfoPtr;

class SSlotNameListRow
	: public SMultiColumnTableRow< FDisplayedSlotNameInfoPtr >
{
public:

	SLATE_BEGIN_ARGS( SSlotNameListRow ) {}

	/** The item for this row **/
	SLATE_ARGUMENT( FDisplayedSlotNameInfoPtr, Item )

	/* Widget used to display the list of morph targets */
	SLATE_ARGUMENT( TSharedPtr<SSkeletonSlotNames>, SlotNameListView )

	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTableView );

	/** Overridden from SMultiColumnTableRow.  Generates a widget for this column of the tree row. */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn( const FName& ColumnName ) override;

private:

	/** Widget used to display the list of slot name */
	TSharedPtr<SSkeletonSlotNames> SlotNameListView;

	/** The notify being displayed by this row */
	FDisplayedSlotNameInfoPtr	Item;
};

void SSlotNameListRow::Construct( const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView )
{
	Item = InArgs._Item;
	SlotNameListView = InArgs._SlotNameListView;

	check( Item.IsValid() );

	SMultiColumnTableRow< FDisplayedSlotNameInfoPtr >::Construct( FSuperRowType::FArguments(), InOwnerTableView );
}

TSharedRef< SWidget > SSlotNameListRow::GenerateWidgetForColumn( const FName& ColumnName )
{
	check( ColumnName == ColumnId_SlotNameLabel );

	// Items can be either Slots or Groups.
	FText ItemText = Item->bIsGroupItem ? FText::Format(LOCTEXT("AnimSlotManagerGroupItem", "(Group) {0}"), FText::FromName(Item->Name))
		: FText::Format(LOCTEXT("AnimSlotManagerSlotItem", "(Slot) {0}"), FText::FromName(Item->Name));

	return
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SExpanderArrow, SharedThis(this))
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(STextBlock)
			.Text(ItemText)
		];
}

/////////////////////////////////////////////////////
// FSkeletonSlotNamesSummoner

FSkeletonSlotNamesSummoner::FSkeletonSlotNamesSummoner(TSharedPtr<class FAssetEditorToolkit> InHostingApp, const TSharedRef<IEditableSkeleton>& InEditableSkeleton, FOnObjectSelected InOnObjectSelected)
	: FWorkflowTabFactory(FPersonaTabs::SkeletonSlotNamesID, InHostingApp)
	, EditableSkeleton(InEditableSkeleton)
	, OnObjectSelected(InOnObjectSelected)
{
	TabLabel = LOCTEXT("AnimSlotManagerTabTitle", "Anim Slot Manager");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Persona.Tabs.AnimSlotManager");

	EnableTabPadding();
	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("SkeletonSlotNamesMenu", "Anim Slots");
	ViewMenuTooltip = LOCTEXT("SkeletonSlotNames_ToolTip", "Manage Skeleton's Slots and Groups.");
}

TSharedRef<SWidget> FSkeletonSlotNamesSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	return SNew(SSkeletonSlotNames, EditableSkeleton.Pin().ToSharedRef())
		.OnObjectSelected(OnObjectSelected);
}

/////////////////////////////////////////////////////
// SSkeletonSlotNames

void SSkeletonSlotNames::Construct(const FArguments& InArgs, const TSharedRef<IEditableSkeleton>& InEditableSkeleton)
{
	EditableSkeleton = InEditableSkeleton;
	OnObjectSelected = InArgs._OnObjectSelected;

	InEditableSkeleton->RegisterOnSlotsChanged(FSimpleMulticastDelegate::FDelegate::CreateSP(this, &SSkeletonSlotNames::RefreshSlotNameListWithFilter));

	// Toolbar
	FToolBarBuilder ToolbarBuilder(TSharedPtr< FUICommandList >(), FMultiBoxCustomization::None);

	// Save USkeleton
	ToolbarBuilder.AddToolBarButton(
		FUIAction(FExecuteAction::CreateSP(this, &SSkeletonSlotNames::OnSaveSkeleton))
		, NAME_None
		, LOCTEXT("AnimSlotManagerToolbarSaveLabel", "Save")
		, LOCTEXT("AnimSlotManagerToolbarSaveTooltip", "Saves changes into Skeleton asset")
		, FSlateIcon(FAppStyle::GetAppStyleSetName(), "AnimSlotManager.SaveSkeleton")
		);

	ToolbarBuilder.AddSeparator();

	// Add Slot
	ToolbarBuilder.AddToolBarButton(
		FUIAction(FExecuteAction::CreateSP(this, &SSkeletonSlotNames::OnAddSlot))
		, NAME_None
		, LOCTEXT("AnimSlotManagerToolbarAddSlotLabel", "Add Slot")
		, LOCTEXT("AnimSlotManagerToolbarAddSlotTooltip", "Create a new unique Slot name")
		, FSlateIcon(FAppStyle::GetAppStyleSetName(), "AnimSlotManager.AddSlot")
		);

	// Add Group
	ToolbarBuilder.AddToolBarButton(
		FUIAction(FExecuteAction::CreateSP(this, &SSkeletonSlotNames::OnAddGroup))
		, NAME_None
		, LOCTEXT("AnimSlotManagerToolbarAddGroupLabel", "Add Group")
		, LOCTEXT("AnimSlotManagerToolbarAddGroupTooltip", "Create a new unique Group name")
		, FSlateIcon(FAppStyle::GetAppStyleSetName(), "AnimSlotManager.AddGroup")
		);

	this->ChildSlot
	[
		SNew( SVerticalBox )

		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			ToolbarBuilder.MakeWidget()
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding( FMargin( 0.0f, 0.0f, 0.0f, 4.0f ) )
		[
			SAssignNew( NameFilterBox, SSearchBox )
			.SelectAllTextWhenFocused( true )
			.OnTextChanged( this, &SSkeletonSlotNames::OnFilterTextChanged )
			.OnTextCommitted( this, &SSkeletonSlotNames::OnFilterTextCommitted )
			.HintText( LOCTEXT( "AnimSlotManagerSlotNameSearchBoxHint", "Slot name filter...") )
		]

		+ SVerticalBox::Slot()
		.FillHeight( 1.0f )		// This is required to make the scrollbar work, as content overflows Slate containers by default
		[
			SAssignNew( SlotNameListView, SSlotNameListType )
			.TreeItemsSource(&NotifyList)
			.OnGenerateRow( this, &SSkeletonSlotNames::GenerateNotifyRow )
			.OnGetChildren(this, &SSkeletonSlotNames::GetChildrenForInfo)
			.OnContextMenuOpening( this, &SSkeletonSlotNames::OnGetContextMenuContent )
			.SelectionMode(ESelectionMode::Single)
			.OnSelectionChanged( this, &SSkeletonSlotNames::OnNotifySelectionChanged )
			.HeaderRow
			(
				SNew( SHeaderRow )
				+ SHeaderRow::Column( ColumnId_SlotNameLabel )
				.DefaultLabel( LOCTEXT( "SlotNameNameLabel", "Slot Name" ) )
			)
		]
	];

	CreateSlotNameList();
}

void SSkeletonSlotNames::OnFilterTextChanged( const FText& SearchText )
{
	FilterText = SearchText;

	RefreshSlotNameListWithFilter();
}

void SSkeletonSlotNames::OnFilterTextCommitted( const FText& SearchText, ETextCommit::Type CommitInfo )
{
	// Just do the same as if the user typed in the box
	OnFilterTextChanged( SearchText );
}

TSharedRef<ITableRow> SSkeletonSlotNames::GenerateNotifyRow(TSharedPtr<FDisplayedSlotNameInfo> InInfo, const TSharedRef<STableViewBase>& OwnerTable)
{
	check( InInfo.IsValid() );

	return
		SNew( SSlotNameListRow, OwnerTable )
		.Item( InInfo )
		.SlotNameListView( SharedThis( this ) );
}

void SSkeletonSlotNames::GetChildrenForInfo(TSharedPtr<FDisplayedSlotNameInfo> InInfo, TArray< TSharedPtr<FDisplayedSlotNameInfo> >& OutChildren)
{
	check(InInfo.IsValid());
	OutChildren = InInfo->Children;
}

TSharedPtr<SWidget> SSkeletonSlotNames::OnGetContextMenuContent() const
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder( bShouldCloseWindowAfterMenuSelection, NULL);

	TArray< TSharedPtr< FDisplayedSlotNameInfo > > SelectedItems = SlotNameListView.Get()->GetSelectedItems();

	bool bHasSelectedItem = (SelectedItems.Num() > 0);
	bool bShowGroupItem = bHasSelectedItem && SelectedItems[0].Get()->bIsGroupItem;
	bool bShowSlotItem = bHasSelectedItem && !SelectedItems[0].Get()->bIsGroupItem;

	if (bShowGroupItem)
	{
		TSharedPtr<FDisplayedSlotNameInfo> SelectedItemPtr = SelectedItems[0];

		MenuBuilder.BeginSection("SlotManagerSlotGroupActions", LOCTEXT("SlotManagerSlotGroupActions", "Slot Group Actions"));
		// Delete Slot Group
		{
			FDisplayedSlotNameInfo* SlotInfo = SelectedItemPtr.Get();

			FUIAction Action = FUIAction(FExecuteAction::CreateSP(const_cast<SSkeletonSlotNames*>(this), &SSkeletonSlotNames::OnDeleteSlotGroup, SlotInfo->Name));
			Action.CanExecuteAction.BindSP(const_cast<SSkeletonSlotNames*>(this), &SSkeletonSlotNames::CanDeleteSlotGroup, SlotInfo->Name);
			const FText Label = LOCTEXT("AnimSlotManagerContextMenuDeleteSlotGroupLabel", "Delete Slot Group");
			const FText ToolTipText = LOCTEXT("AnimSlotManagerContextMenuDeleteSlotGroupTooltip", "Delete this slot group.");
			MenuBuilder.AddMenuEntry(Label, ToolTipText, FSlateIcon(), Action);
		}
		MenuBuilder.EndSection();
	}
	else if (bShowSlotItem)
	{
		TSharedPtr<FDisplayedSlotNameInfo> SelectedItemPtr = SelectedItems[0];

		MenuBuilder.BeginSection("SlotManagerSlotActions", LOCTEXT("SlotManagerSlotActions", "Slot Actions"));
		// Set Slot Group
		{
			MenuBuilder.AddSubMenu(
				FText::Format(LOCTEXT("ContextMenuSetSlotGroupLabel", "Set Slot {0} Group to"), FText::FromName(SelectedItems[0].Get()->Name)),
				FText::Format(LOCTEXT("ContextMenuSetSlotGroupToolTip", "Set Slot {0} Group"), FText::FromName(SelectedItems[0].Get()->Name)),
				FNewMenuDelegate::CreateRaw(const_cast<SSkeletonSlotNames*>(this), &SSkeletonSlotNames::FillSetSlotGroupSubMenu));
		}
		// Rename Slot
		{
			FDisplayedSlotNameInfo* SlotInfo = SelectedItemPtr.Get();

			FUIAction Action = FUIAction(FExecuteAction::CreateSP(const_cast<SSkeletonSlotNames*>(this), &SSkeletonSlotNames::OnRenameSlot, SlotInfo->Name));
			const FText Label = LOCTEXT("AnimSlotManagerContextMenuRenameSlotLabel", "Rename Slot");
			const FText ToolTipText = LOCTEXT("AnimSlotManagerContextMenuRenameSlotTooltip", "Rename this slot");
			MenuBuilder.AddMenuEntry(Label, ToolTipText, FSlateIcon(), Action);
		}
		// Delete Slot
		{
			FDisplayedSlotNameInfo* SlotInfo = SelectedItemPtr.Get();

			FUIAction Action = FUIAction(FExecuteAction::CreateSP(const_cast<SSkeletonSlotNames*>(this), &SSkeletonSlotNames::OnDeleteSlot, SlotInfo->Name));
			const FText Label = LOCTEXT("AnimSlotManagerContextMenuDeleteSlotLabel", "Delete Slot");
			const FText ToolTipText = LOCTEXT("AnimSlotManagerContextMenuDeleteSlotTooltip", "Delete this slot.");
			MenuBuilder.AddMenuEntry(Label, ToolTipText, FSlateIcon(), Action);
		}
		MenuBuilder.EndSection();
	}

	MenuBuilder.BeginSection("SlotManagerGeneralActions", LOCTEXT("SlotManagerGeneralActions", "Slot Manager Actions"));
	// Add Slot
	{
		FUIAction Action = FUIAction(FExecuteAction::CreateSP(const_cast<SSkeletonSlotNames*>(this), &SSkeletonSlotNames::OnAddSlot));
		const FText Label = LOCTEXT("AnimSlotManagerContextMenuAddSlotLabel", "Add Slot");
		const FText ToolTipText = LOCTEXT("AnimSlotManagerContextMenuAddSlotTooltip", "Adds a new Slot");
		MenuBuilder.AddMenuEntry(Label, ToolTipText, FSlateIcon(), Action);
	}
	// Add Group
	{
		FUIAction Action = FUIAction(FExecuteAction::CreateSP(const_cast<SSkeletonSlotNames*>(this), &SSkeletonSlotNames::OnAddGroup));
		const FText Label = LOCTEXT("AnimSlotManagerContextMenuAddGroupLabel", "Add Group");
		const FText ToolTipText = LOCTEXT("AnimSlotManagerContextMenuAddGroupTooltip", "Adds a new Group");
		MenuBuilder.AddMenuEntry(Label, ToolTipText, FSlateIcon(), Action);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void SSkeletonSlotNames::FillSetSlotGroupSubMenu(FMenuBuilder& MenuBuilder)
{
	const TArray<FAnimSlotGroup>& SlotGroups = EditableSkeleton->GetSkeleton().GetSlotGroups();
	for (auto SlotGroup : SlotGroups)
	{
		const FName& GroupName = SlotGroup.GroupName;

		const FText ToolTipText = FText::Format(LOCTEXT("ContextMenuSetSlotSubMenuToolTip", "Changes slot's group to {0}"), FText::FromName(GroupName));
/*		FString Label = Class->GetDisplayNameText().ToString();*/
		const FText Label = FText::FromName(GroupName);

		FUIAction UIAction;
		UIAction.ExecuteAction.BindRaw(this, &SSkeletonSlotNames::ContextMenuOnSetSlot,	GroupName);
		MenuBuilder.AddMenuEntry(Label, ToolTipText, FSlateIcon(), UIAction);
	}
}

void SSkeletonSlotNames::ContextMenuOnSetSlot(FName InNewGroupName)
{
	TArray< TSharedPtr< FDisplayedSlotNameInfo > > SelectedItems = SlotNameListView.Get()->GetSelectedItems();

	bool bHasSelectedItem = (SelectedItems.Num() > 0);
	bool bShowSlotItem = bHasSelectedItem && !SelectedItems[0].Get()->bIsGroupItem;

	if (bShowSlotItem)
	{
		const FName SlotName = SelectedItems[0].Get()->Name;
		if (EditableSkeleton->GetSkeleton().ContainsSlotName(SlotName))
		{
			EditableSkeleton->SetSlotGroupName(SlotName, InNewGroupName);

			RefreshSlotNameListWithFilter();
		}

		// Highlight newly created item.
		TSharedPtr< FDisplayedSlotNameInfo > Item = FindItemNamed(SlotName);
		if (Item.IsValid())
		{
			SlotNameListView->SetSelection(Item);
		}

		FSlateApplication::Get().DismissAllMenus();
	}
}

void SSkeletonSlotNames::OnNotifySelectionChanged(TSharedPtr<FDisplayedSlotNameInfo> Selection, ESelectInfo::Type SelectInfo)
{
	if(Selection.IsValid())
	{
		ShowNotifyInDetailsView(Selection->Name);
	}
}

void SSkeletonSlotNames::OnSaveSkeleton()
{
	TArray< UPackage* > PackagesToSave;
	PackagesToSave.Add(EditableSkeleton->GetSkeleton().GetOutermost());

	FEditorFileUtils::PromptForCheckoutAndSave(PackagesToSave, /*bCheckDirty=*/ false, /*bPromptToSave=*/ false);
}

void SSkeletonSlotNames::OnAddSlot()
{
	TSharedRef<STextEntryPopup> TextEntry =
		SNew(STextEntryPopup)
		.Label(LOCTEXT("NewSlotName_AskSlotName", "New Slot Name"))
		.OnTextCommitted(this, &SSkeletonSlotNames::AddSlotPopUpOnCommit);

	// Show dialog to enter new track name
	FSlateApplication::Get().PushMenu(
		SharedThis(this),
		FWidgetPath(),
		TextEntry,
		FSlateApplication::Get().GetCursorPos(),
		FPopupTransitionEffect(FPopupTransitionEffect::TypeInPopup)
		);
}

void SSkeletonSlotNames::OnAddGroup()
{
	TSharedRef<STextEntryPopup> TextEntry =
		SNew(STextEntryPopup)
		.Label(LOCTEXT("NewGroupName_AskGroupName", "New Group Name"))
		.OnTextCommitted(this, &SSkeletonSlotNames::AddGroupPopUpOnCommit);

	// Show dialog to enter new track name
	FSlateApplication::Get().PushMenu(
		SharedThis(this),
		FWidgetPath(),
		TextEntry,
		FSlateApplication::Get().GetCursorPos(),
		FPopupTransitionEffect(FPopupTransitionEffect::TypeInPopup)
		);
}

void SSkeletonSlotNames::AddSlotPopUpOnCommit(const FText & InNewSlotText, ETextCommit::Type CommitInfo)
{
	if (!InNewSlotText.IsEmpty())
	{
		const FScopedTransaction Transaction(LOCTEXT("NewSlotName_AddSlotName", "Add New Slot Node Name"));

		FName NewSlotName = FName(*InNewSlotText.ToString());
		const USkeleton& Skeleton = EditableSkeleton->GetSkeleton();
		// Keep slot and group names unique
		if (!Skeleton.ContainsSlotName(NewSlotName) && (Skeleton.FindAnimSlotGroup(NewSlotName) == nullptr))
		{
			TArray< TSharedPtr< FDisplayedSlotNameInfo > > SelectedItems = SlotNameListView->GetSelectedItems();
			bool bHasSelectedItem = (SelectedItems.Num() > 0);
			bool bShowGroupItem = bHasSelectedItem && SelectedItems[0].Get()->bIsGroupItem;

			EditableSkeleton->SetSlotGroupName(NewSlotName, bShowGroupItem ? SelectedItems[0].Get()->Name : FAnimSlotGroup::DefaultGroupName);

			RefreshSlotNameListWithFilter();
		}

		// Highlight newly created item.
		TSharedPtr< FDisplayedSlotNameInfo > Item = FindItemNamed(NewSlotName);
		if (Item.IsValid())
		{
			SlotNameListView->SetSelection(Item);
		}

		FSlateApplication::Get().DismissAllMenus();
	}
}

void SSkeletonSlotNames::AddGroupPopUpOnCommit(const FText & InNewGroupText, ETextCommit::Type CommitInfo)
{
	if (!InNewGroupText.IsEmpty())
	{
		FName NewGroupName = FName(*InNewGroupText.ToString());
		const USkeleton& Skeleton = EditableSkeleton->GetSkeleton();
		// Keep slot and group names unique
		if (!Skeleton.ContainsSlotName(NewGroupName) && EditableSkeleton->AddSlotGroupName(NewGroupName))
		{
			RefreshSlotNameListWithFilter();
		}

		// Highlight newly created item.
		TSharedPtr< FDisplayedSlotNameInfo > Item = FindItemNamed(NewGroupName);
		if (Item.IsValid())
		{
			SlotNameListView->SetSelection(Item);
		}

		FSlateApplication::Get().DismissAllMenus();
	}
}

void SSkeletonSlotNames::GetCompatibleAnimBlueprints( TArray<FAssetData>& OutAssets )
{
	//Get the skeleton tag to search for
	const USkeleton& Skeleton = EditableSkeleton->GetSkeleton();
	FString SkeletonExportName = FAssetData(&Skeleton).GetExportTextName();

	// Load the asset registry module
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

	TArray<FAssetData> AssetDataList;
	AssetRegistryModule.Get().GetAssetsByClass(UAnimBlueprint::StaticClass()->GetClassPathName(), AssetDataList, true);

	OutAssets.Empty(AssetDataList.Num());

	for(FAssetData& Data : AssetDataList)
	{
		const FString AssetSkeleton = Data.GetTagValueRef<FString>("TargetSkeleton");
		if(AssetSkeleton == SkeletonExportName)
		{
			OutAssets.Add(Data);
		}
	}
}

void SSkeletonSlotNames::RefreshSlotNameListWithFilter()
{
	CreateSlotNameList( NameFilterBox->GetText().ToString() );
}

void SSkeletonSlotNames::CreateSlotNameList(const FString& SearchText)
{
	NotifyList.Empty();

	const USkeleton& Skeleton = EditableSkeleton->GetSkeleton();
	const TArray<FAnimSlotGroup>& SlotGroups = Skeleton.GetSlotGroups();
	for (auto SlotGroup : SlotGroups)
	{
		const FName& GroupName = SlotGroup.GroupName;
			
		TSharedRef<FDisplayedSlotNameInfo> GroupItem = FDisplayedSlotNameInfo::Make(GroupName, true);
		SlotNameListView->SetItemExpansion(GroupItem, true);
		NotifyList.Add(GroupItem);

		for (auto SlotName : SlotGroup.SlotNames)
		{
			if (SearchText.IsEmpty() || GroupName.ToString().Contains(SearchText) || SlotName.ToString().Contains(SearchText))
			{
				TSharedRef<FDisplayedSlotNameInfo> SlotItem = FDisplayedSlotNameInfo::Make(SlotName, false);
				SlotNameListView->SetItemExpansion(SlotItem, true);
				NotifyList[NotifyList.Num() - 1]->Children.Add(SlotItem);
			}
		}
	}
	
	SlotNameListView->RequestTreeRefresh();
}

TSharedPtr< FDisplayedSlotNameInfo > SSkeletonSlotNames::FindItemNamed(FName ItemName) const
{
	for (auto SlotGroupItem : NotifyList)
	{
		if (SlotGroupItem->Name == ItemName)
		{
			return SlotGroupItem;
		}
		for (auto SlotItem : SlotGroupItem->Children)
		{
			if (SlotItem->Name == ItemName)
			{
				return SlotItem;
			}
		}
	}

	return NULL;
}

void SSkeletonSlotNames::ShowNotifyInDetailsView(FName NotifyName)
{
	// @todo nothing to show now, but in the future
	// we can show the list of montage that are used by this slot node?
}

void SSkeletonSlotNames::GetCompatibleAnimMontages(TArray<struct FAssetData>& OutAssets)
{
	//Get the skeleton tag to search for
	const USkeleton& Skeleton = EditableSkeleton->GetSkeleton();
	FString SkeletonExportName = FAssetData(&Skeleton).GetExportTextName();

	// Load the asset registry module
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

	TArray<FAssetData> AssetDataList;
	AssetRegistryModule.Get().GetAssetsByClass(UAnimMontage::StaticClass()->GetClassPathName(), AssetDataList, true);

	OutAssets.Empty(AssetDataList.Num());

	for( int32 AssetIndex = 0; AssetIndex < AssetDataList.Num(); ++AssetIndex )
	{
		const FAssetData& PossibleAnimMontage = AssetDataList[AssetIndex];
		if( SkeletonExportName == PossibleAnimMontage.GetTagValueRef<FString>("Skeleton") )
		{
			OutAssets.Add( PossibleAnimMontage );
		}
	}
}

UObject* SSkeletonSlotNames::ShowInDetailsView( UClass* EdClass )
{
	UObject *Obj = EditorObjectTracker.GetEditorObjectForClass(EdClass);

	if(Obj != NULL)
	{
		OnObjectSelected.ExecuteIfBound(Obj);
	}
	return Obj;
}

void SSkeletonSlotNames::ClearDetailsView()
{
	OnObjectSelected.ExecuteIfBound(nullptr);
}

void SSkeletonSlotNames::PostUndoRedo()
{
	RefreshSlotNameListWithFilter();
}

void SSkeletonSlotNames::AddReferencedObjects( FReferenceCollector& Collector )
{
	EditorObjectTracker.AddReferencedObjects(Collector);
}

void SSkeletonSlotNames::NotifyUser( FNotificationInfo& NotificationInfo )
{
	TSharedPtr<SNotificationItem> Notification = FSlateNotificationManager::Get().AddNotification( NotificationInfo );
	if ( Notification.IsValid() )
	{
		Notification->SetCompletionState( SNotificationItem::CS_Fail );
	}
}

void SSkeletonSlotNames::OnDeleteSlot(FName SlotName)
{
	TArray<FAssetData> CompatibleMontages;
	TMultiMap<UAnimBlueprint*, UAnimGraphNode_Slot*> CompatibleSlotNodes;
	GetMontagesAndNodesUsingSlot(SlotName, CompatibleMontages, CompatibleSlotNodes);

	if(CompatibleMontages.Num() > 0 || CompatibleSlotNodes.Num() > 0)
	{
		// We can't delete here - still have references. Give the user a chance to fix.
		if(!ReferenceWindow.IsValid())
		{
			// No existing window
			SAssignNew(ReferenceWindow, SWindow)
				.AutoCenter(EAutoCenter::PreferredWorkArea)
				.SizingRule(ESizingRule::Autosized)
				.Title(LOCTEXT("ReferenceWindowTitle", "Slot References"));

			ReferenceWindow->SetContent
				(
				SNew(SBorder)
				.Padding(FMargin(3))
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				[
					SAssignNew(ReferenceWidget, SSlotNameReferenceWindow)
					.ReferencingMontages(&CompatibleMontages)
					.ReferencingNodes(&CompatibleSlotNodes)
					.SlotName(SlotName.ToString())
					.OperationText(LOCTEXT("DeleteOperation", "Delete"))
					.WidgetWindow(ReferenceWindow)
					.OnRetry(FSimpleDelegate::CreateSP(this, &SSkeletonSlotNames::RetryDeleteSlot, SlotName))
				]
			);

			TSharedPtr<SWindow> ParentWindow;
			IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>(TEXT("MainFrame"));
			ParentWindow = MainFrameModule.GetParentWindow();
			
			FSlateApplication::Get().AddWindowAsNativeChild(ReferenceWindow.ToSharedRef(), ParentWindow.ToSharedRef());
			ReferenceWindow->SetOnWindowClosed(FOnWindowClosed::CreateSP(this, &SSkeletonSlotNames::ReferenceWindowClosed));
		}
		else
		{
			TSharedPtr<SSlotNameReferenceWindow> RefWidgetPinned = ReferenceWidget.Pin();
			if(RefWidgetPinned.IsValid())
			{
				FReferenceWindowInfo WindowInfo;
				WindowInfo.ReferencingMontages = &CompatibleMontages;
				WindowInfo.ReferencingNodes = &CompatibleSlotNodes;
				WindowInfo.ItemText = FText::FromName(SlotName);
				WindowInfo.OperationText = LOCTEXT("DeleteOperation", "Delete");
				WindowInfo.RetryDelegate = FSimpleDelegate::CreateSP(this, &SSkeletonSlotNames::RetryDeleteSlot, SlotName);

				RefWidgetPinned->UpdateInfo(WindowInfo);
				ReferenceWindow->BringToFront();
			}
		}
	}
	else
	{
		DeleteSlot(SlotName);
	}
}

void SSkeletonSlotNames::DeleteSlot(const FName& SlotName)
{
	const USkeleton& Skeleton = EditableSkeleton->GetSkeleton();
	if(Skeleton.ContainsSlotName(SlotName))
	{
		EditableSkeleton->DeleteSlotName(SlotName);
		RefreshSlotNameListWithFilter();
	}
}

void SSkeletonSlotNames::GetAnimMontagesUsingSlot(FName SlotName, TArray<FAssetData>& OutMontages)
{
	TArray<FAssetData> SkeletonCompatibleMontages;
	GetCompatibleAnimMontages(SkeletonCompatibleMontages);

	for(FAssetData& MontageData : SkeletonCompatibleMontages)
	{
		UAnimMontage* Montage = Cast<UAnimMontage>(MontageData.GetAsset());

		check(Montage);

		for(FSlotAnimationTrack& SlotTrack : Montage->SlotAnimTracks)
		{
			if(SlotTrack.SlotName == SlotName)
			{
				OutMontages.Add(MontageData);
			}
		}
	}
}

void SSkeletonSlotNames::GetAnimMontagesUsingSlotGroup(FName SlotGroupName, TArray<FAssetData>& OutMontages)
{
	if(const FAnimSlotGroup* Group = EditableSkeleton->GetSkeleton().FindAnimSlotGroup(SlotGroupName))
	{
		for(const FName& SlotName : Group->SlotNames)
		{
			GetAnimMontagesUsingSlot(SlotName, OutMontages);
		}
	}
}

void SSkeletonSlotNames::ReferenceWindowClosed(const TSharedRef<SWindow>& Window)
{
	ReferenceWindow = nullptr;
}

void SSkeletonSlotNames::RetryDeleteSlot(FName SlotName)
{
	TArray<FAssetData> CompatibleMontages;
	TMultiMap<UAnimBlueprint*, UAnimGraphNode_Slot*> CompatibleSlotNodes;
	GetMontagesAndNodesUsingSlot(SlotName, CompatibleMontages, CompatibleSlotNodes);
	
	if(CompatibleMontages.Num() > 0 || CompatibleSlotNodes.Num() > 0)
	{
		// Still can't delete
		TSharedPtr<SSlotNameReferenceWindow> PinnedWidget = ReferenceWidget.Pin();
		if(PinnedWidget.IsValid())
		{
			FReferenceWindowInfo WindowInfo;
			WindowInfo.ReferencingMontages = &CompatibleMontages;
			WindowInfo.ReferencingNodes = &CompatibleSlotNodes;
			WindowInfo.ItemText = FText::FromName(SlotName);
			WindowInfo.OperationText = LOCTEXT("DeleteOperation", "Delete");
			WindowInfo.RetryDelegate = FSimpleDelegate::CreateSP(this, &SSkeletonSlotNames::RetryDeleteSlot, SlotName);

			PinnedWidget->UpdateInfo(WindowInfo);
			ReferenceWindow->BringToFront();
		}
	}
	else
	{
		ReferenceWindow->RequestDestroyWindow();
		DeleteSlot(SlotName);
	}
}

void SSkeletonSlotNames::GetMontagesAndNodesUsingSlot(const FName& SlotName, TArray<FAssetData>& CompatibleMontages, TMultiMap<UAnimBlueprint*, UAnimGraphNode_Slot*> &CompatibleSlotNodes)
{
	FScopedSlowTask SlowTask(3, LOCTEXT("AssetReferenceSlowTaskMessage", "Checking for slot references."));
	SlowTask.MakeDialog();

	TArray<FAssetData> CompatibleBlueprints;
	SlowTask.EnterProgressFrame(1, LOCTEXT("AssetReferenceTask_Blueprints", "Searching Blueprints"));
	GetCompatibleAnimBlueprints(CompatibleBlueprints);

	SlowTask.EnterProgressFrame(1, LOCTEXT("AssetReferenceTask_Montages", "Searching Montages"));
	GetAnimMontagesUsingSlot(SlotName, CompatibleMontages);

	SlowTask.EnterProgressFrame(1, LOCTEXT("AssetReferenceTask_Nodes", "Searching Nodes"));
	for(FAssetData& Data : CompatibleBlueprints)
	{
		TArray<UEdGraph*> BPGraphs;
		UAnimBlueprint* AnimBP = Cast<UAnimBlueprint>(Data.GetAsset());

		AnimBP->GetAllGraphs(BPGraphs);
		for(UEdGraph* Graph : BPGraphs)
		{
			TArray<UAnimGraphNode_Slot*> SlotNodes;
			Graph->GetNodesOfClass(SlotNodes);

			for(UAnimGraphNode_Slot* SlotNode : SlotNodes)
			{
				if(SlotNode->Node.SlotName == SlotName)
				{
					CompatibleSlotNodes.Add(AnimBP, SlotNode);
				}
			}
		}
	}

	// If we end up loading in any previously unsaved assets they can add names to the list so refresh
	RefreshSlotNameListWithFilter();
}

void SSkeletonSlotNames::GetMontagesAndNodesUsingSlotGroup(const FName& SlotGroupName, TArray<FAssetData>& OutMontages, TMultiMap<UAnimBlueprint*, UAnimGraphNode_Slot*> &OutBlueprintSlotMap)
{
	if(const FAnimSlotGroup* SlotGroup = EditableSkeleton->GetSkeleton().FindAnimSlotGroup(SlotGroupName))
	{
		for(const auto& SlotName : SlotGroup->SlotNames)
		{
			GetMontagesAndNodesUsingSlot(SlotName, OutMontages, OutBlueprintSlotMap);
		}
	}
}

void SSkeletonSlotNames::OnRenameSlotPopupCommitted(const FText & InNewSlotText, ETextCommit::Type CommitInfo, FName OldName)
{
	if(CommitInfo == ETextCommit::OnEnter)
	{
		FName NewName(*InNewSlotText.ToString());

		// Need to dismiss menus early or the slow task in GetMontagesAndNodesUsingSlot will fail to show onscreen
		FSlateApplication::Get().DismissAllMenus();

		// Make sure the name doesn't already exist
		if(EditableSkeleton->GetSkeleton().ContainsSlotName(NewName))
		{
			FNotificationInfo Notification(FText::Format(LOCTEXT("ToastRenameFailDesc", "Rename Failed! Slot name {0} already exists in the target skeleton."), FText::FromName(NewName)));
			Notification.ExpireDuration = 3.0f;
			Notification.bFireAndForget = true;

			NotifyUser(Notification);

			return;
		}

		// Validate references
		TArray<FAssetData> CompatibleMontages;
		TMultiMap<UAnimBlueprint*, UAnimGraphNode_Slot*> CompatibleSlotNodes;
		GetMontagesAndNodesUsingSlot(OldName, CompatibleMontages, CompatibleSlotNodes);

		if(CompatibleMontages.Num() > 0 || CompatibleSlotNodes.Num() > 0)
		{
			// We can't rename here - still have references. Give the user a chance to fix.
			if(!ReferenceWindow.IsValid())
			{
				// No existing window
				SAssignNew(ReferenceWindow, SWindow)
					.AutoCenter(EAutoCenter::PreferredWorkArea)
					.SizingRule(ESizingRule::Autosized)
					.Title(LOCTEXT("ReferenceWindowTitle", "Slot References"));

				ReferenceWindow->SetContent
					(
					SNew(SBorder)
					.Padding(FMargin(3))
					.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
					[
						SAssignNew(ReferenceWidget, SSlotNameReferenceWindow)
						.ReferencingMontages(&CompatibleMontages)
						.ReferencingNodes(&CompatibleSlotNodes)
						.SlotName(OldName.ToString())
						.OperationText(LOCTEXT("RenameOperation", "Rename"))
						.WidgetWindow(ReferenceWindow)
						.OnRetry(FSimpleDelegate::CreateSP(this, &SSkeletonSlotNames::RetryRenameSlot, OldName, NewName))
					]
				);

				TSharedPtr<SWindow> ParentWindow;
				IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>(TEXT("MainFrame"));
				ParentWindow = MainFrameModule.GetParentWindow();

				FSlateApplication::Get().AddWindowAsNativeChild(ReferenceWindow.ToSharedRef(), ParentWindow.ToSharedRef());
				ReferenceWindow->SetOnWindowClosed(FOnWindowClosed::CreateSP(this, &SSkeletonSlotNames::ReferenceWindowClosed));
			}
			else
			{
				TSharedPtr<SSlotNameReferenceWindow> RefWidgetPinned = ReferenceWidget.Pin();
				if(RefWidgetPinned.IsValid())
				{
					FReferenceWindowInfo WindowInfo;
					WindowInfo.ReferencingMontages = &CompatibleMontages;
					WindowInfo.ReferencingNodes = &CompatibleSlotNodes;
					WindowInfo.ItemText = FText::FromName(OldName);
					WindowInfo.OperationText = LOCTEXT("RenameOperation", "Rename");
					WindowInfo.RetryDelegate = FSimpleDelegate::CreateSP(this, &SSkeletonSlotNames::RetryRenameSlot, OldName, NewName);

					RefWidgetPinned->UpdateInfo(WindowInfo);
					ReferenceWindow->BringToFront();
				}
			}
		}
		else
		{
			RenameSlot(OldName, NewName);
		}
	}
}

void SSkeletonSlotNames::OnRenameSlot(FName CurrentName)
{
	TSharedRef<STextEntryPopup> TextEntry =
		SNew(STextEntryPopup)
		.Label(LOCTEXT("RenameSlotName_AskSlotName", "New Slot Name"))
		.OnTextCommitted(this, &SSkeletonSlotNames::OnRenameSlotPopupCommitted, CurrentName);

	// Show dialog to enter new track name
	FSlateApplication::Get().PushMenu(
		SharedThis(this),
		FWidgetPath(),
		TextEntry,
		FSlateApplication::Get().GetCursorPos(),
		FPopupTransitionEffect(FPopupTransitionEffect::TypeInPopup)
		);
}

void SSkeletonSlotNames::RenameSlot(FName CurrentName, FName NewName)
{
	if(EditableSkeleton->GetSkeleton().ContainsSlotName(CurrentName))
	{
		EditableSkeleton->RenameSlotName(CurrentName, NewName);
		RefreshSlotNameListWithFilter();
	}
}

void SSkeletonSlotNames::RetryRenameSlot(FName CurrentName, FName NewName)
{
	TArray<FAssetData> CompatibleMontages;
	TMultiMap<UAnimBlueprint*, UAnimGraphNode_Slot*> CompatibleSlotNodes;
	GetMontagesAndNodesUsingSlot(CurrentName, CompatibleMontages, CompatibleSlotNodes);

	if(CompatibleMontages.Num() > 0 || CompatibleSlotNodes.Num() > 0)
	{
		// Still can't rename
		TSharedPtr<SSlotNameReferenceWindow> PinnedWidget = ReferenceWidget.Pin();
		if(PinnedWidget.IsValid())
		{
			FReferenceWindowInfo WindowInfo;
			WindowInfo.ReferencingMontages = &CompatibleMontages;
			WindowInfo.ReferencingNodes = &CompatibleSlotNodes;
			WindowInfo.ItemText = FText::FromName(CurrentName);
			WindowInfo.OperationText = LOCTEXT("DeleteOperation", "Delete");
			WindowInfo.RetryDelegate = FSimpleDelegate::CreateSP(this, &SSkeletonSlotNames::RetryRenameSlot, CurrentName, NewName);

			PinnedWidget->UpdateInfo(WindowInfo);
			ReferenceWindow->BringToFront();
		}
	}
	else
	{
		ReferenceWindow->RequestDestroyWindow();
		RenameSlot(CurrentName, NewName);
	}
}

void SSkeletonSlotNames::OnDeleteSlotGroup(FName GroupName)
{
	TArray<FAssetData> CompatibleMontages;
	TMultiMap<UAnimBlueprint*, UAnimGraphNode_Slot*> CompatibleSlotNodes;
	GetMontagesAndNodesUsingSlotGroup(GroupName, CompatibleMontages, CompatibleSlotNodes);

	if(CompatibleMontages.Num() > 0 || CompatibleSlotNodes.Num() > 0)
	{
		// Can't delete, still referenced
		// We can't rename here - still have references. Give the user a chance to fix.
		if(!ReferenceWindow.IsValid())
		{
			// No existing window
			SAssignNew(ReferenceWindow, SWindow)
				.AutoCenter(EAutoCenter::PreferredWorkArea)
				.SizingRule(ESizingRule::Autosized)
				.Title(LOCTEXT("ReferenceWindowTitle", "Slot References"));

			ReferenceWindow->SetContent
				(
				SNew(SBorder)
				.Padding(FMargin(3))
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				[
					SAssignNew(ReferenceWidget, SSlotNameReferenceWindow)
					.ReferencingMontages(&CompatibleMontages)
					.ReferencingNodes(&CompatibleSlotNodes)
					.SlotName(GroupName.ToString())
					.OperationText(LOCTEXT("DeleteGroupOperation", "Delete Group"))
					.WidgetWindow(ReferenceWindow)
					.OnRetry(FSimpleDelegate::CreateSP(this, &SSkeletonSlotNames::RetryDeleteSlotGroup, GroupName))
				]
			);

			TSharedPtr<SWindow> ParentWindow;
			IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>(TEXT("MainFrame"));
			ParentWindow = MainFrameModule.GetParentWindow();

			FSlateApplication::Get().AddWindowAsNativeChild(ReferenceWindow.ToSharedRef(), ParentWindow.ToSharedRef());
			ReferenceWindow->SetOnWindowClosed(FOnWindowClosed::CreateSP(this, &SSkeletonSlotNames::ReferenceWindowClosed));
		}
		else
		{
			TSharedPtr<SSlotNameReferenceWindow> RefWidgetPinned = ReferenceWidget.Pin();
			if(RefWidgetPinned.IsValid())
			{
				FReferenceWindowInfo WindowInfo;
				WindowInfo.ReferencingMontages = &CompatibleMontages;
				WindowInfo.ReferencingNodes = &CompatibleSlotNodes;
				WindowInfo.ItemText = FText::FromName(GroupName);
				WindowInfo.OperationText = LOCTEXT("DeleteGroupOperation", "Delete Group");
				WindowInfo.RetryDelegate = FSimpleDelegate::CreateSP(this, &SSkeletonSlotNames::RetryDeleteSlotGroup, GroupName);

				RefWidgetPinned->UpdateInfo(WindowInfo);
				ReferenceWindow->BringToFront();
			}
		}
	}
	else
	{
		DeleteSlotGroup(GroupName);
	}
}

void SSkeletonSlotNames::RetryDeleteSlotGroup(FName GroupName)
{
	TArray<FAssetData> CompatibleMontages;
	TMultiMap<UAnimBlueprint*, UAnimGraphNode_Slot*> CompatibleSlotNodes;
	GetMontagesAndNodesUsingSlotGroup(GroupName, CompatibleMontages, CompatibleSlotNodes);

	if(CompatibleMontages.Num() > 0 || CompatibleSlotNodes.Num() > 0)
	{
		// Still can't rename
		TSharedPtr<SSlotNameReferenceWindow> PinnedWidget = ReferenceWidget.Pin();
		if(PinnedWidget.IsValid())
		{
			FReferenceWindowInfo WindowInfo;
			WindowInfo.ReferencingMontages = &CompatibleMontages;
			WindowInfo.ReferencingNodes = &CompatibleSlotNodes;
			WindowInfo.ItemText = FText::FromName(GroupName);
			WindowInfo.OperationText = LOCTEXT("DeleteGroupOperation", "Delete Group");
			WindowInfo.RetryDelegate = FSimpleDelegate::CreateSP(this, &SSkeletonSlotNames::RetryDeleteSlotGroup, GroupName);

			PinnedWidget->UpdateInfo(WindowInfo);
			ReferenceWindow->BringToFront();
		}
	}
	else
	{
		ReferenceWindow->RequestDestroyWindow();
		DeleteSlotGroup(GroupName);
	}
}

bool SSkeletonSlotNames::CanDeleteSlotGroup(FName GroupName)
{
	static const FName DefaultGroupName("DefaultGroup");
	return GroupName != DefaultGroupName;
}

void SSkeletonSlotNames::DeleteSlotGroup(const FName& GroupName)
{
	if(EditableSkeleton->GetSkeleton().FindAnimSlotGroup(GroupName) != nullptr)
	{
		EditableSkeleton->DeleteSlotGroup(GroupName);
		RefreshSlotNameListWithFilter();
	}
}

#undef LOCTEXT_NAMESPACE
