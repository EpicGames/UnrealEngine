// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "UObject/GCObject.h"
#include "Widgets/SWindow.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "WorkflowOrientedApp/WorkflowTabFactory.h"
#include "PersonaDelegates.h"
#include "AssetRegistry/AssetData.h"
#include "IDocumentation.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STreeView.h"
#include "EditorObjectsTracker.h"
#include "EditorUndoClient.h"

class FMenuBuilder;
class IEditableSkeleton;
class SSlotNameReferenceWindow;
class SToolTip;
class UAnimBlueprint;
class UAnimGraphNode_Slot;
struct FNotificationInfo;

#define LOCTEXT_NAMESPACE "SkeletonSlotNames"
/////////////////////////////////////////////////////
// FSkeletonSlotNamesSummoner
struct FSkeletonSlotNamesSummoner : public FWorkflowTabFactory
{
public:
	FSkeletonSlotNamesSummoner(TSharedPtr<class FAssetEditorToolkit> InHostingApp, const TSharedRef<class IEditableSkeleton>& InEditableSkeleton, FOnObjectSelected InOnObjectSelected);

	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;

	// Create a tooltip widget for the tab
	virtual TSharedPtr<SToolTip> CreateTabToolTipWidget(const FWorkflowTabSpawnInfo& Info) const override
	{
		return  IDocumentation::Get()->CreateToolTip(LOCTEXT("WindowTooltip", "This tab lets you modify custom animation SlotName"), NULL, TEXT("Shared/Editors/Persona"), TEXT("AnimationSlotName_Window"));
	}
public:
	TWeakPtr<class IEditableSkeleton> EditableSkeleton;
	FOnObjectSelected OnObjectSelected;
};

//////////////////////////////////////////////////////////////////////////
// FDisplayedSlotNameInfo

class FDisplayedSlotNameInfo
{
public:
	FName Name;

	bool bIsGroupItem;

	TArray< TSharedPtr<FDisplayedSlotNameInfo> > Children;

	/** Handle to editable text block for rename */
	TSharedPtr<SInlineEditableTextBlock> InlineEditableText;

	/** Static function for creating a new item, but ensures that you can only have a TSharedRef to one */
	static TSharedRef<FDisplayedSlotNameInfo> Make(const FName& InItemName, bool InbIsGroupItem)
	{
		return MakeShareable(new FDisplayedSlotNameInfo(InItemName, InbIsGroupItem));
	}

protected:
	/** Hidden constructor, always use Make above */
	FDisplayedSlotNameInfo(const FName& InItemName, bool InbIsGroupItem)
		: Name(InItemName)
		, bIsGroupItem(InbIsGroupItem)
	{}

	/** Hidden constructor, always use Make above */
	FDisplayedSlotNameInfo() {}
};

/** Widgets list type */
typedef STreeView< TSharedPtr<FDisplayedSlotNameInfo> > SSlotNameListType;

class SSkeletonSlotNames : public SCompoundWidget, public FGCObject, public FSelfRegisteringEditorUndoClient
{
public:
	SLATE_BEGIN_ARGS( SSkeletonSlotNames )
	{}

	SLATE_ARGUMENT(FOnObjectSelected, OnObjectSelected)

	SLATE_END_ARGS()
public:
	void Construct(const FArguments& InArgs, const TSharedRef<IEditableSkeleton>& InEditableSkeleton);

	/**
	* Accessor so our rows can grab the filter text for highlighting
	*
	*/
	FText& GetFilterText() { return FilterText; }

	/** Creates an editor object from the given type to be used in a details panel */
	UObject* ShowInDetailsView( UClass* EdClass );

	/** Clears the detail view of whatever we displayed last */
	void ClearDetailsView();

	// FSelfRegisteringEditorUndoClient interface
	virtual void PostUndo(bool bSuccess) override { PostUndoRedo(); }
	virtual void PostRedo(bool bSuccess) override { PostUndoRedo(); }

	/** This triggers a UI repopulation after undo has been called */
	void PostUndoRedo();

	// FGCObject interface start
	virtual void AddReferencedObjects( FReferenceCollector& Collector ) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("SSkeletonSlotNames");
	}
	// FGCObject interface end

private:

	/** Called when the user changes the contents of the search box */
	void OnFilterTextChanged( const FText& SearchText );

	/** Called when the user changes the contents of the search box */
	void OnFilterTextCommitted( const FText& SearchText, ETextCommit::Type CommitInfo );

	/** Delegate handler for generating rows in SlotNameListView */ 
	TSharedRef<ITableRow> GenerateNotifyRow( TSharedPtr<FDisplayedSlotNameInfo> InInfo, const TSharedRef<STableViewBase>& OwnerTable );

	/** Get all children for a given entry in the list */
	void GetChildrenForInfo(TSharedPtr<FDisplayedSlotNameInfo> InInfo, TArray< TSharedPtr<FDisplayedSlotNameInfo> >& OutChildren);

	/** Delegate handler called when the user right clicks in SlotNameListView */
	TSharedPtr<SWidget> OnGetContextMenuContent() const;

	/** Delegate handler for when the user selects something in SlotNameListView */
	void OnNotifySelectionChanged( TSharedPtr<FDisplayedSlotNameInfo> Selection, ESelectInfo::Type SelectInfo );

	// Save Skeleton
	void OnSaveSkeleton();

	// Add a new Slot
	void OnAddSlot();
	void AddSlotPopUpOnCommit(const FText& InNewSlotName, ETextCommit::Type CommitInfo);

	// Add a new Group
	void OnAddGroup();
	void AddGroupPopUpOnCommit(const FText& InNewGroupName, ETextCommit::Type CommitInfo);

	// Delete a slot after checking slot name references and prompting the user to resolve
	void OnDeleteSlot(FName SlotName);
	// Delete a slot immediately without checking references. Call only if sure the name is unreferenced as
	// it will be re-added on next load if it is.
	void DeleteSlot(const FName& SlotName);
	// Retry the validation for deleting a slot
	void RetryDeleteSlot(FName SlotName);

	// Delete a slot group after checking slot name references and prompting the user to resolve
	void OnDeleteSlotGroup(FName GroupName);
	// Delete a slot group immediately without checking references.
	void DeleteSlotGroup(const FName& GroupName);
	// Retry the validation for deleting a slot group
	void RetryDeleteSlotGroup(FName GroupName);
	// Context menu hook for checking whether delete group is enabled
	bool CanDeleteSlotGroup(FName GroupName);

	// Attempt to rename a slot after a name has been given - validating the new name and old references first
	void OnRenameSlotPopupCommitted(const FText& InNewSlotText, ETextCommit::Type CommitInfo, FName OldName);
	// Spawn popup text box for user to enter name
	void OnRenameSlot(FName CurrentName);
	// Rename a slot immediately without checking references
	void RenameSlot(FName CurrentName, FName NewName);
	// Retry the validation for renaming a slot
	void RetryRenameSlot(FName CurrentName, FName NewName);

	// Set Slot Group
	void FillSetSlotGroupSubMenu(FMenuBuilder& MenuBuilder);
	void ContextMenuOnSetSlot(FName InNewGroupName);

	/** Wrapper that populates SlotNameListView using current filter test */
	void RefreshSlotNameListWithFilter();

	TSharedPtr< FDisplayedSlotNameInfo > FindItemNamed(FName ItemName) const;

	/** Populates SlotNameListView based on the skeletons SlotName and the supplied filter text */
	void CreateSlotNameList( const FString& SearchText = FString("") );

	/** handler for user selecting a Notify in SlotNameListView - populates the details panel */
	void ShowNotifyInDetailsView( FName NotifyName );

	/** Populates OutAssets with the AnimSequences that match Personas current skeleton */
	void GetCompatibleAnimMontages( TArray<struct FAssetData>& OutAssets );
	/** Populates OutAssets with the Anim Blueprints that match Personas current skeleton */
	void GetCompatibleAnimBlueprints( TArray<FAssetData>& OutAssets );

	// Get all montages that have an anim track using the given slot
	void GetAnimMontagesUsingSlot(FName SlotName, TArray<FAssetData>& OutMontages);
	// Get all montages that have an anim track using the given slot group
	void GetAnimMontagesUsingSlotGroup(FName SlotGroupName, TArray<FAssetData>& OutMontages);

	// Get all montages using the given slot name and a map of all blueprints/nodes also using the slot name
	void GetMontagesAndNodesUsingSlot(const FName& SlotName, TArray<FAssetData>& CompatibleMontages, TMultiMap<UAnimBlueprint*, UAnimGraphNode_Slot*>& CompatibleSlotNodes);
	// Get all montages using the given slot group and a map of all blueprints/nodes also using the slot group
	void GetMontagesAndNodesUsingSlotGroup(const FName& SlotGroupName, TArray<FAssetData>& OutMontages, TMultiMap<UAnimBlueprint*, UAnimGraphNode_Slot*> &OutBlueprintSlotMap);

	/** Utility function to display notifications to the user */
	void NotifyUser( FNotificationInfo& NotificationInfo );

	// Callback for popup reference window closing
	void ReferenceWindowClosed(const TSharedRef<SWindow>& Window);

	/** The skeleton we are currently editing */
	TSharedPtr<class IEditableSkeleton> EditableSkeleton;

	/** Delegate call to select an object & display its details */
	FOnObjectSelected OnObjectSelected;

	/** SSearchBox to filter the notify list */
	TSharedPtr<SSearchBox>	NameFilterBox;

	/** Widget used to display the list of SlotName */
	TSharedPtr<SSlotNameListType> SlotNameListView;

	/** A list of SlotName. Used by the SlotNameListView. */
	TArray< TSharedPtr<FDisplayedSlotNameInfo> > NotifyList;

	/** Current text typed into NameFilterBox */
	FText FilterText;

	/** Tracks objects created for displaying in the details panel*/
	FEditorObjectTracker EditorObjectTracker;

	/** Stores the window we spawn to notify the user that references exist on deletion */
	TSharedPtr<SWindow> ReferenceWindow;

	/** The actual custom widget inside ReferenceWindow */
	TWeakPtr<SSlotNameReferenceWindow> ReferenceWidget;
};

#undef LOCTEXT_NAMESPACE
