// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSkinWeightProfileImportOptions.h"

#include "PropertyEditorModule.h"
#include "IDetailsView.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SButton.h"
#include "PropertyHandle.h"
#include "DetailLayoutBuilder.h"
#include "Rendering/SkinWeightProfile.h"
#include "DetailWidgetRow.h"
#include "Engine/SkeletalMesh.h"
#include "UObject/UnrealNames.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SSkinWeightProfileImportOptions)

#define LOCTEXT_NAMESPACE "SkinWeightProfileImportOptions"

void FSkinWeightImportOptionsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TSharedPtr<IPropertyHandle> LODIndexHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(USkinWeightImportOptions, LODIndex));
	
	ProfileNameHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(USkinWeightImportOptions, ProfileName));
	DetailBuilder.HideProperty(ProfileNameHandle);

	// Ensure we do not import a profile with a name that is already in use
	UpdateNameRestriction();	

	FDetailWidgetRow& Row = DetailBuilder.AddCustomRowToCategory(ProfileNameHandle, ProfileNameHandle->GetPropertyDisplayName())
	.NameContent()
	[
		ProfileNameHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		SAssignNew(NameEditTextBox, SEditableTextBox)
		.Text(this, &FSkinWeightImportOptionsCustomization::OnGetProfileName)
		.OnTextChanged(this, &FSkinWeightImportOptionsCustomization::OnProfileNameChanged)
		.OnTextCommitted(this, &FSkinWeightImportOptionsCustomization::OnProfileNameCommitted)
		.Font(DetailBuilder.GetDetailFont())
	];

	// Force an update to check the previous/default value
	OnProfileNameChanged(OnGetProfileName());

	LODIndexHandle->SetInstanceMetaData(FName("UIMin"), FString::FromInt(0));
	LODIndexHandle->SetInstanceMetaData(FName("ClampMin"), FString::FromInt(0));

	if (USkeletalMesh* SkeletalMesh = WeakSkeletalMesh.Get())
	{
		LODIndexHandle->SetInstanceMetaData(FName("UIMax"), FString::FromInt(SkeletalMesh->GetLODNum() - 1));
		LODIndexHandle->SetInstanceMetaData(FName("ClampMax"), FString::FromInt(SkeletalMesh->GetLODNum() - 1));
	}
}

void FSkinWeightImportOptionsCustomization::UpdateNameRestriction()
{
	if (USkeletalMesh* SkeletalMesh = WeakSkeletalMesh.Get())
	{
		RestrictedNames.Empty();
		RestrictedNames.Add(FName(NAME_None).ToString());
		const TArray<FSkinWeightProfileInfo>& Profiles = SkeletalMesh->GetSkinWeightProfiles();
		for (const FSkinWeightProfileInfo& ProfileInfo : Profiles)
		{
			RestrictedNames.Add(ProfileInfo.Name.ToString());
		}
	}
}

FText FSkinWeightImportOptionsCustomization::OnGetProfileName() const
{
	FString Value;
	if (ProfileNameHandle->GetValue(Value) == FPropertyAccess::Success)
	{
		return FText::FromString(Value);
	}
	return FText::GetEmpty();
}

void FSkinWeightImportOptionsCustomization::OnProfileNameChanged(const FText& InNewText)
{
	const bool bValidProfileName = IsProfileNameValid(InNewText.ToString());
	if (!bValidProfileName)
	{
		NameEditTextBox->SetError(LOCTEXT("OnProfileNameChanged_NotValid", "This name is already in use"));
	}
	else
	{
		NameEditTextBox->SetError(FText::GetEmpty());
	}
}

void FSkinWeightImportOptionsCustomization::OnProfileNameCommitted(const FText& InNewText, ETextCommit::Type InTextCommit)
{
	if (IsProfileNameValid(InNewText.ToString()) && InTextCommit != ETextCommit::OnCleared)
	{		
		ProfileNameHandle->SetValue(InNewText.ToString());		
	}
}

const bool FSkinWeightImportOptionsCustomization::IsProfileNameValid(const FString& NewName) const
{	
	return !RestrictedNames.Contains(NewName) && !NewName.IsEmpty();
}

void SSkinWeightProfileImportOptions::Construct(const FArguments& InArgs)
{
	ImportSettings = InArgs._ImportSettings;
	WidgetWindow = InArgs._WidgetWindow;
	SkeletalMesh = InArgs._SkeletalMesh;
	DetailsCustomization = MakeShareable(new FSkinWeightImportOptionsCustomization(SkeletalMesh));

	check(ImportSettings);

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	DetailsView->RegisterInstancedCustomPropertyLayout(USkinWeightImportOptions::StaticClass(), FOnGetDetailCustomizationInstance::CreateLambda([this]() { return DetailsCustomization->AsShared(); }));
	DetailsView->SetObject(ImportSettings);
	
	this->ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.Padding(2)
		.MaxHeight(500.0f)
		[
			DetailsView->AsShared()
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Right)
		.Padding(2.f)
		[
			SNew(SUniformGridPanel)
			.SlotPadding(2.f)
			+ SUniformGridPanel::Slot(0, 0)
			[
				SAssignNew(ImportButton, SButton)
				.HAlign(HAlign_Center)
				.Text(LOCTEXT("SkinWeightProfileOptionWindow_Import", "Import"))
				.OnClicked(this, &SSkinWeightProfileImportOptions::OnImport)
				.IsEnabled_Lambda([this]() -> bool 
				{
					const bool bValidProfileName = DetailsCustomization->IsProfileNameValid(DetailsCustomization->OnGetProfileName().ToString());
					return bValidProfileName;
				})
			]
			+ SUniformGridPanel::Slot(1, 0)
			[
				SNew(SButton)
				.HAlign(HAlign_Center)
				.Text(LOCTEXT("SkinWeightProfileOptionWindow_Cancel", "Cancel"))
				.ToolTipText(LOCTEXT("SkinWeightProfileOptionWindow_Cancel_ToolTip", "Cancels importing this SkinWeightProfile file"))
				.OnClicked(this, &SSkinWeightProfileImportOptions::OnCancel)
			]
		]
	];
}


#undef LOCTEXT_NAMESPACE

