// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSlotNameReferenceWindow.h"

#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SScrollBox.h"
#include "AnimGraphNode_Slot.h"
#include "EdGraph/EdGraph.h"
#include "BlueprintEditor.h"
#include "Subsystems/AssetEditorSubsystem.h"

#define LOCTEXT_NAMESPACE "SkeletonSlotNames"

TSharedRef<FDisplayedMontageReferenceInfo> FDisplayedMontageReferenceInfo::Make(const FAssetData& AssetData)
{
	return MakeShareable(new FDisplayedMontageReferenceInfo(AssetData));
}

TSharedRef<FDisplayedBlueprintReferenceInfo> FDisplayedBlueprintReferenceInfo::Make(UAnimBlueprint* Blueprint, UAnimGraphNode_Slot* SlotNode)
{
	return MakeShareable(new FDisplayedBlueprintReferenceInfo(Blueprint, SlotNode));
}

FDisplayedBlueprintReferenceInfo::FDisplayedBlueprintReferenceInfo(UAnimBlueprint* Blueprint, UAnimGraphNode_Slot* SlotNode)
{
	check(Blueprint && SlotNode);
	BlueprintName = Blueprint->GetName();
	GraphName = SlotNode->GetGraph()->GetName();
	NodeName = SlotNode->GetName();

	AnimBlueprint = Blueprint;
	NodeGraph = SlotNode->GetGraph();
	Node = SlotNode;
}

//------------------------------------------------------------------------------------------

void SMontageReferenceListRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
{
	check(InArgs._ReferenceInfo.IsValid());
	ReferenceInfo = InArgs._ReferenceInfo;

	SMultiColumnTableRow<TSharedPtr<FDisplayedMontageReferenceInfo>>::Construct(FSuperRowType::FArguments(), InOwnerTableView);
}

TSharedRef<SWidget> SMontageReferenceListRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	if(ColumnName == TEXT("MontageName"))
	{
		return SNew(SBox)
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock).Text(FText::FromString(ReferenceInfo->AssetData.AssetName.ToString()))
			];
	}
	else if(ColumnName == TEXT("Asset"))
	{
		// Buttons to jump to / view asset
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(0,0,5,0)
			.AutoWidth()
			[
				SNew(SButton)
				.ToolTipText(LOCTEXT("MontageReferenceViewInContentBrowserToolTip", "Highlight this asset in the Content Browser."))
				.OnClicked(this, &SMontageReferenceListRow::OnViewInContentBrowserClicked)
				.DesiredSizeScale(FVector2D(0.75f, 0.75f))
				.Content()
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("Icons.Search"))
				]
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
				.ToolTipText(LOCTEXT("MontageReferenceOpenAssetToolTip", "Open the editor for this asset."))
				.OnClicked(this, &SMontageReferenceListRow::OnOpenAssetClicked)
				.DesiredSizeScale(FVector2D(0.75f, 0.75f))
				.Content()
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("SystemWideCommands.SummonOpenAssetDialog"))
				]
			];
	}

	return SNullWidget::NullWidget;
}

FReply SMontageReferenceListRow::OnViewInContentBrowserClicked()
{
	TArray<FAssetData> Objects;
	Objects.Add(ReferenceInfo->AssetData);
	GEditor->SyncBrowserToObjects(Objects);

	return FReply::Handled();
}

FReply SMontageReferenceListRow::OnOpenAssetClicked()
{
	GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(ReferenceInfo->AssetData.GetAsset());

	return FReply::Handled();
}

//------------------------------------------------------------------------------------------

void SBlueprintReferenceListRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
{
	check(InArgs._ReferenceInfo.IsValid());
	ReferenceInfo = InArgs._ReferenceInfo;

	SMultiColumnTableRow<TSharedPtr<FDisplayedBlueprintReferenceInfo>>::Construct(FSuperRowType::FArguments(), InOwnerTableView);
}

TSharedRef<SWidget> SBlueprintReferenceListRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	if(ColumnName == TEXT("BlueprintName"))
	{
		return SNew(STextBlock).Text(FText::FromString(ReferenceInfo->BlueprintName));
	}
	else if(ColumnName == TEXT("GraphName"))
	{
		return SNew(STextBlock).Text(FText::FromString(ReferenceInfo->GraphName));
	}
	else if(ColumnName == TEXT("NodeName"))
	{
		return SNew(STextBlock).Text(FText::FromString(ReferenceInfo->NodeName));
	}
	else if(ColumnName == TEXT("Asset"))
	{
		// Buttons to jump to / view asset
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0,0,5,0)
			[
				SNew(SButton)
				.ToolTipText(LOCTEXT("MontageReferenceViewInContentBrowserToolTip", "Highlight this asset in the Content Browser."))
				.OnClicked(this, &SBlueprintReferenceListRow::OnViewInContentBrowserClicked)
				.DesiredSizeScale(FVector2D(0.75f, 0.75f))
				.Content()
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("Icons.Search"))
				]
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
				.ToolTipText(LOCTEXT("MontageReferenceOpenAssetToolTip", "Open the editor for this asset."))
				.OnClicked(this, &SBlueprintReferenceListRow::OnOpenAssetClicked)
				.DesiredSizeScale(FVector2D(0.75f, 0.75f))
				.Content()
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("SystemWideCommands.SummonOpenAssetDialog"))
				]
			];
	}

	return SNullWidget::NullWidget;
}

FReply SBlueprintReferenceListRow::OnViewInContentBrowserClicked()
{
	check(ReferenceInfo->AnimBlueprint);
	TArray<UObject*> Objects;
	UObject* BPAsObject = ReferenceInfo->AnimBlueprint;
	Objects.Add(BPAsObject);
	GEditor->SyncBrowserToObjects(Objects);

	return FReply::Handled();
}

FReply SBlueprintReferenceListRow::OnOpenAssetClicked()
{
	GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(ReferenceInfo->AnimBlueprint);
	
	FBlueprintEditor* BPEditor = static_cast<FBlueprintEditor*>(GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(ReferenceInfo->AnimBlueprint, true));
	if(BPEditor)
	{
		// Should only ever get an FPersona object back for an animBP.
		TSharedPtr<SGraphEditor> GraphEditor = BPEditor->OpenGraphAndBringToFront(ReferenceInfo->NodeGraph);
		if(GraphEditor.IsValid())
		{
			// Open the right graph and zoom in on the offending node
			GraphEditor->JumpToNode(ReferenceInfo->Node, false);
		}
	}

	return FReply::Handled();
}

//------------------------------------------------------------------------------------------

void SSlotNameReferenceWindow::Construct(const FArguments& InArgs)
{
	ContainingWindow = InArgs._WidgetWindow;
	
	FReferenceWindowInfo WindowUpdateInfo;
	WindowUpdateInfo.ReferencingMontages = InArgs._ReferencingMontages;
	WindowUpdateInfo.ReferencingNodes = InArgs._ReferencingNodes;
	WindowUpdateInfo.ItemText = FText::FromString(InArgs._SlotName);
	WindowUpdateInfo.OperationText = InArgs._OperationText;
	WindowUpdateInfo.RetryDelegate = InArgs._OnRetry;

	UpdateInfo(WindowUpdateInfo);
}

void SSlotNameReferenceWindow::UpdateInfo(FReferenceWindowInfo& UpdatedInfo)
{
	// Pull any changed data out of the info struct
	if(UpdatedInfo.ReferencingMontages)
	{
		// Sort for nicer display
		UpdatedInfo.ReferencingMontages->Sort([](const FAssetData& A, const FAssetData& B)
		{
			return A.AssetName.LexicalLess(B.AssetName);
		});

		ReferencingMontages.Empty(UpdatedInfo.ReferencingMontages->Num());
		for(FAssetData& Data : *UpdatedInfo.ReferencingMontages)
		{
			ReferencingMontages.Add(FDisplayedMontageReferenceInfo::Make(Data));
		}
	}

	if(UpdatedInfo.ReferencingNodes)
	{
		ReferencingNodes.Empty(UpdatedInfo.ReferencingNodes->Num());
		for(auto& Pair : *UpdatedInfo.ReferencingNodes)
		{
			UAnimBlueprint* AnimBP = Pair.Key;
			UAnimGraphNode_Slot* SlotNode = Pair.Value;
			if(AnimBP && SlotNode)
			{
				ReferencingNodes.Add(FDisplayedBlueprintReferenceInfo::Make(AnimBP, SlotNode));

				ReferencingNodes.Sort([](const TSharedPtr<FDisplayedBlueprintReferenceInfo>& A, const TSharedPtr<FDisplayedBlueprintReferenceInfo>& B)
				{
					return A->BlueprintName < B->BlueprintName;
				});
			}
		}
	}

	if(!UpdatedInfo.OperationText.IsEmpty())
	{
		OperationText = UpdatedInfo.OperationText;
	}

	if(!UpdatedInfo.ItemText.IsEmpty())
	{
		SlotName = UpdatedInfo.ItemText.ToString();
	}

	if(UpdatedInfo.RetryDelegate.IsBound())
	{
		OnRetry = UpdatedInfo.RetryDelegate;
	}

	if(UpdatedInfo.bRefresh)
	{
		RefreshContent();
	}
}

TSharedRef<SWidget> SSlotNameReferenceWindow::GetContent()
{
	TSharedPtr<SWidget> Content;

	SAssignNew(Content, SScrollBox)
	+SScrollBox::Slot()
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.Padding(5.0f)
		.AutoHeight()
		[
			SNew(STextBlock)
			.Text(FText::Format(LOCTEXT("WindowTitle", "{0}: {1}"), OperationText, FText::FromString(SlotName)))
			.Font(FCoreStyle::GetDefaultFontStyle("Bold", 12))
		]
		+SVerticalBox::Slot()
		.Padding(5.0f)
		.AutoHeight()
		[
			// Montage Explanation paragraph
			SNew(STextBlock)
			.AutoWrapText(true)
			.Text(LOCTEXT("SlotReferenceMontageExplanation", "The following montages contain slots that reference the chosen item. Remove or change the referenced item to proceed."))
		]
		+SVerticalBox::Slot()
		.Padding(5.0f)
		.AutoHeight()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.DarkGroupBorder"))
			[
				// List of montages
				SNew(SMontageReferenceList)
				.ListItemsSource(&ReferencingMontages)
				.OnGenerateRow(this, &SSlotNameReferenceWindow::HandleGenerateMontageReferenceRow)
				.SelectionMode(ESelectionMode::None)
				.HeaderRow
				(
				SNew(SHeaderRow)
				+ SHeaderRow::Column("MontageName")
				.DefaultLabel(FText(LOCTEXT("MontageColumnDefaultLabel", "Montage Name")))
				+ SHeaderRow::Column("Asset")
				.DefaultLabel(FText::GetEmpty())
				)
			]
		]
		+SVerticalBox::Slot()
		.Padding(5.0f)
		.AutoHeight()
		[
			// Anim blueprint explanation paragraph
			SNew(STextBlock)
			.AutoWrapText(true)
			.Text(LOCTEXT("SlotReferenceBlueprintExplanation", "The following animation blueprints contain nodes that reference the selected item. Remove or change the nodes listed to proceed"))
		]
		+SVerticalBox::Slot()
		.Padding(5.0f)
		.AutoHeight()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.DarkGroupBorder"))
			[
				// List of montages
				SNew(SBlueprintReferenceList)
				.ListItemsSource(&ReferencingNodes)
				.OnGenerateRow(this, &SSlotNameReferenceWindow::HandleGenerateBlueprintReferenceRow)
				.SelectionMode(ESelectionMode::None)
				.HeaderRow
				(
				SNew(SHeaderRow)
				+ SHeaderRow::Column("BlueprintName")
				.DefaultLabel(FText(LOCTEXT("BlueprintColumnDefaultLabel", "Blueprint Name")))
				+ SHeaderRow::Column("GraphName")
				.DefaultLabel(FText(LOCTEXT("GraphColumnDefaultLabel", "Graph Name")))
				+ SHeaderRow::Column("NodeName")
				.DefaultLabel(FText(LOCTEXT("NodeColumnDefaultLabel", "Node Name")))
				+ SHeaderRow::Column("Asset")
				.DefaultLabel(FText::GetEmpty())
				)
			]
		]
		+ SVerticalBox::Slot()
		.VAlign(VAlign_Bottom)
		.Padding(5.0f)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Center)
			[
				SNew(SButton)
				.Text(LOCTEXT("RetryButton", "Retry"))
				.OnClicked(this, &SSlotNameReferenceWindow::OnRetryClicked)
			]
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Center)
			[
				SNew(SButton)
				.Text(LOCTEXT("CloseButton", "Close"))
				.OnClicked(this, &SSlotNameReferenceWindow::OnCloseClicked)
			]
		]
	];

	return Content->AsShared();
}

void SSlotNameReferenceWindow::RefreshContent()
{
	this->ChildSlot
	[
		GetContent()
	];
}

TSharedRef<ITableRow> SSlotNameReferenceWindow::HandleGenerateMontageReferenceRow(TSharedPtr<FDisplayedMontageReferenceInfo> Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SMontageReferenceListRow, OwnerTable)
			.ReferenceInfo(Item);
}

TSharedRef<ITableRow> SSlotNameReferenceWindow::HandleGenerateBlueprintReferenceRow(TSharedPtr<FDisplayedBlueprintReferenceInfo> Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SBlueprintReferenceListRow, OwnerTable)
			.ReferenceInfo(Item);
}

FReply SSlotNameReferenceWindow::OnRetryClicked()
{
	if(OnRetry.IsBound())
	{
		OnRetry.Execute();
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

FReply SSlotNameReferenceWindow::OnCloseClicked()
{
	if(ContainingWindow.IsValid())
	{
		ContainingWindow->RequestDestroyWindow();
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

#undef LOCTEXT_NAMESPACE
