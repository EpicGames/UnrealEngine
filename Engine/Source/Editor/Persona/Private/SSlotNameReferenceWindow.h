// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "AssetRegistry/AssetData.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "SlotRefWindow"

class SWindow;
class UAnimBlueprint;
class UAnimGraphNode_Base;
class UAnimGraphNode_Slot;
class UEdGraph;

// This type can't exist in a SLATE_ARGUMENT macro so typedef away the template
typedef TMultiMap<UAnimBlueprint*, UAnimGraphNode_Slot*>* NodeMapPtr;

// List display helper for montage references
class FDisplayedMontageReferenceInfo
{
public:
	FAssetData AssetData;

	static TSharedRef<FDisplayedMontageReferenceInfo> Make(const FAssetData& AssetData);

protected:
	/** Hidden constructor, always use Make above */
	FDisplayedMontageReferenceInfo(const FAssetData& InData)
		: AssetData(InData)
	{}

	/** Hidden constructor, always use Make above */
	FDisplayedMontageReferenceInfo()
	{}
};

// List display helper for blueprint references
class FDisplayedBlueprintReferenceInfo
{
public:
	FString BlueprintName;
	FString GraphName;
	FString NodeName;
	UAnimBlueprint* AnimBlueprint;
	UEdGraph* NodeGraph;
	UAnimGraphNode_Base* Node;

	static TSharedRef<FDisplayedBlueprintReferenceInfo> Make(UAnimBlueprint* Blueprint, UAnimGraphNode_Slot* SlotNode);

protected:
	/** Hidden constructor, always use Make above */
	FDisplayedBlueprintReferenceInfo(UAnimBlueprint* Blueprint, UAnimGraphNode_Slot* SlotNode);

	/** Hidden constructor, always use Make above */
	FDisplayedBlueprintReferenceInfo()
	{}
};

typedef SListView<TSharedPtr<FDisplayedMontageReferenceInfo>> SMontageReferenceList;
typedef SListView<TSharedPtr<FDisplayedBlueprintReferenceInfo>> SBlueprintReferenceList;

class SMontageReferenceListRow : public SMultiColumnTableRow < TSharedPtr<FDisplayedMontageReferenceInfo> >
{
public:
	SLATE_BEGIN_ARGS(SMontageReferenceListRow)
	{}
	SLATE_ARGUMENT(TSharedPtr<FDisplayedMontageReferenceInfo>, ReferenceInfo)
		SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView);

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

protected:

	// Handlers for asset interaction buttons
	FReply OnViewInContentBrowserClicked();
	FReply OnOpenAssetClicked();

	TSharedPtr<FDisplayedMontageReferenceInfo> ReferenceInfo;
};

class SBlueprintReferenceListRow : public SMultiColumnTableRow<TSharedPtr<FDisplayedBlueprintReferenceInfo>>
{
public:
	SLATE_BEGIN_ARGS(SBlueprintReferenceListRow) {}
		SLATE_ARGUMENT(TSharedPtr<FDisplayedBlueprintReferenceInfo>, ReferenceInfo)
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView);

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

protected:

	// Handlers for asset interaction buttons
	FReply OnViewInContentBrowserClicked();
	FReply OnOpenAssetClicked();

	TSharedPtr<FDisplayedBlueprintReferenceInfo> ReferenceInfo;
};

// Required information to update a reference window widget, used in conjunction with SSlotNameReferenceWindow::UpdateInfo
// any fields that do not get set by the caller will not be updated and will keep its previous value
struct FReferenceWindowInfo
{
	FReferenceWindowInfo()
		: ReferencingMontages(nullptr)
		, ReferencingNodes(nullptr)
		, bRefresh(true)
	{

	}

	TArray<FAssetData>* ReferencingMontages;
	NodeMapPtr ReferencingNodes;
	FText OperationText;
	FText ItemText;
	FSimpleDelegate RetryDelegate;
	bool bRefresh;
};

// Widget used to slot name references that are blocking an edit operation
class SSlotNameReferenceWindow : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SSlotNameReferenceWindow) {}
		SLATE_ARGUMENT(TArray<FAssetData>*, ReferencingMontages)
		SLATE_ARGUMENT(NodeMapPtr, ReferencingNodes)
		SLATE_ARGUMENT(FString, SlotName)
		SLATE_ARGUMENT(TSharedPtr<SWindow>, WidgetWindow)
		SLATE_ARGUMENT(FText, OperationText)
		SLATE_EVENT(FSimpleDelegate, OnRetry)
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs);

	// Refresh the child content of this widget
	void RefreshContent();

	// Update the reference arrays and optionally rebuild the widget
	void UpdateInfo(FReferenceWindowInfo& UpdatedInfo);

	// The name of the slot we're referencing
	FString SlotName;

protected:

	// Rebuild the internal content of the widget
	TSharedRef<SWidget> GetContent();

	// Row generators for montage and blueprint/node data views
	TSharedRef<ITableRow> HandleGenerateMontageReferenceRow( TSharedPtr<FDisplayedMontageReferenceInfo> Item, const TSharedRef<STableViewBase>& OwnerTable );
	TSharedRef<ITableRow> HandleGenerateBlueprintReferenceRow( TSharedPtr<FDisplayedBlueprintReferenceInfo> Item, const TSharedRef<STableViewBase>& OwnerTable );

	// Button handlers
	FReply OnRetryClicked();
	FReply OnCloseClicked();

	// Operation type to display
	FText OperationText;

	// Objects referencing the given slot name
	TArray<TSharedPtr<FDisplayedMontageReferenceInfo>> ReferencingMontages;
	TArray<TSharedPtr<FDisplayedBlueprintReferenceInfo>> ReferencingNodes;

	// Ptr to the window this widget resides in
	TSharedPtr<SWindow> ContainingWindow;

	// Called when the retry button is clicked. User code determines retry behaviour
	FSimpleDelegate OnRetry;
};

#undef LOCTEXT_NAMESPACE
