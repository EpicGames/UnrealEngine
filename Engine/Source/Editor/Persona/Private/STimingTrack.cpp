// Copyright Epic Games, Inc. All Rights Reserved.

#include "STimingTrack.h"

#include "Containers/Array.h"
#include "HAL/PlatformCrt.h"
#include "Layout/ArrangedChildren.h"
#include "Layout/Children.h"
#include "Layout/Geometry.h"
#include "Math/Box2D.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector2D.h"
#include "SCurveEditor.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SWidget.h"

void STimingTrack::Construct(const FArguments& Args)
{
	STrack::FArguments BaseArgs;
	BaseArgs.ViewInputMin(Args._ViewInputMin);
	BaseArgs.ViewInputMax(Args._ViewInputMax);
	BaseArgs.TrackMinValue(Args._TrackMinValue);
	BaseArgs.TrackMaxValue(Args._TrackMaxValue);
	BaseArgs.TrackNumDiscreteValues(Args._TrackNumDiscreteValues);

	STrack::Construct(BaseArgs);
}

void STimingTrack::OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const
{
	// To make this track look nice and have no overlapping nodes we're going to 
	// treat this as a 1D collision problem and build/resolve islands until everything 
	// is nicely resolved (or until we hit an upper limit as it can't be solved)

	// Helper holding info about a single node
	struct NodeData
	{
		NodeData(TSharedRef<STrackNode>& NodeRef, const FGeometry& Geometry)
			: Node(NodeRef)
		{
			// Separation of the nodes on the track
			const float NodeSeparation = 3.0f;

			Node->CacheTrackGeometry(Geometry);
			FVector2D Offset = Node->GetOffsetRelativeToParent(Geometry);
			FVector2D Size = Node->GetSizeRelativeToParent(Geometry);

			Offset.Y += (Geometry.GetLocalSize().Y - Size.Y) * 0.5f;

			ActualRect = FBox2D(Offset, Offset + Size);
			QueryRect = FBox2D(Offset - FVector2D(NodeSeparation, 0.0f), Offset + Size + FVector2D(NodeSeparation, 0.0f));
		}

		TSharedRef<STrackNode> Node;	// The node widget
		FBox2D ActualRect;				// The actual render rect of the widget
		FBox2D QueryRect;				// An expanded rect used to detect collisions
	};

	// Helper holding list of overlapping nodes
	struct NodeIsland
	{
		TArray<NodeData*> Nodes;
	};

	int32 NumNodes = TrackNodes.Num();

	TArray<NodeData> SortedNodeData;
	SortedNodeData.Reserve(NumNodes);

	// List of collision islands
	TArray<NodeIsland> Islands;
	// Scaling info to translate between local positions and data values
	FTrackScaleInfo ScaleInfo(ViewInputMin.Get(), ViewInputMax.Get(), 0, 0, AllottedGeometry.Size);

	for(int32 TrackIndex = 0; TrackIndex < NumNodes; ++TrackIndex)
	{
		TSharedRef<STrackNode> TrackNode = (TrackNodes[TrackIndex]);

		SortedNodeData.Add(NodeData(TrackNode, AllottedGeometry));
	}

	const static int32 MaxRetries = 5;
	int32 Retries = 0;
	bool bResolved = true;

	while(bResolved && Retries < MaxRetries)
	{
		++Retries;
		bResolved = false;

		for(int32 NodeIdx = 0; NodeIdx < NumNodes; ++NodeIdx)
		{
			NodeData& CurrentNode = SortedNodeData[NodeIdx];
			// Island generation
			NodeIsland* CurrentIsland = &Islands[Islands.AddZeroed()];

			int32 Direction = -1;
			int32 Next = NodeIdx + 1;
			int32 HighestNode = NodeIdx;
			FBox2D& CurrentRect = CurrentNode.ActualRect;
			FBox2D CurrentQueryRect = CurrentNode.QueryRect;

			CurrentIsland->Nodes.Add(&CurrentNode);

			// Walk Nodes
			while(Next >= 0 && Next < NumNodes)
			{
				NodeData& NextNode = SortedNodeData[Next];
				FBox2D& NextRect = NextNode.ActualRect;
				FBox2D& NextQueryRect = NextNode.QueryRect;
				if(NextQueryRect.Intersect(CurrentQueryRect))
				{
					// Add to island
					CurrentIsland->Nodes.Add(&NextNode);
					HighestNode = Next;

					// Expand the current query
					CurrentQueryRect.Max = NextQueryRect.Max;
				}
				else
				{
					// No island, next node
					break;
				}

				++Next;
			}

			// Skip processed nodes (those already in islands)
			NodeIdx = HighestNode;
		}

		// Separation of the nodes on the track
		const float NodeSeparation = 3.0f;

		for(const NodeIsland& Island : Islands)
		{
			bResolved = true;

			// Island resolution
			const int32 NumIslandNodes = Island.Nodes.Num();
			float Width = FMath::Max<float>(static_cast<float>(NumIslandNodes - 1), 0.0f) * NodeSeparation;
			float Centre = 0.0f;

			for(const NodeData* Node : Island.Nodes)
			{
				Width += static_cast<float>(Node->ActualRect.GetSize().X);
				Centre += static_cast<float>(Node->ActualRect.GetCenter().X);
			}
			Centre /= NumIslandNodes;

			const float Begin = Centre - Width / 2.0f;
			for(int32 NodeIdx = 0 ; NodeIdx < NumIslandNodes; ++NodeIdx)
			{
				FBox2D& NodeBox = Island.Nodes[NodeIdx]->ActualRect;
				const float NodeWidth = static_cast<float>(NodeBox.GetSize().X);
				const float SeparationOffset = NodeIdx * NodeSeparation;
				float PositionOffset = 0.0f;
			
				for(int32 PositionNodeIdx = 0 ; PositionNodeIdx < NodeIdx ; ++PositionNodeIdx)
				{
					PositionOffset += static_cast<float>(Island.Nodes[PositionNodeIdx]->ActualRect.GetSize().X);
				}
			
				FBox2D& OriginalRect = Island.Nodes[NodeIdx]->ActualRect;
				OriginalRect.Min.X = Begin + PositionOffset + SeparationOffset;
				OriginalRect.Max.X = OriginalRect.Min.X + NodeWidth;
			
				// Alter Query rect for next pass
				FBox2D& NodeQueryRect = Island.Nodes[NodeIdx]->QueryRect;
				NodeQueryRect.Min = OriginalRect.Min - FVector2D(NodeSeparation, 0.0f);
				NodeQueryRect.Max = OriginalRect.Max + FVector2D(NodeSeparation, 0.0f);
			}
		}
	}

	for(int32 TrackIndex = 0; TrackIndex < NumNodes; ++TrackIndex)
	{
		TSharedRef<STrackNode> TrackNode = (SortedNodeData[TrackIndex].Node);
		if(TrackNode->IsBeingDragged())
		{
			continue;
		}

		FBox2D& Rect = SortedNodeData[TrackIndex].ActualRect;

		ArrangedChildren.AddWidget(AllottedGeometry.MakeChild(TrackNode, Rect.Min, Rect.GetSize()));
	}
}
