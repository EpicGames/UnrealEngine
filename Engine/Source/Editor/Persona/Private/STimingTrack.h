// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Misc/Attribute.h"
#include "STrack.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class FArrangedChildren;
struct FGeometry;

//////////////////////////////////////////////////////////////////////////
// Specialised anim track which arranges overlapping nodes into groups
class STimingTrack : public STrack
{
public:

	SLATE_BEGIN_ARGS(STimingTrack)
	{}

	SLATE_ATTRIBUTE(float, ViewInputMin)
	SLATE_ATTRIBUTE(float, ViewInputMax)
	SLATE_ATTRIBUTE(float, TrackMaxValue)
	SLATE_ATTRIBUTE(float, TrackMinValue)
	SLATE_ATTRIBUTE(int32, TrackNumDiscreteValues)

	SLATE_END_ARGS()

	void Construct(const FArguments& Args);

	virtual void OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const override;

protected:
};
