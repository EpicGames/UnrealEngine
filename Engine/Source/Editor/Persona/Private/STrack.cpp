// Copyright Epic Games, Inc. All Rights Reserved.


#include "STrack.h"

#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "GenericPlatform/ICursor.h"
#include "HAL/PlatformCrt.h"
#include "Input/Events.h"
#include "InputCoreTypes.h"
#include "Layout/ArrangedChildren.h"
#include "Layout/Clipping.h"
#include "Layout/PaintGeometry.h"
#include "Layout/WidgetPath.h"
#include "Math/UnrealMathSSE.h"
#include "Rendering/DrawElements.h"
#include "Rendering/RenderingCommon.h"
#include "SCurveEditor.h"
#include "DragAndDrop/AssetDragDropOp.h"
#include "Styling/CoreStyle.h"
#include "Templates/TypeHash.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SWidget.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/STextBlock.h"

class FChildren;
class FSlateRect;
class FWidgetStyle;
struct FSlateBrush;

const float STrackDefaultHeight = 24.0f;
const float DraggableBarSnapTolerance = 20.0f;
const float NodeHandleWidth = 12.f;
const float NodeHandleHeight = STrackDefaultHeight;

#define LOCTEXT_NAMESPACE "STrack"

DECLARE_DELEGATE_OneParam( FOnDeleteNotify, struct FAnimNotifyEvent*)

//////////////////////////////////////////////////////////////////////////
// FTrackNodeDragDropOp
void FTrackNodeDragDropOp::OnDrop( bool bDropWasHandled, const FPointerEvent& MouseEvent )
{
	if ( bDropWasHandled == false )
	{
		if (OriginalTrackNode.IsValid())
		{
			OriginalTrackNode.Pin()->OnDropCancelled(MouseEvent);
		}
	}
	
	FDragDropOperation::OnDrop(bDropWasHandled, MouseEvent);
}

void FTrackNodeDragDropOp::OnDragged( const class FDragDropEvent& DragDropEvent )
{
	TSharedPtr<STrackNode> Node = OriginalTrackNode.Pin();
	if (Node.IsValid())
	{
		Node->OnDragged(DragDropEvent);
	}

	FVector2D pos;
	pos.X = (DragDropEvent.GetScreenSpacePosition() + Offset).X;
	pos.Y = StartingScreenPos.Y;

	CursorDecoratorWindow->MoveWindowTo(pos);
}

TSharedRef<FTrackNodeDragDropOp> FTrackNodeDragDropOp::New(TSharedRef<STrackNode> TrackNode, const FVector2D &CursorPosition, const FVector2D &ScreenPositionOfNode)
{
	TSharedRef<FTrackNodeDragDropOp> Operation = MakeShareable(new FTrackNodeDragDropOp);
	Operation->OriginalTrackNode = TrackNode;

	Operation->Offset = ScreenPositionOfNode - CursorPosition;
	Operation->StartingScreenPos = ScreenPositionOfNode;

	Operation->Construct();

	return Operation;
}

TSharedPtr<SWidget> FTrackNodeDragDropOp::GetDefaultDecorator() const
{
	return OriginalTrackNode.Pin();
}


//////////////////////////////////////////////////////////////////////////
// STrackNode
void STrackNode::Construct(const FArguments& InArgs)
{
	bSelectedFallback = false;
	bBeingDragged = false;
	LastSize = FVector2D(0.f, 0.f);

	ViewInputMin = InArgs._ViewInputMin;
	ViewInputMax = InArgs._ViewInputMax;

	DataStartPos = InArgs._DataStartPos;
	DataLength = InArgs._DataLength;

	NodeName = InArgs._NodeName;
	NodeColor = InArgs._NodeColor;
	SelectedNodeColor = InArgs._SelectedNodeColor;
	
	OnNodeSelectionChanged = InArgs._OnSelectionChanged;
	OnTrackNodeDragged = InArgs._OnTrackNodeDragged;
	OnTrackNodeDropped = InArgs._OnTrackNodeDropped;
	OnNodeRightClickContextMenu = InArgs._OnNodeRightClickContextMenu;
	OnTrackNodeClicked = InArgs._OnTrackNodeClicked;
	OnTrackNodeDoubleClicked = InArgs._OnTrackNodeDoubleClicked;
	bCenterOnPosition = InArgs._CenterOnPosition;
	
	NodeSelectionSet = InArgs._NodeSelectionSet;
	bAllowDrag = InArgs._AllowDrag;

	Font = FCoreStyle::GetDefaultFontStyle("Regular", 10);

	const FSlateBrush* StyleInfo = FAppStyle::GetBrush( TEXT("SpecialEditableTextImageNormal") ); // FIXME: make slate argument for STrackNode

	if(InArgs._OverrideContent.Widget != SNullWidget::NullWidget)
	{
		bContentOverriden = true;
		// Content was overridden by caller
		this->ChildSlot
		[
			InArgs._OverrideContent.Widget
		];
	}
	else
	{
		bContentOverriden = false;
		// Content not overriden, use default node appearance
		this->ChildSlot
		[
			SNew(SBorder)
			.BorderImage(StyleInfo)
			.ForegroundColor(FLinearColor::Black)
			.BorderBackgroundColor(this, &STrackNode::GetNodeColor)
			.Content()
			[
				SNew(STextBlock)
				.Font(Font)
				.Text(this, &STrackNode::GetNodeText)
			]
		];
	}
}

FVector2D STrackNode::GetOffsetRelativeToParent(const FGeometry& AllottedGeometry) const
{
	FVector2D Result(0.0f, 0.0f);
	FVector2D Size = GetSizeRelativeToParent(AllottedGeometry);
	FTrackScaleInfo ScaleInfo(ViewInputMin.Get(), ViewInputMax.Get(), 0, 0, AllottedGeometry.GetLocalSize());

	if(bCenterOnPosition)
	{
		Result = FVector2D( ScaleInfo.InputToLocalX(DataStartPos.Get()) - (Size.X/2.f) , 0);

	}	
	else
	{
		Result = FVector2D( ScaleInfo.InputToLocalX(DataStartPos.Get()) , 0);
	}

	return Result;
}

FVector2D STrackNode::GetSizeRelativeToParent(const FGeometry& AllottedGeometry) const
{
	if(bContentOverriden)
	{
		return ChildSlot.GetWidget()->GetDesiredSize();
	}
	else
	{
		if(DataLength.Get() > 0.f)
		{
			// Scale us by data size
			FTrackScaleInfo ScaleInfo(ViewInputMin.Get(), ViewInputMax.Get(), 0, 0, AllottedGeometry.Size);
			return FVector2D(ScaleInfo.InputToLocalX(ViewInputMin.Get() + DataLength.Get()), STrackDefaultHeight);
		}
		else
		{
			// Use default hardcoded "knob" size
			return FVector2D(NodeHandleWidth, NodeHandleHeight);
		}
	}
}

FVector2D STrackNode::GetDragDropScreenSpacePosition(const FGeometry& ParentAllottedGeometry, const FDragDropEvent& DragDropEvent) const
{
	FVector2D DragDropPos = DragDropEvent.GetScreenSpacePosition();
	TSharedPtr<FTrackNodeDragDropOp> DragDropOp = StaticCastSharedPtr<FTrackNodeDragDropOp>(DragDropEvent.GetOperation());
	if(DragDropOp.IsValid())
	{
		DragDropPos += DragDropOp->Offset;
	}
	if(bCenterOnPosition)
	{
		// Correct for center-on-position offset
		FVector2D Size = GetSizeRelativeToParent(ParentAllottedGeometry);
		DragDropPos.X += Size.X/2.f;
	}
	return DragDropPos;
}

// mouse interface for tooltip/selection
FReply STrackNode::OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		Select();
		OnTrackNodeClicked.ExecuteIfBound();

		return FReply::Handled().ReleaseMouseCapture();
	}
	else if(MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		Select();
		OnTrackNodeClicked.ExecuteIfBound();
	}

	return FReply::Unhandled();
}

FReply STrackNode::OnMouseButtonDoubleClick(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		OnTrackNodeDoubleClicked.ExecuteIfBound();

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FReply STrackNode::OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	return FReply::Unhandled();
}

void STrackNode::OnMouseEnter( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{

}

void STrackNode::OnMouseLeave( const FPointerEvent& MouseEvent )
{

}

FReply STrackNode::OnDragDetected( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if (bAllowDrag && MouseEvent.IsMouseButtonDown( EKeys::LeftMouseButton ) )
	{
		bBeingDragged = true;

		return BeginDrag( MyGeometry, MouseEvent );
	}
	return FReply::Unhandled();
}

FReply STrackNode::BeginDrag( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{

	FVector2D ScreenCursorPos = MouseEvent.GetScreenSpacePosition();
	FVector2D CursorPos = MyGeometry.AbsoluteToLocal(ScreenCursorPos);
	FVector2D ScreenNodePosition = FVector2D(MyGeometry.AbsolutePosition);// + GetOffsetRelativeToParent(MyGeometry);
	
	bBeingDragged = true;
	LastSize = MyGeometry.GetLocalSize();

	Select();
	OnTrackNodeClicked.ExecuteIfBound();

	//void FTrackNodeDragDropOp(TSharedRef<STrackNode> TrackNode, const FVector2D &CursorPosition, const FVector2D &ScreenPositionOfNode)
	return FReply::Handled().BeginDragDrop(FTrackNodeDragDropOp::New(SharedThis(this), ScreenCursorPos, ScreenNodePosition)).ReleaseMouseCapture();
}

FReply STrackNode::OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if(MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		return FReply::Handled();
	}
	else if ( MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton )
	{
		if(bAllowDrag)
		{
			return FReply::Handled().DetectDrag( SharedThis(this), EKeys::LeftMouseButton ).CaptureMouse(AsShared());
		}

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void STrackNode::Select()
{	
	OnNodeSelectionChanged.ExecuteIfBound();
	if(NodeSelectionSet!=NULL)
	{
		NodeSelectionSet->Empty();
		NodeSelectionSet->Add(this);
	}
	bSelectedFallback = true;
}
void STrackNode::Deselect()
{
	if(NodeSelectionSet!=NULL)
	{
		NodeSelectionSet->Remove(this);
	}
	bSelectedFallback = false;
}
void STrackNode::ToggleSelect()
{
	if(IsSelected()) 
	{
		Select();
	}
	else 
	{
		Deselect();
	}
}

bool STrackNode::IsSelected() const
{
	if(NodeSelectionSet!=NULL)
	{
		return NodeSelectionSet->Contains(this);
	}
	return bSelectedFallback;
}

void STrackNode::OnDropCancelled(const FPointerEvent& MouseEvent)
{
	bBeingDragged = false;
	OnTrackNodeDropped.ExecuteIfBound();
}

void STrackNode::OnDragged( const class FDragDropEvent& DragDropEvent )
{

}

bool STrackNode::HitTest(const FGeometry& AllottedGeometry, FVector2D MouseLocalPose) const
{
	FVector2D Position = GetOffsetRelativeToParent(AllottedGeometry);
	FVector2D Size = GetSizeRelativeToParent(AllottedGeometry);
	return (MouseLocalPose.X >= Position.X && MouseLocalPose.X <= Position.X+Size.X) && (MouseLocalPose.Y >= Position.Y && MouseLocalPose.Y <= Position.Y+Size.Y);
}

FVector2D STrackNode::GetSize() const
{
	// FVector2D DrawSize = FSlateApplication::Get().MeasureString(NodeName.Get(), Font)+FVector2D(5, 5);
	// Returns last painted size
	return LastSize;
}

FVector2D STrackNode::ComputeDesiredSize( float ) const
{
	return GetSize();
}

float STrackNode::GetDataStartPos() const
{
	if(DataStartPos.IsBound())
	{
		return DataStartPos.Get();
	}
	return 0.f;
}

FSlateColor STrackNode::GetNodeColor() const
{
	return IsSelected() ? FSlateColor(SelectedNodeColor.Get()) : FSlateColor(NodeColor.Get());
}

FText STrackNode::GetNodeText() const
{
	return FText::FromString(NodeName.Get());
}

//////////////////////////////////////////////////////////////////////////
// STrack

STrack::STrack()
	: TrackNodes(this)
{
}

void STrack::Construct( const FArguments& InArgs )
{
	TrackColor = InArgs._TrackColor;
	TrackMinValue = InArgs._TrackMinValue;
	TrackMaxValue = InArgs._TrackMaxValue;
	TrackNumDiscreteValues = InArgs._TrackNumDiscreteValues;
	ViewInputMin = InArgs._ViewInputMin;
	ViewInputMax = InArgs._ViewInputMax;
	OnSelectionChanged = InArgs._OnSelectionChanged;
	DraggableBars = InArgs._DraggableBars;
	DraggableBarLabels = InArgs._DraggableBarLabels;
	DraggableBarSnapPositions = InArgs._DraggableBarSnapPositions;
	OnBarDrag = InArgs._OnBarDrag;
	OnBarClicked = InArgs._OnBarClicked;
	OnBarDrop = InArgs._OnBarDrop;
	OnTrackDragDrop = InArgs._OnTrackDragDrop;
	OnAssetDragDrop = InArgs._OnAssetDragDrop;
	OnSummonContextMenu = InArgs._OnSummonContextMenu;
	OnTrackRightClickContextMenu = InArgs._OnTrackRightClickContextMenu;

	DraggableBarIndex = INDEX_NONE;
	bDraggingBar = false;

	SetClipping(EWidgetClipping::ClipToBounds);

	Font = FCoreStyle::GetDefaultFontStyle("Regular", 10);
}

void STrack::OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const
{
	for ( int32 TrackIndex=0; TrackIndex < TrackNodes.Num(); ++TrackIndex )
	{
		TSharedRef<STrackNode> TrackNode = (TrackNodes[TrackIndex]);
		if (TrackNode->bBeingDragged)
		{
			continue;
		}

		// our desired size is sum of all children
		// this isn't precisely correct size, but it is minimal required size
		
		TrackNode->CacheTrackGeometry(AllottedGeometry);
		FVector2D Offset = TrackNode->GetOffsetRelativeToParent(AllottedGeometry);
		FVector2D Size = TrackNode->GetSizeRelativeToParent(AllottedGeometry);
	
		ArrangedChildren.AddWidget( AllottedGeometry.MakeChild(TrackNode, Offset, Size) );
	}
}

FVector2D STrack::ComputeDesiredSize( float ) const
{
	FVector2D Size;
	Size.X = 5000; // FIXME: Setting this really high so that the track takes all the width it can in the montage tool (since its placed in an HBox next to a fixed with column
	Size.Y = STrackDefaultHeight;
	return Size;
}

FChildren* STrack::GetChildren()
{
	return &TrackNodes;
}

int32 STrack::OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const
{
	int32 CustomLayerId = LayerId + 1;
	FPaintGeometry MyGeometry = AllottedGeometry.ToPaintGeometry();

	// Draggable Bars
	for ( int32 I=0; DraggableBars.IsBound() && I < DraggableBars.Get().Num(); I++ )
	{
		// Draw lines
		float XPos = DataToLocalX(DraggableBars.Get()[I], AllottedGeometry);

		TArray<FVector2D> LinePoints;
		LinePoints.Add(FVector2D(XPos, 0.f));
		LinePoints.Add(FVector2D(XPos, AllottedGeometry.Size.Y));

		FSlateDrawElement::MakeLines( 
			OutDrawElements,
			CustomLayerId,
			MyGeometry,
			LinePoints,
			ESlateDrawEffect::None,
			FLinearColor(0.0f, 1.0f, 0.0f, 1.0f)
			);

		// Draw Labels
		if (DraggableBarLabels.IsBound() && DraggableBarLabels.Get().IsValidIndex(I))
		{
			//float EndPos = (DraggableBarIndex.Get().IsValidIndex(I+1) ? DraggableBarIndex.Get()(I+1)

			FPaintGeometry TextGeometry = AllottedGeometry.ToPaintGeometry( AllottedGeometry.GetDrawSize(), FSlateLayoutTransform(FVector2f(XPos + 15.f, 5.f)) );
			FSlateDrawElement::MakeText( 
				OutDrawElements,
				CustomLayerId,
				TextGeometry,
				DraggableBarLabels.Get()[I],
				Font,
				ESlateDrawEffect::None,
				FLinearColor::Black
				);
		}
	}
	// Ghost snap position
	float SnapPos = 0.f;
	if ( GetDraggableBarSnapPosition(AllottedGeometry, SnapPos) )
	{
		SnapPos = DataToLocalX(SnapPos, AllottedGeometry);
		TArray<FVector2D> LinePoints;
		LinePoints.Add(FVector2D(SnapPos, 0.f));
		LinePoints.Add(FVector2D(SnapPos, AllottedGeometry.Size.Y));

		FSlateDrawElement::MakeLines( 
			OutDrawElements,
			CustomLayerId,
			MyGeometry,
			LinePoints,
			ESlateDrawEffect::None,
			FLinearColor(0.5f, 0.0f, 0.0f, 0.5f)
			);
	}
	CustomLayerId++;
	
	return SPanel::OnPaint( Args, AllottedGeometry, MyCullingRect, OutDrawElements, CustomLayerId, InWidgetStyle, bParentEnabled );
}

// drag drop relationship
FReply STrack::OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	const FVector2D CursorPos = MyGeometry.AbsoluteToLocal(DragDropEvent.GetScreenSpacePosition());
	const float CusorDataPos = LocalToDataX(static_cast<float>(CursorPos.X), MyGeometry);

	// Handle TrackNodes that were dropped
	const TSharedPtr<FTrackNodeDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FTrackNodeDragDropOp>();
	if (DragDropOp.IsValid())
	{
		const TSharedPtr<STrackNode> TrackNode = DragDropOp->OriginalTrackNode.Pin();

		const float DataPos = GetNodeDragDropDataPos(MyGeometry, DragDropEvent);
		TrackNode->OnTrackNodeDragged.ExecuteIfBound( DataPos );
		TrackNode->OnTrackNodeDropped.ExecuteIfBound();
	}

	// Call delegate to handle anything else
	OnTrackDragDrop.ExecuteIfBound( DragDropEvent.GetOperation(), CusorDataPos );

	return FReply::Unhandled();
}

FReply STrack::OnDragOver( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	// Handle TrackNodes that were dropped
	TSharedPtr<FTrackNodeDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FTrackNodeDragDropOp>();
	if (DragDropOp.IsValid())
	{
		TSharedPtr<STrackNode> TrackNode = DragDropOp->OriginalTrackNode.Pin();

		float DataPos = GetNodeDragDropDataPos(MyGeometry, DragDropEvent);
		TrackNode->OnTrackNodeDragged.ExecuteIfBound( DataPos );
	}

	return FReply::Unhandled();
}

FReply STrack::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if(DraggableBarIndex != INDEX_NONE)
	{
		bDraggingBar = true;
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

void STrack::OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<FAssetDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FAssetDragDropOp>();
	if (DragDropOp.IsValid())
	{
		if(OnAssetDragDrop.IsBound())
		{
			OnAssetDragDrop.Execute(DragDropOp);
		}
	}
}

void STrack::OnDragLeave(const FDragDropEvent& DragDropEvent)
{
	SPanel::OnDragLeave(DragDropEvent);
	TSharedPtr<FAssetDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FAssetDragDropOp>();
	if (DragDropOp.IsValid())
	{
		DragDropOp->ResetToDefaultToolTip();
	}
}

float STrack::GetNodeDragDropDataPos( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) const
{
	float DataPos = 0.f;
	const TSharedPtr<FTrackNodeDragDropOp> DragDropOp = StaticCastSharedPtr<FTrackNodeDragDropOp>(DragDropEvent.GetOperation());
	if(DragDropOp.IsValid())
	{
		const TSharedPtr<STrackNode> TrackNode = DragDropOp->OriginalTrackNode.Pin();
		if(TrackNode.IsValid())
		{
			const FVector2D CursorPos = MyGeometry.AbsoluteToLocal(TrackNode->GetDragDropScreenSpacePosition(MyGeometry, DragDropEvent));
			DataPos = LocalToDataX(static_cast<float>(CursorPos.X), MyGeometry);
			if(TrackNode->SnapToDragBars())
			{
				const float OriginalX = DataPos;
				DataPos = GetSnappedPosForLocalPos(MyGeometry, static_cast<float>(CursorPos.X));
				TrackNode->OnSnapNodeDataPosition(OriginalX, DataPos);
			}
		}
	}

	return DataPos;
}

float STrack::GetSnappedPosForLocalPos( const FGeometry& MyGeometry, float TrackPos) const
{
	float SnapPos = LocalToDataX(TrackPos, MyGeometry);

	if ( DraggableBars.IsBound() )
	{
		float BestDist = 10.f;

		for (auto SnapIter = DraggableBars.Get().CreateConstIterator(); SnapIter; ++SnapIter)
		{
			// Snap in screen space - not sure if this is best but seems right since its a hardcoded value atm

			float ScreenDist = FMath::Abs(TrackPos - DataToLocalX( *SnapIter, MyGeometry ));
			if ( ScreenDist < BestDist )
			{
				BestDist = ScreenDist;
				SnapPos = *SnapIter;
			}
		}
	}
	return SnapPos;
}

int32 STrack::GetHitNode(const FGeometry& MyGeometry, const FVector2D& CursorPosition)
{	

	for (int32 I=0; I < TrackNodes.Num(); I++)
	{
		TSharedRef<STrackNode> TrackNode = TrackNodes[I];
		if (TrackNode->HitTest(MyGeometry, CursorPosition) )
		{
			return I;
		}
	}

	return INDEX_NONE;
}

FReply STrack::OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if (bDraggingBar && OnBarDrag.IsBound())
	{
		/** Update drag bar position if we are dragging */
		const FVector2D CursorPos = MyGeometry.AbsoluteToLocal( MouseEvent.GetScreenSpacePosition() );
		const float NewDataPos = FMath::Clamp( static_cast<float>(LocalToDataX(static_cast<float>(CursorPos.X), MyGeometry)), TrackMinValue.Get(), TrackMaxValue.Get() );
		OnBarDrag.Execute(DraggableBarIndex, NewDataPos);

		// Update details panel
		OnBarClicked.ExecuteIfBound(DraggableBarIndex);
	}
	else if(DraggableBars.IsBound())
	{
		/** Update what bar is draggable if we arent already dragging */
		UpdateDraggableBarIndex(MyGeometry, MouseEvent.GetScreenSpacePosition());
	}

	return FReply::Unhandled();
}

FReply STrack::OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if ( MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton )
	{
		if (DraggableBarIndex != INDEX_NONE)
		{
			if(!bDraggingBar)
			{
				return FReply::Handled().DetectDrag( SharedThis(this), EKeys::LeftMouseButton );
			}
		}
	}

	return FReply::Unhandled();
}

FReply STrack::OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	bool bLeftMouseButton =  MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton;
	bool bRightMouseButton =  MouseEvent.GetEffectingButton() == EKeys::RightMouseButton;

	if ( bRightMouseButton )
	{
		
		TSharedPtr<SWidget> WidgetToFocus;
		WidgetToFocus = SummonContextMenu(MyGeometry, MouseEvent);

		return (WidgetToFocus.IsValid())
			? FReply::Handled().ReleaseMouseCapture().SetUserFocus( WidgetToFocus.ToSharedRef(), EFocusCause::SetDirectly )
			: FReply::Handled().ReleaseMouseCapture();
	}
	else if ( bLeftMouseButton )
	{
		float SnapPos = 0.f;
		if ( GetDraggableBarSnapPosition(MyGeometry, SnapPos) )
		{
			// Final update on draggable bar to snap position
			OnBarDrag.Execute(DraggableBarIndex, SnapPos);
		}
		
		if(bDraggingBar)
		{
			OnBarDrop.Execute(DraggableBarIndex);
		}
		if (DraggableBarIndex != INDEX_NONE)
		{
			OnBarClicked.ExecuteIfBound(DraggableBarIndex);
		}

		/** Bar dragging */
		DraggableBarIndex = INDEX_NONE;
		bDraggingBar = false;
	}

	return FReply::Unhandled();
}

TSharedPtr<SWidget> STrack::SummonContextMenu(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	bool SummonedContextMenu = false;

	constexpr bool bCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder( bCloseWindowAfterMenuSelection, EditorActions );

	const FVector2D CursorPos = MouseEvent.GetScreenSpacePosition();
	const float DataPos = LocalToDataX( static_cast<float>(MyGeometry.AbsoluteToLocal(CursorPos).X), MyGeometry );

	// Context menu for a node
	const int32 NotifyIndex = GetHitNode(MyGeometry, MyGeometry.AbsoluteToLocal(CursorPos));
	if(NotifyIndex != INDEX_NONE)
	{
		if(TrackNodes[NotifyIndex]->OnNodeRightClickContextMenu.IsBound())
		{
			TrackNodes[NotifyIndex]->OnNodeRightClickContextMenu.Execute(MenuBuilder);
			SummonedContextMenu = true;
		}
	}

	// Context menu for track itself
	if(OnTrackRightClickContextMenu.IsBound())
	{
		SummonedContextMenu = true;
		OnTrackRightClickContextMenu.Execute(MenuBuilder, DataPos, DraggableBarIndex);
	}

	// Build the menu if we actually added anything to it
	TSharedPtr<SWidget> MenuContent;
	if(SummonedContextMenu)
	{
		MenuContent = MenuBuilder.MakeWidget();

		FWidgetPath WidgetPath = MouseEvent.GetEventPath() != nullptr ? *MouseEvent.GetEventPath() : FWidgetPath();
		FSlateApplication::Get().PushMenu(SharedThis(this), WidgetPath, MenuContent.ToSharedRef(), CursorPos, FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu));
	}

	return MenuContent;
}

/** Return true if we should snap and set out position to its position */
bool STrack::GetDraggableBarSnapPosition(const FGeometry& MyGeometry, float &OutPosition) const
{
	if ( bDraggingBar && DraggableBarIndex != INDEX_NONE && DraggableBarSnapPositions.IsBound() )
	{
		float CurX = DraggableBars.Get()[DraggableBarIndex];
		float BestDist=-1.f;
		float BestPos=-1.f;
		for (auto SnapIter = DraggableBarSnapPositions.Get().CreateConstIterator(); SnapIter; ++SnapIter)
		{
			// Snap in screen space - not sure if this is best but seems right since its a hardcoded value atm
			float ScreenDist = FMath::Abs( DataToLocalX( CurX , MyGeometry ) - DataToLocalX( *SnapIter, MyGeometry ));
			if ( ScreenDist < DraggableBarSnapTolerance && (BestDist < 0.f || ScreenDist < BestDist))
			{
				BestDist = ScreenDist;
				BestPos = *SnapIter;
			}
		}
		if ( BestPos >= 0.f )
		{
			OutPosition = BestPos;
			return true;
		}
	}
	return false;
}

void STrack::OnMouseLeave( const FPointerEvent& MouseEvent )
{
	if(bDraggingBar && OnBarDrop.IsBound())
	{
		OnBarDrop.Execute(DraggableBarIndex);
	}
	bDraggingBar = false;
}

void STrack::OnMouseEnter( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent )
{
	if ( MouseEvent.IsMouseButtonDown( EKeys::LeftMouseButton ) )
	{
		UpdateDraggableBarIndex(MyGeometry, MouseEvent.GetScreenSpacePosition());
		if ( DraggableBarIndex != INDEX_NONE )
		{
			bDraggingBar = true;
		}
	}
}

/** Override cursor if over a draggable bar */
FCursorReply STrack::OnCursorQuery( const FGeometry& MyGeometry, const FPointerEvent& CursorEvent ) const
{
	if (DraggableBarIndex != INDEX_NONE)
	{
		return FCursorReply::Cursor( EMouseCursor::ResizeLeftRight );
	}

	if (ViewInputMin.Get() > TrackMinValue.Get() || ViewInputMax.Get() < TrackMaxValue.Get())
	{
		return FCursorReply::Cursor(EMouseCursor::GrabHand);
	}

	return FCursorReply::Unhandled();
}

/** Update DraggableBarIndex but seeing if the mouse is over a draggable bar */
void STrack::UpdateDraggableBarIndex( const FGeometry& MyGeometry, FVector2D CursorScreenPos )
{
	FVector2D CursorPos = MyGeometry.AbsoluteToLocal(CursorScreenPos);

	DraggableBarIndex = INDEX_NONE;
	for ( int32 I=0; DraggableBars.IsBound() && I < DraggableBars.Get().Num(); I++ )
	{
		if( FMath::Abs( DataToLocalX(DraggableBars.Get()[I], MyGeometry) - CursorPos.X ) < 10 )
		{
			DraggableBarIndex = I;
			break;
		}
	}
}

/** Returns Data (Time, etc) to Local cordinate X */
float STrack::DataToLocalX( float Data, const FGeometry& MyGeometry ) const
{
	FTrackScaleInfo ScaleInfo(ViewInputMin.Get(), ViewInputMax.Get(), 0, 0, MyGeometry.GetLocalSize());
	return ScaleInfo.InputToLocalX(Data);
}

/** Returns Local cordinate X to Data (Time, etc) */
float STrack::LocalToDataX( float Input, const FGeometry& MyGeometry ) const
{
	FTrackScaleInfo ScaleInfo(ViewInputMin.Get(), ViewInputMax.Get(), 0, 0, MyGeometry.GetLocalSize());
	return ScaleInfo.LocalXToInput(Input);
}

/** Add child TrackNode to this Track */
void STrack::AddTrackNode( TSharedRef<STrackNode> Node )
{
	TrackNodes.Add(Node);
}

void STrack::ClearTrack()
{
	TrackNodes.Empty();
}

void STrack::GetSelectedNodeIndices(TArray<int32>& OutIndices)
{
	OutIndices.Empty();
	for(int32 i = 0 ; i < TrackNodes.Num() ; ++i)
	{
		TSharedRef<STrackNode> Node = TrackNodes[i];
		if(Node->IsSelected())
		{
			OutIndices.Add(i);
		}
	}
}

#undef LOCTEXT_NAMESPACE
