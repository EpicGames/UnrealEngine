// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/PlatformMath.h"
#include "Input/CursorReply.h"
#include "Input/DragAndDrop.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Layout/Children.h"
#include "Layout/Geometry.h"
#include "Math/Color.h"
#include "Math/Vector2D.h"
#include "Misc/Attribute.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SPanel.h"
#include "DragAndDrop/AssetDragDropOp.h"

class FArrangedChildren;
class FChildren;
class FMenuBuilder;
class FPaintArgs;
class FSlateRect;
class FSlateWindowElementList;
class FUICommandList;
class FWidgetStyle;
class SWidget;
class UObject;
class UAnimSequenceBase;
struct FPointerEvent;
struct FSlateBrush;

//////////////////////////////////////////////////////////////////////////
DECLARE_DELEGATE_OneParam( FOnSelectionChanged, const TArray<UObject*>& )
DECLARE_DELEGATE( FOnNodeSelectionChanged )
DECLARE_DELEGATE( FOnUpdatePanel )

DECLARE_DELEGATE_RetVal_TwoParams( bool, FOnGetBarPos, int32, float& )
DECLARE_DELEGATE_OneParam( FOnBarClicked, int32)
DECLARE_DELEGATE_TwoParams( FOnBarDrag, int32, float)
DECLARE_DELEGATE_OneParam( FOnBarDrop, int32 )
DECLARE_DELEGATE_TwoParams( FOnTrackDragDop, TSharedPtr<FDragDropOperation>, float )
DECLARE_DELEGATE_RetVal_OneParam( bool, FOnAssetDragDrop, TSharedPtr<FAssetDragDropOp>)
DECLARE_DELEGATE_RetVal_TwoParams(bool, FIsAnimAssetValid, const UAnimSequenceBase*, FText* OutReason)

DECLARE_DELEGATE_RetVal( FString, FOnGetNodeName )
DECLARE_DELEGATE_OneParam( FOnTrackNodeDragged, float )
DECLARE_DELEGATE( FOnTrackNodeDropped )
DECLARE_DELEGATE( FOnTrackNodeClicked )
DECLARE_DELEGATE( FOnTrackNodeDoubleClicked )

DECLARE_DELEGATE_RetVal_TwoParams( TSharedPtr<SWidget>, FOnSummonContextMenu, const FGeometry&, const FPointerEvent& );
DECLARE_DELEGATE_ThreeParams( FOnTrackRightClickContextMenu, FMenuBuilder&, float, int32 )
DECLARE_DELEGATE_OneParam( FOnNodeRightClickContextMenu, FMenuBuilder& )

typedef TSet<const class STrackNode*> STrackNodeSelectionSet;

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
class FTrackNodeDragDropOp : public FDragDropOperation
{
public:
	DRAG_DROP_OPERATOR_TYPE(FTrackNodeDragDropOp, FDragDropOperation)

	virtual void OnDrop( bool bDropWasHandled, const FPointerEvent& MouseEvent ) override;

	virtual void OnDragged( const class FDragDropEvent& DragDropEvent ) override;

	static TSharedRef<FTrackNodeDragDropOp> New(TSharedRef<STrackNode> TrackNode, const FVector2D &CursorPosition, const FVector2D &ScreenPositionOfNode);

	/** Gets the widget that will serve as the decorator unless overridden. If you do not override, you will have no decorator */
	virtual TSharedPtr<SWidget> GetDefaultDecorator() const override;

protected:
	/** The window that shows hover text */
	FVector2D						Offset;
	FVector2D						StartingScreenPos;
	TWeakPtr<class STrackNode>		OriginalTrackNode;
	TWeakPtr<class STrack>			OriginalTrack;

	FString GetHoverText() const
	{
		FString HoverText = TEXT("Invalid");
		return HoverText;
	}

	friend class STrack;
	friend class STrackNode;
};

/** class STrackNode for STrack. 
 * This is Children for STrack
 */
class STrackNode : public SCompoundWidget 
{
public:
	SLATE_BEGIN_ARGS( STrackNode )
		: _ViewInputMin()
		, _ViewInputMax()
		, _DataLength()
		, _DataStartPos()
		, _NodeName()
		, _NodeColor()
		, _SelectedNodeColor()
		, _NodeSelectionSet(NULL)
		, _AllowDrag(true)
		, _OnTrackNodeDragged()
		, _OnTrackNodeDropped()
		, _OnSelectionChanged()
		, _OnNodeRightClickContextMenu()
		, _OnTrackNodeClicked()
		, _OnTrackNodeDoubleClicked()
		, _CenterOnPosition(false)
	{
	}

	// have delegates for update functions
	SLATE_ATTRIBUTE( float, ViewInputMin )
	SLATE_ATTRIBUTE( float, ViewInputMax )
	SLATE_ATTRIBUTE( float, DataLength )
	SLATE_ATTRIBUTE( float, DataStartPos )
	SLATE_ATTRIBUTE( FString, NodeName )
	SLATE_ATTRIBUTE( FLinearColor, NodeColor )
	SLATE_ATTRIBUTE( FLinearColor, SelectedNodeColor )
	SLATE_ARGUMENT( STrackNodeSelectionSet *, NodeSelectionSet )
	SLATE_ARGUMENT( bool, AllowDrag )
	SLATE_EVENT( FOnTrackNodeDragged, OnTrackNodeDragged )
	SLATE_EVENT( FOnTrackNodeDropped, OnTrackNodeDropped )
	SLATE_EVENT( FOnNodeSelectionChanged, OnSelectionChanged )
	SLATE_EVENT( FOnNodeRightClickContextMenu, OnNodeRightClickContextMenu )
	SLATE_EVENT( FOnTrackNodeClicked, OnTrackNodeClicked )
	SLATE_EVENT( FOnTrackNodeDoubleClicked, OnTrackNodeDoubleClicked)
	SLATE_ARGUMENT( bool, CenterOnPosition )
	SLATE_NAMED_SLOT(FArguments, OverrideContent)

	SLATE_END_ARGS()

	void Construct(const FArguments& Declaration);

	// mouse interface for tooltip/selection
	virtual FReply OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	virtual FReply OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	virtual void OnMouseEnter( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	virtual void OnMouseLeave( const FPointerEvent& MouseEvent ) override;
	virtual void OnDragged(const class FDragDropEvent& DragDropEvent );
	virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;

	// virtual draw related functions
	virtual FVector2D GetOffsetRelativeToParent(const FGeometry& ParentAllottedGeometry) const;
	virtual FVector2D GetSizeRelativeToParent(const FGeometry& ParentAllottedGeometry) const;
	
	// drag drop relationship
	virtual FReply OnDragDetected( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	virtual FReply OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;

	void OnDropCancelled(const FPointerEvent& MouseEvent);

	virtual FReply BeginDrag( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent );

	virtual FVector2D GetDragDropScreenSpacePosition(const FGeometry& ParentAllottedGeometry, const FDragDropEvent& DragDropEvent) const;

	bool HitTest(const FGeometry& AllottedGeometry, FVector2D MouseLocalPose) const;

	virtual FVector2D GetSize() const;
	
	float GetDataStartPos() const;

	/** Return whether this node should snap to the tracks draggable bars when being dragged */
	virtual bool SnapToDragBars() const {return false;}

	/** Called when the nodes position has been 'snapped' to something */
	virtual void OnSnapNodeDataPosition(float OriginalX, float SnappedX) {}

	/** Cache the supplied geometry as our track geometry */
	void CacheTrackGeometry(const FGeometry& TrackGeometry) { CachedTrackGeometry = TrackGeometry; }

	const FGeometry& GetTrackGeometry() const {return CachedTrackGeometry;}

	bool IsBeingDragged() const {return bBeingDragged;}

protected:
	// Begin SWidget overrides.
	virtual FVector2D ComputeDesiredSize(float) const override;
	// End SWidget overrides.
	
	FSlateColor GetNodeColor() const;

	FText GetNodeText() const; // Temp, remove and replace with correct attribute

	void	ToggleSelect();
	void	Select();
	void	Deselect();
	bool	IsSelected() const;

	
	bool						bSelectedFallback; // only use this if the selection set isnt set
	STrackNodeSelectionSet *	NodeSelectionSet;

	TAttribute<FString>			NodeName;
	TAttribute<float>			DataStartPos;
	TAttribute<float>			DataLength;

	TAttribute<float>			ViewInputMin;
	TAttribute<float>			ViewInputMax;
	FOnNodeSelectionChanged		OnNodeSelectionChanged;

	TAttribute<FLinearColor>	NodeColor;
	TAttribute<FLinearColor>	SelectedNodeColor;

	FOnTrackNodeDragged			OnTrackNodeDragged;
	FOnTrackNodeDropped			OnTrackNodeDropped;
	
	FOnTrackNodeClicked				OnTrackNodeClicked;
	FOnTrackNodeDoubleClicked		OnTrackNodeDoubleClicked;
	FOnNodeRightClickContextMenu	OnNodeRightClickContextMenu;
	
	FSlateFontInfo Font;

	mutable FVector2D	LastSize; //HACK Fixme: Need to update in OnPaint in case we are drag/dropped
	FGeometry			CachedTrackGeometry; //Our parent tracks geometry, so we can calculate scale/position
	
	float				DistanceDragged;
	bool				bBeingDragged;
	bool				bCenterOnPosition;
	bool				bAllowDrag;
	bool				bContentOverriden;

	friend class STrack;
};

//////////////////////////////////////////////////////////////////////////
// STrack

class STrack : public SPanel
{
public:

	SLATE_BEGIN_ARGS( STrack )
		: _ViewInputMin()
		, _ViewInputMax()
		, _TrackMaxValue()
		, _TrackMinValue()
		, _TrackNumDiscreteValues()
		, _TrackColor(FLinearColor::White)
		, _OnSelectionChanged()
		, _DraggableBars()
		, _DraggableBarSnapPositions()
		, _DraggableBarLabels()
		, _OnBarDrag()
		, _OnTrackDragDrop()
		, _OnSummonContextMenu()
		, _OnTrackRightClickContextMenu()
		, _StyleInfo(FAppStyle::GetBrush( TEXT( "Persona.NotifyEditor.NotifyTrackBackground" )))
	{}

	SLATE_ATTRIBUTE( float, ViewInputMin )
		SLATE_ATTRIBUTE( float, ViewInputMax )
		SLATE_ATTRIBUTE( float, TrackMaxValue )
		SLATE_ATTRIBUTE( float, TrackMinValue )
		SLATE_ATTRIBUTE( int32, TrackNumDiscreteValues )
		SLATE_ARGUMENT( FLinearColor, TrackColor )
		SLATE_EVENT( FOnNodeSelectionChanged, OnSelectionChanged )
		SLATE_ATTRIBUTE( TArray<float>, DraggableBars )
		SLATE_ATTRIBUTE( TArray<float>, DraggableBarSnapPositions )
		SLATE_ATTRIBUTE( TArray<FString>, DraggableBarLabels )
		SLATE_EVENT( FOnBarDrag, OnBarDrag)
		SLATE_EVENT( FOnBarClicked, OnBarClicked)
		SLATE_EVENT( FOnTrackDragDop, OnTrackDragDrop )
		SLATE_EVENT( FOnAssetDragDrop, OnAssetDragDrop )
		SLATE_EVENT( FOnBarDrop, OnBarDrop )
		SLATE_EVENT( FOnSummonContextMenu, OnSummonContextMenu )
		SLATE_EVENT( FOnTrackRightClickContextMenu, OnTrackRightClickContextMenu )
		SLATE_ARGUMENT( const FSlateBrush*, StyleInfo )
	SLATE_END_ARGS()

	STrack();

	void			Construct( const FArguments& InArgs );

	virtual void	OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const override;
	virtual int32	OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override;

	virtual FReply	OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;	
	virtual FReply	OnDragOver( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;
	virtual FReply	OnDragDetected( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	virtual void OnDragLeave(const FDragDropEvent& DragDropEvent) override;

	int32			GetHitNode(const FGeometry& MyGeometry, const FVector2D& CursorPosition);

	FReply			OnMouseMove( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	FReply			OnMouseButtonDown( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	FReply			OnMouseButtonUp( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	
	virtual void OnMouseLeave( const FPointerEvent& MouseEvent ) override;
	virtual void OnMouseEnter( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent ) override;
	virtual FCursorReply OnCursorQuery( const FGeometry& MyGeometry, const FPointerEvent& CursorEvent ) const override;

	void				AddTrackNode( TSharedRef<STrackNode> Node );
	void				ClearTrack();
	FVector2D			ComputeDesiredSize(float) const override;
	virtual FChildren*	GetChildren() override;

	void GetSelectedNodeIndices(TArray<int32>& OutIndices);

protected:
	
	bool									GetDraggableBarSnapPosition(const FGeometry& MyGeometry, float &OutPosition) const;
	virtual TSharedPtr<SWidget>				SummonContextMenu(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);
	float									GetNodeDragDropDataPos( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) const;
	float									GetSnappedPosForLocalPos( const FGeometry& MyGeometry, float TrackPos) const;
	void									UpdateDraggableBarIndex( const FGeometry& MyGeometry, FVector2D CursorScreenPos );
	float									DataToLocalX( float Data, const FGeometry& MyGeometry ) const;
	float									LocalToDataX( float Input, const FGeometry& MyGeometry ) const;
	
	TSlotlessChildren< STrackNode >			TrackNodes;

	TAttribute<TArray<float>>				DraggableBars;
	TAttribute<TArray<float>>				DraggableBarSnapPositions;
	TAttribute<TArray<FString>>				DraggableBarLabels;
	FOnBarDrag								OnBarDrag;
	FOnBarDrop								OnBarDrop;
	FOnBarClicked							OnBarClicked;
	TAttribute<FLinearColor>				DraggableBarColor;
	int32									DraggableBarIndex;
	bool									bDraggingBar;

	TAttribute<float>						TrackMaxValue;
	TAttribute<float>						TrackMinValue;
	TAttribute<int32>						TrackNumDiscreteValues; // Discrete values (such as "number of frames in animation" used for accurate grid lines)

	TAttribute<float>						ViewInputMin;
	TAttribute<float>						ViewInputMax;
	TAttribute<FLinearColor>				TrackColor;
	FOnNodeSelectionChanged					OnSelectionChanged;

	FOnGetBarPos							OnGetDraggableBarPos;
	FOnTrackDragDop							OnTrackDragDrop;
	FOnAssetDragDrop						OnAssetDragDrop;
	
	TAttribute<const FSlateBrush*>			StyleInfo;
	FOnSummonContextMenu					OnSummonContextMenu;
	FOnTrackRightClickContextMenu			OnTrackRightClickContextMenu;

	TSharedPtr<FUICommandList> EditorActions;
	FSlateFontInfo Font;
};
