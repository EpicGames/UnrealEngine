// Copyright Epic Games, Inc. All Rights Reserved.


#include "BoneSelectionWidget.h"
#include "Styling/AppStyle.h"
#include "DetailLayoutBuilder.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "IEditableSkeleton.h"
#include "Engine/SkeletalMeshSocket.h"

#define LOCTEXT_NAMESPACE "SBoneSelectionWidget"

/////////////////////////////////////////////////////
void SBoneTreeMenu::Construct(const FArguments& InArgs)
{
	OnSelectionChangedDelegate = InArgs._OnBoneSelectionChanged;
	OnGetReferenceSkeletonDelegate = InArgs._OnGetReferenceSkeleton;
	OnGetSocketListDelegate = InArgs._OnGetSocketList;
	bShowVirtualBones = InArgs._bShowVirtualBones;
	bShowSocket = InArgs._bShowSocket;
	bShowNone = InArgs._bShowNone;

	FText TitleToUse = !InArgs._Title.IsEmpty() ? InArgs._Title  : LOCTEXT("BonePickerTitle", "Select...");

	SAssignNew(TreeView, STreeView<TSharedPtr<FBoneNameInfo>>)
		.TreeItemsSource(&SkeletonTreeInfo)
		.OnGenerateRow(this, &SBoneTreeMenu::MakeTreeRowWidget)
		.OnGetChildren(this, &SBoneTreeMenu::GetChildrenForInfo)
		.OnSelectionChanged(this, &SBoneTreeMenu::OnSelectionChanged)
		.SelectionMode(ESelectionMode::Single);

	RebuildBoneList(InArgs._SelectedBone);

	ChildSlot
	[
		SNew(SBorder)
		.Padding(6.f)
		.BorderImage(FAppStyle::GetBrush("NoBorder"))
		.Content()
		[
			SNew(SBox)
			.WidthOverride(300.f)
			.HeightOverride(512.f)
			.Content()
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(STextBlock)
					.Font(FAppStyle::GetFontStyle("BoldFont"))
					.Text(TitleToUse)
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SSeparator)
					.SeparatorImage(FAppStyle::GetBrush("Menu.Separator"))
					.Orientation(Orient_Horizontal)
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SAssignNew(FilterTextWidget, SSearchBox)
					.SelectAllTextWhenFocused(true)
					.OnTextChanged(this, &SBoneTreeMenu::OnFilterTextChanged)
					.HintText(NSLOCTEXT("BonePicker", "Search", "Search..."))
				]
				+ SVerticalBox::Slot()
				[
					TreeView->AsShared()
				]
			]
		]
	];
}

TSharedPtr<SWidget> SBoneTreeMenu::GetFilterTextWidget()
{
	return FilterTextWidget;
}

TSharedRef<ITableRow> SBoneTreeMenu::MakeTreeRowWidget(TSharedPtr<FBoneNameInfo> InInfo, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(STableRow<TSharedPtr<FBoneNameInfo>>, OwnerTable)
		.Content()
		[
			SNew(STextBlock)
			.HighlightText(FilterText)
			.Text(FText::FromName(InInfo->BoneName))
		];
}

void SBoneTreeMenu::GetChildrenForInfo(TSharedPtr<FBoneNameInfo> InInfo, TArray< TSharedPtr<FBoneNameInfo> >& OutChildren)
{
	OutChildren = InInfo->Children;
}

void SBoneTreeMenu::OnFilterTextChanged(const FText& InFilterText)
{
	FilterText = InFilterText;

	RebuildBoneList(NAME_None);
}

void SBoneTreeMenu::OnSelectionChanged(TSharedPtr<SBoneTreeMenu::FBoneNameInfo> BoneInfo, ESelectInfo::Type SelectInfo)
{
	//Because we recreate all our items on tree refresh we will get a spurious null selection event initially.
	if (BoneInfo.IsValid() && SelectInfo == ESelectInfo::OnMouseClick)
	{
		SelectBone(BoneInfo);
	}
}

void SBoneTreeMenu::SelectBone(TSharedPtr<SBoneTreeMenu::FBoneNameInfo> BoneInfo)
{
	OnSelectionChangedDelegate.ExecuteIfBound(BoneInfo->BoneName);
}

FReply SBoneTreeMenu::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if(InKeyEvent.GetKey() == EKeys::Enter)
	{
		TArray<TSharedPtr<SBoneTreeMenu::FBoneNameInfo>> SelectedItems = TreeView->GetSelectedItems();
		if(SelectedItems.Num() > 0)
		{
			SelectBone(SelectedItems[0]);
			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

void SBoneTreeMenu::RebuildBoneList(const FName& SelectedBone)
{
	SkeletonTreeInfo.Empty();
	SkeletonTreeInfoFlat.Empty();

	if (bShowNone)
	{
		TSharedRef<FBoneNameInfo> NoneBoneInfo = MakeShared<FBoneNameInfo>(NAME_None);
		SkeletonTreeInfo.Add(NoneBoneInfo);
		SkeletonTreeInfoFlat.Add(NoneBoneInfo);
	}

	if (ensure(OnGetReferenceSkeletonDelegate.IsBound()))
	{
		const FReferenceSkeleton& RefSkeleton = OnGetReferenceSkeletonDelegate.Execute();
		const int32 MaxBone = bShowVirtualBones ? RefSkeleton.GetNum() : RefSkeleton.GetRawBoneNum();

		for (int32 BoneIdx = 0; BoneIdx < MaxBone; ++BoneIdx)
		{
			const FName BoneName = RefSkeleton.GetBoneName(BoneIdx);
			TSharedRef<FBoneNameInfo> BoneInfo = MakeShareable(new FBoneNameInfo(BoneName));

			// Filter if Necessary
			if (!FilterText.IsEmpty() && !BoneInfo->BoneName.ToString().Contains(FilterText.ToString()))
			{
				continue;
			}

			int32 ParentIdx = RefSkeleton.GetParentIndex(BoneIdx);
			bool bAddToParent = false;

			if (ParentIdx != INDEX_NONE && FilterText.IsEmpty())
			{
				// We have a parent, search for it in the flat list
				FName ParentName = RefSkeleton.GetBoneName(ParentIdx);

				for (int32 FlatListIdx = 0; FlatListIdx < SkeletonTreeInfoFlat.Num(); ++FlatListIdx)
				{
					TSharedPtr<FBoneNameInfo> InfoEntry = SkeletonTreeInfoFlat[FlatListIdx];
					if (InfoEntry->BoneName == ParentName)
					{
						bAddToParent = true;
						ParentIdx = FlatListIdx;
						break;
					}
				}

				if (bAddToParent)
				{
					SkeletonTreeInfoFlat[ParentIdx]->Children.Add(BoneInfo);
				}
				else
				{
					SkeletonTreeInfo.Add(BoneInfo);
				}
			}
			else
			{
				SkeletonTreeInfo.Add(BoneInfo);
			}

			SkeletonTreeInfoFlat.Add(BoneInfo);
			TreeView->SetItemExpansion(BoneInfo, true);
			if (BoneName == SelectedBone)
			{
				TreeView->SetItemSelection(BoneInfo, true);
				TreeView->RequestScrollIntoView(BoneInfo);
			}
		}
	}


	if (bShowSocket && ensure(OnGetSocketListDelegate.IsBound()))
	{
		const TArray<class USkeletalMeshSocket*>& Sockets = OnGetSocketListDelegate.Execute();
		const int32 MaxSockets = Sockets.Num();

		for (int32 SocketIdx = 0; SocketIdx < MaxSockets; ++SocketIdx)
		{
			const USkeletalMeshSocket* Socket = Sockets[SocketIdx];
			const FName SocketName = Socket->SocketName;
			TSharedRef<FBoneNameInfo> BoneInfo = MakeShareable(new FBoneNameInfo(SocketName));

			// Filter if Necessary
			if (!FilterText.IsEmpty() && !BoneInfo->BoneName.ToString().Contains(FilterText.ToString()))
			{
				continue;
			}

			const FName ParentBoneName = Socket->BoneName;
			if (FilterText.IsEmpty())
			{
				for (int32 FlatListIdx = 0; FlatListIdx < SkeletonTreeInfoFlat.Num(); ++FlatListIdx)
				{
					TSharedPtr<FBoneNameInfo> InfoEntry = SkeletonTreeInfoFlat[FlatListIdx];
					if (InfoEntry->BoneName == ParentBoneName)
					{
						SkeletonTreeInfoFlat[FlatListIdx]->Children.Add(BoneInfo);
						break;
					}
				}
			}
			else
			{
				SkeletonTreeInfo.Add(BoneInfo);
			}

			TreeView->SetItemExpansion(BoneInfo, true);
			if (SocketName == SelectedBone)
			{
				TreeView->SetItemSelection(BoneInfo, true);
				TreeView->RequestScrollIntoView(BoneInfo);
			}
		}
	}
	TreeView->RequestTreeRefresh();
}

/////////////////////////////////////////////////////

void SBoneSelectionWidget::Construct(const FArguments& InArgs)
{
	OnBoneSelectionChanged = InArgs._OnBoneSelectionChanged;
	OnGetSelectedBone = InArgs._OnGetSelectedBone;
	OnGetReferenceSkeleton = InArgs._OnGetReferenceSkeleton;
	OnGetSocketList = InArgs._OnGetSocketList;
	bShowSocket = InArgs._bShowSocket;
	bShowVirtualBones = InArgs._bShowVirtualBones;
	bShowNone = InArgs._bShowNone;

	SuppliedToolTip = InArgs._ToolTipText.Get();

	this->ChildSlot
	[
		SAssignNew(BonePickerButton, SComboButton)
		.OnGetMenuContent(FOnGetContent::CreateSP(this, &SBoneSelectionWidget::CreateSkeletonWidgetMenu))
		.ContentPadding(FMargin(4.0f, 2.0f, 4.0f, 2.0f))
		.ButtonContent()
		[
			SNew(STextBlock)
			.Text(this, &SBoneSelectionWidget::GetCurrentBoneName)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.ToolTipText(this, &SBoneSelectionWidget::GetFinalToolTip)
		]
	];
}

TSharedRef<SWidget> SBoneSelectionWidget::CreateSkeletonWidgetMenu()
{
	bool bMultipleValues = false;
	FName CurrentBoneName;
	if (OnGetSelectedBone.IsBound())
	{
		CurrentBoneName = OnGetSelectedBone.Execute(bMultipleValues);
	}

	TSharedRef<SBoneTreeMenu> MenuWidget = SNew(SBoneTreeMenu)
		.OnBoneSelectionChanged(this, &SBoneSelectionWidget::OnSelectionChanged)
		.OnGetReferenceSkeleton(OnGetReferenceSkeleton)
		.OnGetSocketList(OnGetSocketList)
		.bShowSocket(bShowSocket)
		.bShowNone(bShowNone)
		.SelectedBone(CurrentBoneName)
		.bShowVirtualBones(bShowVirtualBones);

	BonePickerButton->SetMenuContentWidgetToFocus(MenuWidget->GetFilterTextWidget());

	return MenuWidget;
}

void SBoneSelectionWidget::OnSelectionChanged(FName BoneName)
{
	//Because we recreate all our items on tree refresh we will get a spurious null selection event initially.
	if (OnBoneSelectionChanged.IsBound())
	{
		OnBoneSelectionChanged.Execute(BoneName);
	}

	BonePickerButton->SetIsOpen(false);
}

FText SBoneSelectionWidget::GetCurrentBoneName() const
{
	if(OnGetSelectedBone.IsBound())
	{
		bool bMultipleValues = false;
		FName Name = OnGetSelectedBone.Execute(bMultipleValues);
		if(bMultipleValues)
		{
			return LOCTEXT("MultipleValues", "Multiple Values");
		}
		else
		{
			return FText::FromName(Name);
		}
	}

	// @todo implement default solution?
	return FText::GetEmpty();
}

FText SBoneSelectionWidget::GetFinalToolTip() const
{
	return FText::Format(LOCTEXT("BoneClickToolTip", "Bone:{0}\n\n{1}\nClick to choose a different bone"), GetCurrentBoneName(), SuppliedToolTip);
}
#undef LOCTEXT_NAMESPACE

