// Copyright Epic Games, Inc. All Rights Reserved.


#include "ReferenceSectionSelectionWidget.h"

#include "DetailLayoutBuilder.h"
#include "Styling/AppStyle.h"
#include "Rendering/SkeletalMeshLODModel.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SSeparator.h"

#define LOCTEXT_NAMESPACE "SReferenceSectionSelectionWidget"

void SReferenceSectionSelectionWidget::Construct(const FArguments& InArgs)
{
	OnSectionSelectionChanged = InArgs._OnSectionSelectionChanged;
	OnGetSelectedSection = InArgs._OnGetSelectedSection;
	OnGetLodModel = InArgs._OnGetLodModel;
	bHideChunkedSections = InArgs._bHideChunkedSections;

	this->ChildSlot
	[
		SAssignNew(SectionPickerButton, SComboButton)
		.OnGetMenuContent(FOnGetContent::CreateSP(this, &SReferenceSectionSelectionWidget::CreateSectionListWidgetMenu))
		.ContentPadding(FMargin(4.0f, 2.0f, 4.0f, 2.0f))
		.ButtonContent()
		[
			SNew(STextBlock)
			.Text(this, &SReferenceSectionSelectionWidget::GetCurrentSectionIndex)
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
	];
}

const TArray<TSharedPtr<int32>>* SReferenceSectionSelectionWidget::GetSections() const
{
	CacheSectionList.Reset();
	if (OnGetLodModel.IsBound())
	{
		const FSkeletalMeshLODModel& LodModel = OnGetLodModel.Execute();
		for (int32 SectionIndex = 0; SectionIndex < LodModel.Sections.Num(); ++SectionIndex)
		{
			if (LodModel.Sections[SectionIndex].ChunkedParentSectionIndex != INDEX_NONE && bHideChunkedSections)
			{
				continue;
			}
			CacheSectionList.Add(MakeShareable(new int32(SectionIndex)));
		}
	}
	return &CacheSectionList;
}

TSharedRef<ITableRow> SReferenceSectionSelectionWidget::MakeIntegerDisplayWidget(const TSharedPtr<int32> SectionIndex, const TSharedRef<STableViewBase>& OwnerTable) const
{
	const int32 SectionIndexValue = *SectionIndex;
	FString ItemString = FString::FromInt(SectionIndexValue);
	return SNew(SComboRow< TSharedRef<FString> >, OwnerTable)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(1)
			.Padding(2.0f)
			[
				SNew(STextBlock)
				.Text(FText::FromString(ItemString))
				.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
			]
		];
}

TSharedRef<SWidget> SReferenceSectionSelectionWidget::CreateSectionListWidgetMenu()
{
	bool bMultipleValues = false;
	int32 CurrentSectionIndex;
	if (OnGetSelectedSection.IsBound())
	{
		CurrentSectionIndex = OnGetSelectedSection.Execute(bMultipleValues);
	}

	TSharedPtr<SHeaderRow> PlatformColumnHeader = SNew(SHeaderRow);


	//Create a list widget containing all possible section we can reference
	SAssignNew(SectionListView, SListView<TSharedPtr<int32>>)
		.ListItemsSource(GetSections())
		.OnGenerateRow(this, &SReferenceSectionSelectionWidget::MakeIntegerDisplayWidget)
		.Visibility(EVisibility::Visible)
		.SelectionMode(ESelectionMode::Single)
		.OnSelectionChanged(this, &SReferenceSectionSelectionWidget::OnSelectionChanged)
		.HeaderRow(PlatformColumnHeader.ToSharedRef());

	const FText TitleText = FText(LOCTEXT("ReferenceListWidgetMenuTitle", "Choose a Section"));
	TSharedPtr<SBorder> BorderWidget;
	SAssignNew(BorderWidget, SBorder)
	.Padding(6.f)
	.BorderImage(FAppStyle::GetBrush("NoBorder"))
	.Content()
	[
		SNew(SBox)
		.WidthOverride(300.f)
		.HeightOverride(512.f)
		.Content()
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(STextBlock)
				.Font(FAppStyle::GetFontStyle("BoldFont"))
				.Text(TitleText)
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SSeparator)
				.SeparatorImage(FAppStyle::GetBrush("Menu.Separator"))
				.Orientation(Orient_Horizontal)
			]
			+ SVerticalBox::Slot()
			[
				SectionListView->AsShared()
			]
		]
	];

	return BorderWidget.ToSharedRef();
}

void SReferenceSectionSelectionWidget::OnSelectionChanged(TSharedPtr<int32> NewSectionIndex, ESelectInfo::Type SelectInfo)
{
	const int32 SectionIndex = *NewSectionIndex;
	//Because we recreate all our items on tree refresh we will get a spurious null selection event initially.
	if (OnSectionSelectionChanged.IsBound() && OnGetLodModel.IsBound())
	{
		const FSkeletalMeshLODModel& LodModel = OnGetLodModel.Execute();
		int32 SaveSectionIndex = SectionIndex;
		if (LodModel.Sections.IsValidIndex(SectionIndex))
		{
			SaveSectionIndex = LodModel.Sections[SectionIndex].OriginalDataSectionIndex;
		}
		OnSectionSelectionChanged.Execute(SaveSectionIndex);
	}

	SectionPickerButton->SetIsOpen(false);
}

FText SReferenceSectionSelectionWidget::GetCurrentSectionIndex() const
{
	if(OnGetSelectedSection.IsBound())
	{
		bool bMultipleValues = false;
		int32 CurrentSectionIndexValue = OnGetSelectedSection.Execute(bMultipleValues);
		if(bMultipleValues)
		{
			return LOCTEXT("MultipleValues", "Multiple Values");
		}
		else
		{
			int32 DisplaySectionIndex = CurrentSectionIndexValue;
			if (bHideChunkedSections && OnGetLodModel.IsBound())
			{
				const FSkeletalMeshLODModel& LodModel = OnGetLodModel.Execute();
				for (int32 SectionIndex = 0; SectionIndex < LodModel.Sections.Num(); ++SectionIndex)
				{
					if (LodModel.Sections[SectionIndex].ChunkedParentSectionIndex != INDEX_NONE && bHideChunkedSections)
					{
						continue;
					}
					if (LodModel.Sections[SectionIndex].OriginalDataSectionIndex == CurrentSectionIndexValue)
					{
						DisplaySectionIndex = SectionIndex;
						break;
					}
				}
			}
			if (DisplaySectionIndex == INDEX_NONE)
			{
				return LOCTEXT("GetCurrentSectionIndexTextNone", "Section None");
			}
			return FText::Format(LOCTEXT("GetCurrentSectionIndexText", "Section {0}"), DisplaySectionIndex);
		}
	}

	// @todo implement default solution?
	return FText::GetEmpty();
}

#undef LOCTEXT_NAMESPACE

