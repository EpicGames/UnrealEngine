// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletalMeshReferenceSectionDetails.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "Styling/AppStyle.h"
#include "Engine/SkeletalMesh.h"
#include "IDetailCustomization.h"
#include "IDetailPropertyRow.h"
#include "IDetailsView.h"
#include "IPropertyTypeCustomization.h"
#include "DetailWidgetRow.h"
#include "Modules/ModuleManager.h"
#include "PropertyHandle.h"
#include "ReferenceSectionSelectionWidget.h"
#include "Rendering/SkeletalMeshLODModel.h"
#include "Rendering/SkeletalMeshModel.h"
#include "Templates/SharedPointer.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"


//Private persona include
#include "LODInfoUILayout.h"

TSharedRef<IPropertyTypeCustomization> FSectionReferenceCustomization::MakeInstance()
{
	return MakeShareable(new FSectionReferenceCustomization());
}

void FSectionReferenceCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	// set property handle 
	SetPropertyHandle(StructPropertyHandle);
	// set editable LodModel info from struct
	SetEditableLodModel(StructPropertyHandle);
	if (TargetSkeletalMesh != nullptr && TargetLodIndex != INDEX_NONE && SectionIndexProperty->IsValidHandle())
	{
		HeaderRow
		.NameContent()
		[
			StructPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MaxDesiredWidth(0.0f)
		[
			SNew(SReferenceSectionSelectionWidget)
			.ToolTipText(StructPropertyHandle->GetToolTipText())
			.OnSectionSelectionChanged(this, &FSectionReferenceCustomization::OnSectionSelectionChanged)
			.OnGetSelectedSection(this, &FSectionReferenceCustomization::GetSelectedSection)
			.OnGetLodModel(this, &FSectionReferenceCustomization::GetLodModel)
			.bHideChunkedSections(true)
		];
	}
	else
	{
		const bool bEnsureOnInvalidLodModel = true;
		ensureAlways(!bEnsureOnInvalidLodModel);
	}
}

void FSectionReferenceCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{

}

void FSectionReferenceCustomization::SetEditableLodModel(TSharedRef<IPropertyHandle> StructPropertyHandle)
{
	TArray<UObject*> Objects;
	StructPropertyHandle->GetOuterObjects(Objects);

	TargetSkeletalMesh = nullptr;
	TargetLodIndex = INDEX_NONE;

	for (UObject* Outer : Objects)
	{
		//We customize only when we have a ULODInfoUILayout has the parent because we need the lod index to customize the UI
		if (ULODInfoUILayout* LODInfoUILayout = Cast<ULODInfoUILayout>(Outer))
		{
			TargetSkeletalMesh = LODInfoUILayout->GetPersonaToolkit()->GetPreviewMesh();
			check(TargetSkeletalMesh);
			TargetLodIndex = LODInfoUILayout->GetLODIndex();
			break;
		}
	}
}

TSharedPtr<IPropertyHandle> FSectionReferenceCustomization::FindStructMemberProperty(TSharedRef<IPropertyHandle> PropertyHandle, const FName& PropertyName)
{
	uint32 NumChildren = 0;
	PropertyHandle->GetNumChildren(NumChildren);
	for (uint32 ChildIdx = 0; ChildIdx < NumChildren; ++ChildIdx)
	{
		TSharedPtr<IPropertyHandle> ChildHandle = PropertyHandle->GetChildHandle(ChildIdx);
		if (ChildHandle->GetProperty()->GetFName() == PropertyName)
		{
			return ChildHandle;
		}
	}

	return TSharedPtr<IPropertyHandle>();
}

void FSectionReferenceCustomization::SetPropertyHandle(TSharedRef<IPropertyHandle> StructPropertyHandle)
{
	SectionIndexProperty = FindStructMemberProperty(StructPropertyHandle, GET_MEMBER_NAME_CHECKED(FSectionReference, SectionIndex));
	check(SectionIndexProperty->IsValidHandle());
}

void FSectionReferenceCustomization::OnSectionSelectionChanged(int32 SectionIndex)
{
	SectionIndexProperty->SetValue(SectionIndex);
}

int32 FSectionReferenceCustomization::GetSelectedSection(bool& bMultipleValues) const
{
	FString OutText;
	FPropertyAccess::Result Result = SectionIndexProperty->GetValueAsFormattedString(OutText);
	bMultipleValues = (Result == FPropertyAccess::MultipleValues);
	return FCString::Atoi(*OutText);
}

const FSkeletalMeshLODModel& FSectionReferenceCustomization::GetLodModel() const
{
	check(TargetSkeletalMesh && TargetLodIndex != INDEX_NONE && TargetSkeletalMesh->GetImportedModel() && TargetSkeletalMesh->GetImportedModel()->LODModels.IsValidIndex(TargetLodIndex));
	return TargetSkeletalMesh->GetImportedModel()->LODModels[TargetLodIndex];
}
