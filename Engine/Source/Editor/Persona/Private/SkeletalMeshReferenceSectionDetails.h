// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"
#include "IPropertyTypeCustomization.h"
#include "Types/SlateEnums.h"

class FSkeletalMeshLODModel;
class IDetailLayoutBuilder;
class IPropertyHandle;
class SEditableTextBox;
class USkeletalMesh;

class FSectionReferenceCustomization : public IPropertyTypeCustomization
{
public:
	virtual ~FSectionReferenceCustomization()
	{
		SectionIndexProperty = nullptr;
		TargetSkeletalMesh = nullptr;
	}

	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	// IPropertyTypeCustomization interface
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	const FSkeletalMeshLODModel& GetLodModel() const;

protected:
	void SetEditableLodModel(TSharedRef<IPropertyHandle> StructPropertyHandle);
	virtual void SetPropertyHandle(TSharedRef<IPropertyHandle> StructPropertyHandle);
	TSharedPtr<IPropertyHandle> FindStructMemberProperty(TSharedRef<IPropertyHandle> PropertyHandle, const FName& PropertyName);
	// Property to change after section index has been picked
	TSharedPtr<IPropertyHandle> SectionIndexProperty;

	// Target LodModel this widget is referencing
	USkeletalMesh* TargetSkeletalMesh = nullptr;
	int32 TargetLodIndex = INDEX_NONE;

private:

	// Section widget delegates
	virtual void OnSectionSelectionChanged(int32 SectionIndex);
	virtual int32 GetSelectedSection(bool& bMultipleValues) const;
};
