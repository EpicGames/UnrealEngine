// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletalMeshSocketDetails.h"
#include "Modules/ModuleManager.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Animation/Skeleton.h"
#include "Styling/AppStyle.h"
#include "DetailWidgetRow.h"
#include "IDetailPropertyRow.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "IDetailsView.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Engine/SkeletalMeshSocket.h"
#include "AssetSearchBoxUtilPersona.h"
#include "IEditableSkeleton.h"
#include "ISkeletonEditorModule.h"

TSharedRef<IDetailCustomization> FSkeletalMeshSocketDetails::MakeInstance()
{
	return MakeShareable( new FSkeletalMeshSocketDetails );
}

void FSkeletalMeshSocketDetails::CustomizeDetails( IDetailLayoutBuilder& DetailBuilder )
{
	TargetSocket = NULL;

	IDetailCategoryBuilder& SocketCategory = DetailBuilder.EditCategory( TEXT("Socket Parameters"), FText::GetEmpty(), ECategoryPriority::TypeSpecific );

	SocketNameProperty = DetailBuilder.GetProperty( TEXT("SocketName") );
	if( SocketNameProperty.IsValid() && SocketNameProperty->GetProperty() )
	{
		const TArray< TWeakObjectPtr<UObject> >& SelectedObjects = DetailBuilder.GetSelectedObjects();
		if( SelectedObjects.Num() == 1 )
		{
			const TWeakObjectPtr<UObject> BaseClass = SelectedObjects[0];
			const USkeletalMeshSocket* TargetSocketConst = Cast<const USkeletalMeshSocket> (BaseClass.Get());
			if( TargetSocketConst )
			{
				TargetSocket = const_cast<USkeletalMeshSocket*> (TargetSocketConst);
			}
		}

		if (TargetSocket)
		{
			// get the currently chosen bone/socket, if any
			const FText PropertyName = SocketNameProperty->GetPropertyDisplayName();
			FString CurValue;
			SocketNameProperty->GetValue(CurValue);
			if (CurValue == FString("None"))
			{
				CurValue.Empty();
			}
			PreEditSocketName = FText::FromString(CurValue);

			// create the editable text box
			SocketCategory.AddProperty(SocketNameProperty.ToSharedRef())
				.CustomWidget()
				.NameContent()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
				.Padding(FMargin(2, 1, 0, 1))
				[
					SNew(STextBlock)
					.Text(PropertyName)
				.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
				]
				]
			.ValueContent()
				[
					SAssignNew(SocketNameTextBox, SEditableTextBox)
					.Text(FText::FromString(CurValue))
				.HintText(NSLOCTEXT("SkeletalMeshSocketDetails", "SkeletalMeshSocketDetailsHintTextSocketName", "Socket Name..."))
				.OnTextCommitted(this, &FSkeletalMeshSocketDetails::OnSocketNameCommitted)
				.OnTextChanged(this, &FSkeletalMeshSocketDetails::OnSocketNameChanged)
				.ClearKeyboardFocusOnCommit(false)
				];
		}
	}

	ParentBoneProperty = DetailBuilder.GetProperty( TEXT("BoneName") );
	if( TargetSocket && ParentBoneProperty.IsValid() && ParentBoneProperty->GetProperty() )
	{
		if( const UObject* Outer = TargetSocket->GetOuter() )
		{
			SocketCategory.AddProperty( ParentBoneProperty.ToSharedRef() )
			.CustomWidget()
			.NameContent()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.Padding( FMargin( 2, 1, 0, 1 ) )
				[
					SNew(STextBlock)
					.Text(ParentBoneProperty->GetPropertyDisplayName())
					.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
				]
			]
			.ValueContent()
			[
				SNew(SAssetSearchBoxForBones, Outer, ParentBoneProperty)
				.IncludeSocketsForSuggestions(false)
				.MustMatchPossibleSuggestions(true)
				.HintText(NSLOCTEXT( "SkeletalMeshSocketDetails", "SkeletalMeshSocketDetailsHintTextBoneName", "Bone Name..." ))
				.OnTextCommitted( this, &FSkeletalMeshSocketDetails::OnParentBoneNameCommitted )
			];
		}
	}
}

void FSkeletalMeshSocketDetails::OnParentBoneNameCommitted(const FText& InSearchText, ETextCommit::Type CommitInfo)
{
	UObject* Outer = TargetSocket->GetOuter();
	USkeleton* Skeleton = Cast<USkeleton>(Outer);
	USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(Outer);
	if( Skeleton == NULL )
	{
		if( SkeletalMesh )
		{
			Skeleton = SkeletalMesh->GetSkeleton();
		}
	}
	if( Skeleton )
	{
		if( Skeleton->GetReferenceSkeleton().FindBoneIndex( *InSearchText.ToString() ) != INDEX_NONE )
		{
			ISkeletonEditorModule& SkeletonEditorModule = FModuleManager::LoadModuleChecked<ISkeletonEditorModule>("SkeletonEditor");
			TSharedRef<IEditableSkeleton> EditableSkeleton = SkeletonEditorModule.CreateEditableSkeleton(Skeleton);

			EditableSkeleton->SetSocketParent( TargetSocket->SocketName, *InSearchText.ToString(), SkeletalMesh );

			ParentBoneProperty->SetValue( InSearchText.ToString() );
		}
	}
}

void FSkeletalMeshSocketDetails::OnSocketNameChanged(const FText& InSearchText)
{
	UObject* Outer = TargetSocket->GetOuter();
	USkeleton* Skeleton = Cast<USkeleton>(Outer);
	USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(Outer);
	if (Skeleton == NULL)
	{
		if (SkeletalMesh)
		{
			Skeleton = SkeletalMesh->GetSkeleton();
		}
	}
	if (Skeleton)
	{
		ISkeletonEditorModule& SkeletonEditorModule = FModuleManager::LoadModuleChecked<ISkeletonEditorModule>("SkeletonEditor");
		TSharedRef<IEditableSkeleton> EditableSkeleton = SkeletonEditorModule.CreateEditableSkeleton(Skeleton);

		FText OutErrorMessage;
		if( VerifySocketName(EditableSkeleton, TargetSocket, InSearchText, SkeletalMesh, OutErrorMessage) )
		{
			SocketNameTextBox->SetError( FText::GetEmpty() );
		}
		else
		{
			SocketNameTextBox->SetError( OutErrorMessage );
		}
	}
}

void FSkeletalMeshSocketDetails::OnSocketNameCommitted(const FText& InSearchText, ETextCommit::Type CommitInfo)
{
	UObject* Outer = TargetSocket->GetOuter();
	USkeleton* Skeleton = Cast<USkeleton>(Outer);
	USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(Outer);
	if (Skeleton == NULL)
	{
		if (SkeletalMesh)
		{
			Skeleton = SkeletalMesh->GetSkeleton();
		}
	}
	if (Skeleton)
	{
		ISkeletonEditorModule& SkeletonEditorModule = FModuleManager::LoadModuleChecked<ISkeletonEditorModule>("SkeletonEditor");
		TSharedRef<IEditableSkeleton> EditableSkeleton = SkeletonEditorModule.CreateEditableSkeleton(Skeleton);

		FText NewText = FText::TrimPrecedingAndTrailing(InSearchText);

		FText OutErrorMessage;
		if (VerifySocketName(EditableSkeleton, TargetSocket, NewText, SkeletalMesh, OutErrorMessage))
		{
			// tell rename the socket.
			EditableSkeleton->RenameSocket(TargetSocket->SocketName, FName(*NewText.ToString()), SkeletalMesh);
				
			// update the pre-edit socket name and the property
			PreEditSocketName = NewText;
			SocketNameTextBox->SetText(NewText);
		}
		else
		{
			// restore the pre-edit name to the socket text box.
			SocketNameTextBox->SetText(PreEditSocketName);
		}
	}

	if( CommitInfo == ETextCommit::OnUserMovedFocus )
	{
		SocketNameTextBox->SetError( FText::GetEmpty() );
	}
}

bool FSkeletalMeshSocketDetails::VerifySocketName(TSharedRef<IEditableSkeleton> EditableSkeleton, const USkeletalMeshSocket* Socket, const FText& InText, USkeletalMesh* InSkeletalMesh, FText& OutErrorMessage ) const
{
	// You can't have two sockets with the same name on the mesh, nor on the skeleton,
	// but you can have a socket with the same name on the mesh *and* the skeleton.
	bool bVerifyName = true;

	FText NewText = FText::TrimPrecedingAndTrailing(InText);

	if (NewText.IsEmpty())
	{
		OutErrorMessage = NSLOCTEXT( "SkeletalMeshSocketDetails", "EmptySocketName_Error", "Sockets must have a name!");
		bVerifyName = false;
	}
	else
	{
		if (EditableSkeleton->DoesSocketAlreadyExist(Socket, NewText, ESocketParentType::Skeleton, InSkeletalMesh))
		{
			bVerifyName = false;
		}

		if (bVerifyName && EditableSkeleton->DoesSocketAlreadyExist(Socket, NewText, ESocketParentType::Mesh, InSkeletalMesh))
		{
			bVerifyName = false;
		}

		// Needs to be checked on verify.
		if ( !bVerifyName )
		{
			// Tell the user that the socket is a duplicate
			OutErrorMessage = NSLOCTEXT( "SkeletalMeshSocketDetails", "DuplicateSocket_Error", "Socket name in use!");
		}
	}
	return bVerifyName;
}
