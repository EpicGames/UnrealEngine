// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"
#include "Engine/SkeletalMesh.h"
#include "Types/SlateEnums.h"

class IDetailLayoutBuilder;
class IPropertyHandle;
class SEditableTextBox;
class USkeletalMeshSocket;

class FSkeletalMeshSocketDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override;

private:

	/** Handle the committed text for the parent bone */
	void OnParentBoneNameCommitted(const FText& InSearchText, ETextCommit::Type CommitInfo);

	/** Handle the changed text for the socket name */
	void OnSocketNameChanged(const FText& InSearchText);

	/** Handle the committed text for the socket name */
	void OnSocketNameCommitted(const FText& InSearchText, ETextCommit::Type CommitInfo);

	/** Verifies the socket name and supplies and error message if it's invalid **/
	bool VerifySocketName(TSharedRef<class IEditableSkeleton> EditableSkeleton, const USkeletalMeshSocket* Socket, const FText& InText, USkeletalMesh* InSkeletalMesh, FText& OutErrorMessage ) const;

	/** Get the search suggestions */
	TArray<FString> GetSearchSuggestions() const;

	/** The skeletal mesh socket associated with the socket */
	USkeletalMeshSocket* TargetSocket;

	/** The handle to the socket name property */
	TSharedPtr<IPropertyHandle> SocketNameProperty;

	/** The handle to the socket name property */
	TSharedPtr<IPropertyHandle> ParentBoneProperty;

	/** The ptr to the socket name text box */
	TSharedPtr<SEditableTextBox> SocketNameTextBox;

	/** The socket name prior to editing */
	FText PreEditSocketName;
};
