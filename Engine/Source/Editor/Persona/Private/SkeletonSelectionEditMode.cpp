// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkeletonSelectionEditMode.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "AnimationEditorViewportClient.h"
#include "CanvasItem.h"
#include "CanvasTypes.h"
#include "AnimPreviewInstance.h"
#include "ISkeletonTree.h"
#include "AssetEditorModeManager.h"
#include "Engine/SkeletalMeshSocket.h"
#include "EngineUtils.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "Engine/WindDirectionalSource.h"

#include "IPersonaToolkit.h"
#include "IEditableSkeleton.h"

#include "SceneView.h"

#define LOCTEXT_NAMESPACE "SkeletonSelectionEditMode"

namespace SkeletonSelectionModeConstants
{
	/** Distance to trace for physics bodies */
	static const float BodyTraceDistance = 10000.0f;
}

FSkeletonSelectionEditMode::FSkeletonSelectionEditMode()
	: bManipulating(false)
	, bResumeAfterManipulating(false)
	, bInTransaction(false)
{
	// Disable grid drawing for this mode as the viewport handles this
	bDrawGrid = false;
}

const FReferenceSkeleton& FSkeletonSelectionEditMode::GetReferenceSkeletonForComponent(const USkeletalMeshComponent* Component) const
{
	check(Component);
	return Component->GetSkeletalMeshAsset() && !Component->GetSkeletalMeshAsset()->IsCompiling() ? Component->GetSkeletalMeshAsset()->GetRefSkeleton() : GetAnimPreviewScene().GetPersonaToolkit()->GetSkeleton()->GetReferenceSkeleton();
}

bool FSkeletonSelectionEditMode::GetCameraTarget(FSphere& OutTarget) const
{
	bool bHandled = false;

	const UDebugSkelMeshComponent* PreviewMeshComponent = GetAnimPreviewScene().GetPreviewMeshComponent();
	if(PreviewMeshComponent)
	{
		if (GetAnimPreviewScene().GetSelectedBoneIndex() != INDEX_NONE)
		{
			const int32 FocusBoneIndex = GetAnimPreviewScene().GetSelectedBoneIndex();
			const FReferenceSkeleton& ReferenceSkeleton = GetReferenceSkeletonForComponent(PreviewMeshComponent);
			if (FocusBoneIndex != INDEX_NONE && ReferenceSkeleton.IsValidIndex(FocusBoneIndex))
			{
				const FName BoneName = ReferenceSkeleton.GetBoneName(FocusBoneIndex);
				OutTarget.Center = PreviewMeshComponent->GetBoneLocation(BoneName);
				OutTarget.W = 30.0f;
				bHandled = true;
			}
		}

		if (!bHandled && GetAnimPreviewScene().GetSelectedSocket().IsValid())
		{
			USkeletalMeshSocket * Socket = GetAnimPreviewScene().GetSelectedSocket().Socket;
			if (Socket)
			{
				OutTarget.Center = Socket->GetSocketLocation(PreviewMeshComponent);
				OutTarget.W = 30.0f;
				bHandled = true;
			}
		}
	}

	return bHandled;
}

IPersonaPreviewScene& FSkeletonSelectionEditMode::GetAnimPreviewScene() const
{
	return *static_cast<IPersonaPreviewScene*>(static_cast<FAssetEditorModeManager*>(Owner)->GetPreviewScene());
}

void FSkeletonSelectionEditMode::GetOnScreenDebugInfo(TArray<FText>& OutDebugInfo) const
{

}

FSelectedSocketInfo FSkeletonSelectionEditMode::DuplicateAndSelectSocket(const FSelectedSocketInfo& SocketInfoToDuplicate)
{
	USkeletalMesh* SkeletalMesh = GetAnimPreviewScene().GetPreviewMeshComponent()->GetSkeletalMeshAsset();
	USkeletalMeshSocket* NewSocket = GetAnimPreviewScene().GetPersonaToolkit()->GetEditableSkeleton()->DuplicateSocket(SocketInfoToDuplicate, SocketInfoToDuplicate.Socket->BoneName, SkeletalMesh);

	FSelectedSocketInfo NewSocketInfo(NewSocket, SocketInfoToDuplicate.bSocketIsOnSkeleton);
	GetAnimPreviewScene().DeselectAll();
	GetAnimPreviewScene().SetSelectedSocket(NewSocketInfo);

	return NewSocketInfo;
}

bool FSkeletonSelectionEditMode::BeginTransform(const FGizmoState& InState)
{
	bManipulating = bInTransaction = false;
	
	const UDebugSkelMeshComponent* PreviewMeshComponent = GetAnimPreviewScene().GetPreviewMeshComponent();
	const USkeletalMesh* SkeletalMeshAsset = PreviewMeshComponent ? PreviewMeshComponent->GetSkeletalMeshAsset() : nullptr;
	if (!SkeletalMeshAsset)
	{
		return false;
	}

	// transact bone transform?
	const int32 BoneIndex = GetAnimPreviewScene().GetSelectedBoneIndex();
	const FReferenceSkeleton& ReferenceSkeleton = GetReferenceSkeletonForComponent(PreviewMeshComponent);
	if (BoneIndex >= INDEX_NONE && ReferenceSkeleton.IsValidIndex(BoneIndex))
	{
		PreviewMeshComponent->PreviewInstance->SetFlags(RF_Transactional);	// Undo doesn't work without this!
		PreviewMeshComponent->PreviewInstance->Modify();

		// now modify the bone array
		const FName BoneName = ReferenceSkeleton.GetBoneName(BoneIndex);
		PreviewMeshComponent->PreviewInstance->ModifyBone(BoneName);

		bManipulating = bInTransaction = true;
		return true;
	}

	FSelectedSocketInfo SelectedSocketInfo = GetAnimPreviewScene().GetSelectedSocket();
	if (SelectedSocketInfo.IsValid())
	{
		if (GetModeManager()->GetFocusedViewportClient())
		{
			const bool bAltDown = GetModeManager()->GetFocusedViewportClient()->IsAltPressed();
			if (bAltDown)
			{
				// Rather than moving/rotating the selected socket, copy it and move the copy instead
				SelectedSocketInfo = DuplicateAndSelectSocket(SelectedSocketInfo);
			}
		}

		// Socket movement is transactional - we want undo/redo and saving of it
		if (USkeletalMeshSocket* Socket = SelectedSocketInfo.Socket)
		{
			Socket->SetFlags(RF_Transactional);	// Undo doesn't work without this!
			Socket->Modify();
			bManipulating = bInTransaction = true;
			return true;
		}
	}
	
	return false;
}

bool FSkeletonSelectionEditMode::EndTransform(const FGizmoState& InState)
{
	const bool bWasManipulating = bManipulating;
	bManipulating = bInTransaction = false;
	return bWasManipulating;
}

bool FSkeletonSelectionEditMode::StartTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport)
{
	const EAxisList::Type CurrentAxis = InViewportClient->GetCurrentWidgetAxis();
	const UE::Widget::EWidgetMode WidgetMode = InViewportClient->GetWidgetMode();

	const UDebugSkelMeshComponent* PreviewMeshComponent = GetAnimPreviewScene().GetPreviewMeshComponent();
	if(PreviewMeshComponent != nullptr && PreviewMeshComponent->GetSkeletalMeshAsset() != nullptr)
	{
		const int32 BoneIndex = GetAnimPreviewScene().GetSelectedBoneIndex();
		const USkeletalMeshSocket* SelectedSocket = GetAnimPreviewScene().GetSelectedSocket().Socket;
		const AActor* SelectedActor = GetAnimPreviewScene().GetSelectedActor();

		// Retrieve reference skeleton from either current USkeletalMesh, or USkeleton if no mesh is set
		const FReferenceSkeleton& ReferenceSkeleton = GetReferenceSkeletonForComponent(PreviewMeshComponent);
		if ((BoneIndex >= 0 && ReferenceSkeleton.IsValidIndex(BoneIndex)) || SelectedSocket != nullptr || SelectedActor != nullptr)
		{
			if ( ((CurrentAxis & EAxisList::XYZ) | (CurrentAxis & EAxisList::Screen)) != 0)
			{
				FSelectedSocketInfo SelectedSocketInfo = GetAnimPreviewScene().GetSelectedSocket();
				if (SelectedSocketInfo.IsValid())
				{
					const bool bAltDown = InViewportClient->IsAltPressed();

					if (bAltDown)
					{
						// Rather than moving/rotating the selected socket, copy it and move the copy instead
						SelectedSocketInfo = DuplicateAndSelectSocket(SelectedSocketInfo);
					}

					// Socket movement is transactional - we want undo/redo and saving of it
					USkeletalMeshSocket* Socket = SelectedSocketInfo.Socket;

					if (Socket && bInTransaction == false)
					{
						if (WidgetMode == UE::Widget::WM_Rotate)
						{
							GEditor->BeginTransaction(LOCTEXT("AnimationEditorViewport_RotateSocket", "Rotate Socket"));
						}
						else
						{
							GEditor->BeginTransaction(LOCTEXT("AnimationEditorViewport_TranslateSocket", "Translate Socket"));
						}

						Socket->SetFlags(RF_Transactional);	// Undo doesn't work without this!
						Socket->Modify();
						bInTransaction = true;
					}
				}
				else if (BoneIndex >= 0)
				{
					if (bInTransaction == false)
					{
						// we also allow undo/redo of bone manipulations
						if (WidgetMode == UE::Widget::WM_Rotate)
						{
							GEditor->BeginTransaction(LOCTEXT("AnimationEditorViewport_RotateBone", "Rotate Bone"));
						}
						else
						{
							GEditor->BeginTransaction(LOCTEXT("AnimationEditorViewport_TranslateBone", "Translate Bone"));
						}

						PreviewMeshComponent->PreviewInstance->SetFlags(RF_Transactional);	// Undo doesn't work without this!
						PreviewMeshComponent->PreviewInstance->Modify();
						bInTransaction = true;

						// now modify the bone array
						const FName BoneName = ReferenceSkeleton.GetBoneName(BoneIndex);
						PreviewMeshComponent->PreviewInstance->ModifyBone(BoneName);
					}
				}
			}

			bManipulating = true;

			TObjectPtr<UAnimPreviewInstance> PreviewInstance = (PreviewMeshComponent && PreviewMeshComponent->IsPreviewOn()) ? PreviewMeshComponent->PreviewInstance : nullptr;
			if (PreviewInstance)
			{
				bResumeAfterManipulating = PreviewInstance->IsPlaying();
				PreviewInstance->SetPlaying(false);
			}

			return true;
		}
	}
	
	return false;
}

bool FSkeletonSelectionEditMode::EndTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport)
{
	if (bManipulating)
	{
		// Socket movement is transactional - we want undo/redo and saving of it
		if (bInTransaction)
		{
			GEditor->EndTransaction();
			bInTransaction = false;
		}

		bManipulating = false;

		if (bResumeAfterManipulating)
		{
			bResumeAfterManipulating = false;

			const UDebugSkelMeshComponent* PreviewMeshComponent = GetAnimPreviewScene().GetPreviewMeshComponent();
			if (PreviewMeshComponent)
			{
				TObjectPtr<UAnimPreviewInstance> PreviewInstance = (PreviewMeshComponent && PreviewMeshComponent->IsPreviewOn()) ? PreviewMeshComponent->PreviewInstance : nullptr;
				if (PreviewInstance)
				{
					PreviewInstance->SetPlaying(true);
				}
			}
		}

		return true;
	}

	return false;
}

bool FSkeletonSelectionEditMode::InputDelta(FEditorViewportClient* InViewportClient, FViewport* InViewport, FVector& InDrag, FRotator& InRot, FVector& InScale)
{
	const EAxisList::Type CurrentAxis = InViewportClient->GetCurrentWidgetAxis();
	const UE::Widget::EWidgetMode WidgetMode = InViewportClient->GetWidgetMode();
	const ECoordSystem CoordSystem = InViewportClient->GetWidgetCoordSystemSpace();

	bool bHandled = false;

	UDebugSkelMeshComponent* PreviewMeshComponent = GetAnimPreviewScene().GetPreviewMeshComponent();

	if ( bManipulating && CurrentAxis != EAxisList::None )
	{
		bHandled = true;

		int32 BoneIndex = GetAnimPreviewScene().GetSelectedBoneIndex();
		USkeletalMeshSocket* SelectedSocket = GetAnimPreviewScene().GetSelectedSocket().Socket;
		AActor* SelectedActor = GetAnimPreviewScene().GetSelectedActor();
		FAnimNode_ModifyBone* SkelControl = nullptr;

		if ( BoneIndex >= 0 )
		{
			// Retrieve reference skeleton from either current USkeletalMesh, or USkeleton if no mesh is set
			const FReferenceSkeleton& ReferenceSkeleton = GetReferenceSkeletonForComponent(PreviewMeshComponent);
			const FName BoneName = ReferenceSkeleton.GetBoneName(BoneIndex);
			//Get the skeleton control manipulating this bone
			SkelControl = &(PreviewMeshComponent->PreviewInstance->ModifyBone(BoneName));
		}

		if ( SkelControl || SelectedSocket )
		{
			FTransform CurrentSkelControlTM(
				SelectedSocket ? SelectedSocket->RelativeRotation : SkelControl->Rotation,
				SelectedSocket ? SelectedSocket->RelativeLocation : SkelControl->Translation,
				SelectedSocket ? SelectedSocket->RelativeScale : SkelControl->Scale);

			FTransform BaseTM;

			if ( SelectedSocket )
			{
				BaseTM = SelectedSocket->GetSocketTransform( PreviewMeshComponent );
			}
			else
			{
				BaseTM = PreviewMeshComponent->GetBoneTransform( BoneIndex );
			}

			// Remove SkelControl's orientation from BoneMatrix, as we need to translate/rotate in the non-SkelControlled space
			BaseTM = BaseTM.GetRelativeTransformReverse(CurrentSkelControlTM);

			const bool bDoRotation    = WidgetMode == UE::Widget::WM_Rotate    || WidgetMode == UE::Widget::WM_TranslateRotateZ;
			const bool bDoTranslation = WidgetMode == UE::Widget::WM_Translate || WidgetMode == UE::Widget::WM_TranslateRotateZ;
			const bool bDoScale = WidgetMode == UE::Widget::WM_Scale;

			if (bDoRotation)
			{
				FVector RotAxis;
				float RotAngle;
				InRot.Quaternion().ToAxisAndAngle( RotAxis, RotAngle );

				FVector4 BoneSpaceAxis = BaseTM.TransformVectorNoScale( RotAxis );

				//Calculate the new delta rotation
				FQuat DeltaQuat( BoneSpaceAxis, RotAngle );
				DeltaQuat.Normalize();

				FRotator NewRotation = ( CurrentSkelControlTM * FTransform( DeltaQuat )).Rotator();

				if ( SelectedSocket )
				{
					SelectedSocket->RelativeRotation = NewRotation;
				}
				else
				{
					SkelControl->Rotation = NewRotation;
				}
			}

			if (bDoTranslation)
			{
				FVector4 BoneSpaceOffset = BaseTM.TransformVector(InDrag);
				if (SelectedSocket)
				{
					SelectedSocket->RelativeLocation += BoneSpaceOffset;
				}
				else
				{
					SkelControl->Translation += BoneSpaceOffset;
				}
			}
			if(bDoScale)
			{
				FVector4 BoneSpaceScaleOffset;

				if (CoordSystem == COORD_World)
				{
					BoneSpaceScaleOffset = BaseTM.TransformVector(InScale);
				}
				else
				{
					BoneSpaceScaleOffset = InScale;
				}

				if(SelectedSocket)
				{
					SelectedSocket->RelativeScale += BoneSpaceScaleOffset;
				}
				else
				{
					SkelControl->Scale += BoneSpaceScaleOffset;
				}
			}

		}
		else if( SelectedActor != nullptr )
		{
			if (WidgetMode == UE::Widget::WM_Rotate)
			{
				FTransform Transform = SelectedActor->GetTransform();
				FRotator NewRotation = (Transform * FTransform( InRot ) ).Rotator();

				SelectedActor->SetActorRotation( NewRotation );
			}
			else
			{
				FVector Location = SelectedActor->GetActorLocation();
				Location += InDrag;
				SelectedActor->SetActorLocation(Location);
			}
		}

		InViewport->Invalidate();
	}

	return bHandled;
}

FIntPoint FSkeletonSelectionEditMode::GetDPIUnscaledSize(FViewport* Viewport, FViewportClient* Client)
{
	const FIntPoint Size = Viewport->GetSizeXY();
	const float DPIScale = Client->GetDPIScale();
	// (FIntPoint / float) implicitly casts the float to an int if you try to divide it directly
	return FIntPoint(static_cast<int32>(Size.X / DPIScale), static_cast<int32>(Size.Y / DPIScale));
}

void FSkeletonSelectionEditMode::DrawHUD(FEditorViewportClient* ViewportClient, FViewport* Viewport, const FSceneView* View, FCanvas* Canvas)
{
	const UDebugSkelMeshComponent* PreviewMeshComponent = GetAnimPreviewScene().GetPreviewMeshComponent();
	if (PreviewMeshComponent == nullptr)
	{
		return;
	}

	// Retrieve reference skeleton from either current USkeletalMesh, or USkeleton if no mesh is set
	const FReferenceSkeleton& RefSkeleton = PreviewMeshComponent->GetSkeletalMeshAsset() && !PreviewMeshComponent->GetSkeletalMeshAsset()->IsCompiling() ? PreviewMeshComponent->GetSkeletalMeshAsset()->GetRefSkeleton() : GetAnimPreviewScene().GetPersonaToolkit()->GetSkeleton()->GetReferenceSkeleton();
	const int32 BoneIndex = GetAnimPreviewScene().GetSelectedBoneIndex();

	// Draw name of selected bone
	if (RefSkeleton.IsValidIndex(BoneIndex) && (IsSelectedBoneRequired() || RefSkeleton.GetRequiredVirtualBones().Contains(BoneIndex)))
	{
		const FIntPoint ViewPortSize = GetDPIUnscaledSize(Viewport, ViewportClient);
		const int32 HalfX = ViewPortSize.X / 2;
		const int32 HalfY = ViewPortSize.Y / 2;

		const FName BoneName = RefSkeleton.GetBoneName(BoneIndex);

		const FMatrix BoneMatrix = GetBoneTransform(BoneIndex).ToMatrixNoScale();
		const FPlane Proj = View->Project(BoneMatrix.GetOrigin());
		if (Proj.W > 0.f)
		{
			const int32 XPos = HalfX + static_cast<int32>(HalfX * Proj.X);
			const int32 YPos = HalfY + static_cast<int32>(HalfY * Proj.Y * -1);

			FCanvasTextItem TextItem(FVector2D(XPos, YPos), FText::FromString(BoneName.ToString()), GEngine->GetSmallFont(), FLinearColor::White);
			TextItem.EnableShadow(FLinearColor::Black);
			Canvas->DrawItem(TextItem);
		}
	}

	// Draw name of selected socket
	if (GetAnimPreviewScene().GetSelectedSocket().IsValid())
	{
		const USkeletalMeshSocket* Socket = GetAnimPreviewScene().GetSelectedSocket().Socket;

		const FMatrix SocketMatrix = GetSocketTransform(Socket).ToMatrixNoScale();
		const FVector SocketPos = SocketMatrix.GetOrigin();

		const FPlane Proj = View->Project(SocketPos);
		if (Proj.W > 0.f)
		{
			const FIntPoint ViewPortSize = GetDPIUnscaledSize(Viewport, ViewportClient);
			const int32 HalfX = ViewPortSize.X / 2;
			const int32 HalfY = ViewPortSize.Y / 2;

			const int32 XPos = HalfX + static_cast<int32>(HalfX * Proj.X);
			const int32 YPos = HalfY + static_cast<int32>(HalfY * (Proj.Y * -1));
			FCanvasTextItem TextItem(FVector2D(XPos, YPos), FText::FromString(Socket->SocketName.ToString()), GEngine->GetSmallFont(), FLinearColor::White);
			TextItem.EnableShadow(FLinearColor::Black);
			Canvas->DrawItem(TextItem);
		}
	}
}

bool FSkeletonSelectionEditMode::AllowWidgetMove()
{
	return ShouldDrawWidget();
}

bool FSkeletonSelectionEditMode::IsSelectedBoneRequired() const
{
	const UDebugSkelMeshComponent* PreviewMeshComponent = GetAnimPreviewScene().GetPreviewMeshComponent();
	const int32 SelectedBoneIndex = GetAnimPreviewScene().GetSelectedBoneIndex();
	if (SelectedBoneIndex != INDEX_NONE && PreviewMeshComponent->GetSkeletalMeshRenderData())
	{
		//Get current LOD
		const FSkeletalMeshRenderData* SkelMeshRenderData = PreviewMeshComponent->GetSkeletalMeshRenderData();
		if(SkelMeshRenderData->LODRenderData.Num() > 0)
		{
			const int32 LODIndex = FMath::Clamp(PreviewMeshComponent->GetPredictedLODLevel(), 0, SkelMeshRenderData->LODRenderData.Num() - 1);
			const FSkeletalMeshLODRenderData& LODData = SkelMeshRenderData->LODRenderData[LODIndex];

			//Check whether the bone is vertex weighted
			return LODData.RequiredBones.Find(static_cast<FBoneIndexType>(SelectedBoneIndex)) != INDEX_NONE;
		}
	}
	else if (SelectedBoneIndex != INDEX_NONE && !PreviewMeshComponent->GetSkeletalMeshRenderData())
	{
		return true;
	}

	return false;
}

bool FSkeletonSelectionEditMode::ShouldDrawWidget() const
{
	UDebugSkelMeshComponent* PreviewMeshComponent = GetAnimPreviewScene().GetPreviewMeshComponent();
	if (PreviewMeshComponent && PreviewMeshComponent->GetSkeletalMeshAsset() && PreviewMeshComponent->GetSkeletalMeshAsset()->IsCompiling())
	{
		return false;
	}

	if (PreviewMeshComponent && !PreviewMeshComponent->IsAnimBlueprintInstanced())
	{
		return IsSelectedBoneRequired() || GetAnimPreviewScene().GetSelectedSocket().IsValid() || GetAnimPreviewScene().GetSelectedActor() != nullptr;
	}

	return false;
}

bool FSkeletonSelectionEditMode::UsesTransformWidget() const
{
	return true;
}

bool FSkeletonSelectionEditMode::UsesTransformWidget(UE::Widget::EWidgetMode CheckMode) const
{
	return ShouldDrawWidget() && (CheckMode == UE::Widget::WM_Scale || CheckMode == UE::Widget::WM_Translate || CheckMode == UE::Widget::WM_Rotate);
}

FTransform FSkeletonSelectionEditMode::GetWorldSpaceBoneTransform(const FReferenceSkeleton& ReferenceSkeleton, const int32 BoneIndex) const
{
	const TArray<FTransform>& BonePoses = ReferenceSkeleton.GetRefBonePose();

	if (BonePoses.IsValidIndex(BoneIndex))
	{
		FTransform WorldSpacePose = BonePoses[BoneIndex];

		int32 ParentIndex = ReferenceSkeleton.GetParentIndex(BoneIndex);

		while(ParentIndex != INDEX_NONE)
		{
			WorldSpacePose = WorldSpacePose * BonePoses[ParentIndex];
			ParentIndex = ReferenceSkeleton.GetParentIndex(ParentIndex);
		}

		return WorldSpacePose;
	}

	return FTransform::Identity;
}



FTransform FSkeletonSelectionEditMode::GetBoneTransform(const int32 BoneIndex) const
{
	const UDebugSkelMeshComponent* PreviewMeshComponent = GetAnimPreviewScene().GetPreviewMeshComponent();
	if (PreviewMeshComponent && PreviewMeshComponent->GetSkeletalMeshAsset())
	{
		if (!PreviewMeshComponent->GetSkeletalMeshAsset()->IsCompiling())
		{
			return PreviewMeshComponent->GetBoneTransform(BoneIndex);
		}
	}
	else if (const USkeleton* Skeleton = GetAnimPreviewScene().GetPersonaToolkit()->GetSkeleton())
	{
		return GetWorldSpaceBoneTransform(Skeleton->GetReferenceSkeleton(), BoneIndex);
	}

	return FTransform::Identity;
}

FTransform FSkeletonSelectionEditMode::GetSocketTransform(const USkeletalMeshSocket* Socket) const
{
	const UDebugSkelMeshComponent* PreviewMeshComponent = GetAnimPreviewScene().GetPreviewMeshComponent();
	if (PreviewMeshComponent && PreviewMeshComponent->GetSkeletalMeshAsset())
	{
		if (!PreviewMeshComponent->GetSkeletalMeshAsset()->IsCompiling())
		{
			const int32 BoneIndex = PreviewMeshComponent->GetBoneIndex(Socket->BoneName);
			if(BoneIndex != INDEX_NONE)
			{
				const FTransform BoneTM = PreviewMeshComponent->GetBoneTransform(BoneIndex);
				const FTransform RelSocketTM( Socket->RelativeRotation, Socket->RelativeLocation, Socket->RelativeScale );
				return RelSocketTM * BoneTM;
			}
			
		}
	}
	else if (const USkeleton* Skeleton = GetAnimPreviewScene().GetPersonaToolkit()->GetSkeleton())
	{
		const FReferenceSkeleton& ReferenceSkeleton = Skeleton->GetReferenceSkeleton();
		const int32 BoneIndex = ReferenceSkeleton.FindBoneIndex(Socket->BoneName);
		if(BoneIndex != INDEX_NONE)
		{
			const FTransform BoneTM = GetWorldSpaceBoneTransform(Skeleton->GetReferenceSkeleton(), BoneIndex);
			const FTransform RelSocketTM( Socket->RelativeRotation, Socket->RelativeLocation, Socket->RelativeScale );
			return RelSocketTM * BoneTM;
		}
	}

	return FTransform::Identity;
}

bool FSkeletonSelectionEditMode::GetCustomDrawingCoordinateSystem(FMatrix& InMatrix, void* InData)
{
	const bool bIsParentMode = Owner ? Owner->GetCoordSystem() == COORD_Parent : false;
	
	const UDebugSkelMeshComponent* PreviewMeshComponent = GetAnimPreviewScene().GetPreviewMeshComponent();
	if(PreviewMeshComponent)
	{
		// Retrieve reference skeleton from either current USkeletalMesh, or USkeleton if no mesh is set
		const FReferenceSkeleton& ReferenceSkeleton = GetReferenceSkeletonForComponent(PreviewMeshComponent);
		
		int32 BoneIndex = GetAnimPreviewScene().GetSelectedBoneIndex();
		if (BoneIndex != INDEX_NONE && ReferenceSkeleton.IsValidIndex(BoneIndex))
		{
			if (bIsParentMode)
			{
				const int32 ParentIndex = ReferenceSkeleton.GetParentIndex(BoneIndex);
				if (ParentIndex != INDEX_NONE)
				{
					BoneIndex = ParentIndex;
				}
			}
			const FTransform BoneMatrix = GetBoneTransform(BoneIndex);
			InMatrix = BoneMatrix.ToMatrixNoScale().RemoveTranslation();
			return true;
		}
		
		if (GetAnimPreviewScene().GetSelectedSocket().IsValid())
		{
			const USkeletalMeshSocket* Socket = GetAnimPreviewScene().GetSelectedSocket().Socket;
			if (bIsParentMode)
			{
				const int32 SocketBoneIndex = PreviewMeshComponent->GetBoneIndex(Socket->BoneName);
				if (SocketBoneIndex != INDEX_NONE && ReferenceSkeleton.IsValidIndex(SocketBoneIndex))
				{
					const FTransform SocketBoneMatrix = GetBoneTransform(SocketBoneIndex);
					InMatrix = SocketBoneMatrix.ToMatrixNoScale().RemoveTranslation();
					return true;
				}
			}
			
			const FTransform SocketMatrix = GetSocketTransform(Socket);
			InMatrix = SocketMatrix.ToMatrixNoScale().RemoveTranslation();
			return true;
		}
		
		if (const AActor* SelectedActor = GetAnimPreviewScene().GetSelectedActor())
		{
			InMatrix = SelectedActor->GetTransform().ToMatrixNoScale().RemoveTranslation();
			return true;
		}
	}
	
	return false;
}

bool FSkeletonSelectionEditMode::GetCustomInputCoordinateSystem(FMatrix& InMatrix, void* InData)
{
	return GetCustomDrawingCoordinateSystem(InMatrix, InData);
}

FVector FSkeletonSelectionEditMode::GetWidgetLocation() const
{
	const int32 BoneIndex = GetAnimPreviewScene().GetSelectedBoneIndex();
	if (BoneIndex != INDEX_NONE)
	{
		const FMatrix BoneMatrix = GetBoneTransform(BoneIndex).ToMatrixNoScale();
		return BoneMatrix.GetOrigin();
	}
	else if (GetAnimPreviewScene().GetSelectedSocket().IsValid())
	{
		const USkeletalMeshSocket* Socket = GetAnimPreviewScene().GetSelectedSocket().Socket;
		const FMatrix SocketMatrix = GetSocketTransform(Socket).ToMatrixNoScale();
		return SocketMatrix.GetOrigin();
	}
	else if (const AActor* SelectedActor = GetAnimPreviewScene().GetSelectedActor())
	{
		return SelectedActor->GetActorLocation();
	}

	return FVector::ZeroVector;
}

bool FSkeletonSelectionEditMode::HandleClick(FEditorViewportClient* InViewportClient, HHitProxy* HitProxy, const FViewportClick& Click)
{
	GetAnimPreviewScene().DeselectAll();
	USkeletalMeshComponent* MeshComponent = GetAnimPreviewScene().GetPreviewMeshComponent();
	if (MeshComponent)
	{
		MeshComponent->SetSelectedEditorSection(INDEX_NONE);
	}

	if (!HitProxy)
	{
		return false;
	}

	if (HPersonaBoneHitProxy* BoneHitProxy = HitProxyCast<HPersonaBoneHitProxy>(HitProxy))
	{
		GetAnimPreviewScene().SetSelectedBone(BoneHitProxy->BoneName, ESelectInfo::OnMouseClick);
		return true;
	}
	if (HPersonaSocketHitProxy* SocketHitProxy = HitProxyCast<HPersonaSocketHitProxy>(HitProxy))
	{
		if (USkeleton* Skeleton = GetAnimPreviewScene().GetPersonaToolkit()->GetSkeleton())
		{
			FSelectedSocketInfo SocketInfo;
			SocketInfo.Socket = SocketHitProxy->Socket;
			SocketInfo.bSocketIsOnSkeleton = !SocketInfo.Socket->GetOuter()->IsA<USkeletalMesh>();
			GetAnimPreviewScene().SetSelectedSocket(SocketInfo);
			return true;
		}
	}
	if (HActor* ActorHitProxy = HitProxyCast<HActor>(HitProxy))
	{
		if (ActorHitProxy->Actor)
		{
			if (ActorHitProxy->Actor->IsA<AWindDirectionalSource>())
			{
				AWindDirectionalSource* WindSourceActor = CastChecked<AWindDirectionalSource>(ActorHitProxy->Actor);
				if (WindSourceActor->IsSelectable())
				{
					GetAnimPreviewScene().SetSelectedActor(WindSourceActor);
					return true;
				}
			}
		}
		GetAnimPreviewScene().BroadcastMeshClick(ActorHitProxy, Click); // This can pop up menu which redraws viewport and invalidates HitProxy!
		return true;
	}

	return false;
}

bool FSkeletonSelectionEditMode::CanCycleWidgetMode() const
{
	int32 SelectedBoneIndex = GetAnimPreviewScene().GetSelectedBoneIndex();
	USkeletalMeshSocket* SelectedSocket = GetAnimPreviewScene().GetSelectedSocket().Socket;
	AActor* SelectedActor = GetAnimPreviewScene().GetSelectedActor();

	return (SelectedBoneIndex >= 0 || SelectedSocket || SelectedActor != nullptr);
}

bool FSkeletonSelectionEditMode::FrustumSelect(const FConvexVolume& InFrustum, FEditorViewportClient* InViewportClient, bool InSelect)
{
	if (FAnimationViewportClient* AnimViewportClient = static_cast<FAnimationViewportClient*>(InViewportClient))
	{
		const EBoneDrawMode::Type BoneDrawMode = AnimViewportClient->GetBoneDrawMode();
		if (BoneDrawMode != EBoneDrawMode::None)
		{
			const float BoneRadius = AnimViewportClient->GetBoneDrawSize();
			TArray<FName> Selection;
			
			const UDebugSkelMeshComponent* PreviewMeshComponent = GetAnimPreviewScene().GetPreviewMeshComponent();
			const FReferenceSkeleton& RefSkeleton = PreviewMeshComponent->GetReferenceSkeleton();
			const TArray<FTransform>& BoneTransforms = PreviewMeshComponent->GetComponentSpaceTransforms();
			for (int32 Index = 0; Index < BoneTransforms.Num(); ++Index)
			{
				const FVector BoneLocation = BoneTransforms[Index].GetLocation();
				
				// NOTE: the full sphere is used for selection for simplicity, not a "wireframe" version
				if (InFrustum.IntersectSphere(BoneLocation, 0.1f * BoneRadius))
				{
					Selection.AddUnique(RefSkeleton.GetBoneName(Index));
				}
				else
				{
					// otherwise, intersect the line between the bone and its children
					TArray<int32> Children;
					RefSkeleton.GetDirectChildBones(Index, Children);
					for (const int32 ChildIndex : Children)
					{
						if (InFrustum.IntersectLineSegment(BoneLocation, BoneTransforms[ChildIndex].GetLocation()))
						{
							Selection.AddUnique(RefSkeleton.GetBoneName(Index));
							break;
						}
					}
				}
			}
			
			if (!Selection.IsEmpty())
			{
				GetAnimPreviewScene().SetSelectedBones(Selection, ESelectInfo::Direct);
				return true;
			}
		}
	}
	
	return IPersonaEditMode::FrustumSelect(InFrustum, InViewportClient, InSelect);
}

#undef LOCTEXT_NAMESPACE
