// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkinWeightProfileCustomization.h"

#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SButton.h"
#include "Rendering/SkinWeightProfile.h"
#include "Engine/SkeletalMesh.h"
#include "Rendering/SkeletalMeshLODModel.h"
#include "Rendering/SkeletalMeshModel.h"
#include "Components/SkinnedMeshComponent.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "SkeletalMeshTypes.h"

#include "PropertyHandle.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "PropertyCustomizationHelpers.h"
#include "LODUtilities.h"
#include "ComponentReregisterContext.h"
#include "SkeletalMeshTypes.h"
#include "SkinWeightProfileHelpers.h"
#include "ScopedTransaction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Images/SImage.h"
#include "Misc/MessageDialog.h"
#include "IDetailGroup.h"

#define LOCTEXT_NAMESPACE "SkinWeightProfileCustomization"

void FSkinWeightProfileCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{	
	TArray<UObject*> Objects;
	PropertyHandle->GetOuterObjects(Objects);

	// Try and retrieve the outer Skeletal Mesh of which this Profile is part of
	USkeletalMesh* SkeletalMesh = nullptr;
	Objects.FindItemByClass<USkeletalMesh>(&SkeletalMesh);
	if (SkeletalMesh)
	{
		WeakSkeletalMesh = SkeletalMesh;		
	}	

	// Cache the property utilities to use when generating sub-menu
	PropertyUtilities = CustomizationUtils.GetPropertyUtilities();

	/** Show the name of this profile as the top-level array entry label rather than the index */
	NameProperty = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FSkinWeightProfileInfo, Name));
	if (NameProperty->IsValidHandle())
	{
		HeaderRow
		.NameContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(1)
			.VAlign(VAlign_Center)
			[
				// Show the name of the asset or actor
				SNew(STextBlock)
				.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
				.Text_Lambda([this]() -> FText
				{
					FName ProfileName;
					if (NameProperty.IsValid() && NameProperty->GetValue(ProfileName) == FPropertyAccess::Success)
					{
						return FText::FromName(ProfileName);
					}

					return FText::GetEmpty();
				})
			]
		];		
	}

	HeaderRow
	.ValueContent()
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Right)
		[
			// Allows for reimport this skin weight profile and all of the data related to it
			SNew(SComboButton)
			.VAlign(EVerticalAlignment::VAlign_Bottom)
			.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
			.ToolTipText(LOCTEXT("SkinWeightProfileReimportTooltip", "Reimport a Skin Weight Profile (LOD)"))
			.ContentPadding(4.0f)
			.ForegroundColor(FSlateColor::UseForeground())
			.HasDownArrow(false)
			.ButtonContent()
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("Persona.ReimportAsset"))
			]
			.OnGetMenuContent(FOnGetContent::CreateSP(this, &FSkinWeightProfileCustomization::GenerateReimportMenu))
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Right)
		[
			// Allows for removing this skin weight profile and all of the data related to it
			SNew(SComboButton)
			.VAlign(EVerticalAlignment::VAlign_Bottom)
			.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
			.ToolTipText(LOCTEXT("SkinWeightProfileRemoveTooltip", "Remove a Skin Weight Profile (LOD)"))
			.ContentPadding(4.0f)
			.ForegroundColor(FSlateColor::UseForeground())
			.HasDownArrow(false)
			.ButtonContent()
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("Icons.Minus"))
			]
			.OnGetMenuContent(FOnGetContent::CreateSP(this, &FSkinWeightProfileCustomization::GenerateRemoveMenu))
		]
	];
}

void FSkinWeightProfileCustomization::UpdateNameRestriction()
{
	// Retrieve all the names of Skin Weight Profiles in the current Skeletal Mesh
	if (USkeletalMesh* SkeletalMesh = WeakSkeletalMesh.Get())
	{
		RestrictedNames.Empty();
		const TArray<FSkinWeightProfileInfo>& Profiles = SkeletalMesh->GetSkinWeightProfiles();
		RestrictedNames.Add(FName(NAME_None).ToString());
		for (const FSkinWeightProfileInfo& ProfileInfo : Profiles)
		{
			RestrictedNames.Add(ProfileInfo.Name.ToString());
		}
		RestrictedNames.Remove(LastKnownProfileName.ToString());
	}
}

void FSkinWeightProfileCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{	
	uint32 NumChildren = 0;
	if (PropertyHandle->GetNumChildren(NumChildren) == FPropertyAccess::Success)
	{
		for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
		{
			// Customize the Name property for this profile, to ensure it cannot have a name matching with any other profile in WeakSkeletalMesh
			TSharedPtr<IPropertyHandle> ChildProperty = PropertyHandle->GetChildHandle(ChildIndex);
			if (ChildProperty->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(FSkinWeightProfileInfo, Name))
			{
				ChildProperty->GetValue(LastKnownProfileName);
				ChildBuilder.AddCustomRow(LOCTEXT("ProfileNameLabel", "Profile Name"))
				.NameContent()
				[
					ChildProperty->CreatePropertyNameWidget()
				]
				.ValueContent()
				[
					SAssignNew(NameEditTextBox, SEditableTextBox)
					.Text(this, &FSkinWeightProfileCustomization::OnGetProfileName)
					.OnTextChanged(this, &FSkinWeightProfileCustomization::OnProfileNameChanged)
					.OnTextCommitted(this, &FSkinWeightProfileCustomization::OnProfileNameCommitted)
					.Font(CustomizationUtils.GetRegularFont())
				];
			}
			// Customize the DefaultProfile property to make sure only one profile is marked as the default loaded one
			else if (ChildProperty->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(FSkinWeightProfileInfo, DefaultProfile))
			{
				IDetailPropertyRow& Row = ChildBuilder.AddProperty(ChildProperty.ToSharedRef());

				TAttribute<bool> Attribute;
				Attribute.BindRaw(this, &FSkinWeightProfileCustomization::CheckAnyOtherProfileMarkedAsDefault);
				Row.IsEnabled(Attribute);
			}
			// Customize the DefaultProfileFromLODIndex property so it can only be set if the current profile is set as the default one
			else if (ChildProperty->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(FSkinWeightProfileInfo, DefaultProfileFromLODIndex))
			{
				if (USkeletalMesh* SkeletalMesh = WeakSkeletalMesh.Get())
				{
					ChildProperty->SetInstanceMetaData(FName("UIMax"), FString::FromInt(SkeletalMesh->GetLODNum() - 1));
					ChildProperty->SetInstanceMetaData(FName("ClampMax"), FString::FromInt(SkeletalMesh->GetLODNum() - 1));
				}

				IDetailPropertyRow& Row = ChildBuilder.AddProperty(ChildProperty.ToSharedRef());

				TAttribute<bool> Attribute;
				Attribute.BindRaw(this, &FSkinWeightProfileCustomization::IsProfileMarkedAsDefault);
				Row.IsEnabled(Attribute);
			}
			// Customize the per-lod stored source file paths, first so it's read only and uses the LOD index as the label and secondly to add a button allowing the user to reimport the LOD with a different file
			else if (ChildProperty->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(FSkinWeightProfileInfo, PerLODSourceFiles))
			{
				uint32 NumSourceFileChildren = 0;
				if (ChildProperty->GetNumChildren(NumSourceFileChildren) == FPropertyAccess::Success)
				{
					TSharedPtr<IPropertyHandleMap> SourceFilesMapProperty = ChildProperty->AsMap();

					if (SourceFilesMapProperty.IsValid() && NumSourceFileChildren > 0)
					{
						void* MapDataPtr = nullptr;
						if (ChildProperty->GetValueData(MapDataPtr) == FPropertyAccess::Success)
						{
							TMap<int32, FString>* MapPtr = (TMap<int32, FString>*)MapDataPtr;
							if (MapPtr)
							{
								IDetailGroup& SourceFilesGroup = ChildBuilder.AddGroup(GET_MEMBER_NAME_CHECKED(FSkinWeightProfileInfo, PerLODSourceFiles), LOCTEXT("SourceFilesGroupLabel", "Source Files"));
								for (auto Pair : *MapPtr)
								{
									SourceFilesGroup.AddWidgetRow()
									.NameContent()
									[
										SNew(SHorizontalBox)
										+ SHorizontalBox::Slot()
										.AutoWidth()
										[
											SNew(STextBlock)
											.Text(FText::Format(LOCTEXT("SourceFilesEntryLabel", "LOD {0}"), Pair.Key))
											.Font(CustomizationUtils.GetRegularFont())
										]
									]
									.ValueContent()
									[
										SNew(SHorizontalBox)
										+ SHorizontalBox::Slot()
										.AutoWidth()
										.VAlign(VAlign_Center)
										[
											SNew(SEditableTextBox)
											.Text_Lambda([=]() -> FText { return FText::FromString(Pair.Value); })
											.Font(CustomizationUtils.GetRegularFont())
											.IsReadOnly(true)
										]
										+ SHorizontalBox::Slot()
										.AutoWidth()
										.VAlign(VAlign_Center)
										.Padding(4.0f, 0.0f, 0.0f, 0.0f)
										[
											SNew(SButton)
											.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
											.ToolTipText(LOCTEXT("SkinWeightProfileSourceFileTooltip", "Choose a different source import file"))
											.OnClicked_Lambda([this, LODIndex = Pair.Key]()->FReply
											{
												if (USkeletalMesh* Mesh = WeakSkeletalMesh.Get())
												{
													FScopedTransaction ScopedTransaction(LOCTEXT("ReimportSkinProfileLODNewFileTransaction", "Reimport Skin Weight Profile LOD with different file"));
													Mesh->Modify();

													FSkinWeightProfileHelpers::ImportSkinWeightProfileLOD(Mesh, LastKnownProfileName, LODIndex);
													PropertyUtilities->ForceRefresh();
												}

												return FReply::Handled();
											})
											.ContentPadding(2.0f)
											.ForegroundColor(FSlateColor::UseForeground())
											.IsFocusable(false)
											[
												SNew(SImage)
												.Image(FAppStyle::GetBrush("PropertyWindow.Button_Ellipsis"))
												.ColorAndOpacity(FSlateColor::UseForeground())
											]
										]
									];
								}
							}						
						}
					}
				}
			}
			else
			{
				IDetailPropertyRow& Row = ChildBuilder.AddProperty(ChildProperty.ToSharedRef());
			}
		}
	}
}

void FSkinWeightProfileCustomization::RenameProfile()
{
	if (USkeletalMesh* SkeletalMesh = WeakSkeletalMesh.Get())
	{
		FName NewName = NAME_None;
		NameProperty->GetValue(NewName);

		// Make sure the profile isn't in use anywhere 
		FSkinWeightProfileHelpers::ClearSkinWeightProfileInstanceOverrides(SkeletalMesh, LastKnownProfileName);

		FSkinnedMeshComponentRecreateRenderStateContext ReregisterContext(SkeletalMesh);

		// Remove the profile entry on a per-lod basis
		for (FSkeletalMeshLODModel& LODModel : SkeletalMesh->GetImportedModel()->LODModels)
		{
			FImportedSkinWeightProfileData ProfileData;
			if (LODModel.SkinWeightProfiles.RemoveAndCopyValue(LastKnownProfileName, ProfileData))
			{
				LODModel.SkinWeightProfiles.Add(NewName, ProfileData);
			}
		}
		SkeletalMesh->PostEditChange();
	}

	// Refresh the restricted set of names
	UpdateNameRestriction();
}

FText FSkinWeightProfileCustomization::OnGetProfileName() const
{
	return FText::FromName(LastKnownProfileName);
}

void FSkinWeightProfileCustomization::OnProfileNameChanged(const FText& InNewText)
{
	const bool bValidProfileName = IsProfileNameValid(InNewText.ToString());
	if (!bValidProfileName)
	{
		NameEditTextBox->SetError(LOCTEXT("OnProfileNameChanged_NotValid", "This name is already in use"));
	}
	else
	{
		NameEditTextBox->SetError(FText::GetEmpty());
	}
}

void FSkinWeightProfileCustomization::OnProfileNameCommitted(const FText& InNewText, ETextCommit::Type InTextCommit)
{
	if (IsProfileNameValid(InNewText.ToString()) && InTextCommit != ETextCommit::OnCleared)
	{
		const FScopedTransaction Transaction(LOCTEXT("RenameProfile", "Rename Profile"));
		WeakSkeletalMesh->Modify();
		if (NameProperty->SetValue(FName(*InNewText.ToString())) == FPropertyAccess::Success)
		{
			RenameProfile();
			LastKnownProfileName = FName(*InNewText.ToString());
		}
	}

	UpdateNameRestriction();
}

const bool FSkinWeightProfileCustomization::IsProfileNameValid(const FString& NewName)
{
	UpdateNameRestriction();
	return !RestrictedNames.Contains(NewName);
}

TSharedRef<class SWidget> FSkinWeightProfileCustomization::GenerateReimportMenu()
{
	FMenuBuilder ReimportMenuBuilder(true, nullptr, nullptr, true);

	// First entry to re import all the LOD data using their specific files (if applicable)
	ReimportMenuBuilder.AddMenuEntry(LOCTEXT("ReimportOverrideLabel", "Reimport all Skin Weight LODs"), LOCTEXT("ReimportOverrideToolTip", "Reimport all files used for the different LODs"),
		FSlateIcon(), FUIAction(FExecuteAction::CreateLambda([this]()
	{
		if (USkeletalMesh* SkeletalMesh = WeakSkeletalMesh.Get())
		{
			const int32 NumLODs = SkeletalMesh->GetLODNum();
			for (int32 Index = 0; Index < NumLODs; ++Index)
			{
				FScopedTransaction ScopedTransaction(LOCTEXT("ReimportSkinProfiles", "Reimport Skin Weight Profile"));
				SkeletalMesh->Modify();

				FSkinWeightProfileHelpers::ReimportSkinWeightProfileLOD(SkeletalMesh, LastKnownProfileName, Index);
				PropertyUtilities->ForceRefresh();
			}
		}
	})));

	// Possible entries to re import specific LODs
	if (USkeletalMesh* Mesh = WeakSkeletalMesh.Get())
	{
		const int32 NumLODs = Mesh->GetLODNum();
		const FSkinWeightProfileInfo* ProfilePtr = Mesh->GetSkinWeightProfiles().FindByPredicate([this](FSkinWeightProfileInfo Info) { return Info.Name == LastKnownProfileName; });

		// Only show in case we have per-lod stored source files
		if (ProfilePtr && NumLODs > 1 && ProfilePtr->PerLODSourceFiles.Num() > 1)
		{
			ReimportMenuBuilder.AddMenuSeparator();

			const int32 NumLODData = ProfilePtr->PerLODSourceFiles.Num();
			for (auto Pair : ProfilePtr->PerLODSourceFiles)
			{
				const FText Label = FText::Format(LOCTEXT("ReimportLODOverrideLabel", "Reimport LOD {0} Skin Weight "), FText::AsNumber(Pair.Key));
				ReimportMenuBuilder.AddMenuEntry(Label, LOCTEXT("ReimportLODOverrideToolTip", "Reimport data for specific previously imported LOD"),
					FSlateIcon(), FUIAction(FExecuteAction::CreateLambda([this, LODIndex = Pair.Key]()
				{
					if (USkeletalMesh* SkeletalMesh = WeakSkeletalMesh.Get())
					{
						FScopedTransaction ScopedTransaction(LOCTEXT("ReimportSkinProfileLOD", "Reimport Skin Weight Profile LOD"));
						SkeletalMesh->Modify();

						FSkinWeightProfileHelpers::ReimportSkinWeightProfileLOD(SkeletalMesh, LastKnownProfileName, LODIndex);
						PropertyUtilities->ForceRefresh();
					}
				})));
			}
		}
	}

	return ReimportMenuBuilder.MakeWidget();
}

TSharedRef<class SWidget> FSkinWeightProfileCustomization::GenerateRemoveMenu()
{
	FMenuBuilder RemoveMenuBuilder(true, nullptr, nullptr, true);
	
	// Removing the entire skin weight profile
	RemoveMenuBuilder.AddMenuEntry(LOCTEXT("RemoveOverrideLabel", "Remove entire Skin Weight Profile"), LOCTEXT("RemoveOverrideToolTip", "Remove all data for this Skin Weight Profile"),
		FSlateIcon(), FUIAction(FExecuteAction::CreateLambda([this]()
	{
		// Provide user with dialog to make sure they don't delete it on purpose
		const FText Text = FText::Format(LOCTEXT("RemoveOverrideWarning", "Are you sure you want to remove Skin Weight Profile {0}?"), FText::FromName(LastKnownProfileName));
		EAppReturnType::Type Ret = FMessageDialog::Open(EAppMsgType::YesNo, Text);

		if (Ret == EAppReturnType::Yes)
		{
			if (USkeletalMesh* SkeletalMesh = WeakSkeletalMesh.Get())
			{
				//Scope the postedit change
				{
					FScopedSuspendAlternateSkinWeightPreview ScopedSuspendAlternateSkinnWeightPreview(SkeletalMesh);
					FScopedSkeletalMeshPostEditChange ScopedPostEditChange(SkeletalMesh);
					FScopedTransaction ScopedTransaction(LOCTEXT("RemoveSkinProfileTransaction", "Remove Skin Weight Profile"));
					SkeletalMesh->Modify();
					FSkinWeightProfileHelpers::RemoveSkinWeightProfile(SkeletalMesh, LastKnownProfileName);
					UpdateNameRestriction();
				}

				PropertyUtilities->ForceRefresh();
			}
		}
		
	})));

	// Removing specific per-lod skin weight data, if applicable
	if (USkeletalMesh* Mesh = WeakSkeletalMesh.Get())
	{
		const int32 NumLODs = Mesh->GetLODNum();
		const FSkinWeightProfileInfo* ProfilePtr = Mesh->GetSkinWeightProfiles().FindByPredicate([this](FSkinWeightProfileInfo Info) { return Info.Name == LastKnownProfileName; });

		if (ProfilePtr && NumLODs > 1 && ProfilePtr->PerLODSourceFiles.Num() > 1)
		{
			RemoveMenuBuilder.AddMenuSeparator();

			const int32 NumLODData = ProfilePtr->PerLODSourceFiles.Num();
			for (auto Pair : ProfilePtr->PerLODSourceFiles)
			{				
				const FText Label = FText::Format(LOCTEXT("RemoveLODOverrideLabel", "Remove LOD {0} Skin Weight"), FText::AsNumber(Pair.Key));
				RemoveMenuBuilder.AddMenuEntry(Label, LOCTEXT("RemoveOverrideLODToolTip", "Remove data for specific imported LOD"),
					FSlateIcon(), FUIAction(FExecuteAction::CreateLambda([this, LODIndex = Pair.Key]()
				{
					const FText Text = FText::Format(LOCTEXT("RemoveLODOverrideWarning", "Are you sure you want to remove LOD {0} from Skin Weight Profile {1}?"), FText::AsNumber(LODIndex), FText::FromName(LastKnownProfileName));
					EAppReturnType::Type Ret = FMessageDialog::Open(EAppMsgType::YesNo, Text);
					if (Ret == EAppReturnType::Yes)
					{
						if (USkeletalMesh* SkeletalMesh = WeakSkeletalMesh.Get())
						{
							FSkinnedMeshComponentRecreateRenderStateContext ReregisterContext(SkeletalMesh);
							FScopedTransaction ScopedTransaction(LOCTEXT("RemoveSkinProfileLODTransaction", "Remove Skin Weight Profile LOD"));
							SkeletalMesh->Modify();

							FSkinWeightProfileHelpers::RemoveSkinWeightProfileLOD(SkeletalMesh, LastKnownProfileName, LODIndex);

							SkeletalMesh->PostEditChange();
							PropertyUtilities->ForceRefresh();
						}
					}
				})));								
			}
		}
	}

	return RemoveMenuBuilder.MakeWidget();
}

bool FSkinWeightProfileCustomization::CheckAnyOtherProfileMarkedAsDefault() const 
{
	if (USkeletalMesh* SkeletalMesh = WeakSkeletalMesh.Get())
	{
		const TArray<FSkinWeightProfileInfo>& Profiles = SkeletalMesh->GetSkinWeightProfiles();
		for (const FSkinWeightProfileInfo& ProfileInfo : Profiles)
		{
			// Check if a profile other than the current one is marked to be loaded by default
			if ((LastKnownProfileName != ProfileInfo.Name) && ProfileInfo.DefaultProfile.Default)
			{
				return false;
			}
		}
	}

	return true;
}

bool FSkinWeightProfileCustomization::IsProfileMarkedAsDefault() const
{
	if (USkeletalMesh* SkeletalMesh = WeakSkeletalMesh.Get())
	{
		const TArray<FSkinWeightProfileInfo>& Profiles = SkeletalMesh->GetSkinWeightProfiles();
		const FSkinWeightProfileInfo* ProfilePtr = Profiles.FindByPredicate([this](FSkinWeightProfileInfo Info)
		{
			return Info.Name == LastKnownProfileName;
		});

		if (ProfilePtr && ProfilePtr->DefaultProfile.Default)
		{
			return true;
		}
	}

	return false;
}

#undef LOCTEXT_NAMESPACE