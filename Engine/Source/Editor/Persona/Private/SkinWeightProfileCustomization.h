// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "PropertyRestriction.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SCheckBox.h"
#include "IPropertyUtilities.h"

class USkeletalMesh;

/** Details customization for FSkinWeightProfileInfo */
struct FSkinWeightProfileCustomization : public IPropertyTypeCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance() { return MakeShareable(new FSkinWeightProfileCustomization()); }

	/** Begin IPropertyTypeCustomization overrides */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	/** End IPropertyTypeCustomization overrides */

protected:
	/** Check whether or not the currently inputted profile name is not used by any other profile in the mesh */
	void UpdateNameRestriction();
	/** Renaming any profile data stored on a per-lod basis */
	void RenameProfile();

	/** Name of the currently customization profile, last time it was retrieved */
	FName LastKnownProfileName;

	/** Callbacks used by NameEditTextBox widget*/
	FText OnGetProfileName() const;
	void OnProfileNameChanged(const FText& InNewText);
	void OnProfileNameCommitted(const FText& InNewText, ETextCommit::Type InTextCommit);
	const bool IsProfileNameValid(const FString& NewName);

	/** Widgets used for customizing the Profile.Name property */
	TSharedPtr<SEditableTextBox> NameEditTextBox;
	TSharedPtr<IPropertyHandle> NameProperty;
	/** Set of names that are currently used by other profiles in this profile's outer Skeletal Mesh */
	TArray<FString> RestrictedNames;

	/** Generates the sub menu used for reimporting (per-lod) profile data */
	TSharedRef<class SWidget> GenerateReimportMenu();
	/** Generates the sub menu used for removing (per-lod) profile data */
	TSharedRef<class SWidget> GenerateRemoveMenu();

	/** Checks to see if any other profile in the skeletal mesh is currently marked to be loaded by default */
	bool CheckAnyOtherProfileMarkedAsDefault() const;

	/** Checks to see if this profile in the skeletal mesh is currently marked to be loaded by default */
	bool IsProfileMarkedAsDefault() const;
	
	/** Weak-ptr to the currently customized profile's outer Skeletal Mesh */
	TWeakObjectPtr<USkeletalMesh> WeakSkeletalMesh;

	TSharedPtr<IPropertyUtilities> PropertyUtilities;
};
