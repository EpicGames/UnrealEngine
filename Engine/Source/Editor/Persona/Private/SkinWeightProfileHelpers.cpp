// Copyright Epic Games, Inc. All Rights Reserved.

#include "SkinWeightProfileHelpers.h"

#include "SSkinWeightProfileImportOptions.h"
#include "HAL/UnrealMemory.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkinnedAssetCommon.h"
#include "Widgets/SWindow.h"
#include "Modules/ModuleManager.h"
#include "Framework/Application/SlateApplication.h"
#include "Interfaces/IMainFrameModule.h"
#include "Rendering/SkeletalMeshModel.h"
#include "LODUtilities.h"
#include "SkinWeightsUtilities.h"
#include "IMeshReductionInterfaces.h"
#include "ComponentReregisterContext.h"
#include "Components/SkinnedMeshComponent.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "EditorFramework/AssetImportData.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Misc/MessageDialog.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Misc/CoreMisc.h"

#define LOCTEXT_NAMESPACE "SkinWeightProfileHelpers"

void FSkinWeightProfileHelpers::ImportSkinWeightProfile(USkeletalMesh* InSkeletalMesh)
{
	if (InSkeletalMesh)
	{
		const int32 LOD0Index = 0;
		FString PickedFileName = FSkinWeightsUtilities::PickSkinWeightPath(LOD0Index, InSkeletalMesh);

		bool bShouldImport = false;
		USkinWeightImportOptions* ImportSettings = GetMutableDefault<USkinWeightImportOptions>();		

		// Show settings dialog, for user to provide name for the new profile
		if (FPaths::FileExists(PickedFileName))
		{
			TSharedRef<SWindow> Window = SNew(SWindow)
				.Title(LOCTEXT("WindowTitle", "Skin Weight Profile Import Options"))
				.SizingRule(ESizingRule::Autosized);

			TSharedPtr<SSkinWeightProfileImportOptions> OptionsWidget;
			ImportSettings->FilePath = PickedFileName;
			ImportSettings->LODIndex = 0;
			Window->SetContent
			(
				SAssignNew(OptionsWidget, SSkinWeightProfileImportOptions).WidgetWindow(Window)
				.ImportSettings(ImportSettings)
				.WidgetWindow(Window)
				.SkeletalMesh(InSkeletalMesh)
			);

			TSharedPtr<SWindow> ParentWindow;

			if (FModuleManager::Get().IsModuleLoaded("MainFrame"))
			{
				IMainFrameModule& MainFrame = FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame");
				ParentWindow = MainFrame.GetParentWindow();
			}

			FSlateApplication::Get().AddModalWindow(Window, ParentWindow, false);

			// Poll the result of the window interaction, user could have cancelled operation
			bShouldImport = OptionsWidget->ShouldImport();
		}
		
		if (bShouldImport)
		{
			FScopedSuspendAlternateSkinWeightPreview ScopedSuspendAlternateSkinnWeightPreview(InSkeletalMesh);
			FScopedSkeletalMeshPostEditChange ScopedPostEditChange(InSkeletalMesh);
			const FName ProfileName(*ImportSettings->ProfileName);			
			// Try and import the skin weight profile from the provided FBX file path
			const bool bResult = FSkinWeightsUtilities::ImportAlternateSkinWeight(InSkeletalMesh, PickedFileName, ImportSettings->LODIndex, ProfileName, false);
			if (bResult)
			{
				FNotificationInfo NotificationInfo(FText::GetEmpty());
				NotificationInfo.Text = LOCTEXT("ImportSuccessful", "Skin Weights imported successfully!");
				NotificationInfo.ExpireDuration = 2.5f;
				FSlateNotificationManager::Get().AddNotification(NotificationInfo);
			}
			else
			{
				FNotificationInfo NotificationInfo(FText::GetEmpty());
				NotificationInfo.Text = LOCTEXT("ImportFailed", "Failed to import Skin Weights!");
				NotificationInfo.ExpireDuration = 2.5f;
				FSlateNotificationManager::Get().AddNotification(NotificationInfo);
			}

			if(!InSkeletalMesh->HasMeshDescription(ImportSettings->LODIndex))
			{
				FLODUtilities::RegenerateDependentLODs(InSkeletalMesh, ImportSettings->LODIndex, GetTargetPlatformManagerRef().GetRunningTargetPlatform());
			}
		}
	}
}

void FSkinWeightProfileHelpers::ImportSkinWeightProfileLOD(USkeletalMesh* InSkeletalMesh, FName ProfileName, int32 LODIndex)
{
	if (InSkeletalMesh)
	{
		// Pick a FBX file to import the weights from 
		const FString PickedFileName = FSkinWeightsUtilities::PickSkinWeightPath(LODIndex, InSkeletalMesh);
		FScopedSuspendAlternateSkinWeightPreview ScopedSuspendAlternateSkinnWeightPreview(InSkeletalMesh);
		FScopedSkeletalMeshPostEditChange ScopedPostEditChange(InSkeletalMesh);
		// Try and import skin weights for a specific mesh LOD
		const bool bResult = FSkinWeightsUtilities::ImportAlternateSkinWeight(InSkeletalMesh, PickedFileName, LODIndex, ProfileName, false);
		if (bResult)
		{
			if(!InSkeletalMesh->HasMeshDescription(LODIndex))
			{
				FLODUtilities::RegenerateDependentLODs(InSkeletalMesh, LODIndex, GetTargetPlatformManagerRef().GetRunningTargetPlatform());
			}

			FNotificationInfo NotificationInfo(FText::GetEmpty());
			NotificationInfo.Text = FText::Format(LOCTEXT("LODImportSuccessful", "Skin Weights for LOD {0} imported successfully!"), FText::AsNumber(LODIndex));
			NotificationInfo.ExpireDuration = 2.5f;
			FSlateNotificationManager::Get().AddNotification(NotificationInfo);
		}
		else
		{
			FNotificationInfo NotificationInfo(FText::GetEmpty());
			NotificationInfo.Text = FText::Format(LOCTEXT("LODImportFailed", "Failed to import Skin Weights for LOD {0}!"), FText::AsNumber(LODIndex));
			NotificationInfo.ExpireDuration = 2.5f;
			FSlateNotificationManager::Get().AddNotification(NotificationInfo);
		}
	}
}


void FSkinWeightProfileHelpers::ReimportSkinWeightProfileLOD(USkeletalMesh* InSkeletalMesh, const FName& InProfileName, const int32 LODIndex)
{
	const FSkinWeightProfileInfo* Profile = InSkeletalMesh->GetSkinWeightProfiles().FindByPredicate([InProfileName](FSkinWeightProfileInfo Profile) { return Profile.Name == InProfileName; });
	if (Profile)
	{
		if (const FString* PathNamePtr = Profile->PerLODSourceFiles.Find(LODIndex))
		{
			FScopedSuspendAlternateSkinWeightPreview ScopedSuspendAlternateSkinnWeightPreview(InSkeletalMesh);
			FScopedSkeletalMeshPostEditChange ScopedPostEditChange(InSkeletalMesh);
			bool bResult = false;
			const FString& PathName = UAssetImportData::ResolveImportFilename(*PathNamePtr, InSkeletalMesh->GetOutermost());
			
			// Check to see if the source file is still valid
			if (FPaths::FileExists(PathName))
			{
				bResult = FSkinWeightsUtilities::ImportAlternateSkinWeight(InSkeletalMesh, PathName, LODIndex, InProfileName, true);
			}
			else
			{
				FText WarningMessage = FText::Format(LOCTEXT("Warning_SkinWeightsFileMissing", "Previous file {0} containing Skin Weight data for LOD {1} could not be found, do you want to specify a new path?"), FText::FromString(PathName), LODIndex);				
				if (EAppReturnType::Yes == FMessageDialog::Open(EAppMsgType::YesNo, WarningMessage))
				{
					// Otherwise let the user pick a new path
					const FString PickedFileName = FSkinWeightsUtilities::PickSkinWeightPath(LODIndex, InSkeletalMesh);
					bResult = FSkinWeightsUtilities::ImportAlternateSkinWeight(InSkeletalMesh, PickedFileName, LODIndex, InProfileName, true);
				}
			}

			if (bResult)
			{
				if (!InSkeletalMesh->HasMeshDescription(LODIndex))
				{
					// Make sure we regenerate any LOD data that is based off the now re imported LOD
					FLODUtilities::RegenerateDependentLODs(InSkeletalMesh, LODIndex, GetTargetPlatformManagerRef().GetRunningTargetPlatform());
				}

				FNotificationInfo NotificationInfo(FText::GetEmpty());
				NotificationInfo.Text = FText::Format(LOCTEXT("ReimportedSkinWeightsForLOD", "Reimported Skin Weights for LOD {0} succesfully!"), FText::AsNumber(LODIndex));
				NotificationInfo.ExpireDuration = 2.5f;
				FSlateNotificationManager::Get().AddNotification(NotificationInfo);
			}
			else
			{
				FNotificationInfo NotificationInfo(FText::GetEmpty());
				NotificationInfo.Text = FText::Format(LOCTEXT("ReimportSkinWeightsForLODFailed", "Reimporting Skin Weights for LOD {0} failed!"), FText::AsNumber(LODIndex));
				NotificationInfo.ExpireDuration = 2.5f;
				FSlateNotificationManager::Get().AddNotification(NotificationInfo);
			}
		}
	}
}

void FSkinWeightProfileHelpers::RemoveSkinWeightProfile(USkeletalMesh* InSkeletalMesh, const FName& InProfileName)
{
	const int32 NumRemoved = InSkeletalMesh->GetSkinWeightProfiles().RemoveAll([InProfileName](FSkinWeightProfileInfo Profile)
	{
		return Profile.Name == InProfileName;
	});

	// Check whether or not we actually removed a profile
	if (NumRemoved)
	{
		FScopedSuspendAlternateSkinWeightPreview ScopedSuspendAlternateSkinnWeightPreview(InSkeletalMesh);
		FScopedSkeletalMeshPostEditChange ScopedPostEditChange(InSkeletalMesh);
		FSkeletalMeshUpdateContext UpdateContext;
		UpdateContext.SkeletalMesh = InSkeletalMesh;
		
		FSkeletalMeshModel* ImportModel = InSkeletalMesh->GetImportedModel();		
		for (int32 LODIndex = 0; LODIndex < ImportModel->LODModels.Num(); ++LODIndex)
		{
			FSkeletalMeshLODModel& LODModel = ImportModel->LODModels[LODIndex];
			const FSkeletalMeshLODInfo* LODInfo = InSkeletalMesh->GetLODInfo(LODIndex);
			if (LODInfo && LODInfo->bHasBeenSimplified)
			{
				// Delete the alternate influences data
				LODModel.SkinWeightProfiles.Remove(InProfileName);
				if (!InSkeletalMesh->HasMeshDescription(LODIndex))
				{
					// Regenerate this generated LOD
					FLODUtilities::SimplifySkeletalMeshLOD(UpdateContext, LODIndex, GetTargetPlatformManagerRef().GetRunningTargetPlatform());
				}
				// Goto next LOD
				continue;
			}
			
			// Make sure we actually removed the profile data
			if (!FSkinWeightsUtilities::RemoveSkinnedWeightProfileData(InSkeletalMesh, InProfileName, LODIndex))
			{
				FNotificationInfo NotificationInfo(FText::GetEmpty());
				NotificationInfo.Text = FText::Format(LOCTEXT("FailedToRemoveLODWeights", "Failed to remove Skin Weights for LOD {0}!"), FText::AsNumber(LODIndex));
				NotificationInfo.ExpireDuration = 2.5f;
				FSlateNotificationManager::Get().AddNotification(NotificationInfo);
			}
			LODModel.SkinWeightProfiles.Remove(InProfileName);
		}
	}
}

void FSkinWeightProfileHelpers::RemoveSkinWeightProfileLOD(USkeletalMesh* InSkeletalMesh, const FName& InProfileName, const int32 LODIndex)
{
	FSkinWeightProfileInfo* Profile = InSkeletalMesh->GetSkinWeightProfiles().FindByPredicate([InProfileName](FSkinWeightProfileInfo Profile) { return Profile.Name == InProfileName; });
	if (Profile)
	{
		// Remove source path
		Profile->PerLODSourceFiles.Remove(LODIndex);		

		FSkeletalMeshModel* ImportModel = InSkeletalMesh->GetImportedModel();			
		const FSkeletalMeshLODInfo* LODInfo = InSkeletalMesh->GetLODInfo(LODIndex);
		if (LODInfo && LODInfo->bHasBeenSimplified)
		{
			//We cannot remove alternate skin weights profile for a generated LOD
			//It will be removed when the base LOD used to reduce will get their skin weights profile removed
			return;
		}
		FScopedSuspendAlternateSkinWeightPreview ScopedSuspendAlternateSkinnWeightPreview(InSkeletalMesh);
		FScopedSkeletalMeshPostEditChange ScopedPostEditChange(InSkeletalMesh);

		if (FSkinWeightsUtilities::RemoveSkinnedWeightProfileData(InSkeletalMesh, InProfileName, LODIndex))
		{
			FSkinWeightProfileHelpers::ClearSkinWeightProfileInstanceOverrides(InSkeletalMesh, InProfileName);

			if (!InSkeletalMesh->HasMeshDescription(LODIndex))
			{
				//Regenerate dependent LODs if we remove the LOD successfully
				FLODUtilities::RegenerateDependentLODs(InSkeletalMesh, LODIndex, GetTargetPlatformManagerRef().GetRunningTargetPlatform());
			}

			FNotificationInfo NotificationInfo(FText::GetEmpty());
			NotificationInfo.Text = FText::Format(LOCTEXT("RemovedLODWeights", "Removed Skin Weights for LOD {0}!"), FText::AsNumber(LODIndex));
			NotificationInfo.ExpireDuration = 2.5f;
			FSlateNotificationManager::Get().AddNotification(NotificationInfo);
		}
		else
		{
			FNotificationInfo NotificationInfo(FText::GetEmpty());
			NotificationInfo.Text = FText::Format(LOCTEXT("FailedToRemoveLODWeights", "Failed to remove Skin Weights for LOD {0}!"), FText::AsNumber(LODIndex));
			NotificationInfo.ExpireDuration = 2.5f;
			FSlateNotificationManager::Get().AddNotification(NotificationInfo);
		}
	}
}

void FSkinWeightProfileHelpers::ClearSkinWeightProfileInstanceOverrides(USkeletalMesh* InSkeletalMesh, FName InProfileName)
{
	//Make sure all component are unregister (world and render data) avoid post edit change
	FScopedSuspendAlternateSkinWeightPreview ScopedSuspendAlternateSkinnWeightPreview(InSkeletalMesh);
	FScopedSkeletalMeshPostEditChange ScopedPostEditChange(InSkeletalMesh, false, true);
	for (TObjectIterator<USkinnedMeshComponent> It; It; ++It)
	{
		if (Cast<USkeletalMesh>(It->GetSkinnedAsset()) == InSkeletalMesh)
		{
			checkf(!It->IsUnreachable(), TEXT("%s"), *It->GetFullName());

			if (It->GetCurrentSkinWeightProfileName() == InProfileName)
			{
				It->ClearSkinWeightProfile();
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE