// Copyright Epic Games, Inc. All Rights Reserved.

#include "TabSpawners.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "SSkeletonAnimNotifies.h"
#include "SAnimBlueprintParentPlayerList.h"
#include "SSkeletonSlotNames.h"
#include "AdvancedPreviewSceneModule.h"
#include "IAnimationBlueprintEditor.h"
#include "ISkeletonTree.h"
#include "ISkeletonEditorModule.h"
#include "SPersonaDetails.h"
#include "PersonaUtils.h"
#include "SMorphTargetViewer.h"
#include "SAnimCurveViewer.h"
#include "SAnimAttributeView.h"
#include "SAnimationSequenceBrowser.h"
#include "SAnimationEditorViewport.h"
#include "SPoseWatchManager.h"
#include "SRetargetSources.h"
#include "SKismetInspector.h"
#include "Widgets/Input/SButton.h"
#include "PersonaPreviewSceneDescription.h"
#include "IPersonaPreviewScene.h"
#include "PreviewSceneCustomizations.h"
#include "Engine/PreviewMeshCollection.h"
#include "PoseWatchManagerPublicTypes.h"
#include "PoseWatchManagerDefaultMode.h"
#include "SAnimCurveMetadataEditor.h"

#define LOCTEXT_NAMESPACE "PersonaModes"

/////////////////////////////////////////////////////
// FPersonaTabs

// Tab constants
const FName FPersonaTabs::MorphTargetsID("MorphTargetsTab");
const FName FPersonaTabs::AnimCurveViewID("AnimCurveViewerTab");
const FName FPersonaTabs::AnimCurveMetadataEditorID("AnimCurveMetadataEditorTab");
const FName FPersonaTabs::SkeletonTreeViewID("SkeletonTreeView");		//@TODO: Name
// Skeleton Pose manager
const FName FPersonaTabs::RetargetManagerID("RetargetManager");
// Anim Blueprint params
// Explorer
// Class Defaults
const FName FPersonaTabs::AnimBlueprintPreviewEditorID("AnimBlueprintPreviewEditor");

const FName FPersonaTabs::AnimBlueprintParentPlayerEditorID("AnimBlueprintParentPlayerEditor");
// Anim Document
const FName FPersonaTabs::ScrubberID("ScrubberTab");

// Toolbar
const FName FPersonaTabs::PreviewViewportID("Viewport");
const FName FPersonaTabs::PreviewViewport1ID("Viewport1");
const FName FPersonaTabs::PreviewViewport2ID("Viewport2");
const FName FPersonaTabs::PreviewViewport3ID("Viewport3");
const FName FPersonaTabs::AssetBrowserID("SequenceBrowser");	//@TODO: Name
const FName FPersonaTabs::MirrorSetupID("MirrorSetupTab");
const FName FPersonaTabs::AnimBlueprintDebugHistoryID("AnimBlueprintDebugHistoryTab");
const FName FPersonaTabs::AnimAssetPropertiesID("AnimAssetPropertiesTab");
const FName FPersonaTabs::MeshAssetPropertiesID("MeshAssetPropertiesTab");
const FName FPersonaTabs::PreviewManagerID("AnimPreviewSetup");		//@TODO: Name
const FName FPersonaTabs::SkeletonAnimNotifiesID("SkeletonAnimNotifies");
const FName FPersonaTabs::SkeletonSlotNamesID("SkeletonSlotNames");
const FName FPersonaTabs::SkeletonSlotGroupNamesID("SkeletonSlotGroupNames");
const FName FPersonaTabs::BlendProfileManagerID("BlendProfileManager");
const FName FPersonaTabs::AnimMontageSectionsID("AnimMontageSections");

const FName FPersonaTabs::PoseWatchManagerID("PoseWatchManager");

const FName FPersonaTabs::AdvancedPreviewSceneSettingsID("AdvancedPreviewTab");

const FName FPersonaTabs::DetailsID("DetailsTab");

const FName FPersonaTabs::FindReplaceID("FindReplaceTab");

const FName FPersonaTabs::ToolboxID("ToolBox");

const FName FPersonaTabs::AnimAttributeViewID("AnimAttributeViewerTab");

/////////////////////////////////////////////////////
// FPersonaMode

// Mode constants
const FName FPersonaModes::SkeletonDisplayMode( "SkeletonName" );
const FName FPersonaModes::MeshEditMode( "MeshName" );
const FName FPersonaModes::PhysicsEditMode( "PhysicsName" );
const FName FPersonaModes::AnimationEditMode( "AnimationName" );
const FName FPersonaModes::AnimBlueprintEditMode( "GraphName" );

/////////////////////////////////////////////////////
// FPersonaModeSharedData

FPersonaModeSharedData::FPersonaModeSharedData()
	: OrthoZoom(1.0f)
	, bCameraLock(true)
	, CameraFollowMode(EAnimationViewportCameraFollowMode::None)
	, CameraFollowBoneName(NAME_None)
	, bShowReferencePose(false)
	, bShowBones(false)
	, bShowBoneNames(false)
	, bShowSockets(false)
	, bShowBound(false)
	, ViewportType(0)
	, PlaybackSpeedMode(EAnimationPlaybackSpeeds::Normal)
	, LocalAxesMode(0)
{}

void FPersonaModeSharedData::Save(const TSharedRef<FAnimationViewportClient>& InFromViewport)
{
	ViewLocation = InFromViewport->GetViewLocation();
	ViewRotation = InFromViewport->GetViewRotation();
	LookAtLocation = InFromViewport->GetLookAtLocation();
	OrthoZoom = InFromViewport->GetOrthoZoom();
	bCameraLock = InFromViewport->IsCameraLocked();
	CameraFollowMode = InFromViewport->GetCameraFollowMode();
	CameraFollowBoneName = InFromViewport->GetCameraFollowBoneName();
	bShowBound = InFromViewport->IsSetShowBoundsChecked();
	LocalAxesMode = InFromViewport->GetLocalAxesMode();
	ViewportType = InFromViewport->ViewportType;

	const TSharedRef<FAnimationEditorPreviewScene> AnimationEditorPreviewScene =
		StaticCastSharedRef<FAnimationEditorPreviewScene>(InFromViewport->GetPreviewScene());
	PlaybackSpeedMode = AnimationEditorPreviewScene->GetAnimationPlaybackSpeedMode();
}

void FPersonaModeSharedData::Restore(const TSharedRef<FAnimationViewportClient>& InToViewport)
{
	InToViewport->SetViewportType((ELevelViewportType)ViewportType);
	InToViewport->SetViewLocation(ViewLocation);
	InToViewport->SetViewRotation(ViewRotation);
	InToViewport->SetShowBounds(bShowBound);
	InToViewport->SetLocalAxesMode((ELocalAxesMode::Type)LocalAxesMode);
	InToViewport->SetOrthoZoom(OrthoZoom);

	const TSharedRef<FAnimationEditorPreviewScene> AnimationEditorPreviewScene =
		StaticCastSharedRef<FAnimationEditorPreviewScene>(InToViewport->GetPreviewScene());
	AnimationEditorPreviewScene->SetAnimationPlaybackSpeedMode(PlaybackSpeedMode);

	if (bCameraLock)
	{
		InToViewport->SetLookAtLocation(LookAtLocation);
	}
	else if(CameraFollowMode != EAnimationViewportCameraFollowMode::None)
	{
		InToViewport->SetCameraFollowMode(CameraFollowMode, CameraFollowBoneName);
	}
}

/////////////////////////////////////////////////////
// FMorphTargetTabSummoner

FMorphTargetTabSummoner::FMorphTargetTabSummoner(TSharedPtr<class FAssetEditorToolkit> InHostingApp, TSharedRef<IMorphTargetViewer> InViewerWidget)
	: FWorkflowTabFactory(FPersonaTabs::MorphTargetsID, InHostingApp)
	, ViewerWidget(InViewerWidget)
{
	TabLabel = LOCTEXT("MorphTargetTabTitle", "Morph Targets");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Persona.Tabs.MorphTargetPreviewer");

	EnableTabPadding();
	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("MorphTargetTabView", "Morph Targets");
	ViewMenuTooltip = LOCTEXT("MorphTargetTabView_ToolTip", "Shows the morph target viewer");
}

TSharedRef<SWidget> FMorphTargetTabSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	return ViewerWidget->AsWidget();
}
/////////////////////////////////////////////////////
// FAnimCurveViewerTabSummoner

FAnimCurveViewerTabSummoner::FAnimCurveViewerTabSummoner(TSharedPtr<class FAssetEditorToolkit> InHostingApp, const TSharedPtr<IEditableSkeleton>& InEditableSkeleton, const TSharedRef<IPersonaPreviewScene>& InPreviewScene, FOnObjectsSelected InOnObjectsSelected, bool bInSetCurveViewerReadOnly)
	: FWorkflowTabFactory(FPersonaTabs::AnimCurveViewID, InHostingApp)
	, EditableSkeleton(InEditableSkeleton)
	, PreviewScene(InPreviewScene)
	, OnObjectsSelected(InOnObjectsSelected)
	, bSetCurveViewerReadOnly(bInSetCurveViewerReadOnly)
{
	TabLabel = LOCTEXT("AnimCurveViewerTabTitle", "Curve Debugger");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Persona.Tabs.AnimCurveDebugger");

	EnableTabPadding();
	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("AnimCurveTabView", "Curve Debugger");
	ViewMenuTooltip = LOCTEXT("AnimCurveTabView_ToolTip", "Shows the animation curve debugger. This shows the state of animation curves.");
}

TSharedRef<SWidget> FAnimCurveViewerTabSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	return SNew(SAnimCurveViewer, PreviewScene.Pin().ToSharedRef(), OnObjectsSelected)
		.EditableSkeleton(EditableSkeleton.Pin())
		.bIsReadOnly(bSetCurveViewerReadOnly);
}

/////////////////////////////////////////////////////
// FAnimCurveViewerTabSummoner

FAnimCurveMetadataEditorTabSummoner::FAnimCurveMetadataEditorTabSummoner(TSharedPtr<class FAssetEditorToolkit> InHostingApp, UObject* InMetadataHost, const TSharedRef<IPersonaPreviewScene>& InPreviewScene, FOnObjectsSelected InOnObjectsSelected)
	: FWorkflowTabFactory(FPersonaTabs::AnimCurveMetadataEditorID, InHostingApp)
	, MetadataHost(InMetadataHost)
	, PreviewScene(InPreviewScene)
	, OnObjectsSelected(InOnObjectsSelected)
{
	TabLabel = LOCTEXT("AnimCurveMetadataEditorTabTitle", "Curves");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Persona.Tabs.AnimCurveMetadataEditor");

	EnableTabPadding();
	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("AnimCurveMetadataEditorTab", "Animation Curves");
	ViewMenuTooltip = LOCTEXT("AnimCurveMetadataEditorTab_ToolTip", "Shows the animation curve editor. This lets you add, remove, rename and edit curve metadata.");
}

TSharedRef<SWidget> FAnimCurveMetadataEditorTabSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	return SNew(SAnimCurveMetadataEditor, MetadataHost.Get(), PreviewScene.Pin().ToSharedRef(), OnObjectsSelected);
}


/////////////////////////////////////////////////////
// FAnimationAssetBrowserSummoner

FAnimationAssetBrowserSummoner::FAnimationAssetBrowserSummoner(TSharedPtr<class FAssetEditorToolkit> InHostingApp, const TSharedRef<IPersonaToolkit>& InPersonaToolkit, FOnOpenNewAsset InOnOpenNewAsset, FOnAnimationSequenceBrowserCreated InOnAnimationSequenceBrowserCreated, bool bInShowHistory)
	: FWorkflowTabFactory(FPersonaTabs::AssetBrowserID, InHostingApp)
	, PersonaToolkit(InPersonaToolkit)
	, OnOpenNewAsset(InOnOpenNewAsset)
	, OnAnimationSequenceBrowserCreated(InOnAnimationSequenceBrowserCreated)
	, bShowHistory(bInShowHistory)
{
	TabLabel = LOCTEXT("AssetBrowserTabTitle", "Asset Browser");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ContentBrowser.TabIcon");

	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("AssetBrowser", "Asset Browser");
	ViewMenuTooltip = LOCTEXT("AssetBrowser_ToolTip", "Shows the animation asset browser");
}

TSharedRef<SWidget> FAnimationAssetBrowserSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	TSharedRef<SAnimationSequenceBrowser> Widget = SNew(SAnimationSequenceBrowser, PersonaToolkit.Pin().ToSharedRef())
		.OnOpenNewAsset(OnOpenNewAsset)
		.ShowHistory(bShowHistory);

	OnAnimationSequenceBrowserCreated.ExecuteIfBound(Widget);

	return Widget;
}

/////////////////////////////////////////////////////
// FPreviewViewportSummoner

static FName ViewportInstanceToTabName(int32 InViewportIndex)
{
	switch(InViewportIndex)
	{
	default:
	case 0:
		return FPersonaTabs::PreviewViewportID;
	case 1:
		return FPersonaTabs::PreviewViewport1ID;
	case 2:
		return FPersonaTabs::PreviewViewport2ID;
	case 3:
		return FPersonaTabs::PreviewViewport3ID;
	}
}


static FName MakeViewportContextName(FName InContext, int32 InViewportIndex)
{
	return *FString::Printf(TEXT("%s%d"), *InContext.ToString(), InViewportIndex);
}

FPreviewViewportSummoner::FPreviewViewportSummoner(TSharedPtr<class FAssetEditorToolkit> InHostingApp, const FPersonaViewportArgs& InArgs, int32 InViewportIndex)
	: FWorkflowTabFactory(ViewportInstanceToTabName(InViewportIndex), InHostingApp)
	, PreviewScene(InArgs.PreviewScene)
	, BlueprintEditor(InArgs.BlueprintEditor)
	, OnViewportCreated(InArgs.OnViewportCreated)
	, OnGetViewportText(InArgs.OnGetViewportText)
	, Extenders(InArgs.Extenders)
	, ContextName(MakeViewportContextName(InArgs.ContextName, InViewportIndex))
	, ViewportIndex(InViewportIndex)
	, bShowShowMenu(InArgs.bShowShowMenu)
	, bShowLODMenu(InArgs.bShowLODMenu)
	, bShowPlaySpeedMenu(InArgs.bShowPlaySpeedMenu)
	, bShowTimeline(InArgs.bShowTimeline)
	, bShowStats(InArgs.bShowStats)
	, bAlwaysShowTransformToolbar(InArgs.bAlwaysShowTransformToolbar)
	, bShowFloorOptions(InArgs.bShowFloorOptions)
	, bShowTurnTable(InArgs.bShowTurnTable)
	, bShowPhysicsMenu(InArgs.bShowPhysicsMenu)
	, TimelineDelegates(InArgs.TimelineDelegates)
{
	TabLabel = FText::Format(LOCTEXT("ViewportTabTitle", "Viewport {0}"), FText::AsNumber(InViewportIndex + 1));
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Viewports");

	bIsSingleton = true;

	ViewMenuDescription = FText::Format(LOCTEXT("ViewportViewFormat", "Viewport {0}"), FText::AsNumber(InViewportIndex + 1));
	ViewMenuTooltip = LOCTEXT("ViewportView_ToolTip", "Shows the viewport");
}

TSharedRef<SWidget> FPreviewViewportSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	TSharedRef<SAnimationEditorViewportTabBody> NewViewport = SNew(SAnimationEditorViewportTabBody, PreviewScene.Pin().ToSharedRef(), HostingApp.Pin().ToSharedRef(), ViewportIndex)
		.BlueprintEditor(BlueprintEditor.Pin())
		.OnInvokeTab(FOnInvokeTab::CreateSP(HostingApp.Pin().Get(), &FAssetEditorToolkit::InvokeTab))
		.AddMetaData<FTagMetaData>(TEXT("Persona.Viewport"))
		.Extenders(Extenders)
		.ContextName(ContextName)
		.OnGetViewportText(OnGetViewportText)
		.ShowShowMenu(bShowShowMenu)
		.ShowLODMenu(bShowLODMenu)
		.ShowPlaySpeedMenu(bShowPlaySpeedMenu)
		.ShowTimeline(bShowTimeline)
		.ShowStats(bShowStats)
		.AlwaysShowTransformToolbar(bAlwaysShowTransformToolbar)
		.ShowFloorOptions(bShowFloorOptions)
		.ShowTurnTable(bShowTurnTable)
		.ShowPhysicsMenu(bShowPhysicsMenu)
		.TimelineDelegates(TimelineDelegates);

	OnViewportCreated.ExecuteIfBound(NewViewport);

	return NewViewport;
}

FTabSpawnerEntry& FPreviewViewportSummoner::RegisterTabSpawner(TSharedRef<FTabManager> TabManager, const FApplicationMode* CurrentApplicationMode) const
{
	FTabSpawnerEntry& SpawnerEntry = FWorkflowTabFactory::RegisterTabSpawner(TabManager, nullptr);

	if(CurrentApplicationMode)
	{
		// find an existing workspace item or create new
		TSharedPtr<FWorkspaceItem> GroupItem = nullptr;
		
		for(const TSharedRef<FWorkspaceItem>& Item : CurrentApplicationMode->GetWorkspaceMenuCategory()->GetChildItems())
		{
			if(Item->GetDisplayName().ToString() == LOCTEXT("ViewportsSubMenu", "Viewports").ToString())
			{
				GroupItem = Item;
				break;
			}
		}

		if(!GroupItem.IsValid())
		{
			GroupItem = CurrentApplicationMode->GetWorkspaceMenuCategory()->AddGroup(LOCTEXT("ViewportsSubMenu", "Viewports"), LOCTEXT("ViewportsSubMenu_Tooltip", "Open a new viewport on the scene"), FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Viewports"));
		}

		SpawnerEntry.SetGroup(GroupItem.ToSharedRef());
	}

	return SpawnerEntry;
}

/////////////////////////////////////////////////////
// FRetargetManagerTabSummoner

FRetargetSourcesTabSummoner::FRetargetSourcesTabSummoner(
	TSharedPtr<class FAssetEditorToolkit> InHostingApp,
	const TSharedRef<IEditableSkeleton>& InEditableSkeleton,
	FSimpleMulticastDelegate& InOnPostUndo)
	: FWorkflowTabFactory(FPersonaTabs::RetargetManagerID, InHostingApp)
	, EditableSkeleton(InEditableSkeleton)
	, OnPostUndo(InOnPostUndo)
{
	TabLabel = LOCTEXT("RetargetingTabTitle", "Retargeting");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Persona.Tabs.RetargetManager");

	EnableTabPadding();
	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("RetargetingTabView", "Retargeting");
	ViewMenuTooltip = LOCTEXT("RetargetSourcesTabView_ToolTip", "Specify Retarget Sources and Compatible Skeletons.");
}

TSharedRef<SWidget> FRetargetSourcesTabSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	return SNew(SRetargetSources, EditableSkeleton.Pin().ToSharedRef(), OnPostUndo);
}


/////////////////////////////////////////////////////
// SPersonaPreviewPropertyEditor

void SPersonaPreviewPropertyEditor::Construct(const FArguments& InArgs, TSharedRef<IPersonaPreviewScene> InPreviewScene, TSharedRef<FBlueprintEditor> InBPEditor)
{
	PreviewScene = InPreviewScene;
	BPEditor = InBPEditor;

	bPropertyEdited = false;

	SSingleObjectDetailsPanel::Construct(SSingleObjectDetailsPanel::FArguments(), /*bAutomaticallyObserveViaGetObjectToObserve*/ true, /*bAllowSearch*/ true);

	PropertyView->SetIsPropertyEditingEnabledDelegate(FIsPropertyEditingEnabled::CreateStatic([] { return !GIntraFrameDebuggingGameThread; }));
	PropertyView->OnFinishedChangingProperties().Add(FOnFinishedChangingProperties::FDelegate::CreateSP(this, &SPersonaPreviewPropertyEditor::HandlePropertyChanged));
}

UObject* SPersonaPreviewPropertyEditor::GetObjectToObserve() const
{
	// Use the selected debug instance, if we have one.
	if (UBlueprint* Blueprint = BPEditor.Pin()->GetBlueprintObj())
	{
		if (UObject* DebugObject = Blueprint->GetObjectBeingDebugged())
		{
			return DebugObject;
		}
	}

	// If there's no selection, fall-back to preview instance.
	if (UDebugSkelMeshComponent* PreviewMeshComponent = PreviewScene.Pin()->GetPreviewMeshComponent())
	{
		if (PreviewMeshComponent->GetAnimInstance() != nullptr)
		{
			return PreviewMeshComponent->GetAnimInstance();
		}
	}

	return nullptr;
}

TSharedRef<SWidget> SPersonaPreviewPropertyEditor::PopulateSlot(TSharedRef<SWidget> PropertyEditorWidget)
{
	return SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			PropertyEditorWidget
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("Docking.Tab.ContentAreaBrush"))
			.Visibility_Lambda([this]() { return bPropertyEdited ? EVisibility::Visible : EVisibility::Collapsed; })
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.VAlign(VAlign_Center)
				.Padding(2.0f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("AnimBlueprintEditPreviewText", "Changes made to the selected instance only. Changes will not be saved!"))
					.ColorAndOpacity(FLinearColor::Yellow)
					.ShadowOffset(FVector2D::UnitVector)
					.AutoWrapText(true)
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				.Padding(2.0f)
				[
					SNew(SButton)
					.OnClicked(this, &SPersonaPreviewPropertyEditor::HandleApplyChanges)
					.ToolTipText(LOCTEXT("AnimBlueprintEditApplyChanges_Tooltip", "Apply the selected instance's current property values to the defaults."))
					[
						SNew(STextBlock)
						.Text(LOCTEXT("AnimBlueprintEditApplyChanges", "Apply"))
					]
				]
			]
		];
}

void SPersonaPreviewPropertyEditor::HandlePropertyChanged(const FPropertyChangedEvent& InPropertyChangedEvent)
{
	UAnimInstance* AnimInstance = nullptr;

	if (UBlueprint* Blueprint = BPEditor.Pin()->GetBlueprintObj())
	{
		if (UAnimInstance* DebugInstance = Cast<UAnimInstance>(Blueprint->GetObjectBeingDebugged()))
		{
			// Compare the selected debug instance (if we have one) against the CDO.
			AnimInstance = DebugInstance;
		}
		else if (UDebugSkelMeshComponent* PreviewMeshComponent = PreviewScene.Pin()->GetPreviewMeshComponent())
		{
			// If there's no selection, fall-back to preview instance.
			AnimInstance = PreviewMeshComponent->GetAnimInstance();
		}
	}

	if (AnimInstance != nullptr)
	{
		// check to see how many properties have changed
		const int32 NumChangedProperties = PersonaUtils::CopyPropertiesToCDO(AnimInstance, PersonaUtils::FCopyOptions(PersonaUtils::ECopyOptions::PreviewOnly));
		bPropertyEdited = (NumChangedProperties > 0);
	}
}

FReply SPersonaPreviewPropertyEditor::HandleApplyChanges()
{
	// copy preview properties into CDO
	if (UDebugSkelMeshComponent* PreviewMeshComponent = PreviewScene.Pin()->GetPreviewMeshComponent())
	{
		if (UAnimInstance* AnimInstance = PreviewMeshComponent->GetAnimInstance())
		{
			PersonaUtils::CopyPropertiesToCDO(AnimInstance);

			bPropertyEdited = false;
		}
	}

	return FReply::Handled();
}

/////////////////////////////////////////////////////
// FAnimBlueprintPreviewEditorSummoner

FAnimBlueprintPreviewEditorSummoner::FAnimBlueprintPreviewEditorSummoner(TSharedPtr<class FBlueprintEditor> InBlueprintEditor, const TSharedRef<class IPersonaPreviewScene>& InPreviewScene)
	: FWorkflowTabFactory(FPersonaTabs::AnimBlueprintPreviewEditorID, InBlueprintEditor)
	, BlueprintEditor(InBlueprintEditor)
	, PreviewScene(InPreviewScene)
{
	TabLabel = LOCTEXT("AnimBlueprintPreviewTabTitle", "Anim Preview Editor");

	bIsSingleton = true;

	CurrentMode = EAnimBlueprintEditorMode::PreviewMode;

	ViewMenuDescription = LOCTEXT("AnimBlueprintPreviewView", "Preview");
	ViewMenuTooltip = LOCTEXT("AnimBlueprintPreviewView_ToolTip", "Shows the animation preview editor view (as well as class defaults)");
}

TSharedRef<SWidget> FAnimBlueprintPreviewEditorSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	return	SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(FMargin( 5.f, 0.f, 2.f, 0.f ))
				[
					SNew(SCheckBox)
					.Style(FAppStyle::Get(), "RadioButton")
					.IsChecked(this, &FAnimBlueprintPreviewEditorSummoner::IsChecked, EAnimBlueprintEditorMode::PreviewMode)
					.OnCheckStateChanged(const_cast<FAnimBlueprintPreviewEditorSummoner*>(this), &FAnimBlueprintPreviewEditorSummoner::OnCheckedChanged, EAnimBlueprintEditorMode::PreviewMode)
					.ToolTip(IDocumentation::Get()->CreateToolTip(	LOCTEXT("AnimBlueprintPropertyEditorPreviewMode", "Switch to editing the preview instance properties"),
																	NULL,
																	TEXT("Shared/Editors/Persona"),
																	TEXT("AnimBlueprintPropertyEditorPreviewMode")))
					[
						SNew( STextBlock )
						.Text( LOCTEXT("AnimBlueprintDefaultsPreviewMode", "Edit Selected Instance") )
					]
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(FMargin( 2.f, 0.f, 0.f, 0.f ))
				[
					SNew(SCheckBox)
					.Style(FAppStyle::Get(), "RadioButton")
					.IsChecked(this, &FAnimBlueprintPreviewEditorSummoner::IsChecked, EAnimBlueprintEditorMode::DefaultsMode)
					.OnCheckStateChanged(const_cast<FAnimBlueprintPreviewEditorSummoner*>(this), &FAnimBlueprintPreviewEditorSummoner::OnCheckedChanged, EAnimBlueprintEditorMode::DefaultsMode)
					.ToolTip(IDocumentation::Get()->CreateToolTip(	LOCTEXT("AnimBlueprintPropertyEditorDefaultMode", "Switch to editing the class defaults"),
																	NULL,
																	TEXT("Shared/Editors/Persona"),
																	TEXT("AnimBlueprintPropertyEditorDefaultMode")))
					[
						SNew( STextBlock )
						.Text( LOCTEXT("AnimBlueprintDefaultsDefaultsMode", "Edit Defaults") )
					]
				]
			]
			+SVerticalBox::Slot()
			[
				SNew(SOverlay)
				+SOverlay::Slot()
				[
					SNew(SBorder)
					.Padding(0)
					.BorderImage( FAppStyle::GetBrush("NoBorder") )
					.Visibility(this, &FAnimBlueprintPreviewEditorSummoner::IsEditorVisible, EAnimBlueprintEditorMode::PreviewMode)
					[
						SNew(SPersonaPreviewPropertyEditor, PreviewScene.Pin().ToSharedRef(), BlueprintEditor.Pin().ToSharedRef())
					]
				]
				+SOverlay::Slot()
				[
					SNew(SBorder)
					.Padding(FMargin(3.0f, 2.0f))
					.BorderImage( FAppStyle::GetBrush("NoBorder") )
					.Visibility(this, &FAnimBlueprintPreviewEditorSummoner::IsEditorVisible, EAnimBlueprintEditorMode::DefaultsMode)
					[
						BlueprintEditor.Pin()->GetDefaultEditor()
					]
				]
			];
}

FText FAnimBlueprintPreviewEditorSummoner::GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const
{
	return LOCTEXT("AnimBlueprintPreviewEditorTooltip", "The editor lets you change the values of the selected instance");
}

EVisibility FAnimBlueprintPreviewEditorSummoner::IsEditorVisible(EAnimBlueprintEditorMode::Type Mode) const
{
	return CurrentMode == Mode ? EVisibility::Visible: EVisibility::Hidden;
}

ECheckBoxState FAnimBlueprintPreviewEditorSummoner::IsChecked(EAnimBlueprintEditorMode::Type Mode) const
{
	return CurrentMode == Mode ? ECheckBoxState::Checked: ECheckBoxState::Unchecked;
}


void FAnimBlueprintPreviewEditorSummoner::OnCheckedChanged(ECheckBoxState NewType, EAnimBlueprintEditorMode::Type Mode)
{
	if(NewType == ECheckBoxState::Checked)
	{
		CurrentMode = Mode;
	}
}

//////////////////////////////////////////////////////////////////////////
// FAnimBlueprintParentPlayerEditorSummoner

FAnimBlueprintParentPlayerEditorSummoner::FAnimBlueprintParentPlayerEditorSummoner(TSharedPtr<class FBlueprintEditor> InBlueprintEditor, FSimpleMulticastDelegate& InOnPostUndo)
	: FWorkflowTabFactory(FPersonaTabs::AnimBlueprintParentPlayerEditorID, InBlueprintEditor)
	, BlueprintEditor(InBlueprintEditor)
	, OnPostUndo(InOnPostUndo)
{
	TabLabel = LOCTEXT("ParentPlayerOverrideEditor", "Asset Override Editor");
	bIsSingleton = true;
}

TSharedRef<SWidget> FAnimBlueprintParentPlayerEditorSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	return SNew(SAnimBlueprintParentPlayerList, BlueprintEditor.Pin().ToSharedRef(), OnPostUndo);
}

FText FAnimBlueprintParentPlayerEditorSummoner::GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const
{
	return LOCTEXT("AnimSubClassTabToolTip", "Editor for overriding the animation assets referenced by the parent animation graph.");
}

//////////////////////////////////////////////////////////////////////////
// FPoseWatchManagerSummoner

FPoseWatchManagerSummoner::FPoseWatchManagerSummoner(TSharedPtr<class FBlueprintEditor> InBlueprintEditor)
	: FWorkflowTabFactory(FPersonaTabs::PoseWatchManagerID, InBlueprintEditor)
	, BlueprintEditor(InBlueprintEditor)
{
	TabLabel = LOCTEXT("PoseWatchManager", "Pose Watch Manager");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "AnimGraph.PoseWatch.Icon");
	bIsSingleton = true;
}

TSharedRef<SWidget> FPoseWatchManagerSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	FPoseWatchManagerInitializationOptions Options;
	Options.BlueprintEditor = BlueprintEditor;

	return SNew(SPoseWatchManager, Options)
		.IsEnabled(FSlateApplication::Get().GetNormalExecutionAttribute());
}

FText FPoseWatchManagerSummoner::GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const
{
	return LOCTEXT("PoseWatchTabToolTip", "Shows all active pose watches.");
}

/////////////////////////////////////////////////////
// FAdvancedPreviewSceneTabSummoner

FAdvancedPreviewSceneTabSummoner::FAdvancedPreviewSceneTabSummoner(TSharedPtr<class FAssetEditorToolkit> InHostingApp, const TSharedRef<IPersonaPreviewScene>& InPreviewScene)
	: FWorkflowTabFactory(FPersonaTabs::AdvancedPreviewSceneSettingsID, InHostingApp)
	, PreviewScene(InPreviewScene)
{
	TabLabel = LOCTEXT("PreviewSceneSettingsTab", "Preview Scene Settings");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details");	
	bIsSingleton = true;
	
	ViewMenuDescription = LOCTEXT("AdvancedPreviewScene", "Preview Scene Settings");
	ViewMenuTooltip = LOCTEXT("AdvancedPreviewScene_ToolTip", "Shows the advanced preview scene settings");
}

FAdvancedPreviewSceneTabSummoner::FAdvancedPreviewSceneTabSummoner(TSharedPtr<class FAssetEditorToolkit> InHostingApp, const TSharedRef<IPersonaPreviewScene>& InPreviewScene, const FPersonaAdvancedPreviewSceneTabSettings& InDetailCustomizationSettings)
	: FAdvancedPreviewSceneTabSummoner(InHostingApp, InPreviewScene)
{
	DetailCustomizationSettings = InDetailCustomizationSettings;
}

TSharedRef<class IDetailCustomization> FAdvancedPreviewSceneTabSummoner::CustomizePreviewSceneDescription()
{
	TSharedRef<IPersonaPreviewScene> PreviewSceneRef = PreviewScene.Pin().ToSharedRef();
	FString SkeletonName;
	TSharedPtr<IEditableSkeleton> EditableSkeleton = PreviewSceneRef->GetPersonaToolkit()->GetEditableSkeleton();
	if(EditableSkeleton.IsValid())
	{
		SkeletonName = FAssetData(&EditableSkeleton->GetSkeleton()).GetExportTextName();
	}
	return MakeShareable(new FPreviewSceneDescriptionCustomization(SkeletonName, PreviewSceneRef->GetPersonaToolkit(), DetailCustomizationSettings));
}

TSharedRef<class IPropertyTypeCustomization> FAdvancedPreviewSceneTabSummoner::CustomizePreviewMeshCollectionEntry()
{
	return MakeShareable(new FPreviewMeshCollectionEntryCustomization(PreviewScene.Pin().ToSharedRef()));
}

TSharedRef<SWidget> FAdvancedPreviewSceneTabSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	TSharedRef<FAnimationEditorPreviewScene> PreviewSceneRef = StaticCastSharedRef<FAnimationEditorPreviewScene>(PreviewScene.Pin().ToSharedRef());

	TArray<FAdvancedPreviewSceneModule::FDetailCustomizationInfo> DetailsCustomizations;
	TArray<FAdvancedPreviewSceneModule::FPropertyTypeCustomizationInfo> PropertyTypeCustomizations;

	DetailsCustomizations.Add({ UPersonaPreviewSceneDescription::StaticClass(), FOnGetDetailCustomizationInstance::CreateSP(const_cast<FAdvancedPreviewSceneTabSummoner*>(this), &FAdvancedPreviewSceneTabSummoner::CustomizePreviewSceneDescription) });
	PropertyTypeCustomizations.Add({ FPreviewMeshCollectionEntry::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateSP(const_cast<FAdvancedPreviewSceneTabSummoner*>(this), &FAdvancedPreviewSceneTabSummoner::CustomizePreviewMeshCollectionEntry) });

	FAdvancedPreviewSceneModule& AdvancedPreviewSceneModule = FModuleManager::LoadModuleChecked<FAdvancedPreviewSceneModule>("AdvancedPreviewScene");
	return AdvancedPreviewSceneModule.CreateAdvancedPreviewSceneSettingsWidget(PreviewSceneRef, PreviewSceneRef->GetPreviewSceneDescription(), DetailsCustomizations, PropertyTypeCustomizations);
}

FText FAdvancedPreviewSceneTabSummoner::GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const
{
	return LOCTEXT("AdvancedPreviewSettingsToolTip", "The Advanced Preview Settings tab will let you alter the preview scene's settings.");
}

/////////////////////////////////////////////////////
// FPersonaDetailsTabSummoner

FPersonaDetailsTabSummoner::FPersonaDetailsTabSummoner(TSharedPtr<class FAssetEditorToolkit> InHostingApp, FOnDetailsCreated InOnDetailsCreated)
	: FWorkflowTabFactory(FPersonaTabs::DetailsID, InHostingApp)
	, OnDetailsCreated(InOnDetailsCreated)
{
	TabLabel = LOCTEXT("PersonaDetailsTab", "Details");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details");
	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("DetailsDescription", "Details");
	ViewMenuTooltip = LOCTEXT("DetailsToolTip", "Shows the details tab for selected objects.");

	PersonaDetails = SNew(SPersonaDetails);

	OnDetailsCreated.ExecuteIfBound(PersonaDetails->DetailsView.ToSharedRef());
}

TSharedRef<SWidget> FPersonaDetailsTabSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	return PersonaDetails.ToSharedRef();
}

FText FPersonaDetailsTabSummoner::GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const
{
	return LOCTEXT("PersonaDetailsToolTip", "Edit the details of selected objects.");
}


/////////////////////////////////////////////////////
// SAnimAssetPropertiesTabBody

class SAssetPropertiesTabBody : public SSingleObjectDetailsPanel
{
public:
	SLATE_BEGIN_ARGS(SAssetPropertiesTabBody) {}

	SLATE_ARGUMENT(FOnGetAsset, OnGetAsset)

	SLATE_ARGUMENT(FOnDetailsCreated, OnDetailsCreated)

	SLATE_END_ARGS()

private:
	FOnGetAsset OnGetAsset;

public:
	void Construct(const FArguments& InArgs)
	{
		OnGetAsset = InArgs._OnGetAsset;

		SSingleObjectDetailsPanel::Construct(SSingleObjectDetailsPanel::FArguments(), true, true);

		InArgs._OnDetailsCreated.ExecuteIfBound(PropertyView.ToSharedRef());
	}

	virtual EVisibility GetAssetDisplayNameVisibility() const
	{
		return (GetObjectToObserve() != NULL) ? EVisibility::Visible : EVisibility::Collapsed;
	}

	virtual FText GetAssetDisplayName() const
	{
		if (UObject* Object = GetObjectToObserve())
		{
			return FText::FromString(Object->GetName());
		}
		else
		{
			return FText::GetEmpty();
		}
	}

	// SSingleObjectDetailsPanel interface
	virtual UObject* GetObjectToObserve() const override
	{
		if (OnGetAsset.IsBound())
		{
			return OnGetAsset.Execute();
		}
		return nullptr;
	}
	// End of SSingleObjectDetailsPanel interface
};

FAssetPropertiesSummoner::FAssetPropertiesSummoner(TSharedPtr<class FAssetEditorToolkit> InHostingApp, FOnGetAsset InOnGetAsset, FOnDetailsCreated InOnDetailsCreated)
	: FWorkflowTabFactory(FPersonaTabs::AnimAssetPropertiesID, InHostingApp)
	, OnGetAsset(InOnGetAsset)
	, OnDetailsCreated(InOnDetailsCreated)
{
	TabLabel = LOCTEXT("AssetProperties_TabTitle", "Asset Details");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Persona.Tabs.AnimAssetDetails");

	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("AssetProperties_MenuTitle", "Asset Details");
	ViewMenuTooltip = LOCTEXT("AssetProperties_MenuToolTip", "Shows the asset properties");
}

TSharedPtr<SToolTip> FAssetPropertiesSummoner::CreateTabToolTipWidget(const FWorkflowTabSpawnInfo& Info) const
{
	return  IDocumentation::Get()->CreateToolTip(LOCTEXT("AssetPropertiesTooltip", "The Asset Details tab lets you edit properties of the current asset (animation, blend space etc)."), NULL, TEXT("Shared/Editors/Persona"), TEXT("AnimationAssetDetail_Window"));
}

TSharedRef<SWidget> FAssetPropertiesSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	return SNew(SAssetPropertiesTabBody)
		.OnGetAsset(OnGetAsset)
		.OnDetailsCreated(OnDetailsCreated);
}

/////////////////////////////////////////////////////
// FAnimAttributeViewerTabSummoner

FAnimAttributeViewerTabSummoner::FAnimAttributeViewerTabSummoner(TSharedPtr<class FAssetEditorToolkit> InHostingApp, const TSharedRef<IPersonaPreviewScene>& InPreviewScene)
	: FWorkflowTabFactory(FPersonaTabs::AnimAttributeViewID, InHostingApp)
	, PreviewScene(InPreviewScene)
{
	TabLabel = LOCTEXT("AnimAttributeViewTabTitle", "Animation Attributes");	
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "AnimGraph.Attribute.Attributes.Icon");

	EnableTabPadding();
	bIsSingleton = true;

	ViewMenuDescription = LOCTEXT("AnimAttributeTabView", "Animation Attributes");
	ViewMenuTooltip = LOCTEXT("AnimAttributeTabView_ToolTip", "Shows the animation attribute viewer");
}

TSharedRef<SWidget> FAnimAttributeViewerTabSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	return SNew(SAnimAttributeViewer, PreviewScene.Pin().ToSharedRef());
}


#undef LOCTEXT_NAMESPACE

