// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "Layout/Visibility.h"
#include "Widgets/SWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "IPersonaViewport.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "WorkflowOrientedApp/WorkflowTabFactory.h"
#include "WorkflowOrientedApp/WorkflowTabManager.h"
#include "BlueprintEditor.h"
#include "WorkflowOrientedApp/ApplicationMode.h"
#include "IDocumentation.h"
#include "PersonaModule.h"
#include "IPersonaPreviewScene.h"
#include "AnimationEditorViewportClient.h"
#include "SSingleObjectDetailsPanel.h"
#include "PersonaTabs.h"

#define LOCTEXT_NAMESPACE "PersonaMode"

class IEditableSkeleton;
class IPersonaToolkit;
class ISkeletonTree;
class SPersonaDetails;
class SToolTip;

/////////////////////////////////////////////////////

// This is the list of IDs for persona modes
struct FPersonaModes
{
	// Mode constants
	static const FName SkeletonDisplayMode;
	static const FName MeshEditMode;
	static const FName PhysicsEditMode;
	static const FName AnimationEditMode;
	static const FName AnimBlueprintEditMode;
	static FText GetLocalizedMode( const FName InMode )
	{
		static TMap< FName, FText > LocModes;

		if (LocModes.Num() == 0)
		{
			LocModes.Add( SkeletonDisplayMode, NSLOCTEXT("PersonaModes", "SkeletonDisplayMode", "Skeleton") );
			LocModes.Add( MeshEditMode, NSLOCTEXT("PersonaModes", "MeshEditMode", "Mesh") );
			LocModes.Add( PhysicsEditMode, NSLOCTEXT("PersonaModes", "PhysicsEditMode", "Physics") );
			LocModes.Add( AnimationEditMode, NSLOCTEXT("PersonaModes", "AnimationEditMode", "Animation") );
			LocModes.Add( AnimBlueprintEditMode, NSLOCTEXT("PersonaModes", "AnimBlueprintEditMode", "Graph") );
		}

		check( InMode != NAME_None );
		const FText* OutDesc = LocModes.Find( InMode );
		check( OutDesc );
		return *OutDesc;
	}
private:
	FPersonaModes() {}
};

/////////////////////////////////////////////////////
// FPersonaModeSharedData

struct FPersonaModeSharedData : public IPersonaViewportState
{
	FPersonaModeSharedData();

	void Save(const TSharedRef<FAnimationViewportClient>& InFromViewport);
	void Restore(const TSharedRef<FAnimationViewportClient>& InToViewport);

	// camera setup
	FVector				ViewLocation;
	FRotator			ViewRotation;
	float				OrthoZoom;
	
	// orbit setup
	FVector				OrbitZoom;
	FVector				LookAtLocation;
	bool				bCameraLock;
	EAnimationViewportCameraFollowMode CameraFollowMode;
	FName				CameraFollowBoneName;

	// show flags
	bool				bShowReferencePose;
	bool				bShowBones;
	bool				bShowBoneNames;
	bool				bShowSockets;
	bool				bShowBound;

	// viewport setup
	int32				ViewportType;
	EAnimationPlaybackSpeeds::Type PlaybackSpeedMode;
	int32				LocalAxesMode;
};

/////////////////////////////////////////////////////
// FPersonaAppMode

class FPersonaAppMode : public FApplicationMode
{
protected:
	FPersonaAppMode(TSharedPtr<class FPersona> InPersona, FName InModeName);

public:
	// FApplicationMode interface
	virtual void RegisterTabFactories(TSharedPtr<FTabManager> InTabManager) override;
	virtual void PostActivateMode() override;
	// End of FApplicationMode interface

protected:
	TWeakPtr<class FPersona> MyPersona;

	// Set of spawnable tabs in persona mode (@TODO: Multiple lists!)
	FWorkflowAllowedTabSet PersonaTabFactories;
};

/////////////////////////////////////////////////////
// FMorphTargetTabSummoner

struct FMorphTargetTabSummoner : public FWorkflowTabFactory
{
public:
	FMorphTargetTabSummoner(TSharedPtr<class FAssetEditorToolkit> InHostingApp, TSharedRef<IMorphTargetViewer> InViewerWidget);

	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;

	// Create a tooltip widget for the tab
	virtual TSharedPtr<SToolTip> CreateTabToolTipWidget(const FWorkflowTabSpawnInfo& Info) const override
	{
		return  IDocumentation::Get()->CreateToolTip(LOCTEXT("MorphTargetTooltip", "The Morph Target tab lets you preview any morph targets (aka blend shapes) available for the current mesh."), NULL, TEXT("Shared/Editors/Persona"), TEXT("MorphTarget_Window"));
	}

private:
	TSharedRef<IMorphTargetViewer> ViewerWidget;
};

/////////////////////////////////////////////////////
// FAnimCurveViewerTabSummoner

struct FAnimCurveViewerTabSummoner : public FWorkflowTabFactory
{
public:
	FAnimCurveViewerTabSummoner(TSharedPtr<class FAssetEditorToolkit> InHostingApp, const TSharedPtr<IEditableSkeleton>& InEditableSkeleton, const TSharedRef<IPersonaPreviewScene>& InPreviewScene, FOnObjectsSelected InOnObjectsSelected, bool bInSetCurveViewerReadOnly = true);

	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;

	// Create a tooltip widget for the tab
	virtual TSharedPtr<SToolTip> CreateTabToolTipWidget(const FWorkflowTabSpawnInfo& Info) const override
	{
		return  IDocumentation::Get()->CreateToolTip(LOCTEXT("AnimCurveViewTooltip", "The Anim Curve Viewer tab lets you preview any animation curves available for the current mesh from preview asset."), NULL, TEXT("Shared/Editors/Persona"), TEXT("AnimCurveView_Window"));
	}

private:
	TWeakPtr<class IEditableSkeleton> EditableSkeleton;
	TWeakPtr<class IPersonaPreviewScene> PreviewScene;
	FOnObjectsSelected OnObjectsSelected;
	bool bSetCurveViewerReadOnly;
};

/////////////////////////////////////////////////////
// FAnimCurveViewerTabSummoner

struct FAnimCurveMetadataEditorTabSummoner : public FWorkflowTabFactory
{
public:
	FAnimCurveMetadataEditorTabSummoner(TSharedPtr<class FAssetEditorToolkit> InHostingApp, UObject* InMetadataHost, const TSharedRef<IPersonaPreviewScene>& InPreviewScene, FOnObjectsSelected InOnObjectsSelected);

	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;

	virtual TSharedPtr<SToolTip> CreateTabToolTipWidget(const FWorkflowTabSpawnInfo& Info) const override
	{
		return  IDocumentation::Get()->CreateToolTip(LOCTEXT("AnimCurveMetadataEditorTooltip", "The Anim Curve Metadata Editor tab lets you add, remove, rename and edit curve metadata."), NULL, TEXT("Shared/Editors/Persona"), TEXT("AnimCurveMetadataEditor_Window"));
	}

	virtual FText GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const
	{
		return LOCTEXT("AnimCurveTabView_ToolTip", "Shows the animation curve debugger. This shows the state of animation curves.");
	}

private:
	TWeakObjectPtr<UObject> MetadataHost;
	TWeakPtr<class IPersonaPreviewScene> PreviewScene;
	FOnObjectsSelected OnObjectsSelected;
};

/////////////////////////////////////////////////////
// FAnimationAssetBrowserSummoner

struct FAnimationAssetBrowserSummoner : public FWorkflowTabFactory
{
	FAnimationAssetBrowserSummoner(TSharedPtr<class FAssetEditorToolkit> InHostingApp, const TSharedRef<IPersonaToolkit>& InPersonaToolkit, FOnOpenNewAsset InOnOpenNewAsset, FOnAnimationSequenceBrowserCreated InOnAnimationSequenceBrowserCreated, bool bInShowHistory);

	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;

	// Create a tooltip widget for the tab
	virtual TSharedPtr<SToolTip> CreateTabToolTipWidget(const FWorkflowTabSpawnInfo& Info) const override
	{
		return  IDocumentation::Get()->CreateToolTip(LOCTEXT("AnimAssetBrowserTooltip", "The Asset Browser lets you browse all animation-related assets (animations, blend spaces etc)."), NULL, TEXT("Shared/Editors/Persona"), TEXT("AssetBrowser_Window"));
	}

private:
	TWeakPtr<class IPersonaToolkit> PersonaToolkit;
	FOnOpenNewAsset OnOpenNewAsset;
	FOnAnimationSequenceBrowserCreated OnAnimationSequenceBrowserCreated;
	bool bShowHistory;
};

/////////////////////////////////////////////////////
// FPreviewViewportSummoner

struct FPreviewViewportSummoner : public FWorkflowTabFactory
{
	FPreviewViewportSummoner(TSharedPtr<class FAssetEditorToolkit> InHostingApp, const FPersonaViewportArgs& InArgs, int32 InViewportIndex);

	virtual FTabSpawnerEntry& RegisterTabSpawner(TSharedRef<FTabManager> TabManager, const FApplicationMode* CurrentApplicationMode) const;
	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;

	TWeakPtr<ISkeletonTree> SkeletonTree;
	TWeakPtr<IPersonaPreviewScene> PreviewScene;
	TWeakPtr<FBlueprintEditor> BlueprintEditor;
	FOnViewportCreated OnViewportCreated;
	FOnGetViewportText OnGetViewportText;
	TArray<TSharedPtr<FExtender>> Extenders;
	FName ContextName;
	int32 ViewportIndex;
	bool bShowShowMenu;
	bool bShowLODMenu;
	bool bShowPlaySpeedMenu;
	bool bShowTimeline;
	bool bShowStats;
	bool bAlwaysShowTransformToolbar;
	bool bShowFloorOptions;
	bool bShowTurnTable;
	bool bShowPhysicsMenu;
	FAnimationScrubPanelDelegates TimelineDelegates;
};

/////////////////////////////////////////////////////
// FRetargetManagerTabSummoner

struct FRetargetSourcesTabSummoner : public FWorkflowTabFactory
{
public:
	FRetargetSourcesTabSummoner(
		TSharedPtr<class FAssetEditorToolkit> InHostingApp,
		const TSharedRef<IEditableSkeleton>& InEditableSkeleton,
		FSimpleMulticastDelegate& InOnPostUndo);

	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;

	// Create a tooltip widget for the tab
	virtual TSharedPtr<SToolTip> CreateTabToolTipWidget(const FWorkflowTabSpawnInfo& Info) const override
	{
		return IDocumentation::Get()->CreateToolTip(LOCTEXT("RetargetSourceTooltip", "In this panel, you can manage Retarget Sources (for playing animations authored with varying proportions) and Compatible Skeletons (for playing animations from other skeletal meshes)."), NULL, TEXT("Shared/Editors/Persona"), TEXT("RetargetSources"));
	}

private:
	TWeakPtr<class IEditableSkeleton> EditableSkeleton;
	FSimpleMulticastDelegate& OnPostUndo;
};

/////////////////////////////////////////////////////
// SPersonaPreviewPropertyEditor

class SPersonaPreviewPropertyEditor : public SSingleObjectDetailsPanel
{
public:
	SLATE_BEGIN_ARGS(SPersonaPreviewPropertyEditor) {}
	SLATE_END_ARGS()

private:
	// Pointer to preview scene
	TWeakPtr<IPersonaPreviewScene> PreviewScene;
	TWeakPtr<FBlueprintEditor> BPEditor;

public:

	void Construct(const FArguments& InArgs, TSharedRef<IPersonaPreviewScene> InPreviewScene, TSharedRef<FBlueprintEditor> InBPEditor);

	// SSingleObjectDetailsPanel interface
	virtual UObject* GetObjectToObserve() const override;
	virtual TSharedRef<SWidget> PopulateSlot(TSharedRef<SWidget> PropertyEditorWidget) override;
	// End of SSingleObjectDetailsPanel interface

private:
	void HandlePropertyChanged(const FPropertyChangedEvent& InPropertyChangedEvent);
	FReply HandleApplyChanges();

private:
	bool bPropertyEdited;
};

/////////////////////////////////////////////////////
// FAnimBlueprintPreviewEditorSummoner

namespace EAnimBlueprintEditorMode
{
	enum Type
	{
		PreviewMode,
		DefaultsMode
	};
}

struct FAnimBlueprintPreviewEditorSummoner : public FWorkflowTabFactory
{
public:
	FAnimBlueprintPreviewEditorSummoner(TSharedPtr<class FBlueprintEditor> InBlueprintEditor, const TSharedRef<class IPersonaPreviewScene>& InPreviewScene);

	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;

	virtual FText GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const override;

private:
	/** Delegates to customize tab look based on selected mode */
	EVisibility IsEditorVisible(EAnimBlueprintEditorMode::Type Mode) const;
	ECheckBoxState IsChecked(EAnimBlueprintEditorMode::Type Mode) const;

	/** Handle changing of editor mode */
	void OnCheckedChanged(ECheckBoxState NewType, EAnimBlueprintEditorMode::Type Mode);
	EAnimBlueprintEditorMode::Type CurrentMode;

	TWeakPtr<class FBlueprintEditor> BlueprintEditor;
	TWeakPtr<class IPersonaPreviewScene> PreviewScene;
};

//////////////////////////////////////////////////////////////////////////
// FAnimBlueprintParentPlayerEditorSummoner
class FAnimBlueprintParentPlayerEditorSummoner : public FWorkflowTabFactory
{
public:
	FAnimBlueprintParentPlayerEditorSummoner(TSharedPtr<class FBlueprintEditor> InBlueprintEditor, FSimpleMulticastDelegate& InOnPostUndo);

	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;
	virtual FText GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const override;

private:
	TWeakPtr<class FBlueprintEditor> BlueprintEditor;
	FSimpleMulticastDelegate& OnPostUndo;
};

/////////////////////////////////////////////////////
// FPoseWatchManagerSummoner
class FPoseWatchManagerSummoner : public FWorkflowTabFactory
{
public:
	FPoseWatchManagerSummoner(TSharedPtr<class FBlueprintEditor> InBlueprintEditor);

	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;
	virtual FText GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const override;

private:
	TWeakPtr<class FBlueprintEditor> BlueprintEditor;
};


/////////////////////////////////////////////////////
// FAdvancedPreviewSceneTabSummoner

struct FAdvancedPreviewSceneTabSummoner : public FWorkflowTabFactory
{
public:
 	FAdvancedPreviewSceneTabSummoner(TSharedPtr<class FAssetEditorToolkit> InHostingApp, const TSharedRef<class IPersonaPreviewScene>& InPreviewScene);
	FAdvancedPreviewSceneTabSummoner(TSharedPtr<class FAssetEditorToolkit> InHostingApp, const TSharedRef<class IPersonaPreviewScene>& InPreviewScene, const FPersonaAdvancedPreviewSceneTabSettings& InDetailCustomizationSettings);
	
	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;
	virtual FText GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const override;

private:
	/** Customize the details of the scene setup object */
	TSharedRef<class IDetailCustomization> CustomizePreviewSceneDescription();

	/** Customize a preview mesh collection entry */
	TSharedRef<class IPropertyTypeCustomization> CustomizePreviewMeshCollectionEntry();

private:
	TWeakPtr<class IPersonaPreviewScene> PreviewScene;
	FPersonaAdvancedPreviewSceneTabSettings DetailCustomizationSettings;
};

/////////////////////////////////////////////////////
// FPersonaDetailsTabSummoner

struct FPersonaDetailsTabSummoner : public FWorkflowTabFactory
{
public:
	FPersonaDetailsTabSummoner(TSharedPtr<class FAssetEditorToolkit> InHostingApp, FOnDetailsCreated InOnDetailsCreated);
	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;
	virtual FText GetTabToolTipText(const FWorkflowTabSpawnInfo& Info) const override;

private:
	FOnDetailsCreated OnDetailsCreated;
	TSharedPtr<class SPersonaDetails> PersonaDetails;
};

/////////////////////////////////////////////////////
// FAssetPropertiesSummoner

struct FAssetPropertiesSummoner : public FWorkflowTabFactory
{
	FAssetPropertiesSummoner(TSharedPtr<class FAssetEditorToolkit> InHostingApp, FOnGetAsset InOnGetAsset, FOnDetailsCreated InOnDetailsCreated);

	// FWorkflowTabFactory interface
	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;
	virtual TSharedPtr<SToolTip> CreateTabToolTipWidget(const FWorkflowTabSpawnInfo& Info) const override;
	// FWorkflowTabFactory interface

private:
	FOnGetAsset OnGetAsset;
	FOnDetailsCreated OnDetailsCreated;
};

/////////////////////////////////////////////////////
// FAnimAttributeViewerTabSummoner

struct FAnimAttributeViewerTabSummoner : public FWorkflowTabFactory
{
public:
	FAnimAttributeViewerTabSummoner(TSharedPtr<class FAssetEditorToolkit> InHostingApp, const TSharedRef<IPersonaPreviewScene>& InPreviewScene);

	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;

	// Create a tooltip widget for the tab
	virtual TSharedPtr<SToolTip> CreateTabToolTipWidget(const FWorkflowTabSpawnInfo& Info) const override
	{
		return IDocumentation::Get()->CreateToolTip(LOCTEXT("AnimAttributeViewTooltip", "The Animation Attribute Viewer tab lets you preview any Animation Attributes resulting from evaluating the current preview asset."), NULL, TEXT("Shared/Editors/Persona"), TEXT("AnimAttributeView_Window"));
	}

private:
	TWeakPtr<class IEditableSkeleton> EditableSkeleton;
	TWeakPtr<class IPersonaPreviewScene> PreviewScene;
};


#undef LOCTEXT_NAMESPACE
