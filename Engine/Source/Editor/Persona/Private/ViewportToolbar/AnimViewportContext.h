// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SAnimationEditorViewport.h"
#include "AnimViewportContext.generated.h"

UCLASS()
class UAnimViewportContext : public UObject
{
	GENERATED_BODY()

public:
	TWeakPtr<SAnimationEditorViewportTabBody> ViewportTabBody;
	TWeakPtr<IPersonaPreviewScene> PersonaPreviewScene;
};
