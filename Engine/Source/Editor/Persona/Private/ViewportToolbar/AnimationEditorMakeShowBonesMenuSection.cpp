// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationEditorMakeShowBonesMenuSection.h"

#include "AnimationEditorWidgets.h"
#include "AnimViewportContext.h"
#include "AnimViewportShowCommands.h"
#include "SAnimationEditorViewport.h"
#include "ToolMenu.h"
#include "ToolMenuSection.h"

#define LOCTEXT_NAMESPACE "AnimationEditorMakeShowBonesMenuSection"

namespace UE::Persona::ViewportToolbar
{
	void MakeShowBonesMenuSection(UToolMenu* ToolMenu, const EShowBonesMenuEntryFlags Flags)
	{
		if (!ensureMsgf(FAnimViewportShowCommands::IsRegistered(), TEXT("FAnimViewportShowCommands needs to be registered for it to MakeShowBonesMenuSection")))
		{
			return;
		}

		if (!ToolMenu)
		{
			return;
		}

		const TSharedPtr<SAnimationEditorViewportTabBody> AnimEditorViewportTab = [ToolMenu]() -> TSharedPtr<SAnimationEditorViewportTabBody>
			{
				if (UAnimViewportContext* const AnimViewportContext = ToolMenu->FindContext<UAnimViewportContext>())
				{
					return AnimViewportContext->ViewportTabBody.Pin();
				}

				return nullptr;
			}();

		if (!AnimEditorViewportTab.IsValid())
		{
			return;
		}

		FToolMenuSection& Section = ToolMenu->AddSection(
			"AnimViewportPreviewHierarchyBoneDraw", LOCTEXT("CharacterMenu_Actions_BoneDrawing", "Bone Drawing")
		);

		if (Flags == EShowBonesMenuEntryFlags::AllFlags ||
			EnumHasAnyFlags(Flags, EShowBonesMenuEntryFlags::BoneSize))
		{
			TSharedPtr<SWidget> BoneSizeWidget =
				SNew(UE::AnimationEditor::SBoneDrawSizeSetting)
				.AnimEditorViewport(AnimEditorViewportTab);

			Section.AddEntry(FToolMenuEntry::InitWidget(
				"BoneDrawSize", BoneSizeWidget.ToSharedRef(), LOCTEXT("CharacterMenu_Actions_BoneDrawSize", "Bone Draw Size:")
			));
		}

		if (Flags == (EShowBonesMenuEntryFlags::DrawAll | EShowBonesMenuEntryFlags::DrawNone) ||
			Flags == (EShowBonesMenuEntryFlags::DrawAll | EShowBonesMenuEntryFlags::DrawNone | EShowBonesMenuEntryFlags::BoneSize))
		{
			// Show the toggle all/none button if none or all are the only options
			Section.AddMenuEntry(FAnimViewportShowCommands::Get().ToggleShowBoneDrawAll);
		}
		else
		{
			if (Flags == EShowBonesMenuEntryFlags::AllFlags ||
				EnumHasAnyFlags(Flags, EShowBonesMenuEntryFlags::DrawAll))
			{
				Section.AddMenuEntry(FAnimViewportShowCommands::Get().ShowBoneDrawAll);
			}

			if (Flags == EShowBonesMenuEntryFlags::AllFlags ||
				EnumHasAnyFlags(Flags, EShowBonesMenuEntryFlags::DrawSelected))
			{
				Section.AddMenuEntry(FAnimViewportShowCommands::Get().ShowBoneDrawSelected);
			}

			if (Flags == EShowBonesMenuEntryFlags::AllFlags ||
				EnumHasAnyFlags(Flags, EShowBonesMenuEntryFlags::DrawSelectedAndParents))
			{
				Section.AddMenuEntry(FAnimViewportShowCommands::Get().ShowBoneDrawSelectedAndParents);
			}

			if (Flags == EShowBonesMenuEntryFlags::AllFlags ||
				EnumHasAnyFlags(Flags, EShowBonesMenuEntryFlags::DrawSelectedAndChildren))
			{
				Section.AddMenuEntry(FAnimViewportShowCommands::Get().ShowBoneDrawSelectedAndChildren);
			}

			if (Flags == EShowBonesMenuEntryFlags::AllFlags ||
				EnumHasAnyFlags(Flags, EShowBonesMenuEntryFlags::DrawSelectedAndParentsAndChildren))
			{
				Section.AddMenuEntry(FAnimViewportShowCommands::Get().ShowBoneDrawSelectedAndParentsAndChildren);
			}

			if (Flags == EShowBonesMenuEntryFlags::AllFlags ||
				EnumHasAnyFlags(Flags, EShowBonesMenuEntryFlags::DrawNone))
			{
				Section.AddMenuEntry(FAnimViewportShowCommands::Get().ShowBoneDrawNone);
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
