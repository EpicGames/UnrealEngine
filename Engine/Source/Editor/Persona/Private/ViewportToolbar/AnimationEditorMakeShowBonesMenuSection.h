// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/EnumClassFlags.h"

class UToolMenu;

#define UE_API PERSONA_API

namespace UE::Persona::ViewportToolbar
{
	enum class EShowBonesMenuEntryFlags : uint8
	{
		BoneSize = 1 << 0,
		DrawAll = 1 << 1,
		DrawSelected = 1 << 2,
		DrawSelectedAndParents = 1 << 3,
		DrawSelectedAndChildren = 1 << 4,
		DrawSelectedAndParentsAndChildren = 1 << 5,
		DrawNone = 1 << 6,

		AllFlags = 0xFF
	};
	ENUM_CLASS_FLAGS(EShowBonesMenuEntryFlags);

	/** Creates the Show Bones section in a tool menu, optionally with flags what to show */
	PERSONA_API void MakeShowBonesMenuSection(UToolMenu* ToolMenu, const EShowBonesMenuEntryFlags Flags = EShowBonesMenuEntryFlags::AllFlags);
}

#undef UE_API
