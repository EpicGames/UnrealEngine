// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationEditorMenus.h"

#include "AnimPreviewInstance.h"
#include "AnimViewportContext.h"
#include "AnimViewportLODCommands.h"
#include "AnimViewportMenuCommands.h"
#include "AnimViewportPlaybackCommands.h"
#include "AnimViewportShowCommands.h"
#include "Animation/MirrorDataTable.h"
#include "AnimationEditorWidgets.h"
#include "AnimationEditorMakeShowBonesMenuSection.h"
#include "BoneSelectionWidget.h"
#include "BufferVisualizationMenuCommands.h"
#include "NaniteVisualizationMenuCommands.h"
#include "ClothingSimulationFactory.h"
#include "ClothingSimulationInstance.h"
#include "ClothingSystemEditorInterfaceModule.h"
#include "ContentBrowserModule.h"
#include "EditorViewportCommands.h"
#include "IContentBrowserSingleton.h"
#include "IPinnedCommandList.h"
#include "SAnimationEditorViewport.h"
#include "ShowFlagMenuCommands.h"
#include "SimulationEditorExtender.h"
#include "ToolMenus.h"
#include "UICommandList_Pinnable.h"
#include "Rendering/SkinWeightProfile.h"
#include "ViewportToolbar/UnrealEdViewportToolbar.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Input/STextComboBox.h"
#include "Widgets/Input/SVectorInputBox.h"
#include "Runtime/SlateCore/Public/Styling/SlateStyleRegistry.h"

#define LOCTEXT_NAMESPACE "AnimEditorViewportToolbar"

namespace UE::AnimationEditor::Private
{
TSharedRef<SWidget> MakeBoneAsCameraWidget(const TWeakPtr<SAnimationEditorViewportTabBody>& InViewport, const TWeakPtr<SComboButton>& InWeakComboButton)
{
	TSharedPtr<SAnimationEditorViewportTabBody> Viewport = InViewport.Pin();

	if (!Viewport)
	{
		return SNullWidget::NullWidget;
	}

	TSharedPtr<SEditorViewport> ViewportWidget = Viewport->GetViewportWidget();

	if (!ViewportWidget || !ViewportWidget->GetViewportClient())
	{
		return SNullWidget::NullWidget;
	}

	TSharedPtr<SBoneTreeMenu> BoneTreeMenu;


	// clang-format off
	TSharedRef<SWidget> MenuWidget =
		SNew(SBox)
		.MaxDesiredHeight(400.0f)
		[
			SAssignNew(BoneTreeMenu, SBoneTreeMenu)
				.bShowVirtualBones(true)
				.OnBoneSelectionChanged_Lambda(
					[ViewportWeak = Viewport.ToWeakPtr()](FName InBoneName)
					{
						TSharedPtr<SAnimationEditorViewportTabBody> Viewport = ViewportWeak.Pin();
						if (!Viewport)
						{
							return;
						}

						Viewport->SetCameraFollowMode(EAnimationViewportCameraFollowMode::BoneAsCamera, InBoneName);

						FSlateApplication::Get().DismissAllMenus();

						if (const TSharedPtr<IPinnedCommandList>& PinnedCommands = Viewport->GetPinnedCommands())
						{
							PinnedCommands->SetStyle(&FAppStyle::Get(), TEXT("ViewportPinnedCommandList"));
							PinnedCommands->AddCustomWidget(TEXT("BoneAsCameraWidget"));
						}
					})
				.SelectedBone(Viewport->GetCameraFollowBoneName())
				.OnGetReferenceSkeleton_Lambda(
					[ViewportWeak = Viewport.ToWeakPtr()]() -> const FReferenceSkeleton&
					{
						static FReferenceSkeleton EmptySkeleton;

						if (TSharedPtr<SAnimationEditorViewportTabBody> Viewport = ViewportWeak.Pin())
						{
							if (USkeletalMesh* PreviewMesh = Viewport->GetPreviewScene()->GetPreviewMesh())
							{
								return PreviewMesh->GetRefSkeleton();
							}
						}

						return EmptySkeleton;
					})
		];
	// clang-format on

	if (TSharedPtr<SComboButton> ComboButton = InWeakComboButton.Pin())
	{
		ComboButton->SetMenuContentWidgetToFocus(BoneTreeMenu->GetFilterTextWidget());
	}

	return MenuWidget;
}


TSharedRef<SWidget> SetupBoneAsCameraAxisRadioButton(TSharedPtr<SAnimationEditorViewportTabBody>& ViewportTab,
	const FName InName,
	const TAttribute<FText>& InLabel,
	const TAttribute<FText>& InToolTip,
	FAnimationViewportClient::EDirection Direction,
	EAxis::Type Axis)
{
	auto IsCheckedLambda = [ViewportWeak = ViewportTab.ToWeakPtr(), Direction, Axis]() //IsChecked
		{
			if (TSharedPtr<SAnimationEditorViewportTabBody> Viewport = ViewportWeak.Pin())
			{
				const bool bIsSameAxis = Viewport->GetBoneAsCameraAxis(Direction) == Axis;
				return bIsSameAxis ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			}

			return ECheckBoxState::Unchecked;
		};

	auto OnStateChangedLambda = [ViewportWeak = ViewportTab.ToWeakPtr(), Direction, Axis](ECheckBoxState State) //Exec
		{
			if (TSharedPtr<SAnimationEditorViewportTabBody> Viewport = ViewportWeak.Pin())
			{
				Viewport->SetBoneAsCameraAxis(Direction, Axis);
			}
		};

	const ISlateStyle* StyleSet = FSlateStyleRegistry::FindSlateStyle("CoreStyle");

	TSharedRef<SWidget> Button =
		SNew(SCheckBox)
		.Visibility(EVisibility::Visible)
		.Style(StyleSet, "Menu.RadioButton")
		.IsChecked_Lambda(IsCheckedLambda)
		.OnCheckStateChanged_Lambda(OnStateChangedLambda);

	return Button;
}

void SetupBoneAsCameraForward(FToolMenuSection& Section, TSharedPtr<SAnimationEditorViewportTabBody>& ViewportTab)
{
	TSharedRef<SWidget> XRadioButton = SetupBoneAsCameraAxisRadioButton(ViewportTab,
		"CameraForwardX",
		LOCTEXT("BoneAsCameraForwardSectionX_Label", "X"),
		LOCTEXT("BoneAsCameraForwardSectionX_ToolTip", "Select X bone axis to use as camera forward"),
		FAnimationViewportClient::EDirection::Forward,
		EAxis::Type::X);

	TSharedRef<SWidget> YRadioButton = SetupBoneAsCameraAxisRadioButton(ViewportTab,
		"CameraForwardY",
		LOCTEXT("BoneAsCameraForwardSectionY_Label", "Y"),
		LOCTEXT("BoneAsCameraForwardSectionY_ToolTip", "Select Y bone axis to use as camera forward"),
		FAnimationViewportClient::EDirection::Forward,
		EAxis::Type::Y);

	TSharedRef<SWidget> ZRadioButton = SetupBoneAsCameraAxisRadioButton(ViewportTab,
		"CameraForwardZ",
		LOCTEXT("BoneAsCameraForwardSectionZ_Label", "Z"),
		LOCTEXT("BoneAsCameraForwardSectionZ_ToolTip", "Select Z bone axis to use as camera forward"),
		FAnimationViewportClient::EDirection::Forward,
		EAxis::Type::Z);

	TSharedRef<SWidget> CheckboxRow = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		.AutoWidth()
		.Padding(FMargin(10, 0, 0, 0))
		[
			XRadioButton
		]
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		.AutoWidth()
		.Padding(FMargin(10, 0, 50, 0))
		[
			SNew(STextBlock)
				.Text(LOCTEXT("CameraForwardRadioButtonTextX", "X"))
		]
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		.AutoWidth()
		.Padding(FMargin(10, 0, 0, 0))
		[
			YRadioButton
		]
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		.AutoWidth()
		.Padding(FMargin(10, 0, 50, 0))
		[
			SNew(STextBlock)
				.Text(LOCTEXT("CameraForwardRadioButtonTextY", "Y"))
		]
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		.AutoWidth()
		.Padding(FMargin(10, 0, 0, 0))
		[
			ZRadioButton
		]
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		.AutoWidth()
		.Padding(FMargin(10, 0, 50, 0))
		[
			SNew(STextBlock)
				.Text(LOCTEXT("CameraForwardRadioButtonTextZ", "Z"))
		];

	Section.AddEntry(FToolMenuEntry::InitWidget("BoneAsCameraForwardAxisWidget", CheckboxRow, FText()));

}

void SetupBoneAsCameraRight(FToolMenuSection& Section, TSharedPtr<SAnimationEditorViewportTabBody>& ViewportTab)
{
	TSharedRef<SWidget> XRadioButton = SetupBoneAsCameraAxisRadioButton(ViewportTab,
		"CameraRightX",
		LOCTEXT("BoneAsCameraRightSectionX_Label", "X"),
		LOCTEXT("BoneAsCameraRightSectionX_ToolTip", "Select X bone axis to use as camera forward"),
		FAnimationViewportClient::EDirection::Right,
		EAxis::Type::X);

	TSharedRef<SWidget> YRadioButton = SetupBoneAsCameraAxisRadioButton(ViewportTab,
		"CameraRightY",
		LOCTEXT("BoneAsCameraRightSectionY_Label", "Y"),
		LOCTEXT("BoneAsCameraRightSectionY_ToolTip", "Select Y bone axis to use as camera forward"),
		FAnimationViewportClient::EDirection::Right,
		EAxis::Type::Y);

	TSharedRef<SWidget> ZRadioButton = SetupBoneAsCameraAxisRadioButton(ViewportTab,
		"CameraRightZ",
		LOCTEXT("BoneAsCameraRightSectionZ_Label", "Z"),
		LOCTEXT("BoneAsCameraRightSectionZ_ToolTip", "Select Z bone axis to use as camera forward"),
		FAnimationViewportClient::EDirection::Right,
		EAxis::Type::Z);

	TSharedRef<SWidget> CheckboxRow = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		.AutoWidth()
		.Padding(FMargin(10, 0, 0, 0))
		[
			XRadioButton
		]
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		.AutoWidth()
		.Padding(FMargin(10, 0, 50, 0))
		[
			SNew(STextBlock)
				.Text(LOCTEXT("CameraRightRadioButtonTextX", "X"))
		]
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		.AutoWidth()
		.Padding(FMargin(10, 0, 0, 0))
		[
			YRadioButton
		]
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		.AutoWidth()
		.Padding(FMargin(10, 0, 50, 0))
		[
			SNew(STextBlock)
				.Text(LOCTEXT("CameraRightRadioButtonTextY", "Y"))
		]
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		.AutoWidth()
		.Padding(FMargin(10, 0, 0, 0))
		[
			ZRadioButton
		]
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		.AutoWidth()
		.Padding(FMargin(10, 0, 50, 0))
		[
			SNew(STextBlock)
				.Text(LOCTEXT("CameraRightRadioButtonTextZ", "Z"))
		];

	Section.AddEntry(FToolMenuEntry::InitWidget("BoneAsCameraRightAxisWidget", CheckboxRow, FText()));
}

void SetupBoneAsCameraUp(FToolMenuSection& Section, TSharedPtr<SAnimationEditorViewportTabBody>& ViewportTab)
{
	TSharedRef<SWidget> XRadioButton = SetupBoneAsCameraAxisRadioButton(ViewportTab,
		"CameraUpX",
		LOCTEXT("BoneAsCameraUpSectionX_Label", "X"),
		LOCTEXT("BoneAsCameraUpSectionX_ToolTip", "Select X bone axis to use as camera forward"),
		FAnimationViewportClient::EDirection::Up,
		EAxis::Type::X);

	TSharedRef<SWidget> YRadioButton = SetupBoneAsCameraAxisRadioButton(ViewportTab,
		"CameraUpY",
		LOCTEXT("BoneAsCameraUpSectionY_Label", "Y"),
		LOCTEXT("BoneAsCameraUpSectionY_ToolTip", "Select Y bone axis to use as camera forward"),
		FAnimationViewportClient::EDirection::Up,
		EAxis::Type::Y);

	TSharedRef<SWidget> ZRadioButton = SetupBoneAsCameraAxisRadioButton(ViewportTab,
		"CameraUpZ",
		LOCTEXT("BoneAsCameraUpSectionZ_Label", "Z"),
		LOCTEXT("BoneAsCameraUpSectionZ_ToolTip", "Select Z bone axis to use as camera forward"),
		FAnimationViewportClient::EDirection::Up,
		EAxis::Type::Z);

	TSharedRef<SWidget> CheckboxRow = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		.AutoWidth()
		.Padding(FMargin(10, 0, 0, 0))
		[
			XRadioButton
		]
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		.AutoWidth()
		.Padding(FMargin(10, 0, 50, 0))
		[
			SNew(STextBlock)
				.Text(LOCTEXT("CameraUpRadioButtonTextX", "X"))
		]
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		.AutoWidth()
		.Padding(FMargin(10, 0, 0, 0))
		[
			YRadioButton
		]
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		.AutoWidth()
		.Padding(FMargin(10, 0, 50, 0))
		[
			SNew(STextBlock)
				.Text(LOCTEXT("CameraUpRadioButtonTextY", "Y"))
		]
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		.AutoWidth()
		.Padding(FMargin(10, 0, 0, 0))
		[
			ZRadioButton
		]
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		.AutoWidth()
		.Padding(FMargin(10, 0, 50, 0))
		[
			SNew(STextBlock)
				.Text(LOCTEXT("CameraUpRadioButtonTextZ", "Z"))
		];

	Section.AddEntry(FToolMenuEntry::InitWidget("BoneAsCameraUpAxisWidget", CheckboxRow, FText()));
}

void SetupBoneAsCameraPositionOffset(FToolMenuSection& Section, TSharedPtr<SAnimationEditorViewportTabBody>& ViewportTab)
{
	const TSharedRef<SWidget> CameraOffsets =
		SNew(SNumericVectorInputBox<double>)
		.IsEnabled(true)
		.Vector_Lambda([ViewportWeak = ViewportTab.ToWeakPtr()]()
			{
				if (TSharedPtr<SAnimationEditorViewportTabBody> Viewport = ViewportWeak.Pin())
				{
					return Viewport->GetBoneAsCameraPositionOffset();
				}

				return UE::Math::TVector<double>::Zero();
			}
		)
		.OnXChanged_Lambda(
			[ViewportWeak = ViewportTab.ToWeakPtr()](double Value)
			{
				if (TSharedPtr<SAnimationEditorViewportTabBody> Viewport = ViewportWeak.Pin())
				{
					UE::Math::TVector<double> Offset = Viewport->GetBoneAsCameraPositionOffset();
					Offset.SetComponentForAxis(EAxis::Type::X, Value);
					Viewport->SetBoneAsCameraPositionOffset(Offset);
				}
			}
		)
		.OnYChanged_Lambda(
			[ViewportWeak = ViewportTab.ToWeakPtr()](double Value)
			{
				if (TSharedPtr<SAnimationEditorViewportTabBody> Viewport = ViewportWeak.Pin())
				{
					UE::Math::TVector<double> Offset = Viewport->GetBoneAsCameraPositionOffset();
					Offset.SetComponentForAxis(EAxis::Type::Y, Value);
					Viewport->SetBoneAsCameraPositionOffset(Offset);
				}
			}
		)
		.OnZChanged_Lambda(
			[ViewportWeak = ViewportTab.ToWeakPtr()](double Value)
			{
				if (TSharedPtr<SAnimationEditorViewportTabBody> Viewport = ViewportWeak.Pin())
				{
					UE::Math::TVector<double> Offset = Viewport->GetBoneAsCameraPositionOffset();
					Offset.SetComponentForAxis(EAxis::Type::Z, Value);
					Viewport->SetBoneAsCameraPositionOffset(Offset);
				}
			}
		)
		.AllowSpin(true);

	Section.AddEntry(FToolMenuEntry::InitWidget("BoneAsCameraPositionOffsetWidget", CameraOffsets, FText()));
}

void SetupBoneAsCameraRotationOffset(FToolMenuSection& Section, TSharedPtr<SAnimationEditorViewportTabBody>& ViewportTab)
{
	const TSharedRef<SWidget> CameraOffsets =
		SNew(SNumericVectorInputBox<double>)
		.IsEnabled(true)
		.Vector_Lambda([ViewportWeak = ViewportTab.ToWeakPtr()]()
			{
				if (TSharedPtr<SAnimationEditorViewportTabBody> Viewport = ViewportWeak.Pin())
				{
					const FRotator Offset = Viewport->GetBoneAsCameraRotationOffset();
					return FVector(Offset.Roll, Offset.Pitch, Offset.Yaw);
				}

				return UE::Math::TVector<double>::Zero();
			}
		)
		.OnXChanged_Lambda(
			[ViewportWeak = ViewportTab.ToWeakPtr()](double Value)
			{
				if (TSharedPtr<SAnimationEditorViewportTabBody> Viewport = ViewportWeak.Pin())
				{
					FRotator Offset = Viewport->GetBoneAsCameraRotationOffset();
					Offset.SetComponentForAxis(EAxis::Type::X, Value);
					Viewport->SetBoneAsCameraRotationOffset(Offset);
				}
			}
		)
		.OnYChanged_Lambda(
			[ViewportWeak = ViewportTab.ToWeakPtr()](double Value)
			{
				if (TSharedPtr<SAnimationEditorViewportTabBody> Viewport = ViewportWeak.Pin())
				{
					FRotator Offset = Viewport->GetBoneAsCameraRotationOffset();
					Offset.SetComponentForAxis(EAxis::Type::Y, Value);
					Viewport->SetBoneAsCameraRotationOffset(Offset);
				}
			}
		)
		.OnZChanged_Lambda(
			[ViewportWeak = ViewportTab.ToWeakPtr()](double Value)
			{
				if (TSharedPtr<SAnimationEditorViewportTabBody> Viewport = ViewportWeak.Pin())
				{
					FRotator Offset = Viewport->GetBoneAsCameraRotationOffset();
					Offset.SetComponentForAxis(EAxis::Type::Z, Value);
					Viewport->SetBoneAsCameraRotationOffset(Offset);
				}
			}
		)
		.AllowSpin(true);

	Section.AddEntry(FToolMenuEntry::InitWidget("BoneAsCameraRotationOffsetWidget", CameraOffsets, FText()));
}
// Convenience function to retrieve Anim Editor Viewport Tab from UToolMenu
TSharedPtr<SAnimationEditorViewportTabBody> GetAnimationEditorViewportTab(const UToolMenu* InMenu)
{
	if (InMenu)
	{
		if (UAnimViewportContext* const AnimViewportContext = InMenu->FindContext<UAnimViewportContext>())
		{
			return AnimViewportContext->ViewportTabBody.Pin();
		}
	}

	return nullptr;
}

// Convenience function to retrieve Anim Editor Viewport Tab from FToolMenuSection
TSharedPtr<SAnimationEditorViewportTabBody> GetAnimationEditorViewportTab(const FToolMenuSection& InDynamicSection)
{
	if (UAnimViewportContext* const AnimViewportContext = InDynamicSection.FindContext<UAnimViewportContext>())
	{
		return AnimViewportContext->ViewportTabBody.Pin();
	}

	return nullptr;
}

void PopulateLODSubmenu(UToolMenu* InMenu)
{
	if (!InMenu)
	{
		return;
	}

	const TSharedPtr<SAnimationEditorViewportTabBody> AnimEditorViewportTab = GetAnimationEditorViewportTab(InMenu);
	if (!AnimEditorViewportTab)
	{
		return;
	}

	const FAnimViewportLODCommands& Actions = FAnimViewportLODCommands::Get();

	const TWeakObjectPtr<const USkeletalMesh> PreviewMeshWeak = AnimEditorViewportTab->GetPreviewScene()->GetPreviewMesh();

	auto IsLODSimplified = [PreviewMeshWeak](const int32 InLODId) -> bool
	{
		if (const TStrongObjectPtr<const USkeletalMesh>& PreviewMeshPinned = PreviewMeshWeak.Pin())
		{
			return !PreviewMeshPinned->IsCompiling() && PreviewMeshPinned->IsValidLODIndex(InLODId) &&
				   PreviewMeshPinned->GetLODInfo(InLODId)->bHasBeenSimplified;
		}

		return false;
	};

	auto GetLODStatusExtraLabel = [IsLODSimplified, PreviewMeshWeak](const int32 InLODId) -> FText
	{
		if (const TStrongObjectPtr<const USkeletalMesh>& PreviewMeshPinned = PreviewMeshWeak.Pin())
		{
			if (IsLODSimplified(InLODId))
			{
				if (PreviewMeshPinned->HasMeshDescription(InLODId))
				{
					return LOCTEXT("LODStatus_Inline", " (Inline Reduced)");
				}

				return LOCTEXT("LODStatus_Generated", " (Generated)");
			}
		}
		return FText::GetEmpty();
	};

	auto GetLODStatusExtraTooltip = [IsLODSimplified, PreviewMeshWeak](const int32 InLODId) -> FText
	{
		if (const TStrongObjectPtr<const USkeletalMesh>& PreviewMeshPinned = PreviewMeshWeak.Pin())
		{
			if (IsLODSimplified(InLODId))
			{
				if (PreviewMeshPinned->HasMeshDescription(InLODId))
				{
					return LOCTEXT("LODStatusTooltip_Inline", "Generated from the editable geometry stored on this LOD but has been simplified in place.");
				}

				const int32 BaseLOD = PreviewMeshPinned->GetLODInfo(InLODId)->ReductionSettings.BaseLOD;
				return FText::Format(LOCTEXT("LODStatusTooltip_Generated", "Generated from a reduced version of LOD {0}.\nIt contains no editable geometry."), FText::AsNumber(BaseLOD));
			}
		}
		return LOCTEXT("LODStatusTooltip_Default", "Generated from the editable geometry stored on this LOD with no simplification applied.");
	};

	{
		// LOD Models
		FToolMenuSection& LODSection =
			InMenu->AddSection("AnimViewportPreviewLODs", LOCTEXT("ShowLOD_PreviewLabel", "Preview LODs"));
		{
			LODSection.AddMenuEntry(Actions.LODDebug);
			LODSection.AddMenuEntry(Actions.LODAuto);
			LODSection.AddMenuEntry(Actions.LOD0);

			const int32 LODCount = AnimEditorViewportTab->GetLODModelCount();
			for (int32 LODId = 1; LODId < LODCount; ++LODId)
			{
				const FText LODNumber = FText::AsNumber(LODId);
				const FText TitleLabel = FText::Format(LOCTEXT("LODFmt", "LOD {0}{1}"), LODNumber, GetLODStatusExtraLabel(LODId));
				const FText TooltipText = FText::Format(LOCTEXT("LODTooltip", "Force select LOD {0}.\n\n{1}"), LODNumber, GetLODStatusExtraTooltip(LODId));

				FUIAction Action(
					FExecuteAction::CreateSP(
						AnimEditorViewportTab.ToSharedRef(), &SAnimationEditorViewportTabBody::OnSetLODModel, LODId + 1
					),
					FCanExecuteAction(),
					FIsActionChecked::CreateSP(
						AnimEditorViewportTab.ToSharedRef(), &SAnimationEditorViewportTabBody::IsLODModelSelected, LODId + 1
					)
				);

				LODSection.AddMenuEntry(
					FName(TitleLabel.ToString()), TitleLabel, TooltipText, FSlateIcon(), Action, EUserInterfaceActionType::RadioButton
				);
			}
		}
	}
}

void PopulateSkinWeightProfileMenu(UToolMenu* InMenu)
{
	if (!InMenu)
	{
		return;
	}

	const TSharedPtr<SAnimationEditorViewportTabBody> AnimEditorViewportTab = GetAnimationEditorViewportTab(InMenu);
	if (!AnimEditorViewportTab)
	{
		return;
	}

	FText NoProfileLabel = LOCTEXT("SkinWeightProfileMenu_NoProfile", "No Profile");

	const TWeakObjectPtr<const USkeletalMesh> PreviewMeshWeak = AnimEditorViewportTab->GetPreviewScene()->GetPreviewMesh();

	if (const TStrongObjectPtr<const USkeletalMesh>& PreviewMeshPinned = PreviewMeshWeak.Pin())
	{
		const TArray<FSkinWeightProfileInfo>& SkinWeightProfiles = PreviewMeshPinned->GetSkinWeightProfiles();

		FToolMenuSection& ProfilesSection = InMenu->AddSection("SkinWeightActiveProfile", LOCTEXT("SkinWeightProfileMenu_ActiveProfile", "Active Profile"));

		auto AddProfileEntries = [Viewport=AnimEditorViewportTab, NoProfileLabel, &SkinWeightProfiles](FToolMenuSection& InSection, ESkinWeightProfileLayer InLayer)
			{
				FName NoneProfile;
				FUIAction ActionSetBaseProfile(FExecuteAction::CreateSP(Viewport.ToSharedRef(), &SAnimationEditorViewportTabBody::OnSetSkinWeightProfile, NoneProfile, InLayer),
					FCanExecuteAction(),
					FIsActionChecked::CreateSP(Viewport.ToSharedRef(), &SAnimationEditorViewportTabBody::IsSkinWeightProfileSelected, NoneProfile, InLayer));

				InSection.AddMenuEntry(FName(NoProfileLabel.ToString()), NoProfileLabel, FText::GetEmpty(), FSlateIcon(), ActionSetBaseProfile, EUserInterfaceActionType::RadioButton);
				
				// Retrieve all possible skin weight profiles from the component
				for (const FSkinWeightProfileInfo& Profile : SkinWeightProfiles)
				{
					FUIAction Action(FExecuteAction::CreateSP(Viewport.ToSharedRef(), &SAnimationEditorViewportTabBody::OnSetSkinWeightProfile, Profile.Name, InLayer),
						FCanExecuteAction(),
						FIsActionChecked::CreateSP(Viewport.ToSharedRef(), &SAnimationEditorViewportTabBody::IsSkinWeightProfileSelected, Profile.Name, InLayer));
					
					InSection.AddMenuEntry(Profile.Name, FText::FromName(Profile.Name), FText::GetEmpty(), FSlateIcon(), Action, EUserInterfaceActionType::RadioButton);
				}
			};

		AddProfileEntries(ProfilesSection, ESkinWeightProfileLayer::Primary);

		if (SkinWeightProfiles.Num() > 1)
		{
			FToolMenuSection& SecondaryProfilesSection = InMenu->AddSection("SkinWeightActiveSecondaryProfiles", LOCTEXT("SkinWeightProfileMenu_ActiveSecondaryProfile", "Active Secondary Profile"));
			AddProfileEntries(SecondaryProfilesSection, ESkinWeightProfileLayer::Secondary);
		}
	}	
}

void FillCharacterMirrorMenu(UToolMenu* InMenu)
{
	if (!InMenu)
	{
		return;
	}

	TSharedPtr<SAnimationEditorViewportTabBody> ViewportTab = GetAnimationEditorViewportTab(InMenu);
	if (!ViewportTab)
	{
		return;
	}

	UDebugSkelMeshComponent* PreviewComp = ViewportTab->GetPreviewScene()->GetPreviewMeshComponent();
	USkeletalMesh* Mesh = PreviewComp->GetSkeletalMeshAsset();
	UAnimPreviewInstance* PreviewInstance = PreviewComp->PreviewInstance;
	if (Mesh && PreviewInstance)
	{
		USkeleton* Skeleton = Mesh->GetSkeleton();

		FAssetPickerConfig AssetPickerConfig;
		AssetPickerConfig.Filter.ClassPaths.Add(UMirrorDataTable::StaticClass()->GetClassPathName());
		AssetPickerConfig.Filter.bRecursiveClasses = false;
		AssetPickerConfig.bAllowNullSelection = true;
		AssetPickerConfig.OnShouldFilterAsset =
			FOnShouldFilterAsset::CreateUObject(Skeleton, &USkeleton::ShouldFilterAsset, TEXT("Skeleton"));
		AssetPickerConfig.InitialAssetSelection = FAssetData(PreviewInstance->GetMirrorDataTable());
		AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateLambda(
			[ViewportTabWeak = ViewportTab.ToWeakPtr()](const FAssetData& InSelectedMirrorTableData)
			{
				if (const TSharedPtr<SAnimationEditorViewportTabBody>& ViewportTabPinned = ViewportTabWeak.Pin())
				{
					UDebugSkelMeshComponent* PreviewComp = ViewportTabPinned->GetPreviewScene()->GetPreviewMeshComponent();
					USkeletalMesh* Mesh = PreviewComp->GetSkeletalMeshAsset();
					UAnimPreviewInstance* PreviewInstance = PreviewComp->PreviewInstance;
					if (Mesh && PreviewInstance)
					{
						UMirrorDataTable* MirrorDataTable = Cast<UMirrorDataTable>(InSelectedMirrorTableData.GetAsset());
						PreviewInstance->SetMirrorDataTable(MirrorDataTable);
						PreviewComp->OnMirrorDataTableChanged();
					}
				}
			}
		);
		AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;
		AssetPickerConfig.ThumbnailScale = 0.1f;
		AssetPickerConfig.bAddFilterUI = false;

		FContentBrowserModule& ContentBrowserModule =
			FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

		FToolMenuEntry CharacterMirrorMenu =
			FToolMenuEntry::InitWidget("", ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig), FText());

		InMenu->AddMenuEntry("CharacterMirrorMenu", CharacterMirrorMenu);
	}
}

void FillCharacterClothingMenu(UToolMenu* InMenu)
{
	if (!InMenu)
	{
		return;
	}

	// Call into the clothing editor module to customize the menu (this is mainly for debug visualizations and sim-specific options)
	TSharedPtr<SAnimationEditorViewportTabBody> AnimEditorViewportTab = GetAnimationEditorViewportTab(InMenu);
	if (!AnimEditorViewportTab)
	{
		return;
	}

	const FAnimViewportShowCommands& Actions = FAnimViewportShowCommands::Get();

	FToolMenuSection& ClothPreviewSection = InMenu->AddSection("ClothPreview", LOCTEXT("ClothPreview_Label", "Simulation"));
	{
		ClothPreviewSection.AddMenuEntry(Actions.EnableClothSimulation);
		ClothPreviewSection.AddMenuEntry(Actions.ResetClothSimulation);

		TSharedPtr<SWidget> WindWidget =
			SNew(UE::AnimationEditor::SClothWindSettings).AnimEditorViewport(AnimEditorViewportTab);
		ClothPreviewSection.AddEntry(FToolMenuEntry::InitWidget(
			"", WindWidget.ToSharedRef(), LOCTEXT("ClothPreview_WindStrength", "Wind Strength:")
		));

		TSharedPtr<SWidget> GravityWidget =
			SNew(UE::AnimationEditor::SGravitySettings).AnimEditorViewport(AnimEditorViewportTab);
		ClothPreviewSection.AddEntry(FToolMenuEntry::InitWidget(
			"", GravityWidget.ToSharedRef(), LOCTEXT("ClothPreview_GravityScale", "Gravity Scale:")
		));

		ClothPreviewSection.AddMenuEntry(Actions.PauseClothWithAnim);
	}

	FToolMenuSection& ClothAdditionalVisualizationSection = InMenu->AddSection(
		"ClothAdditionalVisualization", LOCTEXT("ClothAdditionalVisualization_Label", "Sections Display Mode")
	);
	{
		ClothAdditionalVisualizationSection.AddMenuEntry(Actions.ShowAllSections);
		ClothAdditionalVisualizationSection.AddMenuEntry(Actions.ShowOnlyClothSections);
		ClothAdditionalVisualizationSection.AddMenuEntry(Actions.HideOnlyClothSections);
	}

	// Call into the clothing editor module to customize the menu (this is mainly for debug visualizations and sim-specific options)
	if (const TSharedPtr<FAnimationViewportClient>& AnimationViewportClient =
			AnimEditorViewportTab->GetAnimationViewportClient())
	{
		TSharedRef<IPersonaPreviewScene> PreviewScene = AnimationViewportClient->GetPreviewScene();
		if (UDebugSkelMeshComponent* PreviewComponent = PreviewScene->GetPreviewMeshComponent())
		{
			if (PreviewComponent->GetClothingSimulationInstances().Num())
			{
				// Currently using FNewToolMenuDelegateLegacy since this extension is being done via
				// ISimulationEditorExtender::ExtendViewportShowMenu(FMenuBuilder&InMenuBuilder, TSharedRef<IPersonaPreviewScene> InPreviewScene)
				// todo: that function might need a new version using UToolMenu
				FToolMenuSection& UnnamedSection = InMenu->AddSection(NAME_None);
				UnnamedSection.AddDynamicEntry(
					"SimulationEditorExtender",
					FNewToolMenuDelegateLegacy::CreateLambda(
						[](FMenuBuilder& InMenuBuilder, UToolMenu* InMenu)
						{
							TSharedPtr<SAnimationEditorViewportTabBody> AnimEditorViewportTab =
								GetAnimationEditorViewportTab(InMenu);
							if (!AnimEditorViewportTab)
							{
								return;
							}

							TSharedRef<FAnimationEditorPreviewScene> PreviewScene =
								AnimEditorViewportTab->GetPreviewScene();

							UDebugSkelMeshComponent* PreviewComponent = PreviewScene->GetPreviewMeshComponent();
							if (!PreviewComponent)
							{
								return;
							}

							FClothingSystemEditorInterfaceModule& ClothingEditorModule =
								FModuleManager::LoadModuleChecked<FClothingSystemEditorInterfaceModule>(
									TEXT("ClothingSystemEditorInterface")
								);
							bool bHasExtender = false;
							for (const FClothingSimulationInstance& ClothingSimulationInstance : PreviewComponent->GetClothingSimulationInstances())
							{
								UClothingSimulationFactory* const ClothingSimulationFactory = ClothingSimulationInstance.GetClothingSimulationFactory();
								if (ClothingSimulationFactory && ClothingSimulationFactory->GetClass())
								{
									if (ISimulationEditorExtender* const Extender = ClothingEditorModule.GetSimulationEditorExtender(
										ClothingSimulationFactory->GetClass()->GetFName()))
									{
										if (!bHasExtender)
										{
											// Calling end section will set bSectionNeedsToBeApplied to false.
											// Without doing so, calling ExtendViewportShowMenu will end up triggering an ensure
											InMenuBuilder.EndSection();
											bHasExtender = true;
										}
										Extender->ExtendViewportShowMenu(InMenuBuilder, PreviewScene);
									}
								}
							}
						}
					)
				);
			}
		}
	}
}

void FillCharacterAdvancedMenu(UToolMenu* InMenu)
{
	if (!InMenu)
	{
		return;
	}

	const FAnimViewportShowCommands& Actions = FAnimViewportShowCommands::Get();

	TSharedPtr<SAnimationEditorViewportTabBody> AnimEditorViewportTab = GetAnimationEditorViewportTab(InMenu);
	if (!AnimEditorViewportTab)
	{
		return;
	}

	// Draw UVs
	{
		FToolMenuSection& UVSection =
			InMenu->AddSection("UVVisualization", LOCTEXT("UVVisualization_Label", "UV Visualization"));

		if (TSharedPtr<STextComboBox> UVChannelComboBox = AnimEditorViewportTab->UVChannelCombo)
		{
			UVSection.AddEntry(FToolMenuEntry::InitWidget("", UVChannelComboBox.ToSharedRef(), FText()));
		}
	}

	// Skinning
	{
		FToolMenuSection& SkinningSection = InMenu->AddSection("Skinning", LOCTEXT("Skinning_Label", "Skinning"));
		SkinningSection.AddMenuEntry(FAnimViewportMenuCommands::Get().SetCPUSkinning);
	}

	// Vertex visualization
	{
		FToolMenuSection& ShowVertexSection =
			InMenu->AddSection("ShowVertex", LOCTEXT("ShowVertex_Label", "Vertex Normal Visualization"));

		// Vertex debug flags
		ShowVertexSection.AddMenuEntry(FAnimViewportMenuCommands::Get().SetShowNormals);
		ShowVertexSection.AddMenuEntry(FAnimViewportMenuCommands::Get().SetShowTangents);
		ShowVertexSection.AddMenuEntry(FAnimViewportMenuCommands::Get().SetShowBinormals);
	}

	// Local Axes
	{
		FToolMenuSection& LocalAxesSection = InMenu->AddSection(
			"AnimViewportPreviewHierarchyLocalAxes", LOCTEXT("ShowMenu_Actions_HierarchyAxes", "Hierarchy Local Axes")
		);

		LocalAxesSection.AddMenuEntry(Actions.ShowLocalAxesAll);
		LocalAxesSection.AddMenuEntry(Actions.ShowLocalAxesSelected);
		LocalAxesSection.AddMenuEntry(Actions.ShowLocalAxesNone);
	}
}

void FillCharacterTimecodeMenu(UToolMenu* InMenu)
{
	if (!InMenu)
	{
		return;
	}

	const FAnimViewportShowCommands& Actions = FAnimViewportShowCommands::Get();
	FToolMenuSection& TimecodeSection = InMenu->AddSection("Timecode", LOCTEXT("Timecode_Label", "Timecode"));
	TimecodeSection.AddMenuEntry(Actions.ShowTimecode);
}

void FillPlaybackMenu(UToolMenu* InMenu)
{
	if (!InMenu)
	{
		return;
	}

	FToolMenuSection& PlaybackSpeedSection =
		InMenu->FindOrAddSection("AnimViewportPlaybackSpeed", LOCTEXT("PlaybackMenu_SpeedLabel", "Playback Speed"));

	const FAnimViewportPlaybackCommands& Actions = FAnimViewportPlaybackCommands::Get();
	for (int32 PlaybackSpeedIndex = 0; PlaybackSpeedIndex < EAnimationPlaybackSpeeds::NumPlaybackSpeeds; ++PlaybackSpeedIndex)
	{
		PlaybackSpeedSection.AddMenuEntry(Actions.PlaybackSpeedCommands[PlaybackSpeedIndex]);
	}

	if (UAnimViewportContext* const AnimViewportContext = InMenu->FindContext<UAnimViewportContext>())
	{
		if (TSharedPtr<IPersonaPreviewScene> PreviewScenePinned =
				AnimViewportContext->PersonaPreviewScene.Pin())
		{
			TSharedPtr<SWidget> AnimSpeedWidget =
				SNew(UE::AnimationEditor::SCustomAnimationSpeedSetting)
					.CustomSpeed_Lambda(
						[PreviewSceneWeak = PreviewScenePinned.ToWeakPtr()]()
						{
							if (TSharedPtr<IPersonaPreviewScene> PreviewScenePinned = PreviewSceneWeak.Pin())
							{
								return PreviewScenePinned->GetCustomAnimationSpeed();
							}

							return 0.0f;
						}
					)
					.OnCustomSpeedChanged_Lambda(
						[PreviewSceneWeak = PreviewScenePinned.ToWeakPtr()](float InCustomSpeed)
						{
							if (TSharedPtr<IPersonaPreviewScene> PreviewScenePinned = PreviewSceneWeak.Pin())
							{
								return PreviewScenePinned->SetCustomAnimationSpeed(InCustomSpeed);
							}
						}
					);

			PlaybackSpeedSection.AddEntry(
				FToolMenuEntry::InitWidget(
					"PlaybackSpeed", AnimSpeedWidget.ToSharedRef(), LOCTEXT("PlaybackMenu_Speed_Custom", "Custom Speed:")
				)
			);
		}
	}
}
} // namespace UE::AnimationEditor::Private

TSharedRef<SWidget> UE::AnimationEditor::MakeFollowBoneWidget(
	const TWeakPtr<SAnimationEditorViewportTabBody>& InViewport, const TWeakPtr<SComboButton>& InWeakComboButton
)
{
	TSharedPtr<SAnimationEditorViewportTabBody> Viewport = InViewport.Pin();

	if (!Viewport)
	{
		return SNullWidget::NullWidget;
	}

	TSharedPtr<SEditorViewport> ViewportWidget = Viewport->GetViewportWidget();

	if (!ViewportWidget || !ViewportWidget->GetViewportClient())
	{
		return SNullWidget::NullWidget;
	}

	TSharedPtr<SBoneTreeMenu> BoneTreeMenu;

	// clang-format off
	TSharedRef<SWidget> MenuWidget =
		SNew(SBox)
		.MaxDesiredHeight(400.0f)
		[
			SAssignNew(BoneTreeMenu, SBoneTreeMenu)
			.bShowVirtualBones(true)
			.OnBoneSelectionChanged_Lambda(
				[ViewportWeak = Viewport.ToWeakPtr()](FName InBoneName)
				{
					TSharedPtr<SAnimationEditorViewportTabBody> Viewport = ViewportWeak.Pin();
					if (!Viewport)
					{
						return;
					}

					Viewport->SetCameraFollowMode(EAnimationViewportCameraFollowMode::Bone, InBoneName);
					FSlateApplication::Get().DismissAllMenus();

					if (const TSharedPtr<IPinnedCommandList>& PinnedCommands = Viewport->GetPinnedCommands())
					{
						PinnedCommands->SetStyle(&FAppStyle::Get(), TEXT("ViewportPinnedCommandList"));
						PinnedCommands->AddCustomWidget(TEXT("FollowBoneWidget"));
					}
				})
			.SelectedBone(Viewport->GetCameraFollowBoneName())
			.OnGetReferenceSkeleton_Lambda(
				[ViewportWeak = Viewport.ToWeakPtr()]() -> const FReferenceSkeleton&
				{
					static FReferenceSkeleton EmptySkeleton;

					if (TSharedPtr<SAnimationEditorViewportTabBody> Viewport = ViewportWeak.Pin())
					{
						if (USkeletalMesh* PreviewMesh = Viewport->GetPreviewScene()->GetPreviewMesh())
						{
							return PreviewMesh->GetRefSkeleton();
						}
					}

					return EmptySkeleton;
				})
		];
	// clang-format on

	if (TSharedPtr<SComboButton> ComboButton = InWeakComboButton.Pin())
	{
		ComboButton->SetMenuContentWidgetToFocus(BoneTreeMenu->GetFilterTextWidget());
	}

	return MenuWidget;
}

FToolMenuEntry UE::AnimationEditor::CreateShowSubmenu()
{
	return UE::UnrealEd::CreateShowSubmenu(FNewToolMenuDelegate::CreateLambda(
		[](UToolMenu* Submenu) -> void
		{
			AddSceneElementsSection(Submenu);
			UE::AnimationEditor::FillShowSubmenu(Submenu);
		}
	));
}

void UE::AnimationEditor::FillShowSubmenu(UToolMenu* InMenu, bool bInShowViewportStatsToggle)
{
	if (!InMenu)
	{
		return;
	}

	InMenu->AddDynamicSection(
		"AnimSection",
		FNewToolMenuDelegate::CreateLambda(
			[bInShowViewportStatsToggle](UToolMenu* InMenu)
			{
				FToolMenuSection& UnnamedSection = InMenu->AddSection(NAME_None);
				
				UnnamedSection.AddMenuEntry(FAnimViewportShowCommands::Get().ShowNaniteFallback);
				
				if (bInShowViewportStatsToggle)
				{
					UnnamedSection.AddSeparator(NAME_None);
					UnnamedSection.AddMenuEntry(
						FEditorViewportCommands::Get().ToggleStats, LOCTEXT("ViewportStatsLabel", "Viewport Stats")
					);
				}
				
				UnnamedSection.AddSeparator(NAME_None);

				// Only include helpful show flags.
				static const FShowFlagFilter ShowFlagFilter =
					FShowFlagFilter(FShowFlagFilter::ExcludeAllFlagsByDefault)
						// General
						.IncludeFlag(FEngineShowFlags::SF_AntiAliasing)
						.IncludeFlag(FEngineShowFlags::SF_Collision)
						.IncludeFlag(FEngineShowFlags::SF_Particles)
						.IncludeFlag(FEngineShowFlags::SF_Translucency)
						.IncludeFlag(FEngineShowFlags::SF_Grid)
						// Post Processing
						.IncludeFlag(FEngineShowFlags::SF_Bloom)
						.IncludeFlag(FEngineShowFlags::SF_DepthOfField)
						.IncludeFlag(FEngineShowFlags::SF_EyeAdaptation)
						.IncludeFlag(FEngineShowFlags::SF_HMDDistortion)
						.IncludeFlag(FEngineShowFlags::SF_MotionBlur)
						.IncludeFlag(FEngineShowFlags::SF_Tonemapper)
						// Lighting Components
						.IncludeGroup(SFG_LightingComponents)
						// Lighting Features
						.IncludeFlag(FEngineShowFlags::SF_AmbientCubemap)
						.IncludeFlag(FEngineShowFlags::SF_DistanceFieldAO)
						.IncludeFlag(FEngineShowFlags::SF_IndirectLightingCache)
						.IncludeFlag(FEngineShowFlags::SF_LightFunctions)
						.IncludeFlag(FEngineShowFlags::SF_LightShafts)
						.IncludeFlag(FEngineShowFlags::SF_ReflectionEnvironment)
						.IncludeFlag(FEngineShowFlags::SF_ScreenSpaceAO)
						.IncludeFlag(FEngineShowFlags::SF_ContactShadows)
						.IncludeFlag(FEngineShowFlags::SF_ScreenSpaceReflections)
						.IncludeFlag(FEngineShowFlags::SF_SubsurfaceScattering)
						.IncludeFlag(FEngineShowFlags::SF_TexturedLightProfiles)
						// Developer
						.IncludeFlag(FEngineShowFlags::SF_Refraction)
						// Advanced
						.IncludeFlag(FEngineShowFlags::SF_DeferredLighting)
						.IncludeFlag(FEngineShowFlags::SF_Selection)
						.IncludeFlag(FEngineShowFlags::SF_SeparateTranslucency)
						.IncludeFlag(FEngineShowFlags::SF_TemporalAA)
						.IncludeFlag(FEngineShowFlags::SF_VertexColors)
						.IncludeFlag(FEngineShowFlags::SF_MeshEdges);

				FShowFlagMenuCommands::Get().BuildShowFlagsMenu(InMenu, ShowFlagFilter);
			}
		)
	);
}

FToolMenuEntry UE::AnimationEditor::CreateTurnTableMenu()
{
	return FToolMenuEntry::InitSubMenu(
		"TurnTable",
		LOCTEXT("TurnTableLabel", "Turn Table"),
		LOCTEXT("TurnTableTooltip", "Set up auto-rotation of preview."),
		FNewToolMenuDelegate::CreateLambda(
			[](UToolMenu* Submenu) -> void
			{
				UE::AnimationEditor::FillTurnTableSubmenu(Submenu);
			}
		),
		false,
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "AnimViewportMenu.TurnTableSpeed")
	);
}

void UE::AnimationEditor::FillTurnTableSubmenu(UToolMenu* InMenu)
{
	if (!InMenu)
	{
		return;
	}

	const FAnimViewportPlaybackCommands& Actions = FAnimViewportPlaybackCommands::Get();
	{
		FToolMenuSection& TurnTableModeSection =
			InMenu->FindOrAddSection("AnimViewportTurnTableMode", LOCTEXT("TurnTableMenu_ModeLabel", "Turn Table Mode"));

		TurnTableModeSection.AddMenuEntry(Actions.PersonaTurnTablePlay);
		TurnTableModeSection.AddMenuEntry(Actions.PersonaTurnTablePause);
		TurnTableModeSection.AddMenuEntry(Actions.PersonaTurnTableStop);
	}

	{
		FToolMenuSection& TurnTableSpeedSection =
			InMenu->FindOrAddSection("AnimViewportTurnTableSpeed", LOCTEXT("TurnTableMenu_SpeedLabel", "Turn Table Speed"));
		for (int i = 0; i < EAnimationPlaybackSpeeds::NumPlaybackSpeeds; ++i)
		{
			TurnTableSpeedSection.AddMenuEntry(Actions.TurnTableSpeeds[i]);
		}

		const TSharedPtr<SAnimationEditorViewportTabBody> AnimEditorViewportTab =
			Private::GetAnimationEditorViewportTab(InMenu);

		// clang-format off
		TSharedPtr<SWidget> AnimSpeedWidget =
			SNew(UE::AnimationEditor::SCustomAnimationSpeedSetting)
			.CustomSpeed_Lambda(
				[ViewportWeak = AnimEditorViewportTab.ToWeakPtr()]()
				{
					if (const TSharedPtr<SAnimationEditorViewportTabBody> Viewport = ViewportWeak.Pin())
					{
						return Viewport->GetCustomTurnTableSpeed();
					}

					return 1.0f;
				})
			.OnCustomSpeedChanged_Lambda(
				[ViewportWeak = AnimEditorViewportTab.ToWeakPtr()](float CustomSpeed)
				{
					if (TSharedPtr<SAnimationEditorViewportTabBody> Viewport = ViewportWeak.Pin())
					{
						Viewport->SetCustomTurnTableSpeed(CustomSpeed);
					}
				});
		// clang-format on

		TurnTableSpeedSection.AddEntry(FToolMenuEntry::InitWidget(
			"AnimSpeed", AnimSpeedWidget.ToSharedRef(), LOCTEXT("PlaybackMenu_Speed_Custom", "Custom Speed:")
		));
	}
}

void UE::AnimationEditor::AddSceneElementsSection(UToolMenu* InMenu)
{
	if (!InMenu)
	{
		return;
	}

	FToolMenuSection& Section =
		InMenu->AddSection("AnimViewportSceneElements", LOCTEXT("CharacterMenu_SceneElements", "Scene Elements"));

	Section.AddSubMenu(
		TEXT("MeshSubMenu"),
		LOCTEXT("CharacterMenu_MeshSubMenu", "Mesh"),
		LOCTEXT("CharacterMenu_MeshSubMenuToolTip", "Mesh-related options"),
		FNewToolMenuDelegate::CreateLambda(
			[](UToolMenu* InSubMenu)
			{
				{
					FToolMenuSection& Section =
						InSubMenu->AddSection("AnimViewportMesh", LOCTEXT("CharacterMenu_Actions_Mesh", "Mesh"));
					Section.AddMenuEntry(FAnimViewportShowCommands::Get().ShowBound);
					Section.AddMenuEntry(FAnimViewportShowCommands::Get().UseInGameBound);
					Section.AddMenuEntry(FAnimViewportShowCommands::Get().UseFixedBounds);
					Section.AddMenuEntry(FAnimViewportShowCommands::Get().UsePreSkinnedBounds);
					Section.AddMenuEntry(FAnimViewportShowCommands::Get().ShowPreviewMesh);
					Section.AddMenuEntry(FAnimViewportShowCommands::Get().ShowMorphTargets);
				}
				{
					FToolMenuSection& Section = InSubMenu->AddSection(
						"AnimViewportMeshInfo", LOCTEXT("CharacterMenu_Actions_MeshInfo", "Mesh Info")
					);
					Section.AddMenuEntry(FAnimViewportShowCommands::Get().ShowDisplayInfoBasic);
					Section.AddMenuEntry(FAnimViewportShowCommands::Get().ShowDisplayInfoDetailed);
					Section.AddMenuEntry(FAnimViewportShowCommands::Get().ShowDisplayInfoSkelControls);
					Section.AddMenuEntry(FAnimViewportShowCommands::Get().HideDisplayInfo);
				}
				{
					FToolMenuSection& Section = InSubMenu->AddSection(
						"AnimViewportPreviewOverlayDraw", LOCTEXT("CharacterMenu_Actions_Overlay", "Mesh Overlay Drawing")
					);
					Section.AddMenuEntry(FAnimViewportShowCommands::Get().ShowOverlayNone);
					Section.AddMenuEntry(FAnimViewportShowCommands::Get().ShowBoneWeight);
					Section.AddMenuEntry(FAnimViewportShowCommands::Get().ShowMorphTargetVerts);
				}
			}
		)
	);

	Section.AddSubMenu(
		"AnimationSubMenu",
		LOCTEXT("CharacterMenu_AnimationSubMenu", "Animation"),
		LOCTEXT("CharacterMenu_AnimationSubMenuToolTip", "Animation-related options"),
		FNewToolMenuDelegate::CreateLambda(
			[](UToolMenu* InSubMenu)
			{
				const TSharedPtr<SAnimationEditorViewportTabBody> AnimEditorViewportTab =
					Private::GetAnimationEditorViewportTab(InSubMenu);
				if (!AnimEditorViewportTab)
				{
					return;
				}

				{
					FToolMenuSection& Section = InSubMenu->AddSection(
						"AnimViewportRootMotion", LOCTEXT("CharacterMenu_RootMotionLabel", "Root Motion")
					);
					Section.AddMenuEntry(FAnimViewportShowCommands::Get().DoNotProcessRootMotion);
					Section.AddMenuEntry(FAnimViewportShowCommands::Get().ProcessRootMotionLoop);
					Section.AddMenuEntry(FAnimViewportShowCommands::Get().ProcessRootMotionLoopAndReset);
				}

				{
					FToolMenuSection& Section = InSubMenu->AddSection(
						"AnimViewportVisualization", LOCTEXT("CharacterMenu_VisualizationsLabel", "Visualizations")
					);
					Section.AddMenuEntry(FAnimViewportShowCommands::Get().ShowNotificationVisualizations);
					Section.AddMenuEntry(FAnimViewportShowCommands::Get().DoNotVisualizeRootMotion);
					Section.AddMenuEntry(FAnimViewportShowCommands::Get().VisualizeRootMotionTrajectory);
					Section.AddMenuEntry(FAnimViewportShowCommands::Get().VisualizeRootMotionTrajectoryAndOrientation);
					Section.AddMenuEntry(FAnimViewportShowCommands::Get().ShowAssetUserDataVisualizations);
				}

				{
					FToolMenuSection& Section = InSubMenu->AddSection(
						"AnimViewportAnimation", LOCTEXT("CharacterMenu_Actions_AnimationAsset", "Animation")
					);
					Section.AddMenuEntry(FAnimViewportShowCommands::Get().ShowRawAnimation);
					Section.AddMenuEntry(FAnimViewportShowCommands::Get().ShowNonRetargetedAnimation);
					Section.AddMenuEntry(FAnimViewportShowCommands::Get().ShowAdditiveBaseBones);
					Section.AddMenuEntry(FAnimViewportShowCommands::Get().ShowSourceRawAnimation);

					TSharedPtr<SEditorViewport> ViewportWidget = AnimEditorViewportTab->GetViewportWidget();
					if (ViewportWidget && ViewportWidget->GetViewportClient())
					{
						if (UDebugSkelMeshComponent* PreviewComponent =
								AnimEditorViewportTab->GetPreviewScene()->GetPreviewMeshComponent())
						{
							FUIAction DisableUnlessPreviewInstance;
							DisableUnlessPreviewInstance.CanExecuteAction = FCanExecuteAction::CreateLambda(
								[PreviewComponentWeak = TWeakObjectPtr<UDebugSkelMeshComponent>(PreviewComponent)]()
								{
									if (UDebugSkelMeshComponent* PreviewComponent = PreviewComponentWeak.Get())
									{
										return (
											PreviewComponent->PreviewInstance
											&& (PreviewComponent->PreviewInstance == PreviewComponent->GetAnimInstance())
										);
									}

									return false;
								}
							);

							Section.AddSubMenu(
								"MirrorSubMenu",
								LOCTEXT("CharacterMenu_AnimationSubMenu_MirrorSubMenu", "Mirror"),
								LOCTEXT(
									"CharacterMenu_AnimationSubMenu_MirrorSubMenuToolTip",
									"Mirror the animation using the selected mirror data table"
								),
								FNewToolMenuChoice(FNewToolMenuDelegate::CreateStatic(
									&UE::AnimationEditor::Private::FillCharacterMirrorMenu
								)),
								DisableUnlessPreviewInstance,
								EUserInterfaceActionType::Button,
								false,
								FSlateIcon(),
								false
							);
						}
					}
					Section.AddMenuEntry(FAnimViewportShowCommands::Get().ShowBakedAnimation);
					Section.AddMenuEntry(FAnimViewportShowCommands::Get().DisablePostProcessBlueprint);
				}
			}
		)
	);

	Section.AddSubMenu(
		"BonesSubMenu",
		LOCTEXT("CharacterMenu_BoneDrawSubMenu", "Bones"),
		LOCTEXT("CharacterMenu_BoneDrawSubMenuToolTip", "Bone Drawing Options"),
		FNewToolMenuDelegate::CreateLambda(
			[](UToolMenu* InSubMenu)
			{
				TSharedPtr<SAnimationEditorViewportTabBody> AnimEditorViewportTab =
					Private::GetAnimationEditorViewportTab(InSubMenu);
				if (!AnimEditorViewportTab)
				{
					return;
				}

				{
					FToolMenuSection& Section =
						InSubMenu->AddSection("BonesAndSockets", LOCTEXT("CharacterMenu_BonesAndSocketsLabel", "Show"));
					Section.AddMenuEntry(FAnimViewportShowCommands::Get().ShowSockets);
					Section.AddMenuEntry(FAnimViewportShowCommands::Get().ShowAttributes);
					Section.AddMenuEntry(FAnimViewportShowCommands::Get().ShowBoneNames);
					Section.AddMenuEntry(FAnimViewportShowCommands::Get().ShowBoneColors);
				}

				{
					UE::Persona::ViewportToolbar::MakeShowBonesMenuSection(InSubMenu);
				}
			}
		)
	);

	Section.AddDynamicEntry(
		"ClothingSubMenu",
		FNewToolMenuSectionDelegate::CreateLambda(
			[](FToolMenuSection& InSection)
			{
				TSharedPtr<SAnimationEditorViewportTabBody> AnimEditorViewportTab =
					Private::GetAnimationEditorViewportTab(InSection);
				if (!AnimEditorViewportTab)
				{
					return;
				}

				TSharedPtr<FAnimationEditorPreviewScene> PreviewScene = AnimEditorViewportTab->GetPreviewScene();
				if (!PreviewScene)
				{
					return;
				}

				UDebugSkelMeshComponent* PreviewComp = PreviewScene->GetPreviewMeshComponent();
				if (PreviewComp && GetDefault<UPersonaOptions>()->bExposeClothingSceneElementMenu)
				{
					constexpr bool bInOpenSubMenuOnClick = false;
					constexpr bool bShouldCloseWindowAfterMenuSelection = false;
					InSection.AddSubMenu(
						"ClothingSubMenu",
						LOCTEXT("CharacterMenu_ClothingSubMenu", "Clothing"),
						LOCTEXT("CharacterMenu_ClothingSubMenuToolTip", "Options relating to clothing"),
						FNewToolMenuChoice(
							FNewToolMenuDelegate::CreateStatic(&UE::AnimationEditor::Private::FillCharacterClothingMenu)
						),
						bInOpenSubMenuOnClick,
						TAttribute<FSlateIcon>(),
						bShouldCloseWindowAfterMenuSelection
					);
				}
			}
		)
	);

	Section.AddSubMenu(
		TEXT("AudioSubMenu"),
		LOCTEXT("CharacterMenu_AudioSubMenu", "Audio"),
		LOCTEXT("CharacterMenu_AudioSubMenuToolTip", "Audio options"),
		FNewToolMenuDelegate::CreateLambda(
			[](UToolMenu* InSubMenu)
			{
				FToolMenuSection& Section =
					InSubMenu->AddSection("AnimViewportAudio", LOCTEXT("CharacterMenu_Audio", "Audio"));
				Section.AddMenuEntry(FAnimViewportShowCommands::Get().MuteAudio);
				Section.AddMenuEntry(FAnimViewportShowCommands::Get().UseAudioAttenuation);
			}
		)
	);
	Section.AddDynamicEntry(
		"Timecode",
		FNewToolMenuSectionDelegate::CreateLambda(
			[](FToolMenuSection& InSection)
			{
				InSection.AddSubMenu(
					TEXT("TimecodeSubMenu"),
					LOCTEXT("CharacterMenu_TimecodeSubMenu", "Timecode"),
					LOCTEXT("CharacterMenu_TimecodeSubMenuToolTip", "Timecode options"),
					FNewToolMenuChoice(FNewToolMenuDelegate::CreateStatic(&UE::AnimationEditor::Private::FillCharacterTimecodeMenu
					))
				);
			}
		)
	);

	Section.AddDynamicEntry(
		"AdvancedSubMenu",
		FNewToolMenuSectionDelegate::CreateLambda(
			[](FToolMenuSection& InSection)
			{
				InSection.AddSubMenu(
					TEXT("AdvancedSubMenu"),
					LOCTEXT("CharacterMenu_AdvancedSubMenu", "Advanced"),
					LOCTEXT("CharacterMenu_AdvancedSubMenuToolTip", "Advanced options"),
					FNewToolMenuChoice(FNewToolMenuDelegate::CreateStatic(&UE::AnimationEditor::Private::FillCharacterAdvancedMenu
					))
				);
			}
		)
	);
	
	Section.Sorter.BindLambda([](const FToolMenuEntry& A, const FToolMenuEntry& B, const FToolMenuContext& Context)
	{
		if (A.Name == "AdvancedSubMenu")
		{
			return false;
		}
		if (B.Name == "AdvancedSubMenu")
		{
			return true;
		}
		
		return A.Label.Get().CompareTo(B.Label.Get()) < 0;
	});
}

FToolMenuEntry UE::AnimationEditor::CreateLODSubmenu()
{
	FToolMenuEntry Entry = FToolMenuEntry::InitDynamicEntry(
		"DynamicLODOptions",
		FNewToolMenuSectionDelegate::CreateLambda(
			[](FToolMenuSection& InDynamicSection) -> void
			{
				TSharedPtr<SAnimationEditorViewportTabBody> AnimEditorViewportTab =
					Private::GetAnimationEditorViewportTab(InDynamicSection);
				if (!AnimEditorViewportTab)
				{
					return;
				}

				// Label updates based on currently selected LOD
				const TAttribute<FText> Label = TAttribute<FText>::CreateLambda(
					[WeakTab = AnimEditorViewportTab.ToWeakPtr()]()
					{
						if (TSharedPtr<SAnimationEditorViewportTabBody> ViewportTab = WeakTab.Pin())
						{
							return GetLODMenuLabel(ViewportTab);
						}

						return LOCTEXT("LODSubmenuLabel", "LOD");
					}
				);

				FToolMenuEntry& Entry = InDynamicSection.AddSubMenu(
					"LOD",
					Label,
					LOCTEXT(
						"LODMenuTooltip", "LOD Options. Control how LODs are displayed.\nShift-clicking items will 'pin' them to the toolbar."
					),
					FNewToolMenuDelegate::CreateLambda(
						[](UToolMenu* Submenu) -> void
						{
							UE::AnimationEditor::Private::PopulateLODSubmenu(Submenu);
						}
					)
				);
				Entry.ToolBarData.ResizeParams.ClippingPriority = 800;
			}
		)
	);

	return Entry;
}

FText UE::AnimationEditor::GetLODMenuLabel(const TWeakPtr<SAnimationEditorViewportTabBody>& InAnimEditorViewportTab)
{
	FText Label = LOCTEXT("LODMenu_AutoLabel", "LOD Auto");

	if (TSharedPtr<SAnimationEditorViewportTabBody> ViewportTab = InAnimEditorViewportTab.Pin())
	{
		int32 LODSelectionType = ViewportTab->GetLODSelection();

		if (ViewportTab->IsTrackingAttachedMeshLOD())
		{
			Label = FText::Format(LOCTEXT("LODMenu_DebugLabel", "LOD Debug ({0})"), FText::AsNumber(LODSelectionType - 1));
		}
		else if (LODSelectionType > 0)
		{
			FString TitleLabel = FString::Printf(TEXT("LOD %d"), LODSelectionType - 1);
			Label = FText::FromString(TitleLabel);
		}
	}
	return Label;
}

TSharedRef<SWidget> UE::AnimationEditor::MakeFloorOffsetWidget(const TWeakPtr<SAnimationEditorViewportTabBody>& InAnimEditorViewportTabWeak
)
{
	constexpr float FOVMin = -100.0f;
	constexpr float FOVMax = 100.0f;

	return
		// clang-format off
		SNew(SBox)
		.HAlign(HAlign_Right)
		[
			SNew(SBox)
			.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
			.WidthOverride(100.0f)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("Menu.WidgetBorder"))
				.Padding(FMargin(1.0f))
				[
					SNew(SSpinBox<float>)
					.Style(&FAppStyle::Get(), "Menu.SpinBox")
					.Font(FAppStyle::GetFontStyle(TEXT("MenuItem.Font")))
					.MinValue(FOVMin)
					.MaxValue(FOVMax)
					.Value_Lambda(
						[InAnimEditorViewportTabWeak]()
						{
							if (TSharedPtr<SAnimationEditorViewportTabBody> ViewportClient = InAnimEditorViewportTabWeak.Pin())
							{
								if (ViewportClient->GetViewportWidget() && ViewportClient->GetViewportWidget()->GetViewportClient())
								{
									FAnimationViewportClient& AnimViewportClient = (FAnimationViewportClient&)ViewportClient->GetLevelViewportClient();
									return AnimViewportClient.GetFloorOffset();
								}
							}
							return 0.0f;
						})
					.OnBeginSliderMovement_Lambda(
						[InAnimEditorViewportTabWeak]()
						{
							if (TSharedPtr<SAnimationEditorViewportTabBody> ViewportClient = InAnimEditorViewportTabWeak.Pin())
							{
								ViewportClient->OnBeginSliderMovementFloorOffset();
							}
						})
					.OnValueChanged_Lambda(
						[InAnimEditorViewportTabWeak](float InNewValue)
						{
							if (TSharedPtr<SAnimationEditorViewportTabBody> ViewportClient = InAnimEditorViewportTabWeak.Pin())
							{
								ViewportClient->OnFloorOffsetChanged(InNewValue);
							}
						})
					.OnValueCommitted_Lambda(
						[InAnimEditorViewportTabWeak](float InNewValue, ETextCommit::Type InCommitType)
						{
							if (TSharedPtr<SAnimationEditorViewportTabBody> ViewportClient = InAnimEditorViewportTabWeak.Pin())
							{
								ViewportClient->OnFloorOffsetCommitted(InNewValue, InCommitType);
							}
						})
					.ToolTipText(LOCTEXT("FloorOffsetToolTip", "Height offset for the floor mesh (stored per-mesh)"))
				]
			]
		];
	// clang-format on
}

void UE::AnimationEditor::ExtendCameraMenu(FName InCameraOptionsMenuName)
{
	UToolMenu* const Menu = UToolMenus::Get()->ExtendMenu(InCameraOptionsMenuName);
	if (!Menu)
	{
		return;
	}

	Menu->AddDynamicSection(
		"AnimEditorCameraExtensionDynamicSection",
		FNewToolMenuDelegate::CreateLambda(
			[](UToolMenu* InDynamicMenu)
			{
				UUnrealEdViewportToolbarContext* const EditorViewportContext =
					InDynamicMenu->FindContext<UUnrealEdViewportToolbarContext>();
				if (!EditorViewportContext)
				{
					return;
				}

				const TSharedPtr<SEditorViewport> EditorViewport = EditorViewportContext->Viewport.Pin();
				if (!EditorViewport)
				{
					return;
				}

				FToolMenuSection& MovementSection = InDynamicMenu->FindOrAddSection("Movement");

				MovementSection.AddSeparator("PositioningSeparator_1");

				MovementSection.AddSubMenu(
					"FollowMode",
					LOCTEXT("CameraFollowModeLabel", "Follow Mode"),
					LOCTEXT("CameraFollowModeTooltip", "Set various camera follow modes"),
					FNewToolMenuDelegate::CreateLambda(
						[](UToolMenu* InMenu)
						{
							UE::AnimationEditor::FillFollowModeSubmenu(InMenu);
						}
					),
					false,
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "AnimViewportMenu.CameraFollow")
				);

				MovementSection.AddMenuEntry(FAnimViewportMenuCommands::Get().TogglePauseAnimationOnCameraMove);

				FToolMenuSection& DefaultCameraSection =
					InDynamicMenu->FindOrAddSection("DefaultCamera", LOCTEXT("DefaultCameraLabel", "Default Camera"));

				DefaultCameraSection.AddMenuEntry(FAnimViewportMenuCommands::Get().JumpToDefaultCamera);
				DefaultCameraSection.AddMenuEntry(FAnimViewportMenuCommands::Get().SaveCameraAsDefault);
				DefaultCameraSection.AddMenuEntry(FAnimViewportMenuCommands::Get().ClearDefaultCamera);
			}
		)
	);
}

void UE::AnimationEditor::FillFollowModeSubmenu(UToolMenu* InMenu)
{
	if (!InMenu)
	{
		return;
	}

	FToolMenuSection& CameraFollowModeSection = InMenu->FindOrAddSection(
		"AnimViewportCameraFollowMode", LOCTEXT("ViewMenu_CameraFollowModeLabel", "Camera Follow Mode")
	);

	CameraFollowModeSection.AddMenuEntry(FAnimViewportMenuCommands::Get().CameraFollowNone);
	CameraFollowModeSection.AddMenuEntry(FAnimViewportMenuCommands::Get().CameraFollowRoot);
	CameraFollowModeSection.AddMenuEntry(FAnimViewportMenuCommands::Get().CameraFollowBounds);

	CameraFollowModeSection.AddSubMenu(
		"CameraFollowBone",
		LOCTEXT("CameraFollowBone_DisplayName", "Orbit Bone"),
		LOCTEXT("CameraFollowBone_ToolTip", "Select a bone for the camera to follow and orbit around"),
		FNewToolMenuDelegate::CreateLambda(
			[](UToolMenu* InSubmenu)
			{
				InSubmenu->AddDynamicSection(
					"CameraFollowModeBoneSubmenu",
					FNewToolMenuDelegate::CreateLambda(
						[](UToolMenu* InSectionMenu)
						{
							TSharedPtr<SAnimationEditorViewportTabBody> ViewportTab =
								Private::GetAnimationEditorViewportTab(InSectionMenu);
							if (!ViewportTab)
							{
								return;
							}

							FToolMenuSection& FollowModeBoneSection = InSectionMenu->FindOrAddSection(
								"CameraFollowModeBoneSection",
								LOCTEXT("CameraFollowModeBoneSection_Label", "Follow Bone Options")
							);
							FollowModeBoneSection.AddEntry(FToolMenuEntry::InitWidget(
								"FollowBoneWidget", UE::AnimationEditor::MakeFollowBoneWidget(ViewportTab, nullptr), FText()
							));

							FollowModeBoneSection.AddMenuEntry(
								"LockRotation",
								LOCTEXT("LockRotation_DisplayName", "Lock Rotation"),
								LOCTEXT("LockRotation_ToolTip", "Keep viewport camera rotation aligned to the orbited bone."),
								FSlateIcon(),
								FUIAction(
									FExecuteAction::CreateLambda(
										[ViewportWeak = ViewportTab.ToWeakPtr()]()
										{
											if (TSharedPtr<SAnimationEditorViewportTabBody> Viewport = ViewportWeak.Pin())
											{
												Viewport->ToggleRotateCameraToFollowBone();
											}
										}
									),
									FCanExecuteAction(),
									FIsActionChecked::CreateLambda(
										[ViewportWeak = ViewportTab.ToWeakPtr()]()
										{
											if (TSharedPtr<SAnimationEditorViewportTabBody> Viewport = ViewportWeak.Pin())
											{
												return Viewport->GetShouldRotateCameraToFollowBone();
											}

											return false;
										}
									)
								),
								EUserInterfaceActionType::ToggleButton
							);
						}
					)
				);
			}
		)
	);

	CameraFollowModeSection.AddSubMenu(
		"BoneAsCamera",
		LOCTEXT("BoneAsCamera_DisplayName", "Bone As Camera"),
		LOCTEXT("BoneAsCamera_ToolTip", "Select a bone for the camera to look out of"),
		FNewToolMenuDelegate::CreateLambda(
			[](UToolMenu* InSubmenu)
			{
				InSubmenu->AddDynamicSection(
					"BoneAsCameraSubmenu",
					FNewToolMenuDelegate::CreateLambda(
						[](UToolMenu* InSectionMenu)
						{
							TSharedPtr<SAnimationEditorViewportTabBody> ViewportTab =
								Private::GetAnimationEditorViewportTab(InSectionMenu);

							if (!ViewportTab)
							{
								return;
							}

							FToolMenuSection& BoneAsCameraModeSelection = InSectionMenu->FindOrAddSection(
								"BoneAsCameraSection",
								LOCTEXT("BoneAsCameraSection_Label", "Bone Selector")
							);

							BoneAsCameraModeSelection.AddEntry(FToolMenuEntry::InitWidget(
								"BoneAsCameraWidget", UE::AnimationEditor::Private::MakeBoneAsCameraWidget(ViewportTab, nullptr), FText()
							));

							FToolMenuSection& BoneAsCameraForwardAxisSection = InSectionMenu->FindOrAddSection(
								"BoneAsCameraForwardAxisSection",
								LOCTEXT("BoneAsCameraForwardSection_Label", "Forward Axis Selector")
							);

							UE::AnimationEditor::Private::SetupBoneAsCameraForward(BoneAsCameraForwardAxisSection, ViewportTab);

							FToolMenuSection& BoneAsCameraRightAxisSection = InSectionMenu->FindOrAddSection(
								"BoneAsCameraRightAxisSection",
								LOCTEXT("BoneAsCameraRightSection_Label", "Right Axis Selector")
							);

							UE::AnimationEditor::Private::SetupBoneAsCameraRight(BoneAsCameraRightAxisSection, ViewportTab);

							FToolMenuSection& BoneAsCameraUpAxisSection = InSectionMenu->FindOrAddSection(
								"BoneAsCameraAxisUpSection",
								LOCTEXT("BoneAsCameraUpSection_Label", "Up Axis Selector")
							);

							UE::AnimationEditor::Private::SetupBoneAsCameraUp(BoneAsCameraUpAxisSection, ViewportTab);

							FToolMenuSection& BoneAsCameraPositionOffsetSection = InSectionMenu->FindOrAddSection(
								"BoneAsCameraPositionOffset",
								LOCTEXT("BoneAsCameraPositoinOffsetSection_Label", "Position Offset Selector")
							);

							UE::AnimationEditor::Private::SetupBoneAsCameraPositionOffset(BoneAsCameraPositionOffsetSection, ViewportTab);

							FToolMenuSection& BoneAsCameraRotationOffsetSection = InSectionMenu->FindOrAddSection(
								"BoneAsCameraRotationOffset",
								LOCTEXT("BoneAsCameraRotationOffsetSection_Label", "Rotation Offset Selector")
							);

							UE::AnimationEditor::Private::SetupBoneAsCameraRotationOffset(BoneAsCameraRotationOffsetSection, ViewportTab);
						}
					)
				);
			}
		),
		false //ShouldCloseWindowAfterSelection
	);
}

void UE::AnimationEditor::ExtendViewModesSubmenu(FName InViewModesSubmenuName)
{
	UToolMenu* const Submenu = UToolMenus::Get()->ExtendMenu(InViewModesSubmenuName);

	Submenu->AddDynamicSection(
		"LevelEditorViewModesExtensionDynamicSection",
		FNewToolMenuDelegate::CreateLambda(
			[](UToolMenu* InDynamicMenu)
			{
				TSharedPtr<SAnimationEditorViewportTabBody> ViewportTab =
					Private::GetAnimationEditorViewportTab(InDynamicMenu);

				if (!ViewportTab)
				{
					return;
				}

				FToolMenuSection& Section = InDynamicMenu->FindOrAddSection("ViewMode");
				Section.AddSubMenu(
					"VisualizeBufferViewMode",
					LOCTEXT("VisualizeBufferViewModeDisplayName", "Buffer Visualization"),
					LOCTEXT("BufferVisualizationMenu_ToolTip", "Select a mode for buffer visualization"),
					FNewMenuDelegate::CreateStatic(&FBufferVisualizationMenuCommands::BuildVisualisationSubMenu),
					FUIAction(
						FExecuteAction(),
						FCanExecuteAction(),
						FIsActionChecked::CreateLambda(
							[WeakTab = ViewportTab.ToWeakPtr()]()
							{
								if (const TSharedPtr<SAnimationEditorViewportTabBody> Viewport = WeakTab.Pin())
								{
									const FEditorViewportClient& ViewportClient = Viewport->GetViewportClient();
									return ViewportClient.IsViewModeEnabled(VMI_VisualizeBuffer);
								}

								return false;
							}
						)
					),
					EUserInterfaceActionType::RadioButton,
					/* bInOpenSubMenuOnClick = */ false,
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "EditorViewport.VisualizeBufferMode")
				);
				Section.AddSubMenu(
					"VisualizeNanite",
					LOCTEXT("VisualizeNaniteDisplayName", "Nanite Visualization"),
					LOCTEXT("NaniteVisualizationMenu_ToolTip", "Select a mode for Nanite visualization"),
					FNewMenuDelegate::CreateStatic(&FNaniteVisualizationMenuCommands::BuildVisualisationSubMenu),
					FUIAction(
						FExecuteAction(),
						FCanExecuteAction(),
						FIsActionChecked::CreateLambda(
							[WeakTab = ViewportTab.ToWeakPtr()]()
							{
								if (const TSharedPtr<SAnimationEditorViewportTabBody> Viewport = WeakTab.Pin())
								{
									FEditorViewportClient& ViewportClient = Viewport->GetViewportClient();
									return ViewportClient.IsViewModeEnabled(VMI_VisualizeNanite);
								}

								return false;
							}
						)
					),
					EUserInterfaceActionType::RadioButton,
					/* bInOpenSubMenuOnClick = */ false,
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "EditorViewport.VisualizeNaniteMode")
				);
			}
		)
	);
}

TSharedRef<FExtender> UE::AnimationEditor::GetViewModesLegacyExtenders(const TWeakPtr<SAnimationEditorViewportTabBody>& InViewport)
{
	TSharedRef<FExtender> Extender(new FExtender());

	TSharedPtr<SAnimationEditorViewportTabBody> ViewportTabPinned = InViewport.Pin();
	if (!ViewportTabPinned)
	{
		return Extender;
	}

	TSharedPtr<SEditorViewport> EditorViewport = ViewportTabPinned->GetViewportWidget();
	if (!EditorViewport)
	{
		return Extender;
	}

	Extender->AddMenuExtension(
		TEXT("ViewMode"),
		EExtensionHook::After,
		EditorViewport->GetCommandList(),
		FMenuExtensionDelegate::CreateLambda(
			[InViewportWeak = InViewport](FMenuBuilder& InMenuBuilder)
			{
				InMenuBuilder.AddSubMenu(
					LOCTEXT("VisualizeBufferViewModeDisplayName", "Buffer Visualization"),
					LOCTEXT("BufferVisualizationMenu_ToolTip", "Select a mode for buffer visualization"),
					FNewMenuDelegate::CreateStatic(&FBufferVisualizationMenuCommands::BuildVisualisationSubMenu),
					FUIAction(
						FExecuteAction(),
						FCanExecuteAction(),
						FIsActionChecked::CreateLambda(
							[InViewportWeak]()
							{
								if (const TSharedPtr<SAnimationEditorViewportTabBody> ViewportPtr = InViewportWeak.Pin())
								{
									const FEditorViewportClient& ViewportClient = ViewportPtr->GetViewportClient();
									return ViewportClient.IsViewModeEnabled(VMI_VisualizeBuffer);
								}
								return false;
							}
						)
					),
					"VisualizeBufferViewMode",
					EUserInterfaceActionType::RadioButton,
					/* bInOpenSubMenuOnClick = */ false,
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "EditorViewport.VisualizeBufferMode")
				);
			}
		)
	);

	return Extender;
}

void UE::AnimationEditor::AddPhysicsMenu(FName InPhysicsSubmenuName, FToolMenuInsert InInsertPosition)
{
	UToolMenu* const Submenu = UToolMenus::Get()->ExtendMenu(InPhysicsSubmenuName);
	if (!Submenu)
	{
		return;
	}

	if (FToolMenuSection* RightSection = Submenu->FindSection("Right"))
	{
		RightSection->AddDynamicEntry(
			"PhysicsDynamic",
			FNewToolMenuSectionDelegate::CreateLambda(
				[InInsertPosition](FToolMenuSection& InSection)
				{
					TSharedPtr<SAnimationEditorViewportTabBody> ViewportTab =
						Private::GetAnimationEditorViewportTab(InSection);
					if (!ViewportTab)
					{
						return;
					}

					if (TSharedPtr<SAnimationEditorViewport> ViewportWidget =
							StaticCastSharedPtr<SAnimationEditorViewport>(ViewportTab->GetViewportWidget()))
					{
						// Only show physics sub menu when needed
						if (ViewportWidget->IsPhysicsEditor())
						{
							FToolMenuEntry Entry = FToolMenuEntry::InitSubMenu(
								"Physics",
								LOCTEXT("PhysicsLabel", "Physics"),
								LOCTEXT("PhysicsTooltip", "Physics Options. Use this to control the physics of the scene."),
								FNewToolMenuDelegate::CreateLambda(
									[](UToolMenu* Submenu) -> void
									{
										UE::AnimationEditor::FillPhysicsSubmenu(Submenu);
									}
								)
							);

							Entry.Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Persona.AssetClass.Physics");
							Entry.ToolBarData.LabelOverride = FText();
							Entry.InsertPosition = InInsertPosition;

							InSection.AddEntry(Entry);
						}
					}
				}
			)
		);
	}
}

void UE::AnimationEditor::FillPhysicsSubmenu(UToolMenu* InMenu)
{
	if (!InMenu)
	{
		return;
	}

	TSharedPtr<SAnimationEditorViewportTabBody> ViewportTab = Private::GetAnimationEditorViewportTab(InMenu);
	TSharedRef<SWidget> PhysicsMenuWidget = GeneratePhysicsMenuWidget(ViewportTab, InMenu->Context.GetAllExtenders());

	InMenu->AddMenuEntry(NAME_None, FToolMenuEntry::InitWidget("Physics", PhysicsMenuWidget, FText()));
}

TSharedRef<SWidget> UE::AnimationEditor::GeneratePhysicsMenuWidget(
	const TWeakPtr<SAnimationEditorViewportTabBody>& InAnimEditorViewportTab, const TSharedPtr<FExtender>& MenuExtender
)
{
	if (TSharedPtr<SAnimationEditorViewportTabBody> ViewportTabPinned = InAnimEditorViewportTab.Pin())
	{
		static const FName MenuName("Persona.AnimViewportPhysicsMenu");
		if (!UToolMenus::Get()->IsMenuRegistered(MenuName))
		{
			UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(MenuName);
			Menu->AddSection("AnimViewportPhysicsMenu", LOCTEXT("ViewMenu_AnimViewportPhysicsMenu", "Physics Menu"));
		}

		FToolMenuContext MenuContext(ViewportTabPinned->GetCommandList(), MenuExtender);
		ViewportTabPinned->GetAssetEditorToolkit()->InitToolMenuContext(MenuContext);
		return UToolMenus::Get()->GenerateWidget(MenuName, MenuContext);
	}

	return SNullWidget::NullWidget;
}

void UE::AnimationEditor::ExtendPreviewSceneSettingsSubmenu(FName InSubmenuName)
{
	UToolMenu* const Submenu = UToolMenus::Get()->ExtendMenu(InSubmenuName);
	if (!Submenu)
	{
		return;
	}

	Submenu->AddDynamicSection(
		"StaticMeshEditorPreviewSceneDynamicSection",
		FNewToolMenuDelegate::CreateLambda(
			[](UToolMenu* InDynamicMenu)
			{
				TSharedPtr<SAnimationEditorViewportTabBody> AnimEditorViewportTab =
					Private::GetAnimationEditorViewportTab(InDynamicMenu);

				if (!AnimEditorViewportTab)
				{
					return;
				}

				// Scene Setup
				{
					FToolMenuSection& SceneSetupSection = InDynamicMenu->FindOrAddSection("PreviewSceneSettings");
					FToolMenuEntry FloorOffsetEntry = FToolMenuEntry::InitWidget(
						"FloorOffset",
						MakeFloorOffsetWidget(AnimEditorViewportTab),
						LOCTEXT("FloorHeightOffset", "Floor Height Offset")
					);
					SceneSetupSection.AddEntry(FloorOffsetEntry);
					SceneSetupSection.AddMenuEntry(FAnimViewportShowCommands::Get().AutoAlignFloorToMesh);
					SceneSetupSection.AddEntry(UE::AnimationEditor::CreateTurnTableMenu());
				}
			}
		)
	);
}

FText UE::AnimationEditor::GetPlaybackMenuLabel(const TWeakPtr<IPersonaPreviewScene>& InPersonaPreviewScene)
{
	FText Label = LOCTEXT("PlaybackError", "Error");
	if (const TSharedPtr<IPersonaPreviewScene>& AnimationEditorPreviewScene = InPersonaPreviewScene.Pin())
	{
		for (int i = 0; i < EAnimationPlaybackSpeeds::NumPlaybackSpeeds; ++i)
		{
			if (AnimationEditorPreviewScene->IsPlaybackSpeedSelected(i))
			{
				const int32 NumFractionalDigits =
					(i == EAnimationPlaybackSpeeds::Quarter || i == EAnimationPlaybackSpeeds::ThreeQuarters) ? 2 : 1;

				const FNumberFormattingOptions FormatOptions = FNumberFormattingOptions()
																   .SetMinimumFractionalDigits(NumFractionalDigits)
																   .SetMaximumFractionalDigits(NumFractionalDigits);

				float CurrentValue = i == EAnimationPlaybackSpeeds::Type::Custom
									   ? AnimationEditorPreviewScene->GetCustomAnimationSpeed()
									   : EAnimationPlaybackSpeeds::Values[i];
				Label = FText::Format(
					LOCTEXT("AnimViewportPlaybackMenuLabel", "x{0}"), FText::AsNumber(CurrentValue, &FormatOptions)
				);
			}
		}
	}
	return Label;
}

FToolMenuEntry UE::AnimationEditor::CreatePlaybackSubmenu()
{
	FToolMenuEntry Entry = FToolMenuEntry::InitDynamicEntry(
		"PlaybackMenu",
		FNewToolMenuSectionDelegate::CreateLambda(
			[](FToolMenuSection& InDynamicSection) -> void
			{
				if (UAnimViewportContext* const AnimViewportContext = InDynamicSection.FindContext<UAnimViewportContext>())
				{
					// Label updates based on currently selected LOD
					const TAttribute<FText> Label = TAttribute<FText>::CreateLambda(
						[PreviewSceneWeak = AnimViewportContext->PersonaPreviewScene]()
						{
							return GetPlaybackMenuLabel(PreviewSceneWeak);
						}
					);

					FToolMenuEntry& Entry = InDynamicSection.AddSubMenu(
						"Playback",
						Label,
						LOCTEXT("PlaybackMenuTooltip", "Playback Speed Options. Control the time dilation of the scene's update."),
						FNewToolMenuDelegate::CreateLambda(
							[](UToolMenu* Submenu) -> void
							{
								UE::AnimationEditor::Private::FillPlaybackMenu(Submenu);
							}
						)
					);

					Entry.ToolBarData.ResizeParams.AllowClipping = false;
				}
			}
		)
	);

	return Entry;
}

TSharedRef<SWidget> UE::AnimationEditor::GeneratePlaybackMenu(
	const TWeakPtr<FAnimationEditorPreviewScene>& InAnimationEditorPreviewScene, const TArray<TSharedPtr<FExtender>>& InExtenders
)
{
	// We generate a menu via UToolMenus, so we can use FillShowSubmenu call from both old and new toolbar
	FName OldPlaybackMenuName = "AnimationEditor.OldViewportToolbar.PlaybackMenu";
	if (!UToolMenus::Get()->IsMenuRegistered(OldPlaybackMenuName))
	{
		if (UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(OldPlaybackMenuName, NAME_None, EMultiBoxType::Menu, false))
		{
			Menu->AddDynamicSection(
				"BaseSection",
				FNewToolMenuDelegate::CreateLambda(
					[](UToolMenu* InMenu)
					{
						UE::AnimationEditor::Private::FillPlaybackMenu(InMenu);
					}
				)
			);
		}
	}

	FToolMenuContext MenuContext;
	{
		TSharedPtr<FExtender> MenuExtender = FExtender::Combine(InExtenders);
		MenuContext.AddExtender(MenuExtender);
		UAnimViewportContext* const ContextObject = NewObject<UAnimViewportContext>();
		MenuContext.AddObject(ContextObject);

		if (TSharedPtr<FAnimationEditorPreviewScene> AnimationPreviewScenePinned = InAnimationEditorPreviewScene.Pin())
		{
			MenuContext.AppendCommandList(AnimationPreviewScenePinned->GetCommandList());
			ContextObject->PersonaPreviewScene = AnimationPreviewScenePinned;
		}
	}

	return UToolMenus::Get()->GenerateWidget(OldPlaybackMenuName, MenuContext);
}

TSharedRef<SWidget> UE::AnimationEditor::CreateFollowModeMenuWidget(const TWeakPtr<SAnimationEditorViewportTabBody>& InAnimEditorViewportTab
)
{
	TSharedPtr<SAnimationEditorViewportTabBody> AnimEditorViewportTab = InAnimEditorViewportTab.Pin();
	if (!AnimEditorViewportTab)
	{
		return SNullWidget::NullWidget;
	}
	// We generate a menu via UToolMenus, so we can use FillShowSubmenu call from both old and new toolbar
	FName OldFollowModeMenuName = "AnimationEditor.OldViewportToolbar.FollowMode";
	if (!UToolMenus::Get()->IsMenuRegistered(OldFollowModeMenuName))
	{
		if (UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(OldFollowModeMenuName, NAME_None, EMultiBoxType::Menu, false))
		{
			Menu->AddDynamicSection(
				"BaseSection",
				FNewToolMenuDelegate::CreateLambda(
					[](UToolMenu* InMenu)
					{
						UE::AnimationEditor::FillFollowModeSubmenu(InMenu);
					}
				)
			);
		}
	}
	FToolMenuContext MenuContext;
	{
		MenuContext.AppendCommandList(AnimEditorViewportTab->GetCommandList());
		UAnimViewportContext* const ContextObject = NewObject<UAnimViewportContext>();
		ContextObject->ViewportTabBody = AnimEditorViewportTab;
		MenuContext.AddObject(ContextObject);
	}
	return UToolMenus::Get()->GenerateWidget(OldFollowModeMenuName, MenuContext);
}

TSharedRef<SWidget> UE::AnimationEditor::GenerateTurnTableMenu(const TWeakPtr<SAnimationEditorViewportTabBody>& InAnimEditorViewportTab
)
{
	TSharedPtr<SAnimationEditorViewportTabBody> AnimEditorViewportTab = InAnimEditorViewportTab.Pin();
	if (!AnimEditorViewportTab)
	{
		return SNullWidget::NullWidget;
	}
	// We generate a menu via UToolMenus, so we can use FillShowSubmenu call from both old and new toolbar
	FName OldShowMenuName = "AnimationEditor.OldViewportToolbar.TurnTable";
	if (!UToolMenus::Get()->IsMenuRegistered(OldShowMenuName))
	{
		if (UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(OldShowMenuName, NAME_None, EMultiBoxType::Menu, false))
		{
			Menu->AddDynamicSection(
				"BaseSection",
				FNewToolMenuDelegate::CreateLambda(
					[](UToolMenu* InMenu)
					{
						UE::AnimationEditor::FillTurnTableSubmenu(InMenu);
					}
				)
			);
		}
	}
	FToolMenuContext MenuContext;
	{
		MenuContext.AppendCommandList(AnimEditorViewportTab->GetCommandList());
		UAnimViewportContext* const ContextObject = NewObject<UAnimViewportContext>();
		ContextObject->ViewportTabBody = AnimEditorViewportTab;
		MenuContext.AddObject(ContextObject);
	}
	return UToolMenus::Get()->GenerateWidget(OldShowMenuName, MenuContext);
}

TSharedRef<SWidget> UE::AnimationEditor::GenerateLODMenuWidget(const TWeakPtr<SAnimationEditorViewportTabBody>& InAnimEditorViewportTab
)
{
	TSharedPtr<SAnimationEditorViewportTabBody> AnimEditorViewportTab = InAnimEditorViewportTab.Pin();
	if (!AnimEditorViewportTab)
	{
		return SNullWidget::NullWidget;
	}
	// We generate a menu via UToolMenus, so we can use FillShowSubmenu call from both old and new toolbar
	FName OldShowMenuName = "AnimationEditor.OldViewportToolbar.LODMenu";
	if (!UToolMenus::Get()->IsMenuRegistered(OldShowMenuName))
	{
		if (UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(OldShowMenuName, NAME_None, EMultiBoxType::Menu, false))
		{
			Menu->AddDynamicSection(
				"BaseSection",
				FNewToolMenuDelegate::CreateLambda(
					[](UToolMenu* InMenu)
					{
						UE::AnimationEditor::Private::PopulateLODSubmenu(InMenu);
					}
				)
			);
		}
	}
	FToolMenuContext MenuContext;
	{
		MenuContext.AppendCommandList(AnimEditorViewportTab->GetCommandList());
		UAnimViewportContext* const ContextObject = NewObject<UAnimViewportContext>();
		ContextObject->ViewportTabBody = AnimEditorViewportTab;
		MenuContext.AddObject(ContextObject);
	}
	return UToolMenus::Get()->GenerateWidget(OldShowMenuName, MenuContext);
}

FToolMenuEntry UE::AnimationEditor::CreateSkinWeightProfileMenu()
{
	FToolMenuEntry Entry = FToolMenuEntry::InitDynamicEntry(
		"DynamicSkinWeightProfileOptions",
		FNewToolMenuSectionDelegate::CreateLambda(
			[](FToolMenuSection& InDynamicSection) -> void
			{
				TSharedPtr<SAnimationEditorViewportTabBody> AnimEditorViewportTab =
					Private::GetAnimationEditorViewportTab(InDynamicSection);
				if (!AnimEditorViewportTab)
				{
					return;
				}

				// Label updates based on currently selected skin weight profile
				const TAttribute<FText> Label = TAttribute<FText>::CreateLambda(
					[WeakTab = AnimEditorViewportTab.ToWeakPtr()]()
					{
						if (TSharedPtr<SAnimationEditorViewportTabBody> ViewportTab = WeakTab.Pin())
						{
							return GetSkinWeightProfileMenuLabel(ViewportTab);
						}

						return LOCTEXT("SkinWeightProfileMenu_NoProfile", "No Profile");
					}
				);

				FToolMenuEntry& Entry = InDynamicSection.AddSubMenu(
					"SkinWeightProfile",
					Label,
					LOCTEXT(
						"SkinWeightProfileMenuTooltip", "Control how skin weight profiles are displayed."
					),
					FNewToolMenuDelegate::CreateLambda(
						[](UToolMenu* Submenu) -> void
						{
							UE::AnimationEditor::Private::PopulateSkinWeightProfileMenu(Submenu);
						}
					),
					false,
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "AnimViewportMenu.SkinWeightProfiles")
				);


				Entry.Visibility = [WeakTab = AnimEditorViewportTab.ToWeakPtr()]() -> bool
					{
						if (TSharedPtr<SAnimationEditorViewportTabBody> ViewportTab = WeakTab.Pin())
						{
							if (const USkeletalMesh* SkeletalMesh = ViewportTab->GetPreviewScene()->GetPreviewMesh())
							{
								if (!SkeletalMesh->IsCompiling())
								{
									return SkeletalMesh->GetNumSkinWeightProfiles() > 0;
								}
							}
						}
						return false;
					};

				// Profiles are a lower priority than other menus. 
				Entry.ToolBarData.ResizeParams.ClippingPriority = 200;
			}
		)
	);

	return Entry;
}

FText UE::AnimationEditor::GetSkinWeightProfileMenuLabel(const TWeakPtr<SAnimationEditorViewportTabBody>& InAnimEditorViewportTab)
{
	FText NoProfileLabel = LOCTEXT("SkinWeightProfileMenu_NoProfile", "No Profile");

	if (TSharedPtr<SAnimationEditorViewportTabBody> ViewportTab = InAnimEditorViewportTab.Pin())
	{
		TArray<FName> ProfileNames = ViewportTab->GetSelectedProfileNames();

		ProfileNames.RemoveAll([](FName InName) { return InName.IsNone(); });

		if (ProfileNames.IsEmpty())
		{
			return NoProfileLabel;
		}

		TArray<FText> ProfileTextNames;
		for (FName Name: ProfileNames)
		{
			ProfileTextNames.Add(FText::FromName(Name));
		}

		return FText::Join(LOCTEXT("SkinWeightProfileMenu_LabelJoin", " / "), ProfileTextNames);
	}
	
	return NoProfileLabel;
}


TSharedRef<SWidget> UE::AnimationEditor::CreateShowMenuWidget(
	const TSharedRef<SEditorViewport>& InViewport, const TArray<TSharedPtr<FExtender>>& InExtenders, bool bInShowViewportStatsToggle
)
{
	static const FName MenuName("Persona.AnimViewportToolBar");
	if (!UToolMenus::Get()->IsMenuRegistered(MenuName))
	{
		UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(MenuName);
		FillShowSubmenu(Menu, bInShowViewportStatsToggle);
	}
	FToolMenuContext MenuContext;
	{
		MenuContext.AppendCommandList(InViewport->GetCommandList());
		TSharedPtr<FExtender> MenuExtender = FExtender::Combine(InExtenders);
		MenuContext.AddExtender(MenuExtender);
	}
	return UToolMenus::Get()->GenerateWidget(MenuName, MenuContext);
}

#undef LOCTEXT_NAMESPACE
