// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPersonaPreviewScene.h"
#include "Widgets/SCompoundWidget.h"
#include "ToolMenuMisc.h"

class FAnimationEditorPreviewScene;
class FExtender;
class IPreviewProfileController;
class SAnimationEditorViewportTabBody;
class SComboButton;
class SEditorViewport;
class UToolMenu;
struct FToolMenuEntry;

namespace UE::AnimationEditor
{
void ExtendCameraMenu(FName InCameraOptionsMenuName);

void FillFollowModeSubmenu(UToolMenu* InMenu);
TSharedRef<SWidget> CreateFollowModeMenuWidget(const TWeakPtr<SAnimationEditorViewportTabBody>& InAnimEditorViewportTab);

TSharedRef<SWidget> MakeFollowBoneWidget(
	const TWeakPtr<SAnimationEditorViewportTabBody>& InViewport, const TWeakPtr<SComboButton>& InWeakComboButton
);

FToolMenuEntry CreateShowSubmenu();
void FillShowSubmenu(UToolMenu* InMenu, bool bInShowViewportStatsToggle = true);
TSharedRef<SWidget> CreateShowMenuWidget(
	const TSharedRef<SEditorViewport>& InViewport, const TArray<TSharedPtr<FExtender>>& InExtenders, bool bInShowViewportStatsToggle
);

FToolMenuEntry CreateTurnTableMenu();
void FillTurnTableSubmenu(UToolMenu* InMenu);
TSharedRef<SWidget> GenerateTurnTableMenu(const TWeakPtr<SAnimationEditorViewportTabBody>& InAnimEditorViewportTab);

void AddSceneElementsSection(UToolMenu* InMenu);

FToolMenuEntry CreateLODSubmenu();
FText GetLODMenuLabel(const TWeakPtr<SAnimationEditorViewportTabBody>& InAnimEditorViewportTab);
TSharedRef<SWidget> GenerateLODMenuWidget(const TWeakPtr<SAnimationEditorViewportTabBody>& InAnimEditorViewportTab);


FToolMenuEntry CreateSkinWeightProfileMenu();
FText GetSkinWeightProfileMenuLabel(const TWeakPtr<SAnimationEditorViewportTabBody>& InAnimEditorViewportTab);

TSharedRef<SWidget> MakeFloorOffsetWidget(const TWeakPtr<SAnimationEditorViewportTabBody>& InAnimEditorViewportTabWeak);

void ExtendViewModesSubmenu(FName InViewModesSubmenuName);
TSharedRef<FExtender> GetViewModesLegacyExtenders(const TWeakPtr<SAnimationEditorViewportTabBody>& InViewport);

void AddPhysicsMenu(FName InPhysicsSubmenuName, FToolMenuInsert InInsertPosition);
void FillPhysicsSubmenu(UToolMenu* InMenu);
TSharedRef<SWidget> GeneratePhysicsMenuWidget(const TWeakPtr<SAnimationEditorViewportTabBody>& InAnimEditorViewportTab, const TSharedPtr<FExtender>& MenuExtender);

void ExtendPreviewSceneSettingsSubmenu(FName InSubmenuName);

FToolMenuEntry CreatePlaybackSubmenu();
TSharedRef<SWidget> GeneratePlaybackMenu(const TWeakPtr<FAnimationEditorPreviewScene>& InAnimationEditorPreviewScene, const TArray<TSharedPtr<FExtender>>& InExtenders);
FText GetPlaybackMenuLabel(const TWeakPtr<IPersonaPreviewScene>& InPersonaPreviewScene);
} // namespace UE::AnimationEditor
