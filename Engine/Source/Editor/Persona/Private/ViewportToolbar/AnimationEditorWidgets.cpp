// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationEditorWidgets.h"

#include "SAnimationEditorViewport.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Input/SSpinBox.h"

#define LOCTEXT_NAMESPACE "AnimEditorViewportToolbar"

void UE::AnimationEditor::SCustomAnimationSpeedSetting::Construct(const FArguments& InArgs)
{
	CustomSpeed = InArgs._CustomSpeed;
	OnCustomSpeedChanged = InArgs._OnCustomSpeedChanged;

	// clang-format off
	ChildSlot
	[
		SNew(SBox)
		.HAlign(HAlign_Right)
		[
			SNew(SBox)
			.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
			.WidthOverride(100.0f)
			[
				SNew(SSpinBox<float>)
				.Font(FAppStyle::GetFontStyle(TEXT("MenuItem.Font")))
				.ToolTipText(LOCTEXT("AnimationCustomSpeed", "Set Custom Speed."))
				.MinValue(0.f)
				.MaxSliderValue(10.f)
				.SupportDynamicSliderMaxValue(true)
				.Value(CustomSpeed)
				.OnValueChanged(OnCustomSpeedChanged)
			]
		]
	];
	// clang-format on
}

void UE::AnimationEditor::SBoneDrawSizeSetting::Construct(const FArguments& InArgs)
{
	AnimViewportPtr = InArgs._AnimEditorViewport;

	TSharedPtr<SAnimationEditorViewportTabBody> AnimViewport = AnimViewportPtr.Pin();
	if (!AnimViewport)
	{
		return;
	}

	// clang-format off
	ChildSlot
	[
		SNew(SBox)
		.HAlign(HAlign_Right)
		[
			SNew(SBox)
			.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
			.WidthOverride(100.0f)
			[
				SNew(SSpinBox<float>)
				.Font(FAppStyle::GetFontStyle(TEXT("MenuItem.Font")))
				.ToolTipText(LOCTEXT("BoneDrawSize_ToolTip", "Change bone size in viewport."))
				.MinValue(0.f)
				.MaxSliderValue(10.f)
				.SupportDynamicSliderMaxValue(true)
				.Value(AnimViewport.ToSharedRef(), &SAnimationEditorViewportTabBody::GetBoneDrawSize)
				.OnValueChanged(SSpinBox<float>::FOnValueChanged::CreateSP(AnimViewport.ToSharedRef(), &SAnimationEditorViewportTabBody::SetBoneDrawSize))
			]
		]
	];
	// clang-format on
}

void UE::AnimationEditor::SClothWindSettings::Construct(const FArguments& InArgs)
{
	AnimViewportPtr = InArgs._AnimEditorViewport;

	TSharedPtr<SAnimationEditorViewportTabBody> AnimViewport = AnimViewportPtr.Pin();
	if (!AnimViewport)
	{
		return;
	}

	// clang-format off
	ChildSlot
	[
		SNew(SBox)
		.HAlign(HAlign_Right)
		[
			SNew(SBox)
			.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
			.WidthOverride(100.0f)
			[
				SNew(SNumericEntryBox<float>)
				.Font(FAppStyle::GetFontStyle(TEXT("MenuItem.Font")))
				.ToolTipText(LOCTEXT("WindStrength_ToolTip", "Change wind strength"))
				.MinValue(0.f)
				.AllowSpin(true)
				.MinSliderValue(0.f)
				.MaxSliderValue(10.f)
				.Value(AnimViewport.ToSharedRef(), &SAnimationEditorViewportTabBody::GetWindStrengthSliderValue)
				.OnValueChanged(SSpinBox<float>::FOnValueChanged::CreateSP(AnimViewport.ToSharedRef(), &SAnimationEditorViewportTabBody::SetWindStrength))
			]
		]
	];
	// clang-format on
}

bool UE::AnimationEditor::SClothWindSettings::IsWindEnabled() const
{
	if (TSharedPtr<SAnimationEditorViewportTabBody> AnimViewport = AnimViewportPtr.Pin())
	{
		return AnimViewport->IsApplyingClothWind();
	}

	return false;
}

void UE::AnimationEditor::SGravitySettings::Construct(const FArguments& InArgs)
{
	AnimViewportPtr = InArgs._AnimEditorViewport;

	TSharedPtr<SAnimationEditorViewportTabBody> AnimViewport = AnimViewportPtr.Pin();
	if (!AnimViewport)
	{
		return;
	}

	// clang-format off
	ChildSlot
	[
		SNew(SBox)
		.HAlign(HAlign_Right)
		[
			SNew(SBox)
			.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
			.WidthOverride(100.0f)
			[
				SNew(SSpinBox<float>)
				.Font(FAppStyle::GetFontStyle(TEXT("MenuItem.Font")))
				.ToolTipText(LOCTEXT("GravityScale_ToolTip", "Change gravity scale"))
				.MinValue(0.f)
				.MaxValue(4.f)
				.Value(AnimViewport.ToSharedRef(), &SAnimationEditorViewportTabBody::GetGravityScaleSliderValue)
				.OnValueChanged(SSpinBox<float>::FOnValueChanged::CreateSP(AnimViewport.ToSharedRef(), &SAnimationEditorViewportTabBody::SetGravityScale))
			]
		]
	];
	// clang-format on
}

FReply UE::AnimationEditor::SGravitySettings::OnDecreaseGravityScale() const
{
	if (TSharedPtr<SAnimationEditorViewportTabBody> AnimViewport = AnimViewportPtr.Pin())
	{
		constexpr float DeltaValue = 0.025f;
		AnimViewport->SetGravityScale(AnimViewport->GetGravityScaleSliderValue() - DeltaValue);
	}

	return FReply::Handled();
}

FReply UE::AnimationEditor::SGravitySettings::OnIncreaseGravityScale() const
{
	if (TSharedPtr<SAnimationEditorViewportTabBody> AnimViewport = AnimViewportPtr.Pin())
	{
		constexpr float DeltaValue = 0.025f;
		AnimViewport->SetGravityScale(AnimViewport->GetGravityScaleSliderValue() + DeltaValue);
	}

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
