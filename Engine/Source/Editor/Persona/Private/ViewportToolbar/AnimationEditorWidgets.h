// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class SAnimationEditorViewportTabBody;

namespace UE::AnimationEditor
{
// Class definition which represents widget to modify animation speed settings
class SCustomAnimationSpeedSetting : public SCompoundWidget
{
public:
	/** Notification for numeric value change */
	DECLARE_DELEGATE_OneParam(FOnCustomSpeedChanged, float);

	SLATE_BEGIN_ARGS(SCustomAnimationSpeedSetting)
	{
	}
	SLATE_ATTRIBUTE(float, CustomSpeed)
	SLATE_EVENT(FOnCustomSpeedChanged, OnCustomSpeedChanged)
SLATE_END_ARGS()

void Construct(const FArguments& InArgs);

protected:
	TAttribute<float> CustomSpeed = 1.0f;
	FOnCustomSpeedChanged OnCustomSpeedChanged;
};

// Class definition which represents widget to modify Bone Draw Size in viewport
class SBoneDrawSizeSetting : public SCompoundWidget
{

public:
	SLATE_BEGIN_ARGS(SBoneDrawSizeSetting)
	{
	}
	SLATE_ARGUMENT(TWeakPtr<SAnimationEditorViewportTabBody>, AnimEditorViewport)
SLATE_END_ARGS()

void Construct(const FArguments& InArgs);

protected:
	TWeakPtr<SAnimationEditorViewportTabBody> AnimViewportPtr;
};


// Class definition which represents widget to modify strength of wind for clothing
class SClothWindSettings : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SClothWindSettings)
		{
		}

		SLATE_ARGUMENT(TWeakPtr<SAnimationEditorViewportTabBody>, AnimEditorViewport)
	SLATE_END_ARGS()

	/** Constructs this widget from its declaration */
	void Construct(const FArguments& InArgs);

protected:
	/** Callback function which determines whether this widget is enabled */
	bool IsWindEnabled() const;

protected:
	/** The viewport hosting this widget */
	TWeakPtr<SAnimationEditorViewportTabBody> AnimViewportPtr;
};

// Class definition which represents widget to modify gravity for preview
class SGravitySettings : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SGravitySettings)
		{
		}

		SLATE_ARGUMENT(TWeakPtr<SAnimationEditorViewportTabBody>, AnimEditorViewport)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

protected:
	FReply OnDecreaseGravityScale() const;

	FReply OnIncreaseGravityScale() const;

protected:
	TWeakPtr<SAnimationEditorViewportTabBody> AnimViewportPtr;
};

}
