// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AssetRegistry/AssetData.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SNullWidget.h"
#include "Templates/SubclassOf.h"
#include "AnimAssetFindReplace.generated.h"

#define UE_API PERSONA_API

class SAnimAssetFindReplace;
struct FToolMenuSection;
class SAutoCompleteSearchBox;

/** Context for toolbar */
UCLASS()
class UAnimAssetFindReplaceContext : public UObject
{
	GENERATED_BODY()

public:
	/** Weak ptr to the find/replace widget */
	TWeakPtr<SAnimAssetFindReplace> Widget;
};

/** The mode configuration that the dialog can take */
enum class EAnimAssetFindReplaceMode : int32
{
	Find,
	Replace,
};

/** Processor base class to allow systems to add their own find/replace functionality */
UCLASS(MinimalAPI, Abstract)
class UAnimAssetFindReplaceProcessor : public UObject
{
	GENERATED_BODY()

	friend class SAnimAssetFindReplace;

private:
	/** Initially setup the processor, binding it to the specified widget */
	UE_API void Initialize(TSharedRef<SAnimAssetFindReplace> InWidget);

public:
	/** Whether this processor supports the specified mode */
	virtual bool SupportsMode(EAnimAssetFindReplaceMode InMode) const PURE_VIRTUAL(UAnimAssetFindReplaceProcessor::SupportsMode, return false;);

	/** Builds results string to display from the supplied asset data */
	virtual FString GetFindResultStringFromAssetData(const FAssetData& InAssetData) const PURE_VIRTUAL(UAnimAssetFindReplaceProcessor::GetFindResultStringFromAssetData, return FString(););

	/** Get all the asset types that are supported by this processor */
	virtual TConstArrayView<UClass*> GetSupportedAssetTypes() const PURE_VIRTUAL(UAnimAssetFindReplaceProcessor::GetSupportedAssetTypes, return TConstArrayView<UClass*>(););

	/**
	 * Whether the asset in question is a valid search result or not
	 *
	 * @param InAssetData		The asset in question
	 * @param bOutIsOldAsset	Whether the asset is 'old', i.e. it needs to be fully loaded to determine whether it is
	 *							filtered, as relevant data is not saved in the asset registry for it
	 */
	virtual bool ShouldFilterOutAsset(const FAssetData& InAssetData, bool& bOutIsOldAsset) const PURE_VIRTUAL(UAnimAssetFindReplaceProcessor::ShouldFilterOutAsset, return false;);

	/** Perform a replace operation on the specified asset */
	virtual void ReplaceInAsset(const FAssetData& InAssetData) const PURE_VIRTUAL(UAnimAssetFindReplaceProcessor::ReplaceInAsset, );

	/** Perform a remove operation on the specified asset */
	virtual void RemoveInAsset(const FAssetData& InAssetData) const PURE_VIRTUAL(UAnimAssetFindReplaceProcessor::RemoveInAsset, );
	
	/** Optional override to extend the toolbar when this processor is selected */
	virtual void ExtendToolbar(FToolMenuSection& InSection) {}

	/** Make or return the widget used to edit find/replace parameters */
	virtual TSharedRef<SWidget> MakeFindReplaceWidget() PURE_VIRTUAL(UAnimAssetFindReplaceProcessor::MakeFindReplaceWidget, return SNullWidget::NullWidget;);

	/** Optional override allowing a widget to be focused when the find/replace widget is initially displayed */
	virtual void FocusInitialWidget() {}
	
	/** Allows control over when the replace option is available to users */
	virtual bool CanCurrentlyReplace() const PURE_VIRTUAL(UAnimAssetFindReplaceProcessor::CanCurrentlyReplace, return false;);

	/** Allows control over when the remove option is available to users */
	virtual bool CanCurrentlyRemove() const PURE_VIRTUAL(UAnimAssetFindReplaceProcessor::CanCurrentlyRemove, return false;);

	/** Called by the UI to deferred-refresh any cached data that this processor holds after asset operations have completed */
	virtual void RefreshCachedData() {}

protected:
	/** Requests the entire find UI to be refreshed */
	UE_API void RequestRefreshUI();

	/** Requests cached data and search results */
	UE_API void RequestRefreshCachedData();

	/** Requests search results to be refreshed */
	UE_API void RequestRefreshSearchResults();

private:
	/** Weak ptr to the find/replace widget */
	TWeakPtr<SAnimAssetFindReplace> Widget;
};

/** Processor for string-based animation asset operations */
UCLASS(MinimalAPI, Abstract)
class UAnimAssetFindReplaceProcessor_StringBase : public UAnimAssetFindReplaceProcessor
{
	GENERATED_BODY()

public:
	UAnimAssetFindReplaceProcessor_StringBase()
		: AutoCompleteItems(MakeShared<TArray<TSharedPtr<FString>>>())
	{}

	/** Set the current find string */
	UE_API void SetFindString(const FString& InString);

	/** Get the current find string */
	FStringView GetFindString() const { return FindString; }

	/** Get the current find string */
	const FString& GetFindStringRef() const { return FindString; }

	/** Set the current replace string */
	UE_API void SetReplaceString(const FString& InString);

	/** Get the current replace string */
	FStringView GetReplaceString() const { return ReplaceString; }

	/** Get the current replace string */
	const FString& GetReplaceStringRef() const { return ReplaceString; }
	
	/** Set the current skeleton filter */
	UE_API void SetSkeletonFilter(const FAssetData& InSkeletonFilter);

	/** Get the current skeleton filter **/
	const FAssetData& GetSkeletonFilter() const { return SkeletonFilter; }

	/** Sets whether to match whole words when searching strings */
	UE_API void SetFindWholeWord(bool bInFindWholeWord);

	/** Gets whether to match whole words when searching strings */
	bool GetFindWholeWord() const { return bFindWholeWord; }

	/** Sets case comparison to use when searching */
	UE_API void SetSearchCase(ESearchCase::Type InSearchCase);

	/** Gets case comparison to use when searching */
	ESearchCase::Type GetSearchCase() const { return SearchCase; }

	/** Optional override point for derived classes to supply auto complete names to display in the UI */
	virtual void GetAutoCompleteNames(TArrayView<FAssetData> InAssetDatas, TSet<FString>& OutUniqueNames) const {}

protected:
	/** UAnimAssetFindReplaceProcessor interface */
	virtual bool SupportsMode(EAnimAssetFindReplaceMode InMode) const override { return true; }
	UE_API virtual void ExtendToolbar(FToolMenuSection& InSection) override;
	UE_API virtual TSharedRef<SWidget> MakeFindReplaceWidget() override;
	UE_API virtual void FocusInitialWidget() override;
	UE_API virtual bool CanCurrentlyReplace() const override;
	UE_API virtual bool CanCurrentlyRemove() const override;
	UE_API virtual void RefreshCachedData() override;

protected:
	/** Check if the supplied string matches the find parameters */
	UE_API bool NameMatches(FStringView InNameStringView) const;

private:
	/** The find string */
	FString FindString;

	/** The replace string */
	FString ReplaceString;

	/** The skeleton to filter by */
	FAssetData SkeletonFilter;

	/** Whether to match whole words when searching strings */
	bool bFindWholeWord = false;

	/** Case comparison to use when searching */
	ESearchCase::Type SearchCase = ESearchCase::IgnoreCase;

	/** Cached auto-complete items */
	TSharedPtr<TArray<TSharedPtr<FString>>> AutoCompleteItems;

	/** Cached find/replace widget */
	TWeakPtr<SWidget> FindReplaceWidget;

	/** Search box widget for 'find' */
	TWeakPtr<SAutoCompleteSearchBox> FindSearchBox;

	/** Search box widget for 'replace' */
	TWeakPtr<SAutoCompleteSearchBox> ReplaceSearchBox;
};

/** Configuration for the find/replace tab */
struct FAnimAssetFindReplaceConfig
{
	/** The initial processor class to use */
	TSubclassOf<UAnimAssetFindReplaceProcessor> InitialProcessorClass = nullptr;
};

/** Public interface to the find/replace widget */
class IAnimAssetFindReplace : public SCompoundWidget
{
public:
	/** Sets the type of the current processor we are using to find/replace with */
	virtual void SetCurrentProcessor(TSubclassOf<UAnimAssetFindReplaceProcessor> InProcessorClass) = 0;

	/** Gets the current processor instance */
	virtual UAnimAssetFindReplaceProcessor* GetCurrentProcessor() const = 0;
	
	/** Gets the processor instance of the specified type */
	virtual UAnimAssetFindReplaceProcessor* GetProcessor(TSubclassOf<UAnimAssetFindReplaceProcessor> InProcessorClass) const = 0;

	/** Gets the processor instance of the specified type */
	template <typename ProcessorType>
	ProcessorType* GetProcessor() const
	{
		return Cast<ProcessorType>(GetProcessor(ProcessorType::StaticClass()));
	}
};

#undef UE_API
