// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnimAssetFindReplace.h"
#include "AnimAssetFindReplaceCurves.generated.h"

/** Find, replace and remove curves across assets */
UCLASS(MinimalAPI, DisplayName="Curves")
class UAnimAssetFindReplaceCurves : public UAnimAssetFindReplaceProcessor_StringBase
{
	GENERATED_BODY()

	// UAnimAssetFindReplaceProcessor interface
	virtual FString GetFindResultStringFromAssetData(const FAssetData& InAssetData) const override;
	virtual TConstArrayView<UClass*> GetSupportedAssetTypes() const override;
	virtual bool ShouldFilterOutAsset(const FAssetData& InAssetData, bool& bOutIsOldAsset) const override;
	virtual void ReplaceInAsset(const FAssetData& InAssetData) const override;
	virtual void RemoveInAsset(const FAssetData& InAssetData) const override;
	virtual void ExtendToolbar(FToolMenuSection& InSection) override;

	// UAnimAssetFindReplaceProcessor_StringBase interface
	virtual void GetAutoCompleteNames(TArrayView<FAssetData> InAssetDatas, TSet<FString>& OutUniqueNames) const override;

public:
	// Set whether to search morph targets when searching for curves
	PERSONA_API void SetSearchMorphTargets(bool bInSearchMorphTargets);

	// Get whether to search morph targets when searching for curves
	bool GetSearchMorphTargets() const { return bSearchMorphTargets; }

	// Set whether to search material parameters when searching for curves
	PERSONA_API void SetSearchMaterials(bool bInSearchMaterials);

	// Get whether to search material parameters when searching for curves
	bool GetSearchMaterials() const { return bSearchMaterials; }

private:
	// Whether to search morph targets when searching for curves
	bool bSearchMorphTargets = false;

	// Whether to search materials when searching for curves
	bool bSearchMaterials = false;
};
