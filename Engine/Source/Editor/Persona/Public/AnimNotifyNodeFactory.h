// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"

#define UE_API PERSONA_API

class SAnimNotifyNode;
class SAnimNotifyTrack;


typedef TFunction<TSharedPtr<class SAnimNotifyNode>(struct FAnimNotifyEvent*)> FAnimNotifyNodeFactoryFunction;

//////////////////////////////////////////////////////////////////////////
// SAnimNotifyNode

struct FAnimNotifyNodeFactory 
{
public:
	/** Allow others to register custom node types for custom notifies. */
	UE_API static FDelegateHandle RegisterFactory(FAnimNotifyNodeFactoryFunction InFactory);

	/** Unregister the Factory */
	UE_API static void UnregisterFactory(FDelegateHandle InHandle);

	/** Get registered factories. */
	UE_API static const TArray<FAnimNotifyNodeFactoryFunction>& GetRegisteredFactories() { return RegisteredFactories; }

private:

	static TArray<FAnimNotifyNodeFactoryFunction> RegisteredFactories;
	static TArray<FDelegateHandle> RegisteredFactoryHandles;
};

#undef UE_API