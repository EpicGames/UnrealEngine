// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"
#include "AnimationEditorPreviewActor.generated.h"

#define UE_API PERSONA_API

UCLASS(MinimalAPI)
class AAnimationEditorPreviewActor : public AActor
{
	GENERATED_BODY()

public:
	/** AActor interface */
	UE_API virtual void K2_DestroyActor() override;
};

#undef UE_API
