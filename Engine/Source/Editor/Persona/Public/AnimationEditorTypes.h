// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

//////////////////////////////////////////////////////////////////////////
// ELocalAxesMode

namespace ELocalAxesMode
{
enum Type
{
	None,
	Selected,
	All,
	NumAxesModes
};
}; // namespace ELocalAxesMode

//////////////////////////////////////////////////////////////////////////
// ELocalAxesMode

namespace EDisplayInfoMode
{
enum Type
{
	None,
	Basic,
	Detailed,
	SkeletalControls,
	NumInfoModes
};
}; // namespace EDisplayInfoMode

namespace EAnimationPlaybackSpeeds
{
enum Type
{
	OneTenth = 0,
	Quarter,
	Half,
	ThreeQuarters,
	Normal,
	Double,
	FiveTimes,
	TenTimes,
	Custom,
	NumPlaybackSpeeds
};

extern float Values[NumPlaybackSpeeds];
}; // namespace EAnimationPlaybackSpeeds
