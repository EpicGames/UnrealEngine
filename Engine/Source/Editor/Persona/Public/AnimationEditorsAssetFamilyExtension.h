// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "AnimationEditorsAssetFamilyExtension.generated.h"

struct FSlateBrush;
class USkeleton;
class USkeletalMesh;

struct IAnimationEditorsAssetFamilyInterface
{
	virtual bool IsAssetTypeInFamily(const TObjectPtr<UClass> InClass) const = 0;

	template <typename T>
	bool IsAssetTypeInFamily() const
	{
		return IsAssetTypeInFamily(T::StaticClass());
	}

	virtual TWeakObjectPtr<const UObject> GetAssetOfType(const TObjectPtr<UClass> InClass) const = 0;

	template <typename T>
	TObjectPtr<const T> GetAssetOfType() const
	{
		TWeakObjectPtr<const UObject> WeakAsset = GetAssetOfType(T::StaticClass());
		if (TStrongObjectPtr<const UObject> StrongAsset = WeakAsset.Pin())
		{
			return CastChecked<T>(StrongAsset.Get());
		}
		return nullptr;
	}

	bool IsAssetTypeInFamilyAndUnassigned(const TObjectPtr<UClass> InClass) const
	{
		if (IsAssetTypeInFamily(InClass))
		{
			return !GetAssetOfType(InClass).IsValid();
		}
		return false;
	}

	template <typename T>
	bool IsAssetTypeInFamilyAndUnassigned() const
	{
		return IsAssetTypeInFamilyAndUnassigned(T::StaticClass());
	}

	virtual bool SetAssetOfType(const TObjectPtr<UClass> InClass, TWeakObjectPtr<const UObject> InObject) = 0;

	template <typename T>
	bool SetAssetOfType(TWeakObjectPtr<const UObject> InObject)
	{
		return SetAssetOfType(T::StaticClass(), InObject);
	}
};

UCLASS(MinimalAPI)
class UAnimationEditorsAssetFamilyExtension : public UObject
{
	GENERATED_BODY()

public:
	// The asset class that this extension is describing
	virtual TObjectPtr<UClass> GetAssetClass() const { return nullptr; }

	// Returns the display name to show for this asset class
	virtual FText GetAssetTypeDisplayName() const { return FText(); }

	// Returns the display icon to show for this asset class
	virtual const FSlateBrush* GetAssetTypeDisplayIcon() const { return nullptr; }

	// Fills OutAssets with the assets of this type that exist
	virtual void FindAssetsOfType(TArray<FAssetData>& OutAssets, const IAnimationEditorsAssetFamilyInterface& AssetFamilyInterface) const {};

	// Returns whether or now the provided asset is compatible
	virtual bool IsAssetCompatible(const FAssetData& InAssetData, const IAnimationEditorsAssetFamilyInterface& AssetFamilyInterface) const { return false; }

	// Sets the rest of the assets in the family
	virtual void FindCounterpartAssets(const UObject* InAsset, IAnimationEditorsAssetFamilyInterface& AssetFamilyInterface) {};

	// Gets the horizontal position of this asset
	virtual void GetPosition(FName& OutBeforeClass, FName& OutAfterClass) const {};
};
