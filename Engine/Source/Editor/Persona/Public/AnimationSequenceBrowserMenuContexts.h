// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"

#include "AnimationSequenceBrowserMenuContexts.generated.h"

class SAnimationSequenceBrowser;

UCLASS(MinimalAPI, BlueprintType)
class UAnimationSequenceBrowserContextMenuContext : public UObject
{
	GENERATED_BODY()

public:
	TWeakPtr<SAnimationSequenceBrowser> OwningAnimSequenceBrowser;

	UPROPERTY()
	TArray<TWeakObjectPtr<UObject>> SelectedObjects;

	UFUNCTION(BlueprintCallable, Category = AnimationEditorExtensions)
	TArray<UObject*> GetSelectedObjects() const
	{
		TArray<UObject*> Result;
		Result.Reserve(SelectedObjects.Num());
		for (const TWeakObjectPtr<UObject>& Object : SelectedObjects)
		{
			Result.Add(Object.Get());
		}
		return Result;
	}
};
