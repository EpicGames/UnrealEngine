// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Animation/BoneSocketReference.h"
#include "Features/IModularFeature.h"
#include "Animation/BlendSpace.h"
#include "Animation/AnimSequence.h"

#include "BlendSpaceAnalysis.generated.h"

#define UE_API PERSONA_API

#define LOCTEXT_NAMESPACE "BlendSpaceAnalysis"
//#define ANALYSIS_VERBOSE_LOG

class UBlendSpace;
class UAnalysisProperties;

/**
* Users wishing to add their own analysis functions and structures should inherit from this, implement the virtual
* functions, and register an instance with IModularFeatures. It may help to look at the implementation of 
* FCoreBlendSpaceAnalysisFeature when doing this.
*/
class IBlendSpaceAnalysisFeature : public IModularFeature
{
public:
	static FName GetModuleFeatureName() { return "BlendSpaceAnalysis"; }

	// This should process the animation according to the analysis properties, or return false if that is not possible.
	virtual bool CalculateSampleValue(float&                     Result,
									  const UBlendSpace&         BlendSpace,
									  const UAnalysisProperties* AnalysisProperties,
									  const UAnimSequence&       Animation,
									  const float                RateScale) const = 0;

	// This should return an instance derived from UAnalysisProperties that is suitable for the Function. The caller
	// will pass in a suitable owning object, outer, that the implementation should assign as owner of the newly created
	// object. 
	virtual UAnalysisProperties* MakeAnalysisProperties(UObject* Outer, const FString& FunctionName) const = 0;

	// This should return the names of the functions handled
	virtual TArray<FString> GetAnalysisFunctions() const = 0;
};

UENUM()
enum class EAnalysisSpace : uint8
{
	World    UMETA(ToolTip = "Analysis is done in world space (relative to the root of the character)"),
	Fixed    UMETA(ToolTip = "Analysis is done in the space of the specified bone or socket based on the first frame of the animation used"),
	Changing UMETA(ToolTip = "Analysis is done in the space of the specified bone or socket based, but velocities are calculated as if this space is not moving"),
	Moving   UMETA(ToolTip = "Analysis is done in the space of the specified bone or socket"),
};

UENUM()
enum class EAnalysisLinearAxis : uint8
{
	PlusX UMETA(DisplayName = "+X", ToolTip = "The axis points in the positive X direction"),
	PlusY UMETA(DisplayName = "+Y", ToolTip = "The axis points in the positive Y direction"),
	PlusZ UMETA(DisplayName = "+Z", ToolTip = "The axis points in the positive Z direction"),
	MinusX UMETA(DisplayName = "-X", ToolTip = "The axis points in the negative X direction"),
	MinusY UMETA(DisplayName = "-Y", ToolTip = "The axis points in the negative Y direction"),
	MinusZ UMETA(DisplayName = "-Z", ToolTip = "The axis points in the negative Z direction"),
};

UENUM()
enum class EEulerCalculationMethod : uint8
{
	AimDirection    UMETA(ToolTip = "Calculates the yaw by looking at the BoneRightAxis. This can provide better yaw values, especially when aiming (e.g. with a weapon that has minimal rotation around its pointing axis) and covering extreme angles up and down, but only if this rightwards facing axis is reliable. It won't work well if the bone is also rolling around its axis."),
	PointDirection  UMETA(ToolTip = "Calculates the yaw based only on the BoneFacingAxis. This will work when you're most interested in the yaw and pitch from a pointing direction, but can produce undesirable results when pointing almost directly up or down."),
};

UENUM()
enum class EAnalysisEulerAxis : uint8
{
	Roll,
	Pitch,
	Yaw,
};


/**
 * This will be used to preserve values as far as possible when switching between analysis functions, so it contains all
 * the parameters used by the engine functions. User defined can inherit from this and add their own - then the
 * user-defined MakeCache function should replace any base class cache that is passed in with their own.
*/
UCLASS(MinimalAPI)
class UCachedAnalysisProperties : public UObject
{
	GENERATED_BODY()
public:
	UE_API void CopyFrom(const UCachedAnalysisProperties& Other);
	EAnalysisLinearAxis     LinearFunctionAxis = EAnalysisLinearAxis::PlusX;
	EAnalysisEulerAxis      EulerFunctionAxis = EAnalysisEulerAxis::Pitch;
	FBoneSocketTarget       BoneSocket1;
	FBoneSocketTarget       BoneSocket2;
	EAnalysisLinearAxis     BoneFacingAxis = EAnalysisLinearAxis::PlusX;
	EAnalysisLinearAxis     BoneRightAxis = EAnalysisLinearAxis::PlusY;
	EAnalysisSpace          Space = EAnalysisSpace::World;
	FBoneSocketTarget       SpaceBoneSocket;
	EAnalysisLinearAxis     CharacterFacingAxis = EAnalysisLinearAxis::PlusY;
	EAnalysisLinearAxis     CharacterUpAxis = EAnalysisLinearAxis::PlusZ;
	float                   StartTimeFraction = 0.0f;
	float                   EndTimeFraction = 1.0f;
};

UCLASS(MinimalAPI, Abstract)
class ULinearAnalysisPropertiesBase : public UAnalysisProperties
{
	GENERATED_BODY()
public:

	/** The bone or socket used for analysis */
	UPROPERTY(EditAnywhere, DisplayName = "Bone/Socket", Category = AnalysisProperties)
	FBoneSocketTarget BoneSocket;

	/**
	* The space in which to perform the analysis. Fixed will use the analysis bone/socket at the first frame
	* of the analysis time range. Changing will use the analysis bone/socket at the relevant frame during the
	* analysis, but calculate velocities assuming that frame isn't moving. Moving will do the same but velocities
	* as well as positions/rotations will be relative to this moving frame.
	*/
	UPROPERTY(EditAnywhere, Category = AnalysisProperties)
	EAnalysisSpace Space = EAnalysisSpace::World;

	/** Bone or socket that defines the analysis space (when it isn't World) */
	UPROPERTY(EditAnywhere, DisplayName = "Analysis Space Bone/Socket", Category = AnalysisProperties, meta = (EditCondition = "Space != EAnalysisSpace::World"))
	FBoneSocketTarget SpaceBoneSocket;

	/** Fraction through each animation at which analysis starts */
	UPROPERTY(EditAnywhere, DisplayName = "Start time fraction", Category = AnalysisProperties, meta = (ClampMin = "0", ClampMax = "1"))
	float StartTimeFraction = 0.0f;

	/** Fraction through each animation at which analysis ends */
	UPROPERTY(EditAnywhere, DisplayName = "End time fraction", Category = AnalysisProperties, meta = (ClampMin = "0", ClampMax = "1"))
	float EndTimeFraction = 1.0f;
};

UCLASS(MinimalAPI)
class ULinearAnalysisProperties : public ULinearAnalysisPropertiesBase
{
	GENERATED_BODY()
public:
	UE_API void InitializeFromCache(TObjectPtr<UCachedAnalysisProperties> Cache) override;
	UE_API void MakeCache(TObjectPtr<UCachedAnalysisProperties>& Cache, UBlendSpace* BlendSpace) override;

	/** Axis for the analysis function */
	UPROPERTY(EditAnywhere, DisplayName = "Axis", Category = AnalysisProperties)
	EAnalysisLinearAxis FunctionAxis = EAnalysisLinearAxis::PlusX;
};


UCLASS(MinimalAPI)
class UEulerAnalysisProperties : public UAnalysisProperties
{
	GENERATED_BODY()
public:
	UE_API void InitializeFromCache(TObjectPtr<UCachedAnalysisProperties> Cache) override;
	UE_API void MakeCache(TObjectPtr<UCachedAnalysisProperties>& Cache, UBlendSpace* BlendSpace) override;

	/** Axis for the analysis function */
	UPROPERTY(EditAnywhere, DisplayName = "Axis", Category = AnalysisProperties)
	EAnalysisEulerAxis FunctionAxis = EAnalysisEulerAxis::Pitch;

	/** The bone or socket used for analysis */
	UPROPERTY(EditAnywhere, DisplayName = "Bone/Socket", Category = AnalysisProperties)
	FBoneSocketTarget BoneSocket;

	/** Used for some analysis functions - specifies the bone/socket axis that points in the facing/forwards direction */
	UPROPERTY(EditAnywhere, Category = AnalysisProperties)
	EAnalysisLinearAxis BoneFacingAxis = EAnalysisLinearAxis::PlusX;

	/** Used for some analysis functions - specifies the bone/socket axis that points to the "right" */
	UPROPERTY(EditAnywhere, Category = AnalysisProperties)
	EAnalysisLinearAxis BoneRightAxis = EAnalysisLinearAxis::PlusY;

	/** Used for some analysis functions - specifies how yaw should be calculated from the bone axes */
	UPROPERTY(EditAnywhere, Category = AnalysisProperties)
	EEulerCalculationMethod EulerCalculationMethod = EEulerCalculationMethod::AimDirection;

	/**
	* The space in which to perform the analysis. Fixed will use the analysis bone/socket at the first frame
	* of the analysis time range. Changing will use the analysis bone/socket at the relevant frame during the
	* analysis, but calculate velocities assuming that frame isn't moving. Moving will do the same but velocities
	* as well as positions/rotations will be relative to this moving frame.
	*/
	UPROPERTY(EditAnywhere, Category = AnalysisProperties)
	EAnalysisSpace Space = EAnalysisSpace::World;

	/** Bone or socket that defines the analysis space (when it isn't World) */
	UPROPERTY(EditAnywhere, DisplayName = "Analysis Space Bone/Socket", Category = AnalysisProperties, meta = (EditCondition = "Space != EAnalysisSpace::World"))
	FBoneSocketTarget SpaceBoneSocket;

	/** World or bone/socket axis that specifies the character's facing direction */
	UPROPERTY(EditAnywhere, Category = AnalysisProperties)
	EAnalysisLinearAxis CharacterFacingAxis = EAnalysisLinearAxis::PlusY;

	/** World or bone/socket axis that specifies the character's up direction */
	UPROPERTY(EditAnywhere, Category = AnalysisProperties)
	EAnalysisLinearAxis CharacterUpAxis = EAnalysisLinearAxis::PlusZ;

	/** Fraction through each animation at which analysis starts */
	UPROPERTY(EditAnywhere, DisplayName = "Start time fraction", Category = AnalysisProperties, meta = (ClampMin = "0", ClampMax = "1"))
	float StartTimeFraction = 0.0f;
	/** Fraction through each animation at which analysis ends */

	UPROPERTY(EditAnywhere, DisplayName = "End time fraction", Category = AnalysisProperties, meta = (ClampMin = "0", ClampMax = "1"))
	float EndTimeFraction = 1.0f;
};

//======================================================================================================================
// The following are helper functions which may be useful when implementing analysis functions
//======================================================================================================================

namespace BlendSpaceAnalysis
{

//======================================================================================================================
// Retrieves the bone index and transform offset given the BoneSocketTarget. Returns true if found
PERSONA_API bool GetBoneInfo(const UAnimSequence&     Animation, 
							 const FBoneSocketTarget& BoneSocket, 
							 FTransform&              BoneOffset, 
							 FName&                   BoneName);

//======================================================================================================================
PERSONA_API FTransform GetBoneTransform(const UAnimSequence& Animation, int32 Key, const FName& BoneName);

//======================================================================================================================
template<typename T>
void CalculateFrameTM(
	bool& bNeedToUpdateFrameTM, FTransform& FrameTM, 
	const int32 SampleKey, const T& AnalysisProperties, const UAnimSequence& Animation)
{
	if (bNeedToUpdateFrameTM)
	{
		FrameTM.SetIdentity();
		if (AnalysisProperties->Space != EAnalysisSpace::World)
		{
			FTransform SpaceBoneOffset;
			FName SpaceBoneName;
			if (GetBoneInfo(Animation, AnalysisProperties->SpaceBoneSocket, SpaceBoneOffset, SpaceBoneName))
			{
				FTransform SpaceBoneTM = GetBoneTransform(Animation, SampleKey, SpaceBoneName);
				FrameTM = SpaceBoneOffset * SpaceBoneTM;
			}
		}

		bNeedToUpdateFrameTM = (
			AnalysisProperties->Space == EAnalysisSpace::Changing || 
			AnalysisProperties->Space == EAnalysisSpace::Moving);
	}
}

//======================================================================================================================
PERSONA_API FVector GetAxisFromTM(const FTransform& TM, EAnalysisLinearAxis Axis);

//======================================================================================================================
template<typename T>
void GetFrameDirs(
	FVector& FrameFacingDir, FVector& FrameUpDir, FVector& FrameRightDir, 
	const FTransform& FrameTM, const T& AnalysisProperties)
{
	FrameFacingDir = GetAxisFromTM(FrameTM, AnalysisProperties->CharacterFacingAxis);
	FrameUpDir = GetAxisFromTM(FrameTM, AnalysisProperties->CharacterUpAxis);
	FrameRightDir = FVector::CrossProduct(FrameUpDir, FrameFacingDir);
}

//======================================================================================================================
/**
* Helper to extract the component from the FVector functions
*/
template<typename FunctionType>
static bool CalculateComponentSampleValue(
	double& Result,
	const FunctionType& Fn,
	const UBlendSpace& BlendSpace,
	const ULinearAnalysisProperties* AnalysisProperties,
	const UAnimSequence& Animation,
	const float	RateScale)
{
	FVector Value = FVector::ZeroVector;
	int32 ComponentIndex = (int32)AnalysisProperties->FunctionAxis;
	if (Fn(Value, BlendSpace, AnalysisProperties, Animation, RateScale))
	{
		Result = Value | BlendSpaceAnalysis::GetAxisFromTM(FTransform::Identity, AnalysisProperties->FunctionAxis);
		return true;
	}
	return false;
}

//======================================================================================================================
/**
* Helper to extract the component from the FVector functions
*/
template<typename FunctionType>
static bool CalculateComponentSampleValue(
	double& Result,
	const FunctionType& Fn,
	const UBlendSpace& BlendSpace,
	const UEulerAnalysisProperties* AnalysisProperties,
	const UAnimSequence& Animation,
	const float RateScale)
{
	FVector Value;
	int32 ComponentIndex = (int32)AnalysisProperties->FunctionAxis; // Roll, Pitch, Yaw -> 0, 1, 2
	if (Fn(Value, BlendSpace, AnalysisProperties, Animation, RateScale))
	{
		Result = Value[ComponentIndex];
		return true;
	}
	return false;
}

//======================================================================================================================
/**
* Helper to extract the component from the FVector functions as a float (pass through to double version)
*/
template<typename FunctionType>
static bool CalculateComponentSampleValue(
	float&               Result,
	const FunctionType&  Fn,
	const UBlendSpace&   BlendSpace,
	const ULinearAnalysisProperties* AnalysisProperties,
	const UAnimSequence& Animation,
	const float          RateScale)
{
	double DoubleResult = Result;
	bool bResult = CalculateComponentSampleValue(DoubleResult, Fn, BlendSpace, AnalysisProperties, Animation, RateScale);
	Result = float(DoubleResult);
	return bResult;
}

//======================================================================================================================
/**
* Helper to extract the component from the FVector functions as a float (pass through to double version)
*/
template<typename FunctionType>
static bool CalculateComponentSampleValue(
	float& Result,
	const FunctionType& Fn,
	const UBlendSpace& BlendSpace,
	const UEulerAnalysisProperties* AnalysisProperties,
	const UAnimSequence& Animation,
	const float          RateScale)
{
	double DoubleResult = Result;
	bool bResult = CalculateComponentSampleValue(DoubleResult, Fn, BlendSpace, AnalysisProperties, Animation, RateScale);
	Result = float(DoubleResult);
	return bResult;
}

//======================================================================================================================
extern PERSONA_API bool CalculatePosition(
	FVector&                         Result,
	const UBlendSpace&               BlendSpace,
	const ULinearAnalysisProperties* AnalysisProperties,
	const UAnimSequence&             Animation,
	const float                      RateScale);

//======================================================================================================================
extern PERSONA_API bool CalculateDeltaPosition(
	FVector&                         Result,
	const UBlendSpace&               BlendSpace,
	const ULinearAnalysisProperties* AnalysisProperties,
	const UAnimSequence&             Animation,
	const float                      RateScale);

//======================================================================================================================
extern PERSONA_API bool CalculateVelocity(
	FVector&             Result,
	const UBlendSpace&   BlendSpace,
	const ULinearAnalysisPropertiesBase* AnalysisProperties,
	const UAnimSequence& Animation,
	const float          RateScale);

//======================================================================================================================
extern PERSONA_API void CalculateBoneOrientation(
	FVector&                   RollPitchYaw,
	const UAnimSequence&       Animation, 
	const int32                Key, 
	const FName                BoneName, 
	const FTransform&          BoneOffset, 
	const UEulerAnalysisProperties* AnalysisProperties,
	const FVector&             FrameFacingDir, 
	const FVector&             FrameRightDir, 
	const FVector&             FrameUpDir);

//======================================================================================================================
// Note that if a looping animation has 56 keys, then its first key is 0 and last is 55, but these will be identical poses.
// Thus it has one fewer intervals/unique keys
extern PERSONA_API bool CalculateOrientation(
	FVector&                        Result,
	const UBlendSpace&              BlendSpace,
	const UEulerAnalysisProperties* AnalysisProperties,
	const UAnimSequence&            Animation,
	const float                     RateScale);

//======================================================================================================================
// Note that if a looping animation has 56 keys, then its first key is 0 and last is 55, but these will be identical poses.
// Thus it has one fewer intervals/unique keys
extern PERSONA_API bool CalculateDeltaOrientation(
	FVector&                        Result,
	const UBlendSpace&              BlendSpace,
	const UEulerAnalysisProperties* AnalysisProperties,
	const UAnimSequence&            Animation,
	const float                     RateScale);

//======================================================================================================================
extern PERSONA_API bool CalculateAngularVelocity(
	FVector&                         Result,
	const UBlendSpace&               BlendSpace,
	const ULinearAnalysisProperties* AnalysisProperties,
	const UAnimSequence&             Animation,
	const float                      RateScale);

//======================================================================================================================
extern PERSONA_API bool CalculateOrientationRate(
	FVector&                        Result,
	const UBlendSpace&              BlendSpace,
	const UEulerAnalysisProperties* AnalysisProperties,
	const UAnimSequence&            Animation,
	const float                     RateScale);


}

#undef LOCTEXT_NAMESPACE

#undef UE_API
