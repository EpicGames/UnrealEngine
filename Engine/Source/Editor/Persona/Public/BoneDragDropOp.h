// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Input/DragAndDrop.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Styling/AppStyle.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Images/SImage.h"

class IEditableSkeleton;

//////////////////////////////////////////////////////////////////////////
// FBoneDragDropOp
class FBoneDragDropOp : public FDragDropOperation
{
public:	
	DRAG_DROP_OPERATOR_TYPE(FBoneDragDropOp, FDragDropOperation)
	
	TWeakPtr<class IEditableSkeleton> EditableSkeleton;
	FName BoneName;

	/** The widget decorator to use */
	virtual TSharedPtr<SWidget> GetDefaultDecorator() const override
	{
		return SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("Graph.ConnectorFeedback.Border"))
			.Content()
			[		
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SImage)
					.Image(this, &FBoneDragDropOp::GetIcon)
				]

				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(STextBlock) 
					.Text( this, &FBoneDragDropOp::GetHoverText )
				]
			];
	}

	FText GetHoverText() const
	{
		return FText::Format(NSLOCTEXT("BoneDragDropOp", "BoneHoverTextFmt", "Bone {0}"), FText::FromString(BoneName.GetPlainNameString()));
	}

	const FSlateBrush* GetIcon( ) const
	{
		return CurrentIconBrush;
	}

	void SetIcon(const FSlateBrush* InIcon)
	{
		CurrentIconBrush = InIcon;
	}

	static TSharedRef<FBoneDragDropOp> New(TSharedRef<class IEditableSkeleton> EditableSkeleton, const FName& InBoneName)
	{
		TSharedRef<FBoneDragDropOp> Operation = MakeShareable(new FBoneDragDropOp);
		Operation->BoneName = InBoneName;
		Operation->EditableSkeleton = EditableSkeleton;
		Operation->SetIcon( FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error")) );
		Operation->Construct();
		return Operation;
	}

private:
	const FSlateBrush* CurrentIconBrush;
};
