// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STreeView.h"

#define UE_API PERSONA_API

class IEditableSkeleton;
class SComboButton;

DECLARE_DELEGATE_OneParam(FOnBoneSelectionChanged, FName);
DECLARE_DELEGATE_RetVal_OneParam(FName, FGetSelectedBone, bool& /*bMultipleValues*/);
DECLARE_DELEGATE_RetVal(const struct FReferenceSkeleton&, FGetReferenceSkeleton);
DECLARE_DELEGATE_RetVal(const TArray<class USkeletalMeshSocket*>&, FGetSocketList);

class SBoneTreeMenu : public SCompoundWidget
{
public:
	// Storage object for bone hierarchy
	struct FBoneNameInfo
	{
		FBoneNameInfo(FName Name) : BoneName(Name) {}

		FName BoneName;
		TArray<TSharedPtr<FBoneNameInfo>> Children;
	};

	SLATE_BEGIN_ARGS(SBoneTreeMenu)
		: _bShowVirtualBones(true)
		, _bShowSocket(false)
		, _bShowNone(false)
		, _OnGetReferenceSkeleton()
		, _OnBoneSelectionChanged()
		, _OnGetSocketList()
		{}

		SLATE_ARGUMENT(FText, Title)
		SLATE_ARGUMENT(bool, bShowVirtualBones)
		SLATE_ARGUMENT(bool, bShowSocket)
		SLATE_ARGUMENT(bool, bShowNone)
		SLATE_ARGUMENT(FName, SelectedBone)
		SLATE_EVENT(FGetReferenceSkeleton, OnGetReferenceSkeleton)
		SLATE_EVENT(FOnBoneSelectionChanged, OnBoneSelectionChanged)
		SLATE_EVENT(FGetSocketList, OnGetSocketList)

	SLATE_END_ARGS();

	/**
	* Construct this widget
	*
	* @param	InArgs	The declaration data for this widget
	*/
	UE_API void Construct(const FArguments& InArgs);

	/** Get the filter text widget, e.g. for focus */
	UE_API TSharedPtr<SWidget> GetFilterTextWidget();

private:
	// SWidget interface
	UE_API virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

	// Using the current filter, repopulate the tree view
	UE_API void RebuildBoneList(const FName& SelectedBone);

	// Make a single tree row widget
	UE_API TSharedRef<ITableRow> MakeTreeRowWidget(TSharedPtr<FBoneNameInfo> InInfo, const TSharedRef<STableViewBase>& OwnerTable);

	// Get the children for the provided bone info
	UE_API void GetChildrenForInfo(TSharedPtr<FBoneNameInfo> InInfo, TArray< TSharedPtr<FBoneNameInfo> >& OutChildren);

	// Called when the user changes the search filter
	UE_API void OnFilterTextChanged(const FText& InFilterText);

	// Handle the tree view selection changing
	UE_API void OnSelectionChanged(TSharedPtr<SBoneTreeMenu::FBoneNameInfo> BoneInfo, ESelectInfo::Type SelectInfo);

	// Select a specific bone, helper for UI handler functions
	UE_API void SelectBone(TSharedPtr<SBoneTreeMenu::FBoneNameInfo> BoneInfo);
	
	// Tree info entries for bone picker
	TArray<TSharedPtr<FBoneNameInfo>> SkeletonTreeInfo;
	// Mirror of SkeletonTreeInfo but flattened for searching
	TArray<TSharedPtr<FBoneNameInfo>> SkeletonTreeInfoFlat;

	// Text to filter bone tree with
	FText FilterText;

	// Tree view used in the button menu
	TSharedPtr<STreeView<TSharedPtr<FBoneNameInfo>>> TreeView;

	//Filter text widget
	TSharedPtr<SSearchBox> FilterTextWidget;

	FOnBoneSelectionChanged OnSelectionChangedDelegate;
	FGetReferenceSkeleton	OnGetReferenceSkeletonDelegate;
	FGetSocketList			OnGetSocketListDelegate;

	bool bShowVirtualBones;
	bool bShowSocket;
	bool bShowNone;
};

class SBoneSelectionWidget : public SCompoundWidget
{
public: 

	SLATE_BEGIN_ARGS( SBoneSelectionWidget )
		: _bShowSocket(false)
		, _bShowVirtualBones(true)
		, _bShowNone(false)
		, _OnBoneSelectionChanged()
		, _OnGetSelectedBone()
		, _OnGetReferenceSkeleton()
		, _OnGetSocketList()
	{}

		SLATE_ARGUMENT(bool, bShowSocket)

		/** Should show skeletons virtual bones in tree */
		SLATE_ARGUMENT(bool, bShowVirtualBones)

		/** Whether or not to show 'None' as a selectable bone name */
		SLATE_ARGUMENT(bool, bShowNone)

		/** set selected bone name */
		SLATE_EVENT(FOnBoneSelectionChanged, OnBoneSelectionChanged);

		/** get selected bone name **/
		SLATE_EVENT(FGetSelectedBone, OnGetSelectedBone);

		/** Get Reference skeleton */
		SLATE_EVENT(FGetReferenceSkeleton, OnGetReferenceSkeleton)

		/** Get Socket List */
		SLATE_EVENT(FGetSocketList, OnGetSocketList)

	SLATE_END_ARGS();

	/**
	 * Construct this widget
	 *
	 * @param	InArgs	The declaration data for this widget
	 */
	UE_API void Construct( const FArguments& InArgs );

private: 

	// Creates the combo button menu when clicked
	UE_API TSharedRef<SWidget> CreateSkeletonWidgetMenu();
	// Called when the user selects a bone name
	UE_API void OnSelectionChanged(FName BoneName);

	// Gets the current bone name, used to get the right name for the combo button
	UE_API FText GetCurrentBoneName() const;

	UE_API FText GetFinalToolTip() const;

	// Base combo button 
	TSharedPtr<SComboButton> BonePickerButton;

	// delegates
	FOnBoneSelectionChanged OnBoneSelectionChanged;
	FGetSelectedBone		OnGetSelectedBone;
	FGetReferenceSkeleton	OnGetReferenceSkeleton;
	FGetSocketList			OnGetSocketList;
	bool bShowSocket;
	bool bShowVirtualBones;
	bool bShowNone;

	// Cache supplied tooltip
	FText SuppliedToolTip;
};

#undef UE_API
