// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"

struct FRichCurve;
enum class ERawCurveTrackTypes : uint8;

/** Interface to the curve editor tab in the anim sequence editor */
class IAnimSequenceCurveEditor : public SCompoundWidget
{
public:
	/** Reset the curves that are edited by this widget */
	virtual void ResetCurves() = 0;

	/** Add a curve for editing */
	virtual void AddCurve(const FText& InCurveDisplayName, const FLinearColor& InCurveColor, const FName& InName, ERawCurveTrackTypes InType, int32 InCurveIndex, FSimpleDelegate InOnCurveModified) = 0;

	/** Remove a curve for editing */
	virtual void RemoveCurve(const FName& InName, ERawCurveTrackTypes InType, int32 InCurveIndex) = 0;

	/** Zoom to fit all the curves */
	virtual void ZoomToFit() = 0;
};