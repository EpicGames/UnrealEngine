// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"

class UAnimationAsset;

class IAnimationSequenceBrowser : public SCompoundWidget
{
public:
	/** Select the specified asset */
	virtual void SelectAsset(UAnimationAsset* InAsset) = 0;

	/** Add to navigation history */
	virtual void AddToHistory(UAnimationAsset* AnimAsset) = 0;

	/** Add filter by skeleton notify name */
	virtual void FilterBySkeletonNotify(const FName& InNotifyName) = 0;
};
