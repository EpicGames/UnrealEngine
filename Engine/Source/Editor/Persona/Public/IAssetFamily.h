// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AssetRegistry/AssetData.h"
#include "Styling/SlateColor.h"

struct FSlateBrush;

/** Represents a group of related assets, e.g. a skeleton, its animations and skeletal meshes */
class IAssetFamily
{
public:
	/** Virtual destructor */
	virtual ~IAssetFamily() = default;

	/** Get all the asset classes this family supports (doesnt need to include derived classes) */
	virtual void GetAssetTypes(TArray<UClass*>& OutAssetTypes) const = 0;

	/** Find the most relevant asset of a specified type */
	virtual FAssetData FindAssetOfType(UClass* AssetType) const = 0;

	/** Find the most relevant asset of a specified type */
	template<typename AssetType>
	FAssetData GetAsset()
	{
		return FindAssetOfType(AssetType::StaticClass());
	}

	/** Find all assets of a specified type */
	virtual void FindAssetsOfType(UClass* AssetType, TArray<FAssetData>& OutAssets) const = 0;

	/** Find all assets of a specified type */
	template<typename AssetType>
	void GetAssets(TArray<FAssetData>& OutAssets)
	{
		FindAssetsOfType(AssetType::StaticClass(), OutAssets);
	}

	/** Gets the name of an asset that will be displayed to a user */
	virtual FText GetAssetTypeDisplayName(UClass* InAssetClass) const = 0;

	/** Gets the slate brush that represents this asset family */
	virtual const FSlateBrush* GetAssetTypeDisplayIcon(UClass* InAssetClass) const = 0;

	/** Gets the color to tint the asset display icon */
	virtual FSlateColor GetAssetTypeDisplayTint(UClass* InAssetClass) const = 0;

	/** Check whether an asset is compatible with this family */
	virtual bool IsAssetCompatible(const FAssetData& InAssetData) const = 0;

	/** @return the outermost superclass of the passed-in class for this asset family */
	virtual UClass* GetAssetFamilyClass(UClass* InClass) const = 0;

	/** Event fired when an asset is opened */
	DECLARE_EVENT_OneParam(IAssetFamily, FOnAssetOpened, UObject*)
	virtual FOnAssetOpened& GetOnAssetOpened() = 0;

	/** Event fired when an asset family changes (e.g. relationships are altered) */
	DECLARE_EVENT(IAssetFamily, FOnAssetFamilyChanged)
	virtual FOnAssetFamilyChanged& GetOnAssetFamilyChanged() = 0;
};
