// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BlendProfilePicker.h"
#include "Delegates/DelegateCombinations.h"
#include "UObject/ObjectPtr.h"

class IBlendProfileProviderInterface;
class SWidget;
class USkeleton;

class IBlendProfilePickerExtender
{
public:
	struct FPickerWidgetArgs
	{
		FPickerWidgetArgs()
			: Outer(nullptr)
			, SupportedBlendProfileModes(EBlendProfilePickerMode::AllModes)
		{
		}

		DECLARE_DELEGATE_TwoParams(FOnBlendProfileProviderChanged, TObjectPtr<UObject> /* NewProfileProvider */, IBlendProfileProviderInterface*);

		// Should be fired when the blend profile provider object has changed
		FOnBlendProfileProviderChanged OnProviderChanged;

		// The initially selected provider object
		TObjectPtr<UObject> InitialSelection;

		// The outer to use for constructing new provider objects
		TObjectPtr<UObject> Outer;

		// Restrict which types of blend profiles are displayed in the picker
		EBlendProfilePickerMode SupportedBlendProfileModes;

		// Optional skeleton to restrict shown blend profiles relating to a particular skeleton
		TObjectPtr<USkeleton> Skeleton;
	};

	// Returns an identifier to match extender instances
	virtual FName GetId() const = 0;

	// Text to display in the details panel when choosing blend profiles of this type
	virtual FText GetDisplayName() const = 0;

	// Construct the picker widget for choosing blend profiles of this type
	virtual TSharedRef<SWidget> ConstructPickerWidget(const FPickerWidgetArgs& InWidgetArgs) const = 0;

	// Return true if the provided object is the matching type for this extender
	virtual bool OwnsBlendProfileProvider(const TObjectPtr<const UObject> InProviderObject) const = 0;
};