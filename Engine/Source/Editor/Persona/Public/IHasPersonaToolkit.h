// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/** Interface implemented by objects that hold a persona toolkit */
class IHasPersonaToolkit
{
public:
	/** Get the toolkit held by this object */
	virtual TSharedRef<class IPersonaToolkit> GetPersonaToolkit() const = 0;
};
