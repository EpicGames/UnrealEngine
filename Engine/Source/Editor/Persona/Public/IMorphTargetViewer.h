// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"

/** Interface to the morph target viewer in the skeletal mesh editor */
class IMorphTargetViewer
{
public:
	
	virtual TArray<FName> GetSelectedMorphTargetNames() const = 0;
	virtual TSharedRef<SWidget> AsWidget() = 0;

	virtual ~IMorphTargetViewer() = default;
};