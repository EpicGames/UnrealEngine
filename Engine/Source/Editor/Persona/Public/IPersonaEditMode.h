// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimationEditMode.h"

#define UE_API PERSONA_API

class IPersonaEditMode : public FAnimationEditMode
{
	// These implementations are duplicated in the Persona module so that code that links only to Persona and not to
	// the AnimationEditMode module can work properly
public:
	UE_API IPersonaEditMode();
	UE_API virtual ~IPersonaEditMode() override;
	UE_API virtual void Enter() override;
	UE_API virtual void Exit() override;
	UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
};

#undef UE_API
