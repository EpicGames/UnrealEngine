// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AssetEditorModeManager.h"

#include "IPersonaEditorModeManager.generated.h"

#define UE_API PERSONA_API

UINTERFACE(MinimalAPI)
class UPersonaManagerContext : public UInterface
{
	GENERATED_BODY()
};

/** Persona-specific extensions to the asset editor mode manager */
class IPersonaManagerContext
{
	GENERATED_BODY()
public:
	/** 
	 * Get a camera target for when the user focuses the viewport
	 * @param OutTarget		The target object bounds
	 * @return true if the location is valid
	 */
	virtual bool GetCameraTarget(FSphere& OutTarget) const = 0;

	/** 
	 * Get debug info for any editor modes that are active
	 * @param	OutDebugText	The text to draw
	 */
	virtual void GetOnScreenDebugInfo(TArray<FText>& OutDebugText) const = 0;
};

class IPersonaEditorModeManager;

UCLASS(MinimalAPI)
class UPersonaEditorModeManagerContext : public UObject, public IPersonaManagerContext
{
	GENERATED_BODY()
public:
	// Only for use by FAnimationViewportClient::GetPersonaModeManager for compatibility
	UE_API IPersonaEditorModeManager* GetPersonaEditorModeManager() const;
	UE_API virtual bool GetCameraTarget(FSphere& OutTarget) const override;
	UE_API virtual void GetOnScreenDebugInfo(TArray<FText>& OutDebugText) const override;
	UE_API virtual void SetFocusInViewport();
private:
	IPersonaEditorModeManager* ModeManager = nullptr;
	static UPersonaEditorModeManagerContext* CreateFor(IPersonaEditorModeManager* InModeManager)
	{
		UPersonaEditorModeManagerContext* NewPersonaContext = NewObject<UPersonaEditorModeManagerContext>();
		NewPersonaContext->ModeManager = InModeManager;
		return NewPersonaContext;
	}
	friend IPersonaEditorModeManager;
};

class IPersonaEditorModeManager : public FAssetEditorModeManager, public IPersonaManagerContext
{
public:
	IPersonaEditorModeManager();
	virtual ~IPersonaEditorModeManager() override;
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual UTypedElementSelectionSet* GetEditorSelectionSet() const override;

private:
	TObjectPtr<UPersonaEditorModeManagerContext> PersonaModeManagerContext{UPersonaEditorModeManagerContext::CreateFor(this)};
	TObjectPtr<UTypedElementSelectionSet> SelectionSet;
};

#undef UE_API
