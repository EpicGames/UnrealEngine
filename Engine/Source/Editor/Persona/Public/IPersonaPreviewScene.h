// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AdvancedPreviewScene.h"
#include "Containers/ArrayView.h"
#include "Types/SlateEnums.h"
#include "Animation/AnimBlueprint.h"
#include "PersonaSelectionProxies.h"
#include "AnimationEditorTypes.h"

class UAnimationAsset;
class UDebugSkelMeshComponent;
class USkeletalMesh;
struct FSelectedSocketInfo;
struct HActor;
struct FViewportClick;
class FEditorCameraController;
class ISkeletonTreeItem;
class IEditableSkeleton;

// called when animation asset has been changed
DECLARE_MULTICAST_DELEGATE_OneParam(FOnAnimChangedMulticaster, UAnimationAsset*);

// anim changed 
typedef FOnAnimChangedMulticaster::FDelegate FOnAnimChanged;

// Called when the preview mesh has been changed;
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnPreviewMeshChangedMulticaster, USkeletalMesh* /*OldPreviewMesh*/, USkeletalMesh* /*NewPreviewMesh*/);

// preview mesh changed 
typedef FOnPreviewMeshChangedMulticaster::FDelegate FOnPreviewMeshChanged;

DECLARE_MULTICAST_DELEGATE_TwoParams(FOnMeshClickMulticaster, HActor*, const FViewportClick&);

typedef FOnMeshClickMulticaster::FDelegate FOnMeshClick;

//The selected LOD changed
DECLARE_MULTICAST_DELEGATE(FOnSelectedLODChangedMulticaster);
typedef FOnSelectedLODChangedMulticaster::FDelegate FOnSelectedLODChanged;

//The selected bone changed
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnSelectedBoneChangedMulticaster, const FName& /*InBoneName*/, ESelectInfo::Type /*InSelectInfo*/);
typedef FOnSelectedBoneChangedMulticaster::FDelegate FOnSelectedBoneChanged;

//The bones selection changed
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnSelectedBonesChangedMulticaster, const TArray<FName>& /*InBoneNames*/, ESelectInfo::Type /*InSelectInfo*/);
typedef FOnSelectedBonesChangedMulticaster::FDelegate FOnSelectedBonesChanged;

//The selected socket changed
DECLARE_MULTICAST_DELEGATE_OneParam(FOnSelectedSocketChangedMulticaster, const FSelectedSocketInfo& /*InSocketInfo*/);
typedef FOnSelectedSocketChangedMulticaster::FDelegate FOnSelectedSocketChanged;

//The delegate to check if the attach component can be removed
DECLARE_DELEGATE_RetVal_OneParam(bool, FOnRemoveAttachedComponentFilter, const USceneComponent* /*InComponent*/);

/** Modes that the preview scene defaults to (usually depending on asset editor context) */
enum class EPreviewSceneDefaultAnimationMode : int32
{
	ReferencePose,

	Animation,

	AnimationBlueprint,

	// each mesh component defines their custom pose 
	Custom, 
};

class IPersonaPreviewScene : public FAdvancedPreviewScene
{
public:
	/** Constructor only here to pass ConstructionValues to base constructor */
	IPersonaPreviewScene(ConstructionValues CVS)
		: FAdvancedPreviewScene(CVS)
	{}

	/** Unregister this scene from the Undo chain. Necessary for certain editor windows. */
	virtual void UnregisterForUndo() = 0;

	/** Get the persona toolkit we are associated with */
	virtual TSharedRef<class IPersonaToolkit> GetPersonaToolkit() const = 0;

	/** Flag that we want our views to be updated */
	virtual void InvalidateViews() = 0;

	/** Request our views to focus on the current item */
	virtual void FocusViews() = 0;

	/** Get the skeletal mesh component we are using for preview, if any. */
	virtual UDebugSkelMeshComponent* GetPreviewMeshComponent() const = 0;

	/** Get array of all skeletal mesh components in the preview scene. */
	virtual TArray<UDebugSkelMeshComponent*> GetAllPreviewMeshComponents() const = 0;

	/** Run a lambda function on each preview mesh in the scene */
	virtual void ForEachPreviewMesh(TFunction<void (UDebugSkelMeshComponent*)> PerMeshFunction) = 0;

	/** Set the skeletal mesh component we are going to preview. */
	virtual void SetPreviewMeshComponent(UDebugSkelMeshComponent* InSkeletalMeshComponent) = 0;

	/** Set the additional meshes used by this preview scene (sets the additional meshes on the skeleton) */
	virtual void SetAdditionalMeshes(class UDataAsset* InAdditionalMeshes) = 0;

	/** Set whether additional meshes are selectable */
	virtual void SetAdditionalMeshesSelectable(bool bSelectable) = 0;

	/** Refreshes the additional meshes displayed in this preview scene */
	virtual void RefreshAdditionalMeshes(bool bAllowOverrideBaseMesh) = 0;

	/** Set the animation asset to preview **/
	virtual void SetPreviewAnimationAsset(UAnimationAsset* AnimAsset, bool bEnablePreview = true) = 0;

	/** Get the animation asset we are previewing */
	virtual UAnimationAsset* GetPreviewAnimationAsset() const = 0;

	/** Set the preview mesh for this scene (does not set the preview mesh on the skeleton/asset) */
	virtual void SetPreviewMesh(USkeletalMesh* NewPreviewMesh, bool bAllowOverrideBaseMesh = true) = 0;

	/** Get the preview mesh for this scene (does go via skeleton/asset) */
	virtual USkeletalMesh* GetPreviewMesh() const = 0;

	/** Set the preview animation blueprint and an optional overlay (for sub-layers) */
	virtual void SetPreviewAnimationBlueprint(UAnimBlueprint* InAnimBlueprint, UAnimBlueprint* InOverlayOrSubAnimBlueprint) = 0;

	/** Show the reference pose of the displayed skeletal mesh. Otherwise display the default. Optionally reset bone transforms, if any. */
	virtual void ShowReferencePose(bool bShowRefPose, bool bResetBoneTransforms = false) = 0;

	/* Are we currently displaying the ref pose */
	virtual bool IsShowReferencePoseEnabled() const = 0;

	/** Attaches an object to the preview component using the supplied attach name, returning whether it was successfully attached or not */
	virtual bool AttachObjectToPreviewComponent(UObject* Object, FName AttachTo) = 0;

	/** Removes a currently attached object from the preview component */
	virtual void RemoveAttachedObjectFromPreviewComponent(UObject* Object, FName AttachedTo) = 0;

	/** Sets the selected bone on the preview component */
	UE_DEPRECATED(4.26, "Please call/implement SetSelectedBone with ESelectInfo")
	virtual void SetSelectedBone(const FName& BoneName) final { SetSelectedBone(BoneName, ESelectInfo::Direct); }

	/** Sets the selected bone on the preview component */
	virtual void SetSelectedBone(const FName& BoneName, ESelectInfo::Type InSelectInfo) = 0;

	/** Sets the selected bones on the preview component */
	virtual void SetSelectedBones(const TArray<FName>& InBoneNames, ESelectInfo::Type InSelectInfo) = 0;

	/** Clears the selected bone on the preview component */
	virtual void ClearSelectedBone() = 0;

	/** Sets the selected socket on the preview component */
	virtual void SetSelectedSocket(const FSelectedSocketInfo& SocketInfo) = 0;

	/** Clears the selected socket on the preview component */
	virtual void ClearSelectedSocket() = 0;

	/** Sets the selected actor */
	virtual void SetSelectedActor(AActor* InActor) = 0;

	/** Clears the selected actor */
	virtual void ClearSelectedActor() = 0;

	/** Clears all selection on the preview component */
	virtual void DeselectAll() = 0;

	/** Registers a delegate to be called after the preview animation has been changed */
	virtual void RegisterOnAnimChanged(const FOnAnimChanged& Delegate) = 0;

	/** Unregisters a delegate to be called after the preview animation has been changed */
	virtual void UnregisterOnAnimChanged(FDelegateUserObject Thing) = 0;

	/** Registers a delegate to be called when the preview mesh is changed */
	virtual void RegisterOnPreviewMeshChanged(const FOnPreviewMeshChanged& Delegate) = 0;

	/** Unregisters a delegate to be called when the preview mesh is changed */
	virtual void UnregisterOnPreviewMeshChanged(FDelegateUserObject Thing) = 0;

	/** Registers a delegate to be called when the preview mesh's LOD has changed */
	virtual void RegisterOnLODChanged(const FSimpleDelegate& Delegate) = 0;

	/** Unregisters a delegate to be called when the preview mesh's LOD has changed */
	virtual void UnregisterOnLODChanged(FDelegateUserObject Thing) = 0;

	/** Broadcasts that we want to force an update, after preview mesh's LOD has changed */
	virtual void BroadcastOnLODChanged() = 0;

	/** Registers a delegate to be called when the preview mesh's morph targets has changed */
	virtual void RegisterOnMorphTargetsChanged(const FSimpleDelegate& Delegate) = 0;

	/** Unregisters a delegate to be called when the preview mesh's morph targets has changed */
	virtual void UnregisterOnMorphTargetsChanged(FDelegateUserObject Thing) = 0;

	/** Broadcasts that the preview mesh morph targets has changed */
	virtual void BroadcastOnMorphTargetsChanged() = 0;

	/** Registers a delegate to be called when the view is invalidated */
	virtual void RegisterOnInvalidateViews(const FSimpleDelegate& Delegate) = 0;

	/** Unregisters a delegate to be called when the view is invalidated */
	virtual void UnregisterOnInvalidateViews(FDelegateUserObject Thing) = 0;

	/** Registers a delegate to be called when the view should be focused */
	virtual void RegisterOnFocusViews(const FSimpleDelegate& Delegate) = 0;

	/** Unregisters a delegate to be called when the view should be focused */
	virtual void UnregisterOnFocusViews(FDelegateUserObject Thing) = 0;

	/** Registers a delegate to be called when the preview mesh is clicked */
	virtual void RegisterOnMeshClick(const FOnMeshClick& Delegate) = 0;

	/** Unregisters a delegate to be called when the preview mesh is clicked */
	virtual void UnregisterOnMeshClick(FDelegateUserObject Thing) = 0;

	/** Registers a delegate called when the bones selection has changed */
	virtual FDelegateHandle RegisterOnSelectedBonesChanged(const FOnSelectedBonesChanged& InDelegate) = 0;

	/** Unregisters a delegate called when the bones selection has changed */
	virtual void UnregisterOnSelectedBonesChanged(FDelegateHandle InHandle) = 0;
	
	/** Registers a delegate to be called when the currently selected socket has changed */
	virtual FDelegateHandle RegisterOnSelectedSocketChanged(const FOnSelectedSocketChanged& Delegate) = 0;

	/** Unregisters a delegate called when the currently selected socket has changed */
	virtual void UnregisterOnSelectedSocketChanged(FDelegateHandle InHandle) = 0;

	/** Registers a delegate to be called when all sockets/bones are deselected */
	virtual FDelegateHandle RegisterOnDeselectAll(const FSimpleDelegate& Delegate) = 0;

	/** unregisters a delegate called when all sockets/bones are deselected */
	virtual void UnregisterOnDeselectAll(FDelegateHandle InHandle) = 0;

	/** Broadcasts that the preview mesh was clicked */
	virtual bool BroadcastMeshClick(HActor* HitProxy, const FViewportClick& Click) = 0;

	/** Set the default mode this preview scene appears in. Optionally show the default mode. */
	virtual void SetDefaultAnimationMode(EPreviewSceneDefaultAnimationMode Mode, bool bShowNow = true) = 0;

	/** Show the mode specifed by SetDefaultAnimationMode() */
	virtual void ShowDefaultMode() = 0;

	/** Enable wind. Useful when simulating cloth. */
	virtual void EnableWind(bool bEnableWind) = 0;

	/** Check whether wind is enabled */
	virtual bool IsWindEnabled() const = 0;

	/** Set the wind strength */
	virtual void SetWindStrength(float InWindStrength) = 0;

	/** Get the wind strength */
	virtual float GetWindStrength() const = 0;

	/** Set the gravity scale */
	virtual void SetGravityScale(float InGravityScale) = 0;

	/** Get the gravity scale */
	virtual float GetGravityScale() const = 0;

	/** Get the currently selected actor */
	virtual AActor* GetSelectedActor() const = 0;

	/** Get the currently selected socket */
	virtual FSelectedSocketInfo GetSelectedSocket() const = 0;

	/** Get the currently selected bone index */
	virtual int32 GetSelectedBoneIndex() const = 0;

	/** Toggle the playback of animation, if any */
	virtual void TogglePlayback() = 0;

	/** Get the main actor */
	virtual AActor* GetActor() const = 0;

	/** Set the main actor */
	virtual void SetActor(AActor* InActor) = 0;

	/** Get whether or not to ignore mesh hit proxies */
	virtual bool AllowMeshHitProxies() const = 0;

	/** Set whether or not to ignore mesh hit proxies */
	virtual void SetAllowMeshHitProxies(bool bState) = 0;

	/** Register callback to be able to be notify when the select LOD is change */
	virtual void RegisterOnSelectedLODChanged(const FOnSelectedLODChanged &Delegate) = 0;
	/** Unregister callback to free up the ressources */
	virtual void UnRegisterOnSelectedLODChanged(FDelegateUserObject Thing) = 0;
	/** Broadcast select LOD changed */
	virtual void BroadcastOnSelectedLODChanged() = 0;

	/** Register callback for when the camera override is changed */
	virtual void RegisterOnCameraOverrideChanged(const FSimpleDelegate& Delegate) = 0;

	/** Unregister callback for when the camera override is changed */
	virtual void UnregisterOnCameraOverrideChanged(FDelegateUserObject Thing) = 0;

	/** Function to override the editor camera for this scene */
	virtual void SetCameraOverride(TSharedPtr<FEditorCameraController> NewCamera) = 0;

	/** Get the current camera override */
	virtual TSharedPtr<FEditorCameraController> GetCurrentCameraOverride() const = 0;

	/** Register a callback for just before the preview scene is ticked */
	virtual void RegisterOnPreTick(const FSimpleDelegate& Delegate) = 0;

	/** Unregister a callback for just before the preview scene is ticked */
	virtual void UnregisterOnPreTick(FDelegateUserObject Thing) = 0;

	/** Register a callback for just after the preview scene is ticked */
	virtual void RegisterOnPostTick(const FSimpleDelegate& Delegate) = 0;

	/** Unregister a callback for just after the preview scene is ticked */
	virtual void UnregisterOnPostTick(FDelegateUserObject Thing) = 0;

	/** setter/getter for can remove attach component */
	virtual void SetRemoveAttachedComponentFilter(const FOnRemoveAttachedComponentFilter& Delegate) = 0;
	virtual void ClearRemoveAttachedComponentFilter() = 0;

	/** Let the preview scene know that it should tick (because it is visible) */
	virtual void FlagTickable() = 0;
	/** Handle syncing selection with the skeleton tree */
	virtual void HandleSkeletonTreeSelectionChanged(const TArrayView<TSharedPtr<ISkeletonTreeItem>>& InSelectedItems, ESelectInfo::Type InSelectInfo) = 0;

	/** Replaces the current editable skeleton. This is not a safe operation unless you're working
	 * detached from other Persona components (eg. skeleton list).
	 */
	virtual void SetEditableSkeleton(TSharedPtr<IEditableSkeleton> InEditableSkeleton) = 0;
	
	/** Set the current custom animation playback speed */
	virtual void SetCustomAnimationSpeed(float InAnimationSpeed) = 0;

	/** Get the current custom animation playback speed */
	virtual float GetCustomAnimationSpeed() const = 0;

	/** Retrieve the current animation playback speed mode */
	virtual EAnimationPlaybackSpeeds::Type GetAnimationPlaybackSpeedMode() const = 0;

	/** Set the playback speed mode */
	virtual void SetAnimationPlaybackSpeedMode(EAnimationPlaybackSpeeds::Type InMode) = 0;

	/** Function to return whether the supplied playback speed is the current active one */
	virtual bool IsPlaybackSpeedSelected(int32 InPlaybackSpeedMode) const = 0;
};

