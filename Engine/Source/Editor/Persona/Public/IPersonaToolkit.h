// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class USkeleton;
class IEditableSkeleton;
class UDebugSkelMeshComponent;
class USkeletalMesh;
class UAnimBlueprint;
class UAnimationAsset;
class IPersonaPreviewScene;
struct FPersonaPreviewSceneDetailsOptions;
class IDetailLayoutBuilder;

/**
 * Reusable 'Persona' features for asset editors concerned with USkeleton-related assets
 */
class IPersonaToolkit
{
public:
	/** Virtual destructor */
	virtual ~IPersonaToolkit() = default;

	/** Get the skeleton that we are editing */
	virtual USkeleton* GetSkeleton() const = 0;

	/** Get the editable skeleton that we are editing */
	virtual TSharedPtr<IEditableSkeleton> GetEditableSkeleton() const = 0;

	/** Get the preview component that we are using */
	virtual UDebugSkelMeshComponent* GetPreviewMeshComponent() const = 0;

	/** Get the skeletal mesh that we are editing */
	virtual USkeletalMesh* GetMesh() const = 0;

	/** Set the skeletal mesh we are editing */
	virtual void SetMesh(USkeletalMesh* InSkeletalMesh) = 0;

	/** Get the anim blueprint that we are editing */
	virtual UAnimBlueprint* GetAnimBlueprint() const = 0;

	/** Get the animation asset that we are editing */
	virtual UAnimationAsset* GetAnimationAsset() const = 0;

	/** Set the animation asset we are editing */
	virtual void SetAnimationAsset(UAnimationAsset* InAnimationAsset) = 0;

	/** Get the preview scene that we are using */
	virtual TSharedRef<IPersonaPreviewScene> GetPreviewScene() const = 0;

	/**
	 * Get the preview mesh, according to context (mesh, skeleton or animation etc.).
	 * Note that this returns the preview mesh that is set for the asset currently being edited, which may be different
	 * from the one displayed in the viewport.
	 */
	virtual USkeletalMesh* GetPreviewMesh() const = 0;

	/** 
	 * Set the preview mesh, according to context (mesh, skeleton or animation etc.)
	 * Note that this sets the mesh in the asset and in the viewport (and may re-open the asset editor to apply this).
	 * @param	InSkeletalMesh			The mesh to set
	 * @param	bSetPreviewMeshInAsset	If true, the mesh will be written to the asset so it can be permanently saved. 
	 *									Otherwise the change is merely transient and will reset next time the editor is opened.
	 */
	virtual void SetPreviewMesh(USkeletalMesh* InSkeletalMesh, bool bSetPreviewMeshInAsset = true) = 0;

	/** Set the preview anim blueprint, used to preview sub layers in context. */
	virtual void SetPreviewAnimationBlueprint(UAnimBlueprint* InAnimBlueprint) = 0;

	/** Get the preview anim blueprint, used to preview sub layers in context. */
	virtual UAnimBlueprint* GetPreviewAnimationBlueprint() const = 0;

	/** Retrieve editor custom data. Return INDEX_NONE if the key is invalid */
	virtual int32 GetCustomData(const int32 Key) const { return INDEX_NONE; }
	
	/*
	 * Store the custom data using the key.
	 * Remark:
	 * The custom data memory should be clear when the editor is close by the user, this is not persistent data.
	 * Currently we use it to store the state of the editor UI to restore it properly when a refresh happen.
	 */
	virtual void SetCustomData(const int32 Key, const int32 CustomData) {}

	/** Callback to customize the Preview Scene Settings details tab. Different asset types require unique options. */
	virtual void CustomizeSceneSettings(IDetailLayoutBuilder& DetailBuilder) {};
	
	/** Get the context in which this toolkit is being used (usually the class name of the asset) */
	virtual FName GetContext() const = 0;

	/** Returns true if the preview mesh can use skeletal meshes that don't share the same skeleton as the one being edited. */
	virtual bool CanPreviewMeshUseDifferentSkeleton() const = 0;
};
