// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Logging/TokenizedMessage.h"

class FEditorViewportClient;
class IPinnedCommandList;

DECLARE_DELEGATE_RetVal_TwoParams(FReply, FPersonaViewportKeyDownDelegate, const FGeometry&, const FKeyEvent&);

/** Opaque state interface for saving and restoring viewport state */
struct IPersonaViewportState
{
};

struct FPersonaViewportNotificationOptions
{
	FPersonaViewportNotificationOptions() = default;
	
	// in general on GetVisibility should always be provided
	explicit FPersonaViewportNotificationOptions(const TAttribute<EVisibility> InOnGetVisibility)
		:OnGetVisibility(InOnGetVisibility) {};

	TAttribute<EVisibility> OnGetVisibility;
	TAttribute<const FSlateBrush*> OnGetBrushOverride;
};

/** Abstract viewport that can save and restore state */
class IPersonaViewport : public SCompoundWidget
{
public:
	/** Save the viewport state */
	virtual TSharedRef<IPersonaViewportState> SaveState() const = 0;

	/** Restore the viewport state */
	virtual void RestoreState(TSharedRef<IPersonaViewportState> InState) = 0;

	/** Get the viewport client contained within this viewport */
	virtual FEditorViewportClient& GetViewportClient() const = 0;

	/** Get the pinned commands list for this viewport */
	virtual TSharedRef<IPinnedCommandList> GetPinnedCommandList() const = 0;

	/** 
	 * Add a notification widget 
	 * @param	InSeverity				The severity of the message
	 * @param	InCanBeDismissed		Whether the message can be manually dismissed
	 * @param	InNotificationWidget	The widget showing the notification
	 * @param	InOptions				Specify GetVisibility callback and other optional callback overrides
	 * @return the widget containing the notification
	 */
	virtual TWeakPtr<SWidget> AddNotification(TAttribute<EMessageSeverity::Type> InSeverity, TAttribute<bool> InCanBeDismissed, const TSharedRef<SWidget>& InNotificationWidget, FPersonaViewportNotificationOptions InOptions) = 0;
	
	/** 
	* Remove a notification widget 
	* @param	InContainingWidget		The containing widget returned from AddNotification()
	*/
	virtual void RemoveNotification(const TWeakPtr<SWidget>& InContainingWidget) = 0;

	/** 
	 * Adds an extender to the viewport's toolbar
	 * @param MenuToExtend        The name of the toolbar menu to extend
	 * @param MenuBuilderDelegate The delegate to use when filling the menu
	 */
	 UE_DEPRECATED(5.7, "Please extend the menu using UToolMenus instead.")
	virtual void AddToolbarExtender(FName MenuToExtend, FMenuExtensionDelegate MenuBuilderDelegate) {}

	/**
	 * Returns the delegate broadcasted two in case we have an unhandled key
	 */
	virtual FPersonaViewportKeyDownDelegate& GetKeyDownDelegate() = 0;

	/** Overlay a widget over the whole viewport */
	virtual void AddOverlayWidget( TSharedRef<SWidget> InOverlaidWidget, int32 ZOrder=INDEX_NONE ) = 0;

	/** Remove an overlay widget from the viewport */
	virtual void RemoveOverlayWidget( TSharedRef<SWidget> InOverlaidWidget ) = 0;
};
