// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Features/IModularFeature.h"
#include "Modules/ModuleInterface.h"

enum class EConstraintTransformComponentFlags : uint8;

class IPhysicsAssetRenderInterface : public IModuleInterface, public IModularFeature
{
	// virtual ~IPhysicsAssetRenderInterface() = default;

public:
	static FName GetModularFeatureName()
	{
		static FName FeatureName = FName(TEXT("PhysicsAssetRenderInterface"));
		return FeatureName;
	}

	virtual void DebugDraw(class USkeletalMeshComponent* const SkeletalMeshComponent, class UPhysicsAsset* const PhysicsAsset, class FPrimitiveDrawInterface* PDI) = 0;
	virtual void DebugDrawBodies(class USkeletalMeshComponent* const SkeletalMeshComponent, class UPhysicsAsset* const PhysicsAsset, FPrimitiveDrawInterface* PDI, const FColor& PrimitiveColorOverride) = 0;
	virtual void DebugDrawConstraints(class USkeletalMeshComponent* const SkeletalMeshComponent, class UPhysicsAsset* const PhysicsAsset, FPrimitiveDrawInterface* PDI) = 0;

	virtual void SaveConfig() = 0;

	virtual void ToggleShowAllBodies(class UPhysicsAsset* const PhysicsAsset) = 0;
	virtual void ToggleShowAllConstraints(class UPhysicsAsset* const PhysicsAsset) = 0;
	virtual bool AreAnyBodiesHidden(class UPhysicsAsset* const PhysicsAsset) = 0;
	virtual bool AreAnyConstraintsHidden(class UPhysicsAsset* const PhysicsAsset) = 0;

	virtual EConstraintTransformComponentFlags GetConstraintViewportManipulationFlags(class UPhysicsAsset* const PhysicsAsset) = 0;
	virtual bool IsDisplayingConstraintTransformComponentRelativeToDefault(class UPhysicsAsset* const PhysicsAsset, const EConstraintTransformComponentFlags ComponentFlags) = 0;
	virtual void SetDisplayConstraintTransformComponentRelativeToDefault(class UPhysicsAsset* const PhysicsAsset, const EConstraintTransformComponentFlags ComponentFlags, const bool bShouldDisplayRelativeToDefault) = 0;
};
