// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Framework/TypedElementHandle.h"
#include "WorkflowOrientedApp/WorkflowCentricApplication.h"

#define UE_API PERSONA_API

class ISkeletonTreeItem;
class UTypedElementSelectionSet;

DECLARE_MULTICAST_DELEGATE_OneParam(FOnAttachToolkit, const TSharedRef<IToolkit>&);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnDetachToolkit, const TSharedRef<IToolkit>&);

/** Persona asset editor toolkit wrapper, used to auto inject the persona editor mode manager */
class FPersonaAssetEditorToolkit : public FWorkflowCentricApplication
{
public:
	/** FAssetEditorToolkit interface  */
	UE_API virtual void CreateEditorModeManager() override;

	// IToolkitHost Interface
	UE_API virtual void OnToolkitHostingStarted(const TSharedRef<class IToolkit>& Toolkit) override;
	UE_API virtual void OnToolkitHostingFinished(const TSharedRef<class IToolkit>& Toolkit) override;

	// Returns the currently hosted toolkit. Can be invalid if no toolkit is being hosted.
	TSharedPtr<IToolkit> GetHostedToolkit() const { return HostedToolkit; }
	FOnAttachToolkit& GetOnAttachToolkit() { return OnAttachToolkit; }
	FOnDetachToolkit& GetOnDetachToolkit() { return OnDetachToolkit; }
	
	UE_API virtual void OnClose() override;

protected:
	UE_API virtual void OnEditorModeIdChanged(const FEditorModeID& ModeChangedID, bool bIsEnteringMode);
	
	UE_API virtual void RouteTreeSelectionToModeManager(const TConstArrayView<TSharedPtr<ISkeletonTreeItem>>& InItems);

	/** Called when the editor's TypedElementSelectionSet changes. Override to update details views, etc. */
	UE_API virtual void OnSelectionSetChanged(const UTypedElementSelectionSet* InSelectionSet);

protected:
	FOnAttachToolkit OnAttachToolkit;
	FOnDetachToolkit OnDetachToolkit;
	
	// The toolkit we're currently hosting.
	TSharedPtr<IToolkit> HostedToolkit;

private:
	FDelegateHandle SelectionSetChangedHandle;

	/** Element handles routed by the last call to RouteTreeSelectionToModeManager, so we only deselect our own elements. */
	TArray<FTypedElementHandle> RoutedSelectionHandles;
};

#undef UE_API
