// Copyright Epic Games, Inc. All Rights Reserved.

#include "PersonaCommonCommands.h"

#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"
#include "InputCoreTypes.h"

#define LOCTEXT_NAMESPACE "PersonaCommonCommands"

void FPersonaCommonCommands::RegisterCommands()
{
	UI_COMMAND(TogglePlay, "Play/Pause", "Play or pause the current animation", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::SpaceBar));
}

const FPersonaCommonCommands& FPersonaCommonCommands::Get()
{
	return TCommands<FPersonaCommonCommands>::Get();
}

#undef LOCTEXT_NAMESPACE
