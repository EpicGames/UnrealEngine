// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/Commands.h"
#include "HAL/Platform.h"
#include "Internationalization/Internationalization.h"
#include "Styling/AppStyle.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

class FUICommandInfo;

class PERSONA_API FPersonaCommonCommands : public TCommands<FPersonaCommonCommands>
{
public:
	FPersonaCommonCommands()
		: TCommands<FPersonaCommonCommands>(TEXT("PersonaCommon"), NSLOCTEXT("Contexts", "PersonaCommon", "Persona Common"), NAME_None, FAppStyle::GetAppStyleSetName())
	{
	}

	virtual void RegisterCommands() override;
	FORCENOINLINE static const FPersonaCommonCommands& Get();

public:
	// Toggle playback
	TSharedPtr<FUICommandInfo> TogglePlay;
};
