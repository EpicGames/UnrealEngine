// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IAnimationEditor.h"
#include "Animation/AnimData/AttributeIdentifier.h"

class SAnimationEditorViewportTabBody;
struct FTabId;
class UAnimSequenceBase;
class ITimeSliderController;

// Called when a blend profile is selected in the blend profile tab
DECLARE_MULTICAST_DELEGATE_OneParam(FOnSelectBlendProfile, class UBlendProfile*);

// Called when the preview viewport is created
DECLARE_MULTICAST_DELEGATE_OneParam(FOnCreateViewport, TWeakPtr<class SAnimationEditorViewportTabBody>);

// Called to invoke a specified tab
DECLARE_DELEGATE_OneParam(FOnInvokeTab, const FTabId&);

// Called to display a selected object in a details panel
DECLARE_DELEGATE_OneParam(FOnObjectsSelected, const TArray<UObject*>& /*InObjects*/);

// Called to display a selected object in a details panel
DECLARE_DELEGATE_OneParam(FOnObjectSelected, UObject* /*InObject*/);

// Called to replace the currently edited asset or open it (depending on context)
DECLARE_DELEGATE_OneParam(FOnOpenNewAsset, UObject* /* InAsset */);

// Called to get an object (used by the asset details panel)
DECLARE_DELEGATE_RetVal(UObject*, FOnGetAsset);

// Called to invoke the curve editor
DECLARE_DELEGATE_ThreeParams(FOnEditCurves, UAnimSequenceBase* /*InAnimSequence*/, const TArray<IAnimationEditor::FCurveEditInfo>& /*InCurveInfo*/, const TSharedPtr<ITimeSliderController>& /*InTimeSliderController*/);

// Called when the user navigates with pageup
DECLARE_DELEGATE(FOnBlendSpaceNavigateUp);

// Called when the user navigates with pagedown
DECLARE_DELEGATE(FOnBlendSpaceNavigateDown);

// Called when the blendspace canvas is double clicked
DECLARE_DELEGATE(FOnBlendSpaceCanvasDoubleClicked);

// Called when a blendspace sample point is double clicked
DECLARE_DELEGATE_OneParam(FOnBlendSpaceSampleDoubleClicked, int32 /*SampleIndex*/);

// Called when a blendspace sample point is removed
DECLARE_DELEGATE_OneParam(FOnBlendSpaceSampleRemoved, const int32 /*SampleIndex*/);

// Called when a blendspace sample point is added. Returns the new SampleIndex
DECLARE_DELEGATE_RetVal_ThreeParams(int32, FOnBlendSpaceSampleAdded, UAnimSequence* /*InSequence*/, const FVector& /*InSamplePoint*/, bool /*bRunAnalysis*/);

// Called when a blendspace sample point is added
DECLARE_DELEGATE_ThreeParams(FOnBlendSpaceSampleDuplicated, int32 /*SampleIndex*/, const FVector& /*InSamplePoint*/, bool /*bRunAnalysis*/);

// Called when a blendspace sample point is replaced
DECLARE_DELEGATE_TwoParams(FOnBlendSpaceSampleReplaced, const int32 /*SampleIndex*/, UAnimSequence* /*InSequence*/);

// Called to get the overridden name of a blend sample
DECLARE_DELEGATE_RetVal_OneParam(FName, FOnGetBlendSpaceSampleName, int32 /*SampleIndex*/);

// Called to extend a sample's tooltip
DECLARE_DELEGATE_RetVal_OneParam(TSharedRef<SWidget>, FOnExtendBlendSpaceSampleTooltip, int32 /*SampleIndex*/);

// Called to let the blendspace editor UI set the preview position of an external blendspace node
DECLARE_DELEGATE_OneParam(FOnSetBlendSpacePreviewPosition, FVector /*InBlendSample*/);

// Called by the curve picker to indicate that a curve was chosen by the user
DECLARE_DELEGATE_OneParam(FOnCurvePicked, const FName& /*PickedName*/);

// Called by the curve picker to filter out curve names that are displayed to the user
DECLARE_DELEGATE_RetVal_OneParam(bool, FIsCurveNameMarkedForExclusion, const FName& /*CurveName*/);

// Called by the notify picker to indicate that a notify was chosen by the user
DECLARE_DELEGATE_OneParam(FOnNotifyPicked, const FName& /*PickedName*/);

// Called by the curve picker to indicate the curves chosen by the user
DECLARE_DELEGATE_OneParam(FOnCurvesPicked, const TConstArrayView<FName> /*Picked Curves*/);

// Called by the attribute picker to indicate the attributes chosen by the user
DECLARE_DELEGATE_OneParam(FOnAttributesPicked, const TConstArrayView<FAnimationAttributeIdentifier>);