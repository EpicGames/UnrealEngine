// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "Modules/ModuleInterface.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "Animation/AnimSequence.h"
#include "Editor.h"
#include "PersonaDelegates.h"
#include "Factories/FbxImportUI.h"
#include "SPersonaToolBox.h"
#include "IBlendProfilePickerExtender.h"

#define UE_API PERSONA_API

class IMorphTargetViewer;
class FBlueprintEditor;
class IDetailsView;
class IEditableSkeleton;
class IPersonaPreviewScene;
class IPersonaToolkit;
class UAnimBlueprint;
class USkeletalMeshComponent;
class UPhysicsAsset;
class IPinnedCommandList;
class FWorkflowAllowedTabSet;
class IAssetFamily;
class FWorkflowTabFactory;
class UBlendSpace;
class IAnimSequenceCurveEditor;
class IAnimationEditor;
class IDetailLayoutBuilder;
class FPreviewSceneDescriptionCustomization;
struct FAnimAssetFindReplaceConfig;

extern const FName PersonaAppName;

// Editor mode constants
struct FPersonaEditModes
{
	/** Selection/manipulation of bones & sockets */
	UE_API const static FEditorModeID SkeletonSelection;
};

DECLARE_DELEGATE_TwoParams(FIsRecordingActive, USkeletalMeshComponent* /*Component*/, bool& /* bIsRecording */);
DECLARE_DELEGATE_OneParam(FRecord, USkeletalMeshComponent* /*Component*/);
DECLARE_DELEGATE_OneParam(FStopRecording, USkeletalMeshComponent* /*Component*/);
DECLARE_DELEGATE_TwoParams(FGetCurrentRecording, USkeletalMeshComponent* /*Component*/, class UAnimSequence*& /* OutRecording */);
DECLARE_DELEGATE_TwoParams(FGetCurrentRecordingTime, USkeletalMeshComponent* /*Component*/, float& /* OutTime */);
DECLARE_DELEGATE_TwoParams(FTickRecording, USkeletalMeshComponent* /*Component*/, float /* DeltaSeconds */);

/** Called back when a viewport is created */
DECLARE_DELEGATE_OneParam(FOnViewportCreated, const TSharedRef<class IPersonaViewport>&);

/** Called back when a details panel is created */
DECLARE_DELEGATE_OneParam(FOnDetailsCreated, const TSharedRef<IDetailsView>&);

/** Called back when an anim sequence browser is created */
DECLARE_DELEGATE_OneParam(FOnAnimationSequenceBrowserCreated, const TSharedRef<IAnimationSequenceBrowser>&);

/** Called back when a Persona preview scene is created */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnPreviewSceneCreated, const TSharedRef<IPersonaPreviewScene>&);

/** Called back when a Persona preview scene settings are customized */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnPreviewSceneSettingsCustomized, IDetailLayoutBuilder& DetailBuilder);

/** Called back to register tabs */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnRegisterTabs, FWorkflowAllowedTabSet&, TSharedPtr<FAssetEditorToolkit>);

/** Called back to register common layout extensions */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnRegisterLayoutExtensions, FLayoutExtender&);

/** Initialization parameters for persona toolkits */
struct FPersonaToolkitArgs
{
	/** 
	 * Delegate called when the preview scene is created, used to setup the scene 
	 * If this is not set, then a default scene will be set up.
	 */
	FOnPreviewSceneCreated::FDelegate OnPreviewSceneCreated;

	/** Whether to create a preview scene */
	bool bCreatePreviewScene = true;

	/** 
	 * Delegate called when the preview scene settings are being customized, supplies the IDetailLayoutBuilder
	 * for the user to customize the layout however they wish. */
	FOnPreviewSceneSettingsCustomized::FDelegate OnPreviewSceneSettingsCustomized;

	/**
	 * Set to true if the preview mesh can be associated with a skeleton different from the one being inspected
	 * by Persona. Used for editors that are mostly skeleton agnostic.
	 */
	bool bPreviewMeshCanUseDifferentSkeleton = false;

	FPersonaToolkitArgs() = default;
};

struct FAnimDocumentArgs
{
	FAnimDocumentArgs(const TSharedRef<class IPersonaPreviewScene>& InPreviewScene, const TSharedRef<class IPersonaToolkit>& InPersonaToolkit, const TSharedRef<class IEditableSkeleton>& InEditableSkeleton, FSimpleMulticastDelegate& InOnSectionsChanged)
		: PreviewScene(InPreviewScene)
		, PersonaToolkit(InPersonaToolkit)
		, EditableSkeleton(InEditableSkeleton)
		, OnSectionsChanged(InOnSectionsChanged)
	{}

	/** Required args */
	TWeakPtr<class IPersonaPreviewScene> PreviewScene;
	TWeakPtr<class IPersonaToolkit> PersonaToolkit;
	TWeakPtr<class IEditableSkeleton> EditableSkeleton;
	FSimpleMulticastDelegate& OnSectionsChanged;

	/** Optional args */
	FOnObjectsSelected OnDespatchObjectsSelected;
	FOnInvokeTab OnDespatchInvokeTab;
	FSimpleDelegate OnDespatchSectionsChanged;
};

struct FBlendSpaceEditorArgs
{
	// Called when a blendspace sample point is removed
	FOnBlendSpaceSampleRemoved OnBlendSpaceSampleRemoved;

	// Called when a blendspace sample point is added
	FOnBlendSpaceSampleAdded OnBlendSpaceSampleAdded;
	
	// Called when a blendspace sample point is replaced
	FOnBlendSpaceSampleReplaced OnBlendSpaceSampleReplaced;

	// Called when the blendspace canvas is double clicked
	FOnBlendSpaceNavigateUp OnBlendSpaceNavigateUp;

	// Called when the blendspace canvas is double clicked
	FOnBlendSpaceNavigateDown OnBlendSpaceNavigateDown;

	// Called when the blendspace canvas is double clicked
	FOnBlendSpaceCanvasDoubleClicked OnBlendSpaceCanvasDoubleClicked;

	// Called when a blendspace sample point is double clicked
	FOnBlendSpaceSampleDoubleClicked OnBlendSpaceSampleDoubleClicked;

	// Called to get the overridden name of a blend sample
	FOnGetBlendSpaceSampleName OnGetBlendSpaceSampleName;

	// Allows the target preview position to be programmatically driven
	TAttribute<FVector> PreviewPosition;

	// Allows the current position to be programmatically driven
	TAttribute<FVector> PreviewFilteredPosition;

	// Allows an external widget to be inserted into a sample's tooltip
	FOnExtendBlendSpaceSampleTooltip OnExtendSampleTooltip;

	// Allows preview position to drive external node
	FOnSetBlendSpacePreviewPosition OnSetPreviewPosition;

	// Status bar to display hint messages in
	FName StatusBarName = TEXT("AssetEditor.AnimationEditor.MainMenu");
};

struct FBlendSpacePreviewArgs
{
	TAttribute<const UBlendSpace*> PreviewBlendSpace;

	// Allows the target preview position to be programmatically driven
	TAttribute<FVector> PreviewPosition;

	// Allows the current preview position to be programmatically driven
	TAttribute<FVector> PreviewFilteredPosition;

	// Called to get the overridden name of a blend sample
	FOnGetBlendSpaceSampleName OnGetBlendSpaceSampleName;
};

/** Places that viewport text can be placed */
enum class EViewportCorner : uint8
{
	TopLeft,
	TopRight,
	BottomLeft,
	BottomRight,
};

/** Delegate used to provide custom text for the viewport corners */
DECLARE_DELEGATE_RetVal_OneParam(FText, FOnGetViewportText, EViewportCorner /*InViewportCorner*/);

DECLARE_DELEGATE_RetVal(TOptional<bool>, FAnimationScrubPanel_IsRecordingActive);
DECLARE_DELEGATE_RetVal(TOptional<EVisibility>, FAnimationScrubPanel_GetRecordingVisibility);
DECLARE_DELEGATE_RetVal(bool, FAnimationScrubPanel_StartRecording)
DECLARE_DELEGATE_RetVal(bool, FAnimationScrubPanel_StopRecording);
DECLARE_DELEGATE_RetVal(TOptional<int32>, FAnimationScrubPanel_GetPlaybackMode);
DECLARE_DELEGATE_RetVal_OneParam(bool, FAnimationScrubPanel_SetPlaybackMode, int32 /* playback mode */);
DECLARE_DELEGATE_RetVal(TOptional<float>, FAnimationScrubPanel_GetPlaybackTime);
DECLARE_DELEGATE_RetVal_TwoParams(bool, FAnimationScrubPanel_SetPlaybackTime, float /* InTime */, bool /* StopPlayback */);
DECLARE_DELEGATE_RetVal(bool, FAnimationScrubPanel_StepForward);
DECLARE_DELEGATE_RetVal(bool, FAnimationScrubPanel_StepBackward);
DECLARE_DELEGATE_RetVal(TOptional<bool>, FAnimationScrubPanel_GetIsLooping);
DECLARE_DELEGATE_RetVal_OneParam(bool, FAnimationScrubPanel_SetIsLooping, bool /* IsLooping */);
DECLARE_DELEGATE_RetVal(TOptional<FVector2f>, FAnimationScrubPanel_GetPlaybackTimeRange);
DECLARE_DELEGATE_RetVal(TOptional<uint32>, FAnimationScrubPanel_GetNumberOfKeys);

/** Struct to provide delegates to change the timeline behavior for certain clients */
struct FAnimationScrubPanelDelegates
{
	/** Determines if recording is currently active */
	FAnimationScrubPanel_IsRecordingActive IsRecordingActiveDelegate;

	/** Determines if the recording button should be visible */
	FAnimationScrubPanel_GetRecordingVisibility GetRecordingVisibilityDelegate;

	/** Starts the recording */
	FAnimationScrubPanel_StartRecording StartRecordingDelegate;

	/** Stops the recording */	
	FAnimationScrubPanel_StopRecording StopRecordingDelegate;

	/** Determines if the playback is currently active */
	FAnimationScrubPanel_GetPlaybackMode GetPlaybackModeDelegate;

	/** Sets the playback mode */
	FAnimationScrubPanel_SetPlaybackMode SetPlaybackModeDelegate;

	/** Returns the current playback time */
	FAnimationScrubPanel_GetPlaybackTime GetPlaybackTimeDelegate;

	/** Seeks to a given time */
	FAnimationScrubPanel_SetPlaybackTime SetPlaybackTimeDelegate;

	/** steps on frame / key forward */
	FAnimationScrubPanel_StepForward StepForwardDelegate;

	/** steps on frame / key backward */
	FAnimationScrubPanel_StepBackward StepBackwardDelegate;

	/** Returns the current playback looping flag */
	FAnimationScrubPanel_GetIsLooping GetIsLoopingDelegate;

	/** Sets the playback looping flag */
	FAnimationScrubPanel_SetIsLooping SetIsLoopingDelegate;

	/** Returns the playback time range */
	FAnimationScrubPanel_GetPlaybackTimeRange GetPlaybackTimeRangeDelegate;

	/** Returns the number of keys for this playback */
	FAnimationScrubPanel_GetNumberOfKeys GetNumberOfKeysDelegate;
};

/** Arguments used to create a persona viewport tab */
struct FPersonaViewportArgs
{
	FPersonaViewportArgs(const TSharedRef<class IPersonaPreviewScene>& InPreviewScene)
		: PreviewScene(InPreviewScene)
		, ContextName(NAME_None)
		, bShowShowMenu(true)
		, bShowLODMenu(true)
		, bShowPlaySpeedMenu(true)
		, bShowTimeline(true)
		, bShowStats(true)
		, bAlwaysShowTransformToolbar(false)
		, bShowFloorOptions(true)
		, bShowTurnTable(true)
		, bShowPhysicsMenu(false)
	{}

	/** Required args */
	TSharedRef<class IPersonaPreviewScene> PreviewScene;

	/** Optional blueprint editor that we can be embedded in */
	TSharedPtr<class FBlueprintEditor> BlueprintEditor;

	/** Delegate fired when the viewport is created */
	FOnViewportCreated OnViewportCreated;
	
	/** Menu extenders */
	TArray<TSharedPtr<FExtender>> Extenders;

	/** Delegate used to customize viewport corner text */
	FOnGetViewportText OnGetViewportText;

	/** The context in which we are constructed. Used to persist various settings. */
	FName ContextName;

	/** Whether to show the 'Show' menu */
	bool bShowShowMenu;

	/** Whether to show the 'LOD' menu */
	bool bShowLODMenu;

	/** Whether to show the 'Play Speed' menu */
	bool bShowPlaySpeedMenu;

	/** Whether to show the animation timeline */
	bool bShowTimeline;

	/** Whether to show in-viewport stats */
	bool bShowStats;

	/** Whether we should always show the transform toolbar for this viewport */
	bool bAlwaysShowTransformToolbar;

	/** Whether to show options relating to floor height */
	bool bShowFloorOptions;

	/** Whether to show options relating to turntable */
	bool bShowTurnTable;

	/** Whether to show options relating to physics */
	bool bShowPhysicsMenu;

	/** A structure providing delegates to change the timeline behavior */
	FAnimationScrubPanelDelegates TimelineDelegates;
};

/**
 * Some settings for the detail customization inside the advanced preview scene settings tab.
 * This allows you to hide specific categories.
 */
struct FPersonaAdvancedPreviewSceneTabSettings
{
	bool bHideAnimationCategory = false;
	bool bHideAnimationBlueprintCategory = false;
	bool bHideMeshCategory = false;
	bool bHidePhysicsCategory = false;
	bool bHideAdditionalMeshesCategory = false;
};

/**
 * Persona module manages the lifetime of all instances of Persona editors.
 */
class FPersonaModule : public IModuleInterface,
	public IHasMenuExtensibility
{
public:
	/**
	 * Called right after the module's DLL has been loaded and the module object has been created
	 */
	virtual void StartupModule();

	/**
	 * Called before the module is unloaded, right before the module object is destroyed.
	 */
	virtual void ShutdownModule();

	/** Create a re-usable toolkit that multiple asset editors that are concerned with USkeleton-related data can use */
	virtual TSharedRef<IPersonaToolkit> CreatePersonaToolkit(UObject* InAsset, const FPersonaToolkitArgs& PersonaToolkitArgs, USkeleton* InSkeleton = nullptr) const;
	virtual TSharedRef<IPersonaToolkit> CreatePersonaToolkit(USkeleton* InSkeleton, const FPersonaToolkitArgs& PersonaToolkitArgs) const;
	virtual TSharedRef<IPersonaToolkit> CreatePersonaToolkit(UAnimationAsset* InAnimationAsset, const FPersonaToolkitArgs& PersonaToolkitArgs) const;
	virtual TSharedRef<IPersonaToolkit> CreatePersonaToolkit(USkeletalMesh* InSkeletalMesh, const FPersonaToolkitArgs& PersonaToolkitArgs) const;
	virtual TSharedRef<IPersonaToolkit> CreatePersonaToolkit(UAnimBlueprint* InAnimBlueprint, const FPersonaToolkitArgs& PersonaToolkitArgs) const;
	virtual TSharedRef<IPersonaToolkit> CreatePersonaToolkit(UPhysicsAsset* InPhysicsAsset, const FPersonaToolkitArgs& PersonaToolkitArgs) const;

	/** Create an asset family for the supplied persona asset */
	virtual TSharedRef<IAssetFamily> CreatePersonaAssetFamily(const UObject* InAsset) const;

	/** Broadcast event that all asset families need to change */
	virtual void BroadcastAssetFamilyChange() const;

	/** Record that an asset was opened (forward to relevant asset families) */
	virtual void RecordAssetOpened(const FAssetData& InAssetData) const;
	
	/** Create a shortcut widget for an asset family */
	virtual TSharedRef<SWidget> CreateAssetFamilyShortcutWidget(const TSharedRef<class FWorkflowCentricApplication>& InHostingApp, const TSharedRef<class IAssetFamily>& InAssetFamily) const;

	/** Create a details panel tab factory */
	virtual TSharedRef<FWorkflowTabFactory> CreateDetailsTabFactory(const TSharedRef<class FWorkflowCentricApplication>& InHostingApp, FOnDetailsCreated InOnDetailsCreated) const;

	/** Create a persona viewport tab factory */
	virtual TSharedRef<FWorkflowTabFactory> CreatePersonaViewportTabFactory(const TSharedRef<class FWorkflowCentricApplication>& InHostingApp, const FPersonaViewportArgs& InArgs) const;

	/** Register 4 Persona viewport tab factories */
	virtual void RegisterPersonaViewportTabFactories(FWorkflowAllowedTabSet& TabSet, const TSharedRef<class FWorkflowCentricApplication>& InHostingApp, const FPersonaViewportArgs& InArgs) const;

	/** Create an anim notifies tab factory */
	virtual TSharedRef<FWorkflowTabFactory> CreateAnimNotifiesTabFactory(const TSharedRef<class FWorkflowCentricApplication>& InHostingApp, const TSharedRef<class IEditableSkeleton>& InEditableSkeleton, FOnObjectsSelected InOnObjectsSelected) const;

	/** Create a skeleton curve viewer tab factory */
	virtual TSharedRef<FWorkflowTabFactory> CreateCurveViewerTabFactory(const TSharedRef<class FWorkflowCentricApplication>& InHostingApp, const TSharedPtr<class IEditableSkeleton>& InEditableSkeleton, const TSharedRef<class IPersonaPreviewScene>& InPreviewScene, FOnObjectsSelected InOnObjectsSelected, bool bInReadOnlyCurveViewer = true) const;
	
	/** Create a skeleton curve metadata editor tab factory */
	virtual TSharedRef<FWorkflowTabFactory> CreateCurveMetadataEditorTabFactory(const TSharedRef<class FWorkflowCentricApplication>& InHostingApp, UObject* InMetadataHost, const TSharedRef<class IPersonaPreviewScene>& InPreviewScene, FOnObjectsSelected InOnObjectsSelected) const;

	/** Create a retarget sources tab factory */
	virtual TSharedRef<FWorkflowTabFactory> CreateRetargetSourcesTabFactory(const TSharedRef<class FWorkflowCentricApplication>& InHostingApp, const TSharedRef<class IEditableSkeleton>& InEditableSkeleton, FSimpleMulticastDelegate& InOnPostUndo) const;

	/** Create a tab factory used to configure preview scene settings */
	virtual TSharedRef<FWorkflowTabFactory> CreateAdvancedPreviewSceneTabFactory(const TSharedRef<class FWorkflowCentricApplication>& InHostingApp, const TSharedRef<IPersonaPreviewScene>& InPreviewScene) const;
	virtual TSharedRef<FWorkflowTabFactory> CreateAdvancedPreviewSceneTabFactory(const TSharedRef<class FWorkflowCentricApplication>& InHostingApp, const TSharedRef<IPersonaPreviewScene>& InPreviewScene, const FPersonaAdvancedPreviewSceneTabSettings& DetailCustomizationSettings) const;

	/** Create a tab factory for the animation asset browser */
	virtual TSharedRef<FWorkflowTabFactory> CreateAnimationAssetBrowserTabFactory(const TSharedRef<class FWorkflowCentricApplication>& InHostingApp, const TSharedRef<IPersonaToolkit>& InPersonaToolkit, FOnOpenNewAsset InOnOpenNewAsset, FOnAnimationSequenceBrowserCreated InOnAnimationSequenceBrowserCreated, bool bInShowHistory) const;

	/** Create a tab factory for editing a single object (like an animation asset) */
	virtual TSharedRef<FWorkflowTabFactory> CreateAssetDetailsTabFactory(const TSharedRef<class FWorkflowCentricApplication>& InHostingApp, FOnGetAsset InOnGetAsset, FOnDetailsCreated InOnDetailsCreated) const;

	/** Create a tab factory for previewing morph targets */
	virtual TSharedRef<FWorkflowTabFactory> CreateMorphTargetTabFactory(const TSharedRef<class FWorkflowCentricApplication>& InHostingApp, const TSharedRef<IPersonaPreviewScene>& InPreviewScene, FSimpleMulticastDelegate& OnPostUndo) const;

	/** Create a tab factory for previewing morph targets */
	virtual TSharedRef<FWorkflowTabFactory> CreateMorphTargetTabFactory(const TSharedRef<class FWorkflowCentricApplication>& InHostingApp, TSharedRef<IMorphTargetViewer> InViewerWidget) const;

	/** Create the default morph target viewer widget*/
	virtual TSharedRef<IMorphTargetViewer> CreateDefaultMorphTargetViewerWidget(const TSharedRef<IPersonaPreviewScene>& InPreviewScene, FSimpleMulticastDelegate& OnPostUndo) const;
	
	/** Create a tab factory for editing anim blueprint preview & defaults */
	virtual TSharedRef<FWorkflowTabFactory> CreateAnimBlueprintPreviewTabFactory(const TSharedRef<class FBlueprintEditor>& InBlueprintEditor, const TSharedRef<IPersonaPreviewScene>& InPreviewScene) const;

	/** Create a tab factory for the pose watch manager */
	virtual TSharedRef<FWorkflowTabFactory> CreatePoseWatchTabFactory(const TSharedRef<class FBlueprintEditor>& InBlueprintEditor) const;

	/** Create a tab factory for editing anim blueprint parent overrides */
	virtual TSharedRef<FWorkflowTabFactory> CreateAnimBlueprintAssetOverridesTabFactory(const TSharedRef<class FBlueprintEditor>& InBlueprintEditor, UAnimBlueprint* InAnimBlueprint, FSimpleMulticastDelegate& InOnPostUndo) const;

	/** Create a tab factory for editing slot names and groups */
	virtual TSharedRef<FWorkflowTabFactory> CreateSkeletonSlotNamesTabFactory(const TSharedRef<class FWorkflowCentricApplication>& InHostingApp, const TSharedRef<class IEditableSkeleton>& InEditableSkeleton, FOnObjectSelected InOnObjectSelected) const;

	/** Create a toolbox tab factory */
	virtual TSharedRef<FWorkflowTabFactory> CreatePersonaToolboxTabFactory(const TSharedRef<class FPersonaAssetEditorToolkit>& InHostingApp) const;

	/** Create a attributes viewer tab factory */
	virtual TSharedRef<FWorkflowTabFactory> CreateAttributeViewerTabFactory(const TSharedRef<class FWorkflowCentricApplication>& InHostingApp, const TSharedRef<class IPersonaPreviewScene>& InPreviewScene) const;

	/** Create a widget to preview a blendspace */
	virtual TSharedRef<SWidget> CreateBlendSpacePreviewWidget(const FBlendSpacePreviewArgs& InArgs) const;

	/** Create a widget to edit a blendspace */
	virtual TSharedRef<SWidget> CreateBlendSpaceEditWidget(UBlendSpace* InBlendSpace, const FBlendSpaceEditorArgs& InArgs) const;

	/** Create a tab factory for editing montage sections */
	virtual TSharedRef<FWorkflowTabFactory> CreateAnimMontageSectionsTabFactory(const TSharedRef<class FWorkflowCentricApplication>& InHostingApp, const TSharedRef<IPersonaToolkit>& InPersonaToolkit, FSimpleMulticastDelegate& InOnSectionsChanged) const;

	/** Create a tab factory for finding and replacing in anim data */
	virtual TSharedRef<FWorkflowTabFactory> CreateAnimAssetFindReplaceTabFactory(const TSharedRef<FWorkflowCentricApplication>& InHostingApp, const FAnimAssetFindReplaceConfig& InConfig) const;
	
	/** Create a widget for finding and replacing in anim data */
	virtual TSharedRef<SWidget> CreateFindReplaceWidget(const FAnimAssetFindReplaceConfig& InConfig) const;
	
	/** Create a widget that acts as a document for an animation asset */
	virtual TSharedRef<SWidget> CreateEditorWidgetForAnimDocument(const TSharedRef<IAnimationEditor>& InHostingApp, UObject* InAnimAsset, const FAnimDocumentArgs& InArgs, FString& OutDocumentLink);

	/** Create a widget that acts as a curve document for an animation asset */
	virtual TSharedRef<IAnimSequenceCurveEditor> CreateCurveWidgetForAnimDocument(const TSharedRef<class FWorkflowCentricApplication>& InHostingApp, const TSharedRef<IPersonaPreviewScene>& InPreviewScene, UAnimSequenceBase* InAnimSequence, const TSharedPtr<ITimeSliderController>& InExternalTimeSliderController, const TSharedPtr<FTabManager>& InTabManager);

	/** Customize a skeletal mesh details panel */
	virtual void CustomizeMeshDetails(const TSharedRef<IDetailsView>& InDetailsView, const TSharedRef<IPersonaToolkit>& InPersonaToolkit);

	/** Gets the extensibility managers for outside entities to extend persona editor's menus and toolbars */
	virtual TSharedPtr<FExtensibilityManager> GetMenuExtensibilityManager() {return MenuExtensibilityManager;}
	virtual TSharedPtr<FExtensibilityManager> GetToolBarExtensibilityManager() {return ToolBarExtensibilityManager;}

	/** Import a new asset using the supplied skeleton */
	virtual void ImportNewAsset(USkeleton* InSkeleton, EFBXImportType DefaultImportType);

	/** Check all animations & skeletal meshes for curve metadata usage */
	virtual void TestSkeletonCurveMetaDataForUse(const TSharedRef<IEditableSkeleton>& InEditableSkeleton) const;

	/** Apply Compression to list of animations and optionally asks to pick an overrides to the bone compression settings */
	virtual void ApplyCompression(TArray<TWeakObjectPtr<class UAnimSequence>>& AnimSequences, bool bPickBoneSettingsOverride);

	/** Export to FBX files of the list of animations */
	virtual bool ExportToFBX(TArray<TWeakObjectPtr<class UAnimSequence>>& AnimSequences, USkeletalMesh* SkeletalMesh);

	/** Add looping interpolation to the list of animations */
	virtual void AddLoopingInterpolation(TArray<TWeakObjectPtr<class UAnimSequence>>& AnimSequences);

	UE_DEPRECATED(4.24, "Function renamed, please use CustomizeBlueprintEditorDetails")
	virtual void CustomizeSlotNodeDetails(const TSharedRef<class IDetailsView>& InDetailsView, FOnInvokeTab InOnInvokeTab) { CustomizeBlueprintEditorDetails(InDetailsView, InOnInvokeTab); }

	/** Customize the details of a slot node for the specified details view */
	virtual void CustomizeBlueprintEditorDetails(const TSharedRef<class IDetailsView>& InDetailsView, FOnInvokeTab InOnInvokeTab);

	/** Create a Persona editor mode manager. Should be destroyed using plain ol' delete. Note: Only FPersonaEditMode-derived modes should be used with this manager! */
	virtual class IPersonaEditorModeManager* CreatePersonaEditorModeManager();

	/** Delegate used to query whether recording is active */
	virtual FIsRecordingActive& OnIsRecordingActive() { return IsRecordingActiveDelegate; }

	/** Delegate used to start recording animation */
	virtual FRecord& OnRecord() { return RecordDelegate; }

	/** Delegate used to stop recording animation */
	virtual FStopRecording& OnStopRecording() { return StopRecordingDelegate; }

	/** Delegate used to get the currently recording animation */
	virtual FGetCurrentRecording& OnGetCurrentRecording() { return GetCurrentRecordingDelegate; }

	/** Delegate used to get the currently recording animation time */
	virtual FGetCurrentRecordingTime& OnGetCurrentRecordingTime() { return GetCurrentRecordingTimeDelegate; }

	/** Delegate broadcast when a preview scene is created */
	virtual FOnPreviewSceneCreated& OnPreviewSceneCreated() { return OnPreviewSceneCreatedDelegate; }

	/** Delegates that allow customizing the mesh details layout and are called from within PersonaMeshDetails' CustomizeDetails(). */
	DECLARE_DELEGATE_TwoParams(FOnCustomizeMeshDetails, IDetailLayoutBuilder& DetailLayout, TWeakObjectPtr<USkeletalMesh> SkeletalMesh);
	virtual TArray<FOnCustomizeMeshDetails>& GetCustomizeMeshDetailsDelegates() { return CustomizeMeshDetailLayoutDelegates; }

	/** Settings for AddCommonToolbarExtensions */
	struct FCommonToolMenuExtensionArgs
	{
		FCommonToolMenuExtensionArgs()
			: bPreviewMesh(true)
			, bPreviewAnimation(true)
			, bReferencePose(false)
			, bCreateAsset(true)
		{}

		/** Adds a shortcut to setup a preview mesh to override the current display */
		bool bPreviewMesh;

		/** Adds a shortcut to setup a preview animation to override the current display */
		bool bPreviewAnimation;

		/** Adds a shortcut to set the character back to reference pose (also clears all bone modifications) */
		bool bReferencePose;

		/** Adds a combo menu to allow other anim assets to be created */
		bool bCreateAsset;
	};

	typedef FCommonToolMenuExtensionArgs FCommonToolbarExtensionArgs;

	/** Add common menu extensions */
	virtual void AddCommonMenuExtensions(UToolMenu* InToolMenu, const FCommonToolMenuExtensionArgs& InArgs = FCommonToolMenuExtensionArgs());

	/** Add common toolbar extensions */
	virtual void AddCommonToolbarExtensions(UToolMenu* InToolMenu, const FCommonToolMenuExtensionArgs& InArgs = FCommonToolMenuExtensionArgs());

	/** Add common toobar extensions (legacy support) - DEPRECATED */
	virtual void AddCommonToolbarExtensions(FToolBarBuilder& InToolbarBuilder, TSharedRef<IPersonaToolkit> PersonaToolkit, const FCommonToolbarExtensionArgs& InArgs = FCommonToolbarExtensionArgs());

	/** Register common layout extensions */
	virtual FOnRegisterLayoutExtensions& OnRegisterLayoutExtensions() { return OnRegisterLayoutExtensionsDelegate; }

	/** Register common tabs */
	virtual FOnRegisterTabs& OnRegisterTabs() { return OnRegisterTabsDelegate; }

	/** Create a widget that can choose multiple curve names. Derives available names from the asset registry list of assets that use the specified skeleton. */
	virtual TSharedRef<SWidget> CreateMultiCurvePicker(const USkeleton* InSkeleton, FOnCurvesPicked InOnCurvesPicked, FIsCurveNameMarkedForExclusion InIsCurveNameMarkedForExclusion = FIsCurveNameMarkedForExclusion());

	/** Create a widget that can choose a curve name. Derives available names from the asset registry list of assets that use the specified skeleton. */
	virtual TSharedRef<SWidget> CreateCurvePicker(const USkeleton* InSkeleton, FOnCurvePicked InOnCurvePicked, FIsCurveNameMarkedForExclusion InIsCurveNameMarkedForExclusion = FIsCurveNameMarkedForExclusion());

	/** Create a widget that can choose an attribute. Derives available attributes from the asset registry list of assets that use the specified skeleton. */
	virtual TSharedRef<SWidget> CreateAttributePicker(
		const TObjectPtr<const USkeleton> InSkeleton,
		FOnAttributesPicked InOnAttributesPicked,
		const bool bEnableMultiSelect,
		const bool bCanShowOtherSkeletonAttributes);

	/** Create a widget that can choose a notify name. @see SSkeletonAnimNotifies. */
	virtual TSharedRef<SWidget> CreateSkeletonNotifyPicker(FOnNotifyPicked InOnNotifyPicked, TSharedPtr<IEditableSkeleton> InEditableSkeleton = nullptr, TSharedPtr<FAssetEditorToolkit> InHostingApp = nullptr, bool bInShowSyncMarkers = false, bool bInShowNotifies = true, bool bInShowCompatibleSkeletonAssets = true, bool bInShowOtherAssets = true);

	// Struct allowing the anim notify system to interact with an asset type
	struct FNotifyHostAssetParameters
	{
		// Delegate called to replace a notify event with another
		using FOnReplaceNotify = TDelegate<void(UObject* InAsset, const FString& InFindString, const FString& InReplaceString, bool bFindWholeWord, ESearchCase::Type InSearchCase)>;
		FOnReplaceNotify OnReplaceNotify;

		// Delegate called to remove a notify event
		using FOnRemoveNotify = TDelegate<void(UObject* InAsset, const FString& InFindString, bool bFindWholeWord, ESearchCase::Type InSearchCase)>;
		FOnRemoveNotify OnRemoveNotify;
	};

	// Register an asset type that can contain named anim notifies
	virtual void RegisterNotifyHostAsset(const FTopLevelAssetPath& InAssetClassPath, const FNotifyHostAssetParameters& InParameters);

	// Unregister an asset type that can contain named anim notifies
	virtual void UnregisterNotifyHostAsset(const FTopLevelAssetPath& InAssetClassPath);

	// Find registered notify host parameters by asset class
	virtual const FNotifyHostAssetParameters* FindNotifyHostAsset(UObject* InAsset);

	// Get all the registered notify host class paths
	virtual TArray<FTopLevelAssetPath> GetAllNotifyHostAssetClassPaths() const;

	// Get the version information for the notify host asset registry
	virtual uint32 GetNotifyHostAssetVersion() const;

public:
	// Register a new blend provider in the editor picker for FBlendProfileInterfaceWrapper
	PERSONA_API void RegisterBlendProfilePickerExtender(TSharedPtr<IBlendProfilePickerExtender> Extender);

	PERSONA_API void UnregisterBlendProfilePickerExtender(const FName ExtenderId);

	const TArray<TSharedPtr<IBlendProfilePickerExtender>>& GetCustomBlendProfiles() const;

private:
	TArray<TSharedPtr<IBlendProfilePickerExtender>> CustomBlendProfiles;

	/** When a new anim notify blueprint is created, this will handle post creation work such as adding non-event default nodes */
	void HandleNewAnimNotifyBlueprintCreated(UBlueprint* InBlueprint);

	/** When a new anim notify state blueprint is created, this will handle post creation work such as adding non-event default nodes */
	void HandleNewAnimNotifyStateBlueprintCreated(UBlueprint* InBlueprint);

	/** Options for asset creation */
	enum class EPoseSourceOption : uint8
	{
		ReferencePose,
		CurrentPose,
		CurrentAnimation_AnimData,
		CurrentAnimation_PreviewMesh,
		Max
	};

	static TSharedRef< SWidget > GenerateCreateAssetMenu(TWeakPtr<IPersonaToolkit> InWeakPersonaToolkit);

	static void FillCreateAnimationMenu(FMenuBuilder& MenuBuilder, TWeakPtr<IPersonaToolkit> InWeakPersonaToolkit);

	static void FillCreateAnimationFromCurrentAnimationMenu(FMenuBuilder& MenuBuilder, TWeakPtr<IPersonaToolkit> InWeakPersonaToolkit);

	static void FillCreatePoseAssetMenu(FMenuBuilder& MenuBuilder, TWeakPtr<IPersonaToolkit> InWeakPersonaToolkit);

	static void FillInsertPoseMenu(FMenuBuilder& MenuBuilder, TWeakPtr<IPersonaToolkit> InWeakPersonaToolkit);

	static void InsertCurrentPoseToAsset(const FAssetData& NewPoseAssetData, TWeakPtr<IPersonaToolkit> InWeakPersonaToolkit);

	static bool CreateAnimation(const TArray<UObject*> NewAssets, const EPoseSourceOption Option, TWeakPtr<IPersonaToolkit> InWeakPersonaToolkit);

	static bool CreatePoseAsset(const TArray<UObject*> NewAssets, const EPoseSourceOption Option, TWeakPtr<IPersonaToolkit> InWeakPersonaToolkit);
	
	static bool HandleAssetCreated(const TArray<UObject*> NewAssets);

	void RegisterToolMenuExtensions();
private:
	TSharedPtr<FExtensibilityManager> MenuExtensibilityManager;
	TSharedPtr<FExtensibilityManager> ToolBarExtensibilityManager;
	TArray<FOnCustomizeMeshDetails> CustomizeMeshDetailLayoutDelegates;

	/** Delegate used to query whether recording is active */
	FIsRecordingActive IsRecordingActiveDelegate;

	/** Delegate used to start recording animation */
	FRecord RecordDelegate;

	/** Delegate used to stop recording animation */
	FStopRecording StopRecordingDelegate;

	/** Delegate used to get the currently recording animation */
	FGetCurrentRecording GetCurrentRecordingDelegate;

	/** Delegate used to get the currently recording animation time */
	FGetCurrentRecordingTime GetCurrentRecordingTimeDelegate;

	/** Delegate used to tick the skelmesh component recording */
	FTickRecording TickRecordingDelegate;

	/** Delegate broadcast when a preview scene is created */
	FOnPreviewSceneCreated OnPreviewSceneCreatedDelegate;

	/** Delegate broadcast to register common layout extensions */
	FOnRegisterLayoutExtensions OnRegisterLayoutExtensionsDelegate;

	/** Delegate broadcast to register common tabs */
	FOnRegisterTabs OnRegisterTabsDelegate;

	/** Parameters for notify hosts */
	TMap<FTopLevelAssetPath, FNotifyHostAssetParameters> NotifyHostParameters;

	/** Version counter for NotifyHostParameters */
	uint32 NotifyHostAssetVersion = 0;
};

#undef UE_API
