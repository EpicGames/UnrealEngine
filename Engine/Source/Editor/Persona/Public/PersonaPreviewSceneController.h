// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "PersonaPreviewSceneController.generated.h"

#define UE_API PERSONA_API

class UPersonaPreviewSceneDescription;
class IPersonaPreviewScene;

// Base class for preview scene controller (controls what the preview scene in persona does) 
UCLASS(MinimalAPI, Abstract)
class UPersonaPreviewSceneController : public UObject
{
public:
	GENERATED_BODY()

	// Called when this preview controller is activated
	virtual void InitializeView(UPersonaPreviewSceneDescription* SceneDescription, IPersonaPreviewScene* PreviewScene) const PURE_VIRTUAL( UPersonaPreviewSceneController::InitializeView, );
	// Called when this preview controller is deactivated
	virtual void UninitializeView(UPersonaPreviewSceneDescription* SceneDescription, IPersonaPreviewScene* PreviewScene) const PURE_VIRTUAL(UPersonaPreviewSceneController::UninitializeView, );

	//Called when populating the preview scene settings details panel to allow customization of the controllers properties
	UE_API virtual IDetailPropertyRow* AddPreviewControllerPropertyToDetails(const TSharedRef<class IPersonaToolkit>& PersonaToolkit, IDetailLayoutBuilder& DetailBuilder, IDetailCategoryBuilder& Category, const FProperty* Property, const EPropertyLocation::Type PropertyLocation);
};

#undef UE_API
