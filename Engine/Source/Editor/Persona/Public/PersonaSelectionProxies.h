// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenericPlatform/ICursor.h"
#include "HAL/Platform.h"
#include "HitProxies.h"
#include "UObject/NameTypes.h"

#define UE_API PERSONA_API

class USkeletalMeshSocket;


struct HPersonaBoneHitProxy : public HHitProxy
{
	DECLARE_HIT_PROXY( UE_API )

	int32 BoneIndex;
	FName BoneName;

	HPersonaBoneHitProxy(int32 InBoneIndex, FName InBoneName)
		: HHitProxy(HPP_Foreground)
		, BoneIndex(InBoneIndex)
		, BoneName(InBoneName)
	{}

	// HHitProxy interface
	virtual EMouseCursor::Type GetMouseCursor() override { return EMouseCursor::Crosshairs; }
	// End of HHitProxy interface
};

struct HPersonaSocketHitProxy : public HHitProxy
{
	DECLARE_HIT_PROXY( UE_API )

	USkeletalMeshSocket* Socket;

	HPersonaSocketHitProxy(USkeletalMeshSocket* InSocket)
		: HHitProxy(HPP_Foreground)
	{
		Socket = InSocket;
	}

	// HHitProxy interface
	virtual EMouseCursor::Type GetMouseCursor() override { return EMouseCursor::Crosshairs; }
	// End of HHitProxy interface
};

#undef UE_API
