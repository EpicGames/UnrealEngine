// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#define UE_API PERSONA_API

// Tab constants
struct FPersonaTabs
{
	// Selection Details
	static UE_API const FName MorphTargetsID;
	static UE_API const FName AnimCurveViewID;
	static UE_API const FName AnimCurveMetadataEditorID;
	static UE_API const FName SkeletonTreeViewID;
	// Skeleton Pose manager
	static UE_API const FName RetargetManagerID;
	static UE_API const FName RigManagerID;
	// Skeleton/Sockets
	// Anim Blueprint Params
	// Explorer
	// Class Defaults
	static UE_API const FName AnimBlueprintPreviewEditorID;
	static UE_API const FName AnimBlueprintParentPlayerEditorID;
	// Anim Document
	static UE_API const FName ScrubberID;
	// Toolbar
	static UE_API const FName PreviewViewportID;
	static UE_API const FName PreviewViewport1ID;
	static UE_API const FName PreviewViewport2ID;
	static UE_API const FName PreviewViewport3ID;
	static UE_API const FName AssetBrowserID;
	static UE_API const FName MirrorSetupID;
	static UE_API const FName AnimBlueprintDebugHistoryID;
	static UE_API const FName AnimAssetPropertiesID;
	static UE_API const FName MeshAssetPropertiesID;
	static UE_API const FName PreviewManagerID;
	static UE_API const FName SkeletonAnimNotifiesID;
	static UE_API const FName SkeletonSlotNamesID;
	static UE_API const FName SkeletonSlotGroupNamesID;
	static UE_API const FName CurveNameManagerID;
	static UE_API const FName BlendProfileManagerID;
	static UE_API const FName AnimMontageSectionsID;
	static UE_API const FName PoseWatchManagerID;

	// Advanced Preview Scene
	static UE_API const FName AdvancedPreviewSceneSettingsID;
	static UE_API const FName DetailsID;
	static UE_API const FName FindReplaceID;

	static UE_API const FName ToolboxID;

	static UE_API const FName AnimAttributeViewID;
private:
	FPersonaTabs() {}
};

#undef UE_API
