// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/SharedPointer.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "PersonaModule.h"

#include "PersonaToolMenuContext.generated.h"

#define UE_API PERSONA_API

class IPersonaToolkit;
class UAnimBlueprint;
class UAnimationAsset;
class UDebugSkelMeshComponent;
class USkeletalMesh;
class USkeleton;
struct FFrame;

UCLASS(MinimalAPI, BlueprintType)
class UPersonaToolMenuContext : public UObject
{
	GENERATED_BODY()
public:
	
	/** Get the skeleton that we are editing */
	UFUNCTION(BlueprintCallable, Category = PersonaEditorExtensions)
	UE_API USkeleton* GetSkeleton() const;

	/** Get the preview component that we are using */
	UFUNCTION(BlueprintCallable, Category = PersonaEditorExtensions)
	UE_API UDebugSkelMeshComponent* GetPreviewMeshComponent() const;

	/** Get the skeletal mesh that we are editing */
	UFUNCTION(BlueprintCallable, Category = PersonaEditorExtensions)
	UE_API USkeletalMesh* GetMesh() const;

	/** Get the anim blueprint that we are editing */
	UFUNCTION(BlueprintCallable, Category = PersonaEditorExtensions)
	UE_API UAnimBlueprint* GetAnimBlueprint() const;

	/** Get the animation asset that we are editing */
	UFUNCTION(BlueprintCallable, Category = PersonaEditorExtensions)
	UE_API UAnimationAsset* GetAnimationAsset() const;

	/** Get the persona toolkit */
	UE_API void SetToolkit(TSharedRef<IPersonaToolkit> InToolkit);

	/** Get a weak ptr to the persona toolkit */
	TWeakPtr<IPersonaToolkit> GetToolkit() const { return WeakToolkit; }

protected:
	UE_API bool HasValidToolkit() const;
private:
	TWeakPtr<IPersonaToolkit> WeakToolkit;
};

#undef UE_API
