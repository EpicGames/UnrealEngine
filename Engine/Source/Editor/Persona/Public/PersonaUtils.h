// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/SlateDelegates.h"
#include "ClassViewerModule.h"

class UAnimInstance;
class USceneComponent;
class UAnimBlueprint;
class SWidget;
class UAnimSequenceBase;

namespace PersonaUtils
{

/** Get the viewport scene component for the specified attached asset */
PERSONA_API USceneComponent* GetComponentForAttachedObject(USceneComponent* PreviewComponent, class UObject* Object, const FName& AttachedTo);

/** Options for CopyPropertiesToCDO */
enum ECopyOptions : int32
{
	/** Default copy options */
	Default = 0,

	/** Set this option to preview the changes and not actually copy anything.  This will count the number of properties that would be copied. */
	PreviewOnly = 1 << 0,

	/** Call PostEditChangeProperty for each modified property */
	CallPostEditChangeProperty = 1 << 1,

	/** Copy only Edit and Interp properties.  Otherwise we copy all properties by default */
	OnlyCopyEditOrInterpProperties = 1 << 2,

	/** Filters out Blueprint Read-only properties */
	FilterBlueprintReadOnly = 1 << 3,
};

/** Copy options structure for CopyPropertiesToCDO */
struct FCopyOptions
{
	/** Implicit construction for an options enumeration */
	FCopyOptions(const ECopyOptions InFlags) : Flags(InFlags) {}

	/** Check whether we can copy the specified property */
	bool CanCopyProperty(FProperty& Property, UObject& Object) const
	{
		return !PropertyFilter || PropertyFilter(Property, Object);
	}

	/** User-specified flags for the copy */
	ECopyOptions Flags;

	/** User-specified custom property filter predicate */
	TFunction<bool(FProperty&, UObject&)> PropertyFilter;
};

/** Copy modified properties from the specified anim instance back to its CDO */
PERSONA_API int32 CopyPropertiesToCDO(UAnimInstance* InAnimInstance, const FCopyOptions& Options = FCopyOptions(ECopyOptions::Default));

/** Set the correct object to debug depending on whether we are running a sub-layer etc. */
PERSONA_API void SetObjectBeingDebugged(UAnimBlueprint* InAnimBlueprint, UAnimInstance* InAnimInstance);

/** Make an add button widget, as seen in the animation timeline */
PERSONA_API TSharedRef<SWidget> MakeTrackButton(FText HoverText, FOnGetContent MenuContent, const TAttribute<bool>& HoverState);

/** Make AnimNotify class picker widget */
PERSONA_API TSharedRef<SWidget> MakeAnimNotifyPicker(UAnimSequenceBase* Sequence, const FOnClassPicked& OnClassPicked);

/** Make AnimNotifyState class picker widget */
PERSONA_API TSharedRef<SWidget> MakeAnimNotifyStatePicker(UAnimSequenceBase* Sequence, const FOnClassPicked& OnClassPicked);

}
