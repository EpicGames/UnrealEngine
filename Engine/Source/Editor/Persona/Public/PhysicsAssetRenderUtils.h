// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IPhysicsAssetRenderInterface.h"
#include "UObject/ObjectMacros.h"
#include "Math/Color.h"
#include "PhysicsEngine/ConstraintInstance.h"
#include "PhysicsEngine/ShapeElem.h"
#include "SceneManagement.h"
#include "PhysicsAssetRenderUtils.generated.h"

#define UE_API PERSONA_API

enum class EPhysicsAssetEditorCenterOfMassViewMode : uint8;
enum class EPhysicsAssetEditorConstraintViewMode : uint8;
enum class EPhysicsAssetEditorCollisionViewMode : uint8;

class HHitProxy;
class UPhysicsAsset;
class USkeletalMeshComponent;

//////////////////////////////////////////////////////////////////////////
// FPhysicsAssetRenderSettings

/**
	Per Physics Asset parameters that determine how debug draw functions 
	should render that asset in an editor viewport.
	
	These parameters are used across different editor modes to ensure the 
	debug draw is consistent. This makes it easier to create or debug 
	physics assets whilst switching between editor modes.
*/
USTRUCT()
struct FPhysicsAssetRenderSettings
{
	GENERATED_BODY()

public:

	UE_API FPhysicsAssetRenderSettings();
	UE_API void InitPhysicsAssetRenderSettings(class UMaterialInterface* InBoneUnselectedMaterial, class UMaterialInterface* InBoneNoCollisionMaterial);

	/** Accessors/helper methods */
	UE_API bool IsBodyHidden(const int32 BodyIndex) const;
	UE_API bool IsConstraintHidden(const int32 ConstraintIndex) const;
	UE_API bool AreAnyBodiesHidden() const;
	UE_API bool AreAnyConstraintsHidden() const;
	UE_API void HideBody(const int32 BodyIndex);
	UE_API void ShowBody(const int32 BodyIndex);
	UE_API void HideConstraint(const int32 ConstraintIndex);
	UE_API void ShowConstraint(const int32 ConstraintIndex);
	UE_API void ShowAllBodies();
	UE_API void ShowAllConstraints();
	UE_API void ShowAll();
	UE_API void HideAllBodies(const UPhysicsAsset* const PhysicsAsset);
	UE_API void HideAllConstraints(const UPhysicsAsset* const PhysicsAsset);
	UE_API void HideAll(const UPhysicsAsset* const PhysicsAsset);
	UE_API void ToggleShowBody(const int32 BodyIndex);
	UE_API void ToggleShowConstraint(const int32 ConstraintIndex);
	UE_API void ToggleShowAllBodies(const UPhysicsAsset* const PhysicsAsset);
	UE_API void ToggleShowAllConstraints(const UPhysicsAsset* const PhysicsAsset);

	UE_API void SetHiddenBodies(const TArray<int32>& InHiddenBodies);
	UE_API void SetHiddenConstraints(const TArray<int32>& InHiddenConstraints);

	/** Returns a set of flags describing which components of the selected constraint's transforms are being manipulated in the view port. */
	UE_API EConstraintTransformComponentFlags GetConstraintViewportManipulationFlags() const;

	/** Returns true if all the constraint transform components specified by the flags should be displayed as an offset from the default (snapped) transforms. */
	UE_API bool IsDisplayingConstraintTransformComponentRelativeToDefault(const EConstraintTransformComponentFlags ComponentFlags) const;

	/** Set how the constraint transform components specified by the flags should be displayed, in the frame of their associated physics body (false) or as an offset from the default (snapped) transforms (true). */
	UE_API void SetDisplayConstraintTransformComponentRelativeToDefault(const EConstraintTransformComponentFlags ComponentFlags, const bool bShouldDisplayRelativeToDefault);

	UE_API void ResetEditorViewportOptions();

	// Physics Asset Editor Viewport Options
	UPROPERTY()
	EPhysicsAssetEditorCenterOfMassViewMode CenterOfMassViewMode;

	UPROPERTY()
	EPhysicsAssetEditorCollisionViewMode CollisionViewMode;

	UPROPERTY()
	EPhysicsAssetEditorConstraintViewMode ConstraintViewMode;

	// Flags that determine which parts of the constraints transforms (parent frame, child frame, position and rotation) are currently begin manipulated in the viewport.
	UPROPERTY(Transient)
	EConstraintTransformComponentFlags ConstraintViewportManipulationFlags;

	// Flags that determine which parts of the constraints transforms (parent/child position/rotation) should be displayed as an offset from the default (snapped) transforms.
	UPROPERTY()
	EConstraintTransformComponentFlags ConstraintTransformComponentDisplayRelativeToDefaultFlags;

	UPROPERTY()
	float ConstraintDrawSize;

	UPROPERTY()
	float PhysicsBlend;

	UPROPERTY()
	bool bHideKinematicBodies;

	UPROPERTY()
	bool bHideSimulatedBodies;
	
	UPROPERTY()
	bool bHideBodyMass;

	UPROPERTY()
	bool bRenderOnlySelectedConstraints;

	UPROPERTY()
	bool bShowCOM_DEPRECATED;

	UPROPERTY()
	bool bShowConstraintsAsPoints;

	UPROPERTY()
	bool bHighlightOverlapingBodies;

	UPROPERTY()
	bool bDrawViolatedLimits;
	
	UPROPERTY()
	bool bHideCenterOfMassForKinematicBodies;

	// Draw colors
	UPROPERTY()
	FColor BoneUnselectedColor;

	UPROPERTY()
	FColor NoCollisionColor;

	UPROPERTY()
	FColor COMRenderColor;

	UPROPERTY()
	float COMRenderSize;

	UPROPERTY()
	float COMRenderLineThickness;

	UPROPERTY()
	float COMRenderMassTextOffsetScreenspace;

	UPROPERTY()
	float InfluenceLineLength;

	// Materials
	UPROPERTY()
	TObjectPtr<class UMaterialInterface> BoneUnselectedMaterial;

	UPROPERTY()
	TObjectPtr<class UMaterialInterface> BoneNoCollisionMaterial;

	UPROPERTY()
	TArray<int32> HiddenBodies;

	UPROPERTY()
	TArray<int32> HiddenConstraints;

};

//////////////////////////////////////////////////////////////////////////
// UPhysicsAssetRenderUtilities

/** Factory class for FPhysicsAssetRenderSettings. */
UCLASS(MinimalAPI, config=EditorPerProjectUserSettings)
class UPhysicsAssetRenderUtilities : public UObject
{
	GENERATED_BODY()

public:

	UE_API UPhysicsAssetRenderUtilities();
	UE_API virtual ~UPhysicsAssetRenderUtilities();

	static UE_API void Initialise();

	/** Returns an existing render settings object or creates and returns a new one if none exists. */
	static UE_API FPhysicsAssetRenderSettings* GetSettings(const UPhysicsAsset* InPhysicsAsset);
	static UE_API FPhysicsAssetRenderSettings* GetSettings(const FString& InPhysicsAssetPathName);
	static UE_API FPhysicsAssetRenderSettings* GetSettings(const uint32 InPhysicsAssetPathNameHash);
	
	static UE_API uint32 GetPathNameHash(const UPhysicsAsset* InPhysicsAsset);
	static UE_API uint32 GetPathNameHash(const FString& InPhysicsAssetPathName);

	UE_API void OnAssetRenamed(FAssetData const& AssetInfo, const FString& InOldPhysicsAssetPathName);
	UE_API void OnAssetRemoved(UObject* Object);

private:

	UE_API void InitialiseImpl();
	UE_API FPhysicsAssetRenderSettings* GetSettingsImpl(const uint32 InPhysicsAssetPathNameHash);

	UPROPERTY()
	TMap< uint32, FPhysicsAssetRenderSettings > IdToSettingsMap;

	UPROPERTY(transient)
	TObjectPtr<class UMaterialInterface> BoneUnselectedMaterial;

	UPROPERTY(transient)
	TObjectPtr<class UMaterialInterface> BoneNoCollisionMaterial;

	IPhysicsAssetRenderInterface* PhysicsAssetRenderInterface;
};

//////////////////////////////////////////////////////////////////////////
// PhysicsAssetRender

/**
	Namespace containing functions for debug draw of Physics Assets in the editor viewport.
*/
namespace PhysicsAssetRender
{
	template< typename TReturnType > using GetPrimitiveRef = TFunctionRef< TReturnType (const int32 BodyIndex, const EAggCollisionShape::Type PrimitiveType, const int32 PrimitiveIndex, const FPhysicsAssetRenderSettings& Settings) >;
	using GetPrimitiveTransformRef = TFunctionRef< FTransform (const UPhysicsAsset* PhysicsAsset, const FTransform& BoneTM, const int32 BodyIndex, const EAggCollisionShape::Type PrimType, const int32 PrimIndex, const float Scale) >;
	using CreateBodyHitProxyFn = TFunctionRef< HHitProxy* (const int32 BodyIndex, const EAggCollisionShape::Type PrimitiveType, const int32 PrimitiveIndex) >;
	using CreateConstraintHitProxyFn = TFunctionRef< HHitProxy* (const int32 InConstraintIndex) >;
	using CreateCoMHitProxyFn = TFunctionRef< HHitProxy* (const int32 InBodyIndex) >;
	using COMAccessorFunctionFn = TFunctionRef< TPair< bool, FVector >(const int32) >;
	using IsSelectedFn = TFunction< bool(const uint32) >;

	/** Debug draw Physics Asset bodies and constraints using the default callbacks */
	PERSONA_API void DebugDraw(class USkeletalMeshComponent* const SkeletalMeshComponent, class UPhysicsAsset* const PhysicsAsset, FPrimitiveDrawInterface* PDI);
	
	/** Debug draw Physics Asset bodies using the supplied custom callbacks */
	PERSONA_API void DebugDrawBodies(class USkeletalMeshComponent* const SkeletalMeshComponent, class UPhysicsAsset* const PhysicsAsset, FPrimitiveDrawInterface* PDI, GetPrimitiveRef< FColor > GetPrimitiveColor, GetPrimitiveRef< class UMaterialInterface* > GetPrimitiveMaterial, GetPrimitiveTransformRef GetPrimitiveTransform, CreateBodyHitProxyFn CreateHitProxy);

	/** Debug draw the Center of Masses for all bodies in the supplied Skeletal Mesh */
	PERSONA_API void DebugDrawCenterOfMass(USkeletalMeshComponent* const SkeletalMeshComponent, class UPhysicsAsset* const PhysicsAsset, FPrimitiveDrawInterface* PDI, TFunctionRef< FVector(const uint32) > GetCoMPosition, TFunctionRef< bool(const uint32) > IsSelected, TFunctionRef< bool(const uint32) > IsHidden, CreateCoMHitProxyFn CreateHitProxy);

	/** Debug draw Physics Asset constraints using the supplied custom callbacks */
	PERSONA_API void DebugDrawConstraints(class USkeletalMeshComponent* const SkeletalMeshComponent, class UPhysicsAsset* const PhysicsAsset, FPrimitiveDrawInterface* PDI, IsSelectedFn IsSelected, const bool bRunningSimulation, CreateConstraintHitProxyFn CreateHitProxy);
	
	/** Default callbacks used by DebugDraw */
	PERSONA_API FTransform GetPrimitiveTransform(const UPhysicsAsset* PhysicsAsset, const FTransform& BoneTM, const int32 BodyIndex, const EAggCollisionShape::Type PrimType, const int32 PrimIndex, const float Scale);
	PERSONA_API FColor GetPrimitiveColor(const int32 BodyIndex, const EAggCollisionShape::Type PrimitiveType, const int32 PrimitiveIndex, const FPhysicsAssetRenderSettings& Settings);
	PERSONA_API class UMaterialInterface* GetPrimitiveMaterial(const int32 BodyIndex, const EAggCollisionShape::Type PrimitiveType, const int32 PrimitiveIndex, const FPhysicsAssetRenderSettings& Settings);
}

class FPhysicsAssetRenderInterface : public IPhysicsAssetRenderInterface
{
public:
	virtual void DebugDraw(USkeletalMeshComponent* const SkeletalMeshComponent, UPhysicsAsset* const PhysicsAsset, FPrimitiveDrawInterface* PDI) override;
	virtual void DebugDrawBodies(USkeletalMeshComponent* const SkeletalMeshComponent, UPhysicsAsset* const PhysicsAsset, FPrimitiveDrawInterface* PDI, const FColor& PrimitiveColorOverride) override;
	virtual void DebugDrawConstraints(USkeletalMeshComponent* const SkeletalMeshComponent, UPhysicsAsset* const PhysicsAsset, FPrimitiveDrawInterface* PDI) override;

	virtual void SaveConfig() override;

	virtual void ToggleShowAllBodies(UPhysicsAsset* const PhysicsAsset) override;
	virtual void ToggleShowAllConstraints(UPhysicsAsset* const PhysicsAsset) override;
	virtual bool AreAnyBodiesHidden(UPhysicsAsset* const PhysicsAsset) override;
	virtual bool AreAnyConstraintsHidden(UPhysicsAsset* const PhysicsAsset) override;

	/** Returns a set of flags describing which components of the selected constraint's transforms are being manipulated in the viewport. */
	virtual EConstraintTransformComponentFlags GetConstraintViewportManipulationFlags(class UPhysicsAsset* const PhysicsAsset) override;

	/** Returns true if the constraint transform component specified by the flags should be displayed as an offset from the default (snapped) transforms. */
	virtual bool IsDisplayingConstraintTransformComponentRelativeToDefault(class UPhysicsAsset* const PhysicsAsset, const EConstraintTransformComponentFlags ComponentFlags) override;

	/** Change how the constraint transform component specified by the flags should be displayed, in the frame of their associated physics body or as an offset from the default (snapped) transforms. */
	virtual void SetDisplayConstraintTransformComponentRelativeToDefault(class UPhysicsAsset* const PhysicsAsset, const EConstraintTransformComponentFlags ComponentFlags, const bool bShouldDisplayRelativeToDefault) override;
};

PERSONA_API bool IsBodyKinematic(const UPhysicsAsset* const PhysicsAsset, const int32 BodyIndex); // TODO - where is the best place for this fn to live ?

#undef UE_API
