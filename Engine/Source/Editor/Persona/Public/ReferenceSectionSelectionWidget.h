// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STreeView.h"
#include "Widgets/Views/SListView.h"

#define UE_API PERSONA_API

class FSkeletalMeshLODModel;
class IEditableSkeleton;
class SComboButton;

DECLARE_DELEGATE_OneParam(FOnSectionSelectionChanged, int32);
DECLARE_DELEGATE_RetVal_OneParam(int32, FGetSelectedSection, bool& /*bMultipleValues*/);
DECLARE_DELEGATE_RetVal(const FSkeletalMeshLODModel&, FGetLodModel);

class SReferenceSectionSelectionWidget : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SReferenceSectionSelectionWidget)
		: _OnSectionSelectionChanged()
		, _OnGetSelectedSection()
		, _OnGetLodModel()
	{}

		/** Should we hide chunked sections when showing the sections list we can select */
		SLATE_ARGUMENT(bool, bHideChunkedSections)

		/** set selected section index */
		SLATE_EVENT(FOnSectionSelectionChanged, OnSectionSelectionChanged);

		/** get selected section index **/
		SLATE_EVENT(FGetSelectedSection, OnGetSelectedSection);

		SLATE_EVENT(FGetLodModel, OnGetLodModel)

	SLATE_END_ARGS();

	/**
	 * Construct this widget
	 *
	 * @param	InArgs	The declaration data for this widget
	 */
	UE_API void Construct( const FArguments& InArgs );

private: 

	// Creates the combo button menu when clicked
	UE_API TSharedRef<SWidget> CreateSectionListWidgetMenu();
	UE_API TSharedRef<ITableRow> MakeIntegerDisplayWidget(const TSharedPtr<int32> SectionIndex, const TSharedRef<STableViewBase>& OwnerTable) const;

	TSharedPtr<SListView<TSharedPtr<int32>>> SectionListView;
	// Called when the user selects a bone name
	UE_API void OnSelectionChanged(TSharedPtr<int32> NewSectionIndex, ESelectInfo::Type SelectInfo);

	UE_API const TArray<TSharedPtr<int32>>* GetSections() const;

	UE_API FText GetCurrentSectionIndex() const;

	// Base combo button 
	TSharedPtr<SComboButton> SectionPickerButton;

	// delegates
	FOnSectionSelectionChanged OnSectionSelectionChanged;
	FGetSelectedSection		OnGetSelectedSection;
	FGetLodModel	OnGetLodModel;

	bool bHideChunkedSections = false;
	mutable TArray<TSharedPtr<int32>> CacheSectionList;
};

#undef UE_API
