// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IStructureDetailsView.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/SCompoundWidget.h"
#include "Animation/AttributesRuntime.h"
#include "Animation/AnimData/AttributeIdentifier.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"

#define UE_API PERSONA_API

class ITableRow;
class STableViewBase;
class IStructureDetailsView;
class FStructOnScope;
class SHeaderRow;
class SScrollBox;
class SPoseWatchPicker;
class UAnimInstance;

namespace EColumnSortMode
{
	enum Type;
}


class FAnimAttributeEntry : public TSharedFromThis<FAnimAttributeEntry>
{
public:
	static TSharedRef<FAnimAttributeEntry> MakeEntry(const FAnimationAttributeIdentifier& InIdentifier, const FName& InSnapshotDisplayName);

	FAnimAttributeEntry() = default;
	FAnimAttributeEntry(const FAnimationAttributeIdentifier& InIdentifier, const FName& InSnapshotDisplayName);
	
	TSharedRef<ITableRow> MakeTableRowWidget(const TSharedRef<STableViewBase>& InOwnerTable);

	FName GetName() const { return Identifier.GetName(); }
	FName GetBoneName() const { return Identifier.GetBoneName(); }
	int32 GetBoneIndex() const { return Identifier.GetBoneIndex(); }
	FName GetTypeName() const { return CachedTypeName; }
	UScriptStruct* GetScriptStruct() const { return Identifier.GetType(); }
	FName GetSnapshotDisplayName() const { return SnapshotDisplayName; }
	
	FName GetDisplayName() const;
	
	UE::Anim::FAttributeId GetAttributeId() const
	{
		return UE::Anim::FAttributeId(GetName(), FCompactPoseBoneIndex(GetBoneIndex()));
	};

	const FAnimationAttributeIdentifier& GetAnimationAttributeIdentifier() const
	{
		return Identifier;
	};

	bool operator==(const FAnimAttributeEntry& InOther) const
	{
		return Identifier == InOther.Identifier && SnapshotDisplayName == InOther.SnapshotDisplayName;
	}

	bool operator!=(const FAnimAttributeEntry& InOther) const
	{
		return !(*this == InOther);
	}
	
	
private:
	FAnimationAttributeIdentifier Identifier;
	
	FName SnapshotDisplayName;

	FName CachedTypeName;
};

class SAnimAttributeEntry : public SMultiColumnTableRow<TSharedPtr<FAnimAttributeEntry>>
{
	SLATE_BEGIN_ARGS( SAnimAttributeEntry )
	{}
	
	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable, TSharedRef<FAnimAttributeEntry> InEntry);

	/** Overridden from SMultiColumnTableRow.  Generates a widget for this column of the tree row. */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;
	
	FText GetEntryName()const ;
	FText GetEntryBoneName() const;
	FText GetEntryTypeName() const;
	FText GetEntrySnapshotDisplayName() const;
private:
	TWeakPtr<FAnimAttributeEntry> Entry;
};

class SAnimAttributeView : public SCompoundWidget
{
private:	
	static UE_API TSharedRef<IStructureDetailsView> CreateValueViewWidget();
	
	static UE_API TSharedRef<ITableRow> MakeTableRowWidget(
		TSharedPtr<FAnimAttributeEntry> InItem,
		const TSharedRef<STableViewBase>& InOwnerTable);

	static UE_API FName GetSnapshotColumnDisplayName(const TArray<FName>& InSnapshotNames);

public:
	DECLARE_DELEGATE_RetVal_OneParam(FName, FOnGetAttributeSnapshotColumnDisplayName, const TArray<FName>& /** Snapshot Names */)
	
	SLATE_BEGIN_ARGS( SAnimAttributeView )
		: _SnapshotColumnLabelOverride(FText::FromString(TEXT("Direction")))
		{}
		// override what is displayed in the snapshot column, given a set of snapshots that contains the attribute
		SLATE_EVENT(FOnGetAttributeSnapshotColumnDisplayName, OnGetAttributeSnapshotColumnDisplayName)
	
		// override the label on the snapshot column, a typical choice is "Direction"
		SLATE_ATTRIBUTE(FText, SnapshotColumnLabelOverride)
	SLATE_END_ARGS()
	
	UE_API SAnimAttributeView();
	
	UE_API void Construct(const FArguments& InArgs);

	void DisplayNewAttributeContainerSnapshots(
		const TArray<TTuple<FName, const UE::Anim::FMeshAttributeContainer&>>& InSnapshots, 
		const USkeletalMeshComponent* InOwningComponent)
	{
		if (!ensure(InSnapshots.Num() != 0))
		{
			ClearListView();
			return;
		}
		
		// we need the SKM to look up bone names
		if (!InOwningComponent || !InOwningComponent->GetSkeletalMeshAsset())
		{
			ClearListView();
			return;
		}

		if (ShouldInvalidateListViewCache(InSnapshots, InOwningComponent))
		{
			CachedNumSnapshots = InSnapshots.Num();
			CachedSnapshotNameIndexMap.Reset();
			
			CachedAttributeIdentifierLists.Reset(InSnapshots.Num());
			CachedAttributeSnapshotMap.Reset();
			
			CachedAttributeIdentifierLists.AddDefaulted( InSnapshots.Num());
			
			for (int32 SnapshotIndex = 0; SnapshotIndex < InSnapshots.Num(); SnapshotIndex++)
			{
				const TTuple<FName, const UE::Anim::FMeshAttributeContainer&> Snapshot = InSnapshots[SnapshotIndex];
				CachedSnapshotNameIndexMap.FindOrAdd(Snapshot.Key) = SnapshotIndex;
				
				TTuple<FName, TArray<FAnimationAttributeIdentifier>>& CachedIdentifierList = CachedAttributeIdentifierLists[SnapshotIndex];
				CachedIdentifierList.Key = Snapshot.Key;

				const UE::Anim::FMeshAttributeContainer& AttributeContainer = Snapshot.Value;
				TArray<FAnimationAttributeIdentifier>& CachedIdentifiers = CachedIdentifierList.Value;

				CachedIdentifiers.AddDefaulted(AttributeContainer.Num());
				
				const TArray<TWeakObjectPtr<UScriptStruct>>& Types = AttributeContainer.GetUniqueTypes();

				int32 CachedListIdentifierIndex = 0;
				for (int32 TypeIndex = 0; TypeIndex < Types.Num(); TypeIndex++)
				{
					const TArray<UE::Anim::FAttributeId>& Ids = AttributeContainer.GetKeys(TypeIndex);
				
					for (int32 IdIndex = 0; IdIndex < Ids.Num(); IdIndex++)
					{
						const UE::Anim::FAttributeId& Id = Ids[IdIndex];
						const FName BoneName = InOwningComponent->GetSkeletalMeshAsset()->GetRefSkeleton().GetBoneName(Id.GetIndex());
					
						const FAnimationAttributeIdentifier Identifier(
							Id.GetName(), Id.GetIndex() ,BoneName, Types[TypeIndex].Get());

						CachedIdentifiers[CachedListIdentifierIndex] = Identifier;

						CachedAttributeSnapshotMap.FindOrAdd(Identifier).Add(Snapshot.Key);
					
						CachedListIdentifierIndex++;
					}
				}
			}

			// filtered list should also be refreshed since it depends on the cache
			RefreshFilteredAttributeEntries();

			// delay value view refresh until tick since this function can be called from animation thread
			bShouldRefreshValueView = true;

			return;
		}

		if (SelectedAttribute.IsSet())
		{
			for (const FAttributeValueView& ValueView : SelectedAttributeSnapshotValueViews)
			{
				const int32* SnapshotIndex = CachedSnapshotNameIndexMap.Find(ValueView.SnapshotName);
				if (ensure(SnapshotIndex) && ensure(InSnapshots.IsValidIndex(*SnapshotIndex)))
				{
					const TTuple<FName, const UE::Anim::FMeshAttributeContainer&> Snapshot = InSnapshots[*SnapshotIndex];
					const UE::Anim::FMeshAttributeContainer& AttributeContainer = Snapshot.Value;

					ValueView.UpdateValue(AttributeContainer);
				}
			}
		}	
	}

	void ClearListView();
	
	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	
private:

	template <typename ContainerType>
	bool ShouldInvalidateListViewCache(
		const TArray<TTuple<FName, const ContainerType&>>& InSnapshots,
		const USkeletalMeshComponent* InOwningComponent)
	{
		// we need the SKM to look up bone names
		// it should have been checked
		check(InOwningComponent);

		if (InSnapshots.Num() != CachedAttributeIdentifierLists.Num())
		{
			return true;
		}
	
		for (int32 SnapshotIndex = 0; SnapshotIndex < InSnapshots.Num(); SnapshotIndex++)
		{
			const TTuple<FName, const ContainerType&> Snapshot = InSnapshots[SnapshotIndex];
			const TTuple<FName, TArray<FAnimationAttributeIdentifier>>& CachedIdList = CachedAttributeIdentifierLists[SnapshotIndex];
		
			if (Snapshot.Key != CachedIdList.Key)
			{
				return true;
			}

			if (Snapshot.Value.Num() != CachedIdList.Value.Num())
			{
				return true;
			}
		}

		for (int32 SnapshotIndex = 0; SnapshotIndex < InSnapshots.Num(); SnapshotIndex++)
		{
			const TTuple<FName, const ContainerType&> Snapshot = InSnapshots[SnapshotIndex];
			const TTuple<FName, TArray<FAnimationAttributeIdentifier>> CachedIdentifierList = CachedAttributeIdentifierLists[SnapshotIndex];

			const ContainerType& AttributeContainer = Snapshot.Value;
			const TArray<FAnimationAttributeIdentifier>& CachedIdentifiers = CachedIdentifierList.Value;
		
			const TArray<TWeakObjectPtr<UScriptStruct>>& Types = AttributeContainer.GetUniqueTypes();

			int32 CachedListIdentifierIndex = 0;
			for (int32 TypeIndex = 0; TypeIndex < Types.Num(); TypeIndex++)
			{
				const TArray<UE::Anim::FAttributeId>& Ids = AttributeContainer.GetKeys(TypeIndex);
			
				for (int32 IdIndex = 0; IdIndex < Ids.Num(); IdIndex++)
				{
					const UE::Anim::FAttributeId& Id = Ids[IdIndex];
					const FName BoneName = InOwningComponent->GetSkeletalMeshAsset()->GetRefSkeleton().GetBoneName(Id.GetIndex());
				
					FAnimationAttributeIdentifier Identifier(
						Id.GetName(), Id.GetIndex() ,BoneName, Types[TypeIndex].Get());

					if (!(Identifier == CachedIdentifiers[CachedListIdentifierIndex]))
					{
						return true;
					}

					CachedListIdentifierIndex++;
				}
			}
		}

		return false;
	}

	
	UE_API void OnSelectionChanged(TSharedPtr<FAnimAttributeEntry> InEntry, ESelectInfo::Type InSelectType);

	UE_API void OnFilterTextChanged(const FText& InText);

	UE_API EColumnSortMode::Type GetSortModeForColumn(FName InColumnId) const;
	
	UE_API void OnSortAttributeEntries(
		EColumnSortPriority::Type InPriority,
		const FName& InColumnId,
		EColumnSortMode::Type InSortMode);

	UE_API void ExecuteSort();
	
	UE_API void RefreshFilteredAttributeEntries();

	UE_API void RefreshValueView();

private:
	/** list view */
	TSharedPtr<SListView<TSharedPtr<FAnimAttributeEntry>>> AttributeListView;
	bool bShouldRefreshListView;

	TSharedPtr<SHeaderRow> HeaderRow;
	
	FName ColumnIdToSort;
	EColumnSortMode::Type ActiveSortMode;
	FOnGetAttributeSnapshotColumnDisplayName OnGetAttributeSnapshotColumnDisplayName;
	TAttribute<FText> SnapshotColumnLabelOverride;

	int32 CachedNumSnapshots;
	// cache all attributes in the attribute container that the list view is observing
	// such that we can use it to detect if a change to the attribute container occured
	// and refresh the list accordingly
	TArray<TTuple<FName, TArray<FAnimationAttributeIdentifier>>> CachedAttributeIdentifierLists;

	// for each attribute, save the name of the attribute container snapshot that contains it
	TMap<FAnimationAttributeIdentifier, TArray<FName>> CachedAttributeSnapshotMap;

	TMap<FName, int32> CachedSnapshotNameIndexMap;

	// attributes to be displayed
	TArray<TSharedPtr<FAnimAttributeEntry>> FilteredAttributeEntries;

	FString FilterText;

	
	/** value view */
	TSharedPtr<SScrollBox> ValueViewBox;
	bool bShouldRefreshValueView;
	TOptional<FAnimAttributeEntry> SelectedAttribute;

	struct FAttributeValueView
	{
		FAttributeValueView(FName InSnapshotName, const FAnimAttributeEntry& InSelectedAttribute);

		template<typename ContainerType>
		void UpdateValue(const ContainerType& InAttributeContainer) const
		{
			const uint8* ValuePtr = InAttributeContainer.Find(SubjectAttribute.GetScriptStruct(), SubjectAttribute.GetAttributeId());

			if (ensure(ValuePtr))
			{
				FMemory::Memcpy(
					StructData->GetStructMemory(),
					ValuePtr,
					StructData->GetStruct()->GetStructureSize());
			}
		}


		FAnimAttributeEntry SubjectAttribute;
		FName SnapshotName;
		TSharedPtr<FStructOnScope> StructData;
		TSharedPtr<IStructureDetailsView> ViewWidget;
	};
	
	TArray<FAttributeValueView> SelectedAttributeSnapshotValueViews;
};


inline void SAnimAttributeView::ClearListView()
{
	CachedNumSnapshots = 0;
	CachedAttributeIdentifierLists.Reset();
	CachedAttributeSnapshotMap.Reset();
	CachedSnapshotNameIndexMap.Reset();
	FilteredAttributeEntries.Reset();

	bShouldRefreshListView = true;
}

class SAnimAttributeViewer : public SCompoundWidget
{
	SLATE_BEGIN_ARGS( SAnimAttributeViewer ) {}
	SLATE_END_ARGS()
	
	UE_API void Construct( const FArguments& InArgs, const TSharedRef<class IPersonaPreviewScene>& InPreviewScene);	
	virtual ~SAnimAttributeViewer() override = default;
	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

protected:
	UE_API UAnimInstance* GetAnimInstance() const;
protected:
	/** Pointer to the preview scene we are bound to */
	TWeakPtr<class IPersonaPreviewScene> PreviewScenePtr;

	TSharedPtr<SAnimAttributeView> AttributeView;	
	TSharedPtr<SPoseWatchPicker> PoseWatchPicker;
};

#undef UE_API
