// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/UObjectGlobals.h"
#include "Misc/Attribute.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Widgets/SLeafWidget.h"
#include "Styling/AppStyle.h"
#include "Framework/MarqueeRect.h"
#include "AssetRegistry/AssetData.h"
#include "Framework/Commands/Commands.h"
#include "EditorUndoClient.h"
#include "Containers/ArrayView.h"
#include "SAnimNotifyShared.h"
#include "SAnimTimingPanelShared.h"

#define UE_API PERSONA_API

class SAnimNotifyNode;
class SAnimNotifyTrack;
class SOverlay;

DECLARE_DELEGATE_OneParam(FOnDeleteNotify, struct FAnimNotifyEvent*)
DECLARE_DELEGATE_RetVal_FourParams(FReply, FOnNotifyNodeDragStarted, TSharedRef<SAnimNotifyNode>, const FPointerEvent&, const FVector2D&, const bool)
DECLARE_DELEGATE_RetVal_FiveParams(FReply, FOnNotifyNodesDragStarted, TArray<TSharedPtr<SAnimNotifyNode>>, TSharedRef<SWidget>, const FVector2D&, const FVector2D&, const bool)
DECLARE_DELEGATE_RetVal(float, FOnGetDraggedNodePos)

//////////////////////////////////////////////////////////////////////////
// SAnimNotifyNode

class SAnimNotifyNode : public SLeafWidget
{
public:
	SLATE_BEGIN_ARGS(SAnimNotifyNode) {}
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& Declaration);
	
	/** Set the owning Sequence */
	UE_API void SetSequence(class UAnimSequenceBase* InSequence);

	/** Set the owning Anim Notify Event */
	UE_API void SetAnimNotify(FAnimNotifyEvent* InAnimNotifyEvent);

	/** Set the AnimSyncMarker */
	UE_API void SetAnimSyncMarker(FAnimSyncMarker* InMarker);

	/** Set event when Drag Started */
	UE_API void SetOnNotifyNodeDragStarted(FOnNotifyNodeDragStarted InDragStarted);

	/** Set event when notify state handle being dragged */
	UE_API void SetOnNotifyStateHandleBeingDragged(FOnNotifyStateHandleBeingDragged InHandleBeingDragged);

	/** Set event when update panel */
	UE_API void SetOnUpdatePanel(FOnUpdatePanel InOnUpdatePanel);

	/** Set event when pan track request */
	UE_API void SetPanTrackRequest(FPanTrackRequest InPanTrackRequest);

	/** Set event when track selection changes */
	UE_API void SetOnTrackSelectionChanged(FOnTrackSelectionChanged InOnTrackSelectionChanged);

	/** Set the view input min */
	UE_API void SetViewInputMin(TAttribute<float> InViewInputMin);

	/** Set the view input max */
	UE_API void SetViewInputMax(TAttribute<float> InViewInputMax);

	/** Set the state end timing node */
	UE_API void SetStateEndTimingNode(TSharedPtr<SAnimTimingNode> InStateEndTimingNode);

	/** Set event when snapping position */
	UE_API void SetOnSnapPosition(FOnSnapPosition InOnSnapPosition);

	// SWidget interface
	UE_API virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	UE_API virtual void OnFocusLost(const FFocusEvent& InFocusEvent) override;
	UE_API virtual bool SupportsKeyboardFocus() const override;
	UE_API virtual FCursorReply OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const override;
	// End of SWidget interface

	// SNodePanel::SNode interface
	void UpdateSizeAndPosition(const FGeometry& AllottedGeometry);
	FVector2D GetWidgetPosition() const;
	FVector2D GetNotifyPosition() const;
	FVector2D GetNotifyPositionOffset() const;
	FVector2D GetSize() const;
	bool HitTest(const FGeometry& AllottedGeometry, FVector2D MouseLocalPose) const;

	// Extra hit testing to decide whether or not the duration handles were hit on a state node
	ENotifyStateHandleHit::Type DurationHandleHitTest(const FVector2D& CursorScreenPosition) const;

	UObject* GetObjectBeingDisplayed() const;
	// End of SNodePanel::SNode

	UE_API virtual FVector2D ComputeDesiredSize(float) const override;
	UE_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const /*override*/;

	/** Helpers to draw scrub handles and snap offsets */
	void DrawHandleOffset(const float& Offset, const float& HandleCentre, FSlateWindowElementList& OutDrawElements, int32 MarkerLayer, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FLinearColor NodeColour) const;
	void DrawScrubHandle(float ScrubHandleCentre, FSlateWindowElementList& OutDrawElements, int32 ScrubHandleID, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FLinearColor NodeColour) const;

	FLinearColor GetNotifyColor() const;
	FText GetNotifyText() const;

	/** Node object interface */
	INotifyNodeObjectInterface* NodeObjectInterface;

	/** In object storage for our interface struct, saves us having to dynamically allocate what will be a very small struct*/
	uint8 NodeObjectInterfaceStorage[MAX_NOTIFY_NODE_OBJECT_INTERFACE_SIZE];

	/** Helper function to create our node interface object */
	template<typename InterfaceType, typename ParamType>
	void MakeNodeInterface(ParamType& InParam)
	{
		check(sizeof(InterfaceType) <= MAX_NOTIFY_NODE_OBJECT_INTERFACE_SIZE); //Not enough space, check definiton of MAX_NOTIFY_NODE_OBJECT_INTERFACE_SIZE
		NodeObjectInterface = new(NodeObjectInterfaceStorage)InterfaceType(InParam);
	}

	void DropCancelled();

	/** Returns the size of this notifies duration in screen space */
	float GetDurationSize() const { return NotifyDurationSizeX; }

	/** Sets the position the mouse was at when this node was last hit */
	void SetLastMouseDownPosition(const FVector2D& CursorPosition) { LastMouseDownPosition = CursorPosition; }

	/** The minimum possible duration that a notify state can have */
	static const float MinimumStateDuration;

	const FVector2D& GetScreenPosition() const
	{
		return ScreenPosition;
	}

	const float GetLastSnappedTime() const
	{
		return LastSnappedTime;
	}

	void ClearLastSnappedTime()
	{
		LastSnappedTime = -1.0f;
	}

	void SetLastSnappedTime(float NewSnapTime)
	{
		LastSnappedTime = NewSnapTime;
	}

private:
	FText GetNodeTooltip() const;

	/** Detects any overflow on the anim notify track and requests a track pan */
	float HandleOverflowPan(const FVector2D& ScreenCursorPos, float TrackScreenSpaceXPosition, float TrackScreenSpaceMin, float TrackScreenSpaceMax);

	/** Finds a snap position if possible for the provided scrub handle, if it is not possible, returns -1.0f */
	float GetScrubHandleSnapPosition(float NotifyInputX, ENotifyStateHandleHit::Type HandleToCheck, TOptional<EAnimEventTriggerOffsets::Type>& OffsetForSnap);

	UE_API virtual FReply OnFocusReceived(const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent) override;

protected:
	/** The sequence that the AnimNotifyEvent for Notify lives in */
	UAnimSequenceBase* Sequence;
	FSlateFontInfo Font;

	TAttribute<float>			ViewInputMin;
	TAttribute<float>			ViewInputMax;
	FVector2f					CachedAllotedGeometrySize;
	FVector2D					ScreenPosition;
	float						LastSnappedTime;

	bool						bDrawTooltipToRight;
	bool						bBeingDragged;
	bool						bSelected;

	// Index for undo transactions for dragging, as a check to make sure it's active
	int32						DragMarkerTransactionIdx;

	/** The scrub handle currently being dragged, if any */
	ENotifyStateHandleHit::Type CurrentDragHandle;

	float						NotifyTimePositionX;
	float						NotifyDurationSizeX;
	float						NotifyScrubHandleCentre;

	float						WidgetX;
	FVector2D					WidgetSize;

	FVector2D					TextSize;
	float						LabelWidth;
	FVector2D					BranchingPointIconSize;

	/** Last position the user clicked in the widget */
	FVector2D					LastMouseDownPosition;

	/** Delegate that is called when the user initiates dragging */
	FOnNotifyNodeDragStarted	OnNodeDragStarted;

	/** Delegate that is called when a notify state handle is being dragged */
	FOnNotifyStateHandleBeingDragged	OnNotifyStateHandleBeingDragged;

	/** Delegate to pan the track, needed if the markers are dragged out of the track */
	FPanTrackRequest			PanTrackRequest;

	/** Delegate used to snap positions */
	FOnSnapPosition				OnSnapPosition;

	/** Delegate to signal selection changing */
	FOnTrackSelectionChanged	OnSelectionChanged;

	/** Delegate to redraw the notify panel */
	FOnUpdatePanel				OnUpdatePanel;

	/** Cached owning track geometry */
	FGeometry CachedTrackGeometry;

	TSharedPtr<SOverlay> EndMarkerNodeOverlay;

	friend class SAnimNotifyTrack;
};

#undef UE_API