// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimSequence.h"
#include "Animation/AnimTypes.h"
#include "Animation/AnimNotifies/AnimNotify.h"
#include "Animation/AnimNotifies/AnimNotifyState.h"
#include "Input/Events.h"
#include "Internationalization/Internationalization.h"

#define LOCTEXT_NAMESPACE "AnimNotifyPanel"

class SAnimNotifyNode;

namespace ENotifyStateHandleHit
{
	enum Type
	{
		Start,
		End,
		None
	};
}

DECLARE_DELEGATE_FourParams(FOnNotifyStateHandleBeingDragged, TSharedPtr<SAnimNotifyNode> /*NotifyNode*/, const FPointerEvent& /*PointerEvent*/, ENotifyStateHandleHit::Type /*Handle*/, float /*Time*/)
DECLARE_DELEGATE_FourParams(FOnNotifyNodesBeingDragged, const TArray<TSharedPtr<SAnimNotifyNode>>& /*NotifyNodes*/, const class FDragDropEvent& /*DragDropEvent*/, float /*XPosition*/, float /*Time*/)
DECLARE_DELEGATE_TwoParams(FPanTrackRequest, int32, FVector2f)
DECLARE_DELEGATE(FOnTrackSelectionChanged)
DECLARE_DELEGATE(FOnUpdatePanel)
DECLARE_DELEGATE_RetVal_FourParams(bool, FOnSnapPosition, float& /*InOutTimeToSnap*/, float /*InSnapMargin*/, TArrayView<const FName> /*InSkippedSnapTypes*/, FName& /*OutSnapType*/)

namespace ENotifyNodeObjectTypes
{
	enum Type
	{
		NOTIFY,
		SYNC_MARKER
	};
};

struct INotifyNodeObjectInterface
{
	static FText MakeTooltipFromTime(const UAnimSequenceBase* InSequence, float InSeconds, float InDuration)
	{
		const FText Frame = FText::AsNumber(InSequence->GetFrameAtTime(InSeconds));
		const FText Seconds = FText::AsNumber(InSeconds);

		if (InDuration > 0.0f)
		{
			const FText Duration = FText::AsNumber(InDuration);
			return FText::Format(LOCTEXT("NodeToolTipLong", "@ {0} sec (frame {1}) for {2} sec"), Seconds, Frame, Duration);
		}
		else
		{
			return FText::Format(LOCTEXT("NodeToolTipShort", "@ {0} sec (frame {1})"), Seconds, Frame);
		}
	}

	virtual ENotifyNodeObjectTypes::Type GetType() const = 0;
	virtual FAnimNotifyEvent* GetNotifyEvent() = 0;
	virtual int GetTrackIndex() const = 0;
	virtual float GetTime(EAnimLinkMethod::Type ReferenceFrame = EAnimLinkMethod::Absolute) const = 0;
	virtual float GetDuration() = 0;
	virtual FName GetName() = 0;
	virtual TOptional<FLinearColor> GetEditorColor() = 0;
	virtual FText GetNodeTooltip(const UAnimSequenceBase* Sequence) = 0;
	virtual TOptional<UObject*> GetObjectBeingDisplayed() = 0;
	virtual bool IsBranchingPoint() = 0;
	bool operator<(const INotifyNodeObjectInterface& Rhs) const { return GetTime() < Rhs.GetTime(); }

	virtual void SetTime(float Time, EAnimLinkMethod::Type ReferenceFrame = EAnimLinkMethod::Absolute) = 0;
	virtual void SetDuration(float Duration) = 0;

	virtual void HandleDrop(class UAnimSequenceBase* Sequence, float Time, int32 TrackIndex) = 0;
	virtual void CacheName() = 0;

	virtual void Delete(UAnimSequenceBase* Seq) = 0;
	virtual void MarkForDelete(UAnimSequenceBase* Seq) = 0;

	virtual void ExportForCopy(UAnimSequenceBase* Seq, FString& StrValue) const = 0;

	virtual FGuid GetGuid() const = 0;
};

struct FNotifyNodeInterface : public INotifyNodeObjectInterface
{
	FAnimNotifyEvent* NotifyEvent;

	// Cached notify name (can be generated by blueprints so want to cache this instead of hitting VM) 
	FName CachedNotifyName;

	// Stable Guid that allows us to refer to notify event
	FGuid Guid;

	FNotifyNodeInterface(FAnimNotifyEvent* InAnimNotifyEvent) : NotifyEvent(InAnimNotifyEvent), Guid(NotifyEvent->Guid) {}
	virtual ENotifyNodeObjectTypes::Type GetType() const override { return ENotifyNodeObjectTypes::NOTIFY; }
	virtual FAnimNotifyEvent* GetNotifyEvent() override { return NotifyEvent; }
	virtual int GetTrackIndex() const override { return NotifyEvent->TrackIndex; }
	virtual float GetTime(EAnimLinkMethod::Type ReferenceFrame = EAnimLinkMethod::Absolute) const override { return NotifyEvent->GetTime(ReferenceFrame); }
	virtual float GetDuration() override { return NotifyEvent->GetDuration(); }
	virtual FName GetName() override { return CachedNotifyName; }
	virtual bool IsBranchingPoint() override { return NotifyEvent->IsBranchingPoint(); }
	virtual TOptional<FLinearColor> GetEditorColor() override
	{
		TOptional<FLinearColor> ReturnColour;
		if (NotifyEvent->Notify)
		{
			ReturnColour = NotifyEvent->Notify->GetEditorColor();
		}
		else if (NotifyEvent->NotifyStateClass)
		{
			ReturnColour = NotifyEvent->NotifyStateClass->GetEditorColor();
		}
		return ReturnColour;
	}

	virtual FText GetNodeTooltip(const UAnimSequenceBase* Sequence) override
	{
		FText ToolTipText = MakeTooltipFromTime(Sequence, NotifyEvent->GetTime(), NotifyEvent->GetDuration());

		if (NotifyEvent->IsBranchingPoint())
		{
			ToolTipText = FText::Format(LOCTEXT("AnimNotify_ToolTipBranchingPoint", "{0} (BranchingPoint)"), ToolTipText);
		}

		UObject* NotifyToDisplayClassOf = NotifyEvent->Notify;
		if (NotifyToDisplayClassOf == nullptr)
		{
			NotifyToDisplayClassOf = NotifyEvent->NotifyStateClass;
		}

		if (NotifyToDisplayClassOf != nullptr)
		{
			ToolTipText = FText::Format(LOCTEXT("AnimNotify_ToolTipNotifyClass", "{0}\nClass: {1}"), ToolTipText, NotifyToDisplayClassOf->GetClass()->GetDisplayNameText());
		}

		return ToolTipText;
	}

	virtual TOptional<UObject*> GetObjectBeingDisplayed() override
	{
		if (NotifyEvent->Notify)
		{
			return TOptional<UObject*>(NotifyEvent->Notify);
		}

		if (NotifyEvent->NotifyStateClass)
		{
			return TOptional<UObject*>(NotifyEvent->NotifyStateClass);
		}
		return TOptional<UObject*>();
	}

	virtual void SetTime(float Time, EAnimLinkMethod::Type ReferenceFrame = EAnimLinkMethod::Absolute) override { NotifyEvent->SetTime(Time, ReferenceFrame); }
	virtual void SetDuration(float Duration) override { NotifyEvent->SetDuration(Duration); }

	virtual void HandleDrop(class UAnimSequenceBase* Sequence, float Time, int32 TrackIndex) override
	{
		float EventDuration = NotifyEvent->GetDuration();

		NotifyEvent->Link(Sequence, Time, NotifyEvent->GetSlotIndex());
		NotifyEvent->RefreshTriggerOffset(Sequence->CalculateOffsetForNotify(NotifyEvent->GetTime()));

		if (EventDuration > 0.0f)
		{
			NotifyEvent->EndLink.Link(Sequence, NotifyEvent->GetTime() + EventDuration, NotifyEvent->GetSlotIndex());
			NotifyEvent->RefreshEndTriggerOffset(Sequence->CalculateOffsetForNotify(NotifyEvent->EndLink.GetTime()));
		}
		else
		{
			NotifyEvent->EndTriggerTimeOffset = 0.0f;
		}

		NotifyEvent->TrackIndex = TrackIndex;
	}

	virtual void CacheName() override
	{
		if (NotifyEvent->Notify)
		{
			CachedNotifyName = FName(*NotifyEvent->Notify->GetNotifyName());
		}
		else if (NotifyEvent->NotifyStateClass)
		{
			CachedNotifyName = FName(*NotifyEvent->NotifyStateClass->GetNotifyName());
		}
		else
		{
			CachedNotifyName = NotifyEvent->NotifyName;
		}
	}

	virtual void Delete(UAnimSequenceBase* Seq) override
	{
		for (int32 I = 0; I < Seq->Notifies.Num(); ++I)
		{
			if (NotifyEvent == &(Seq->Notifies[I]))
			{
				Seq->Notifies.RemoveAt(I);
				Seq->PostEditChange();
				Seq->MarkPackageDirty();
				break;
			}
		}
	}


	virtual void MarkForDelete(UAnimSequenceBase* Seq) override
	{
		for (int32 I = 0; I < Seq->Notifies.Num(); ++I)
		{
			if (NotifyEvent == &(Seq->Notifies[I]))
			{
				Seq->Notifies[I].Guid = FGuid();
				break;
			}
		}
	}

	virtual void ExportForCopy(UAnimSequenceBase* Seq, FString& StrValue) const override
	{
		int32 Index = INDEX_NONE;
		for (int32 NotifyIdx = 0; NotifyIdx < Seq->Notifies.Num(); ++NotifyIdx)
		{
			if (NotifyEvent == &Seq->Notifies[NotifyIdx])
			{
				Index = NotifyIdx;
				break;
			}
		}

		check(Index != INDEX_NONE);

		FArrayProperty* ArrayProperty = NULL;
		uint8* PropertyData = Seq->FindNotifyPropertyData(Index, ArrayProperty);
		if (PropertyData && ArrayProperty)
		{
			ArrayProperty->Inner->ExportTextItem_Direct(StrValue, PropertyData, PropertyData, Seq, PPF_Copy);
		}
	}

	virtual FGuid GetGuid() const override
	{
		return Guid;
	}

	static void RemoveInvalidNotifies(UAnimSequenceBase* SeqBase)
	{
		SeqBase->Notifies.RemoveAll([](const FAnimNotifyEvent& InNotifyEvent) { return !InNotifyEvent.Guid.IsValid(); });
		SeqBase->PostEditChange();
		SeqBase->MarkPackageDirty();
	}
};

struct FSyncMarkerNodeInterface : public INotifyNodeObjectInterface
{
	FAnimSyncMarker* SyncMarker;

	// Stable Guid that allows us to refer to sync marker event
	FGuid Guid;

	FSyncMarkerNodeInterface(FAnimSyncMarker* InSyncMarker) : SyncMarker(InSyncMarker), Guid(SyncMarker->Guid) {}
	virtual ENotifyNodeObjectTypes::Type GetType() const override { return ENotifyNodeObjectTypes::SYNC_MARKER; }
	virtual FAnimNotifyEvent* GetNotifyEvent() override { return NULL; }
	virtual int GetTrackIndex() const override { return SyncMarker->TrackIndex; }
	virtual float GetTime(EAnimLinkMethod::Type ReferenceFrame = EAnimLinkMethod::Absolute) const override { return SyncMarker->Time; }
	virtual float GetDuration() override { return 0.f; }
	virtual FName GetName() override { return SyncMarker->MarkerName; }
	virtual bool IsBranchingPoint() override { return false; }
	virtual TOptional<FLinearColor> GetEditorColor() override
	{
		// Choose a random color based on the marker name
		// We don't use FLinearColor::MakeRandomSeededColor because it uses random RGB
		// which makes dull colors
		// Note: the 157 multiplier ensures similar hashes have different colors (see FLinearColor::MakeRandomColor)
		const uint8 Saturation = 255;
		const uint8 Luminosity = 255;
		const uint8 Hue = (uint8)GetTypeHash(SyncMarker->MarkerName.ToString()) * 157;
		return FLinearColor::MakeFromHSV8(Hue, Saturation, Luminosity);
	}

	virtual FText GetNodeTooltip(const UAnimSequenceBase* Sequence) override
	{
		return MakeTooltipFromTime(Sequence, SyncMarker->Time, 0.f);
	}

	virtual TOptional<UObject*> GetObjectBeingDisplayed() override
	{
		return TOptional<UObject*>();
	}

	virtual void SetTime(float Time, EAnimLinkMethod::Type ReferenceFrame = EAnimLinkMethod::Absolute) override { SyncMarker->Time = Time; }
	virtual void SetDuration(float Duration) override {}

	virtual void HandleDrop(class UAnimSequenceBase* Sequence, float Time, int32 TrackIndex) override
	{
		SyncMarker->Time = Time;
		SyncMarker->TrackIndex = TrackIndex;
	}

	virtual void CacheName() override {}

	virtual void Delete(UAnimSequenceBase* SeqBase) override
	{
		if (UAnimSequence* Seq = Cast<UAnimSequence>(SeqBase))
		{
			for (int32 I = 0; I < Seq->AuthoredSyncMarkers.Num(); ++I)
			{
				if (SyncMarker == &(Seq->AuthoredSyncMarkers[I]))
				{
					Seq->AuthoredSyncMarkers.RemoveAt(I);
					Seq->PostEditChange();
					Seq->MarkPackageDirty();
					break;
				}
			}
		}
	}

	virtual void MarkForDelete(UAnimSequenceBase* SeqBase) override
	{
		if (UAnimSequence* Seq = Cast<UAnimSequence>(SeqBase))
		{
			for (int32 I = 0; I < Seq->AuthoredSyncMarkers.Num(); ++I)
			{
				if (SyncMarker == &(Seq->AuthoredSyncMarkers[I]))
				{
					Seq->AuthoredSyncMarkers[I].Guid = FGuid();
					break;
				}
			}
		}
	}

	virtual void ExportForCopy(UAnimSequenceBase* SeqBase, FString& StrValue) const override
	{
		if (UAnimSequence* Seq = Cast<UAnimSequence>(SeqBase))
		{
			int32 Index = INDEX_NONE;
			for (int32 SyncMarkerIdx = 0; SyncMarkerIdx < Seq->AuthoredSyncMarkers.Num(); ++SyncMarkerIdx)
			{
				if (SyncMarker == &Seq->AuthoredSyncMarkers[SyncMarkerIdx])
				{
					Index = SyncMarkerIdx;
					break;
				}
			}

			check(Index != INDEX_NONE);

			FArrayProperty* ArrayProperty = NULL;
			uint8* PropertyData = Seq->FindSyncMarkerPropertyData(Index, ArrayProperty);
			if (PropertyData && ArrayProperty)
			{
				ArrayProperty->Inner->ExportTextItem_Direct(StrValue, PropertyData, PropertyData, Seq, PPF_Copy);
			}
		}
	}

	virtual FGuid GetGuid() const override
	{
		return Guid;
	}

	static void RemoveInvalidSyncMarkers(UAnimSequenceBase* SeqBase)
	{
		if (UAnimSequence* Seq = Cast<UAnimSequence>(SeqBase))
		{
			Seq->AuthoredSyncMarkers.RemoveAll([](const FAnimSyncMarker& InSyncMarker) { return !InSyncMarker.Guid.IsValid(); });
			Seq->PostEditChange();
			Seq->MarkPackageDirty();
		}
	}
};

// Struct that allows us to get the max value of 2 numbers at compile time
template<int32 A, int32 B>
struct NotifyCompileTimeMax
{
	enum Max { VALUE = (A > B) ? A : B };
};

// Size of biggest object that we can store in our node, if new node interfaces are added they should be part of this calculation
const int32 MAX_NOTIFY_NODE_OBJECT_INTERFACE_SIZE = NotifyCompileTimeMax<sizeof(FNotifyNodeInterface), sizeof(FSyncMarkerNodeInterface)>::VALUE;

#undef LOCTEXT_NAMESPACE
