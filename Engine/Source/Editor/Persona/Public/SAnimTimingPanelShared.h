// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once
#include "Widgets/SCompoundWidget.h"


namespace ENotifyTimingElementType
{
	enum Type
	{
		QueuedNotify,
		BranchPointNotify,
		NotifyStateBegin,
		NotifyStateEnd,
		Section,
		Max,
	};
};

struct FTimingRelevantElementBase
{
	virtual ~FTimingRelevantElementBase() = default;

	virtual FName GetTypeName()
	{
		return FName(TEXT("BASE"));
	}

	virtual float GetElementTime() const
	{
		return -1.0f;
	}

	virtual int32 GetElementSortPriority() const
	{
		return 0;
	}

	virtual ENotifyTimingElementType::Type GetType()
	{
		return ENotifyTimingElementType::Max;
	}

	// Get a list of descriptions key/values to describe the element.
	// Intended for UI/Tooltip use
	virtual void GetDescriptionItems(TMap<FString, FText>& Items)
	{

	}

	// Comparison for sorting lists of elements
	virtual bool Compare(const FTimingRelevantElementBase& Other)
	{
		if (FMath::IsNearlyEqual(GetElementTime(), Other.GetElementTime(), SMALL_NUMBER))
		{
			return GetElementSortPriority() < Other.GetElementSortPriority();
		}

		return GetElementTime() < Other.GetElementTime();
	}

	// Where in the order for the sequence this element will trigger
	int32 TriggerIdx;
};

//////////////////////////////////////////////////////////////////////////
// The content of SAnimTimingTrackNode, separated to be used in non STrack widgets
//////////////////////////////////////////////////////////////////////////
class SAnimTimingNode : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAnimTimingNode)
		: _InElement()
		, _bUseTooltip(true)
		{
		}

		SLATE_ARGUMENT(TSharedPtr<FTimingRelevantElementBase>, InElement)
		SLATE_ARGUMENT(bool, bUseTooltip)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	virtual FVector2D ComputeDesiredSize(float) const override;

protected:

	// The observed element
	TSharedPtr<FTimingRelevantElementBase> Element;
};