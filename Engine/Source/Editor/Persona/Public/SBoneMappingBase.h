// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Input/SComboButton.h"
#include "BoneSelectionWidget.h"

#define UE_API PERSONA_API

class IEditableSkeleton;
class URig;
class USkeleton;
template <typename ItemType> class SListView;

//////////////////////////////////////////////////////////////////////////
// FDisplayedBoneMappingInfo

class FDisplayedBoneMappingInfo
{
public:
	FName Name;
	FString DisplayName;

	/** Static function for creating a new item, but ensures that you can only have a TSharedRef to one */
	static TSharedRef<FDisplayedBoneMappingInfo> Make(const FName NodeName, const FString DisplayName)
	{
		return MakeShareable(new FDisplayedBoneMappingInfo(NodeName, DisplayName));
	}

	FName GetNodeName() const
	{
		return Name;
	}

	FString GetDisplayName() const
	{
		return DisplayName;
	}

protected:
	/** Hidden constructor, always use Make above */
	FDisplayedBoneMappingInfo(const FName InNodeName, const FString InDisplayName)
		: Name( InNodeName )
		, DisplayName( InDisplayName )
	{}

	/** Hidden constructor, always use Make above */
	FDisplayedBoneMappingInfo() {}
};

typedef SListView< TSharedPtr<FDisplayedBoneMappingInfo> > SBoneMappingListType;

//////////////////////////////////////////////////////////////////////////
// SBoneMappingListRow

typedef TSharedPtr< FDisplayedBoneMappingInfo > FDisplayedBoneMappingInfoPtr;

DECLARE_DELEGATE_TwoParams(FOnBoneMappingChanged, FName /** NodeName */, FName /** BoneName **/);
DECLARE_DELEGATE_RetVal_OneParam(FName, FOnGetBoneMapping, FName /** Node Name **/);
DECLARE_DELEGATE_RetVal(FText&, FOnGetFilteredText)
DECLARE_DELEGATE_TwoParams(FOnCreateBoneMapping, const FString&, TArray< TSharedPtr<FDisplayedBoneMappingInfo> >&)

class SBoneMappingListRow
	: public SMultiColumnTableRow< FDisplayedBoneMappingInfoPtr >
{
public:

	SLATE_BEGIN_ARGS(SBoneMappingListRow) {}

	/** The item for this row **/
	SLATE_ARGUMENT(FDisplayedBoneMappingInfoPtr, Item)

		/* Widget used to display the list of retarget sources*/
		SLATE_ARGUMENT(TSharedPtr<SBoneMappingListType>, BoneMappingListView)

		SLATE_EVENT(FOnBoneMappingChanged, OnBoneMappingChanged)

		SLATE_EVENT(FOnGetBoneMapping, OnGetBoneMapping)

		SLATE_EVENT(FGetReferenceSkeleton, OnGetReferenceSkeleton)

		SLATE_EVENT(FOnGetFilteredText, OnGetFilteredText)

		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTableView);

	/** Overridden from SMultiColumnTableRow.  Generates a widget for this column of the tree row. */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

private:

	/** Widget used to display the list of retarget sources*/
	TSharedPtr<SBoneMappingListType> BoneMappingListView;

	/** The name and weight of the retarget source*/
	FDisplayedBoneMappingInfoPtr	Item;

	// Bone tree widget delegates
	void OnBoneSelectionChanged(FName Name);
	FReply OnClearButtonClicked();
	FName GetSelectedBone(bool& bMultipleValues) const;
	FText GetFilterText() const;

	FOnBoneMappingChanged	OnBoneMappingChanged;
	FOnGetBoneMapping		OnGetBoneMapping;
	FGetReferenceSkeleton	OnGetReferenceSkeleton;
	FOnGetFilteredText		OnGetFilteredText;
};

//////////////////////////////////////////////////////////////////////////
// SBoneMappingBase

class SBoneMappingBase : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SBoneMappingBase )
	{}

	SLATE_EVENT(FOnBoneMappingChanged, OnBoneMappingChanged)
	SLATE_EVENT(FOnGetBoneMapping, OnGetBoneMapping)
	SLATE_EVENT(FGetReferenceSkeleton, OnGetReferenceSkeleton)
	SLATE_EVENT(FOnCreateBoneMapping, OnCreateBoneMapping)

	SLATE_END_ARGS()

	/**
	* Slate construction function
	*
	* @param InArgs - Arguments passed from Slate
	*
	*/
	UE_API void Construct( const FArguments& InArgs, FSimpleMulticastDelegate& InOnPostUndo );

	/**
	* Filters the SListView when the user changes the search text box (NameFilterBox)
	*
	* @param SearchText - The text the user has typed
	*
	*/
	UE_API void OnFilterTextChanged( const FText& SearchText );

	/**
	* Filters the SListView when the user hits enter or clears the search box
	* Simply calls OnFilterTextChanged
	*
	* @param SearchText - The text the user has typed
	* @param CommitInfo - Not used
	*
	*/
	UE_API void OnFilterTextCommitted( const FText& SearchText, ETextCommit::Type CommitInfo );

	/**
	* Create a widget for an entry in the tree from an info
	*
	* @param InInfo - Shared pointer to the morph target we're generating a row for
	* @param OwnerTable - The table that owns this row
	*
	* @return A new Slate widget, containing the UI for this row
	*/
	UE_API TSharedRef<ITableRow> GenerateBoneMappingRow(TSharedPtr<FDisplayedBoneMappingInfo> InInfo, const TSharedRef<STableViewBase>& OwnerTable);

	/**
	* Handler for the delete of retarget source
	*/
	UE_API void RefreshBoneMappingList();
private:

	/**
	* Accessor so our rows can grab the filtertext for highlighting
	*
	*/
	FText& GetFilterText() { return FilterText; }

	/** Box to filter to a specific morph target name */
	TSharedPtr<SSearchBox>	NameFilterBox;

	/** Widget used to display the list of retarget sources */
	TSharedPtr<SBoneMappingListType> BoneMappingListView;

	/** A list of retarget sources. Used by the BoneMappingListView. */
	TArray< TSharedPtr<FDisplayedBoneMappingInfo> > BoneMappingList;

	/** Current text typed into NameFilterBox */
	FText FilterText;

	/** Delegate for undo/redo transaction **/
	UE_API void PostUndo();

	UE_API void OnBoneMappingChanged( FName NodeName, FName BoneName );
	UE_API FName GetBoneMapping( FName NodeName );

	FGetReferenceSkeleton	OnGetReferenceSkeletonDelegate;
	FOnBoneMappingChanged	OnBoneMappingChangedDelegate;
	FOnGetBoneMapping		OnGetBoneMappingDelegate;
	FOnCreateBoneMapping	OnCreateBoneMappingDelegate;
};

#undef UE_API
