// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "ClothingAssetFactoryInterface.h"
#include "IDetailCustomization.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Engine/SkeletalMesh.h"

#define UE_API PERSONA_API

DECLARE_DELEGATE_OneParam(FOnCreateClothingRequested, FSkeletalMeshClothBuildParams&);

class FClothCreateSettingsCustomization : public IDetailCustomization
{
public:

	FClothCreateSettingsCustomization(TWeakObjectPtr<USkeletalMesh> InMeshPtr, bool bInIsSubImport)
		: bIsSubImport(bInIsSubImport)
		, MeshPtr(InMeshPtr)
		, ParamsStruct(nullptr)
	{};

	static TSharedRef<IDetailCustomization> MakeInstance(TWeakObjectPtr<USkeletalMesh> MeshPtr, bool bIsSubImport)
	{
		return MakeShareable(new FClothCreateSettingsCustomization(MeshPtr, bIsSubImport));
	}

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

protected:

	TSharedRef<SWidget> OnGetTargetAssetMenu();
	FText GetTargetAssetText() const;
	void OnAssetSelected(int32 InMeshClothingIndex);

	TSharedRef<SWidget> OnGetTargetLodMenu();
	FText GetTargetLodText() const;
	void OnLodSelected(int32 InLodIndex);
	bool CanSelectLod() const;

	bool bIsSubImport;
	TWeakObjectPtr<USkeletalMesh> MeshPtr;
	FSkeletalMeshClothBuildParams* ParamsStruct;
};

class SCreateClothingSettingsPanel : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SCreateClothingSettingsPanel)
		: _LodIndex(INDEX_NONE)
		, _SectionIndex(INDEX_NONE)
		, _bIsSubImport(false)
	{}

		// Name of the mesh we're operating on
		SLATE_ARGUMENT(FString, MeshName)
		// Mesh LOD index we want to target
		SLATE_ARGUMENT(int32, LodIndex)
		// Mesh section index we want to targe
		SLATE_ARGUMENT(int32, SectionIndex)
		// Weak ptr to the mesh we're building for
		SLATE_ARGUMENT(TWeakObjectPtr<USkeletalMesh>, Mesh)
		// Whether this window is for a sub import (importing a LOD or replacing a LOD)
		SLATE_ARGUMENT(bool, bIsSubImport)
		// Callback to handle create request
		SLATE_EVENT(FOnCreateClothingRequested, OnCreateRequested)

	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs);

private:

	// Params struct to hold request data
	FSkeletalMeshClothBuildParams BuildParams;

	// Whether or not to show sub params (lod imports)
	bool bIsSubImport;

	// Create button functionality
	UE_API FText GetCreateButtonTooltip() const;
	UE_API bool CanCreateClothing() const;

	// Handlers for panel buttons
	UE_API FReply OnCreateClicked();

	// Called when the create button is clicked, so external caller can handle the request
	FOnCreateClothingRequested OnCreateDelegate;

};

#undef UE_API
