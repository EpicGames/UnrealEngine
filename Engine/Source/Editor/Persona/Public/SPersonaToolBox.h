// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PersonaAssetEditorToolkit.h"

#include "CoreMinimal.h"
#include "StatusBarSubsystem.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/SCompoundWidget.h"
#include "WorkflowOrientedApp/WorkflowTabFactory.h"

#include "Delegates/DelegateCombinations.h"
#include "Templates/SharedPointer.h"
#include "Toolkits/IToolkit.h"

#define UE_API PERSONA_API

class FModeToolkit;
class FPersonaAssetEditorToolkit;
class IToolkit;
class SBorder;


struct FToolBoxSummoner : public FWorkflowTabFactory
{
public:
	FToolBoxSummoner(TSharedPtr<FPersonaAssetEditorToolkit> InPersonaEditorToolkit);

	virtual TSharedRef<SDockTab> SpawnTab(const FWorkflowTabSpawnInfo& Info) const override;
	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;
	virtual bool CanSpawnTab(const FSpawnTabArgs& SpawnArgs, TWeakPtr<FTabManager> WeakTabManager) const override;

protected:
	TWeakPtr<FPersonaAssetEditorToolkit> PersonaAssetEditorToolkit;
};

/** Widget to surface an Edit Mode its widget, similar to LevelEditor approach */
class SPersonaToolbox : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SPersonaToolbox ) {}
	SLATE_END_ARGS()

	UE_API ~SPersonaToolbox();

	UE_API void Construct(const FArguments& InArgs, const TSharedRef<FPersonaAssetEditorToolkit>& InOwningEditor);

	// Called to attach a toolkit to this toolbox widget (e.g. when the mode changes).
	UE_API void AttachToolkit(const TSharedRef<IToolkit>& InToolkit);

	// Called to detach a toolkit from this toolbox widget (e.g. when the mode changes).
	UE_API void DetachToolkit(const TSharedRef<IToolkit>& InToolkit);

	// Sets the owning tab so that we can update the tab icon based on what the toolkit provides.
	UE_API void SetOwningTab(const TSharedRef<SDockTab>& InOwningTab);

private:	
	UE_API void UpdateInlineContent(const TSharedPtr<IToolkit>& Toolkit, TSharedPtr<SWidget> InlineContent);

	UE_API void UpdatePalette(const TWeakPtr<FModeToolkit>& InWeakToolkit) const;

	// Whether to show the inline content or not.
	UE_API EVisibility GetInlineContentHolderVisibility() const;

	// The owning tab where the toolkit's interface is hosted. We need this to update the icons
	// and title as the toolkit changes.
	TWeakPtr<SDockTab> OwningTab;

	TWeakPtr<FPersonaAssetEditorToolkit> PersonaEditor;

	// Inline content area for editor modes
	TSharedPtr<SBorder> InlineContentHolder;

	// The container holding the mode toolbar
	TSharedPtr<SBorder> ModeToolBarContainer;

	// The active tool header area
	TSharedPtr<SBorder> ModeToolHeader;

	// The global status bar to show the name of the active tool.
	FStatusBarMessageHandle StatusBarMessageHandle;
};

#undef UE_API
