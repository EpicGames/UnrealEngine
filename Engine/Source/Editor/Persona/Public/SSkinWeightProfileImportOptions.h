// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/SWindow.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Text/STextBlock.h"
#include "IDetailCustomization.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "Widgets/Input/SEditableTextBox.h"
#endif
#include "PropertyHandle.h"
#include "Engine/SkeletalMesh.h"

#include "SSkinWeightProfileImportOptions.generated.h"

// Forward declares
class IDetailsView;
class SEditableTextBox;

UCLASS(Config = EditorPerProjectUserSettings)
class USkinWeightImportOptions : public UObject
{
	GENERATED_BODY()

public:
	/** Name of the to-be-imported Skin Weights Profile */
	UPROPERTY(EditAnywhere, Category=SkinWeights, Config)
	FString ProfileName;

	/** File path to FBX file containing Mesh with alternative set of Skin Weights */
	UPROPERTY(VisibleAnywhere, Category = SkinWeights)
	FString FilePath;
	
	/** Target LOD index this file corresponds to */
	UPROPERTY(EditAnywhere, Category = SkinWeights, meta=(DisplayName="LOD Index"))
	int32 LODIndex;
};

/** Details customization for the import object, used to hide certain properties when needed and ensure we do not get duplicate profile names */
class FSkinWeightImportOptionsCustomization : public IDetailCustomization
{
public:
	FSkinWeightImportOptionsCustomization(USkeletalMesh* InSkeletalMesh) : WeakSkeletalMesh(InSkeletalMesh) {}
	static TSharedRef<IDetailCustomization> MakeInstance(USkeletalMesh* InSkeletalMesh) { return MakeShareable(new FSkinWeightImportOptionsCustomization(InSkeletalMesh)); }
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

	FText OnGetProfileName() const;
	const bool IsProfileNameValid(const FString& NewName) const;
protected:
	TSharedPtr<SEditableTextBox> NameEditTextBox;
	TSharedPtr<IPropertyHandle> ProfileNameHandle;
	TWeakObjectPtr<USkeletalMesh> WeakSkeletalMesh;
	TArray<FString> RestrictedNames;

	void UpdateNameRestriction();
	
	void OnProfileNameChanged(const FText& InNewText);
	void OnProfileNameCommitted(const FText& InNewText, ETextCommit::Type InTextCommit);
};

class SSkinWeightProfileImportOptions : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SSkinWeightProfileImportOptions)
		: _ImportSettings(nullptr)
		, _WidgetWindow()
		, _SkeletalMesh()
	{}
		SLATE_ARGUMENT(USkinWeightImportOptions*, ImportSettings)
		SLATE_ARGUMENT(TSharedPtr<SWindow>, WidgetWindow)
		SLATE_ARGUMENT(USkeletalMesh*, SkeletalMesh)
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs);
	virtual bool SupportsKeyboardFocus() const override { return true; }

	FReply OnImport()
	{
		bShouldImport = true;
		if (WidgetWindow.IsValid())
		{
			WidgetWindow.Pin()->RequestDestroyWindow();
		}
		return FReply::Handled();
	}

	FReply OnCancel()
	{
		bShouldImport = false;
		if (WidgetWindow.IsValid())
		{
			WidgetWindow.Pin()->RequestDestroyWindow();
		}
		return FReply::Handled();
	}

	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override
	{
		if (InKeyEvent.GetKey() == EKeys::Escape)
		{
			return OnCancel();
		}

		return FReply::Unhandled();
	}

	bool ShouldImport() const
	{
		return bShouldImport;
	}


	SSkinWeightProfileImportOptions()
		: ImportSettings(nullptr)
		, bShouldImport(false)
	{}

private:
	USkinWeightImportOptions* ImportSettings;
	USkeletalMesh* SkeletalMesh;
	TWeakPtr<SWindow> WidgetWindow;
	TSharedPtr<SButton> ImportButton;
	TSharedPtr<IDetailsView> DetailsView;
	TSharedPtr<FSkinWeightImportOptionsCustomization> DetailsCustomization;
	bool bShouldImport;
};