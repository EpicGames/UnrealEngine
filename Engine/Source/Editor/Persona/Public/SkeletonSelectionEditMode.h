// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UnrealWidgetFwd.h"
#include "IPersonaEditMode.h"

class FCanvas;
class FEditorViewportClient;
class FPrimitiveDrawInterface;
class FSceneView;
class FViewport;
class USkeletalMeshSocket;
class USkeletalMeshComponent;
struct FViewportClick;
struct FSelectedSocketInfo;
class FViewportClient;
struct FReferenceSkeleton;
struct FGizmoState;
class UViewportInteractionsBehaviorSource;

class FSkeletonSelectionEditMode : public IPersonaEditMode
{
public:
	FSkeletonSelectionEditMode();

	/** IPersonaEditMode interface */
	virtual bool GetCameraTarget(FSphere& OutTarget) const override;
	virtual class IPersonaPreviewScene& GetAnimPreviewScene() const override;
	virtual void GetOnScreenDebugInfo(TArray<FText>& OutDebugInfo) const override;

	/** FEdMode interface */
	virtual bool StartTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport) override;
	virtual bool EndTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport) override;
	virtual bool InputDelta(FEditorViewportClient* InViewportClient, FViewport* InViewport, FVector& InDrag, FRotator& InRot, FVector& InScale) override;
	virtual void DrawHUD(FEditorViewportClient* ViewportClient, FViewport* Viewport, const FSceneView* View, FCanvas* Canvas) override;
	virtual bool AllowWidgetMove() override;
	virtual bool ShouldDrawWidget() const override;
	virtual bool UsesTransformWidget() const override;
	virtual bool UsesTransformWidget(UE::Widget::EWidgetMode CheckMode) const override;
	virtual FVector GetWidgetLocation() const override;
	virtual bool GetCustomDrawingCoordinateSystem(FMatrix& InMatrix, void* InData) override;
	virtual bool GetCustomInputCoordinateSystem(FMatrix& InMatrix, void* InData) override;
	virtual bool HandleClick(FEditorViewportClient* InViewportClient, HHitProxy *HitProxy, const FViewportClick &Click) override;
	virtual bool CanCycleWidgetMode() const override;
	virtual bool FrustumSelect(const FConvexVolume& InFrustum, FEditorViewportClient* InViewportClient, bool InSelect = true) override;
	
	virtual bool BeginTransform(const FGizmoState& InState) override;
	virtual bool EndTransform(const FGizmoState& InState) override;
	
protected:
	FTransform GetBoneTransform(const int32 BoneIndex) const;
	FTransform GetSocketTransform(const USkeletalMeshSocket* Socket) const;
	FTransform GetWorldSpaceBoneTransform(const FReferenceSkeleton& ReferenceSkeleton, const int32 BoneIndex) const;
	const FReferenceSkeleton& GetReferenceSkeletonForComponent(const USkeletalMeshComponent* Component) const;
	
private:
	/** Duplicates and selects a socket when we alt-drag */
	FSelectedSocketInfo DuplicateAndSelectSocket(const FSelectedSocketInfo& SocketInfoToDuplicate);

	/** Check whether the currently selected bone is in the required bones list */
	bool IsSelectedBoneRequired() const;

	/** Unscale a viewport's size by its DPI factor */
	static FIntPoint GetDPIUnscaledSize(FViewport* Viewport, FViewportClient* Client);

private:
	/** Whether we are currently in a manipulation  */
	bool bManipulating;	

	/** Whether the preview animation should resume after the manipulating has ended */
	bool bResumeAfterManipulating;

	/** Whether we are currently in a transaction  */
	bool bInTransaction;
};
