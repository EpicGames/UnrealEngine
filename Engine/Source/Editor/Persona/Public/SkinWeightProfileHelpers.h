// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Rendering/SkinWeightProfile.h"

class USkeletalMesh;
class FSkeletalMeshLODModel;

/** Set of editor-only helper functions used by various bits of UI related to Skin Weight profiles */
struct FSkinWeightProfileHelpers
{
	/** Tries to import a new set of Skin Weights for the given Skeletal Mesh from an FBX file */
	static void ImportSkinWeightProfile(USkeletalMesh* InSkeletalMesh);

	/** Tries to import a new set of Skin Weights for the given Skeletal Mesh at the given LOD index from an FBX file */
	static void ImportSkinWeightProfileLOD(USkeletalMesh* InSkeletalMesh, FName ProfileName, int32 LODIndex);

	/** Tries to re import the previously imported skin weights for the given Skeletal Mesh, Profile name and LOD Index */
	static void ReimportSkinWeightProfileLOD(USkeletalMesh* InSkeletalMesh, const FName& InProfileName, const int32 LODIndex);

	/** Tries to remove the previously imported skin weights for all LODs from the given Skeletal Mesh and Profile name */
	static void RemoveSkinWeightProfile(USkeletalMesh* InSkeletalMesh, const FName& InProfileName);

	/** Tries to remove the previously imported skin weights for all LODs from the given Skeletal Mesh, Profile name and LOD index*/
	static void RemoveSkinWeightProfileLOD(USkeletalMesh* InSkeletalMesh, const FName& InProfileName, const int32 LODIndex);

	/** Goes through every component using the given Skeletal Mesh and checks whether or not it currently has the Skin Weight Profile set (for either preview or at runtime) */
	static void ClearSkinWeightProfileInstanceOverrides(USkeletalMesh* InSkeletalMesh, FName InProfileName);

protected:
	/** Creates a copy of the skin weights from SourceMesh into a Skin Weight profile as part of TargetMesh */
	static bool CopySkinWeightsToProfile(USkeletalMesh* SourceMesh, USkeletalMesh* TargetMesh, const int32 LODIndex, const FName ProfileName);
};