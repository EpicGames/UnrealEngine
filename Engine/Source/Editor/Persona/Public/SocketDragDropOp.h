// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Input/DragAndDrop.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "Engine/SkeletalMeshSocket.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Images/SImage.h"
#include "Styling/AppStyle.h"

//////////////////////////////////////////////////////////////////////////
// FSocketDragDropOp
class FSocketDragDropOp : public FDragDropOperation
{
public:	
	
	DRAG_DROP_OPERATOR_TYPE(FSocketDragDropOp, FDragDropOperation)

	/** The widget decorator to use */
	virtual TSharedPtr<SWidget> GetDefaultDecorator() const override
	{
		return SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("Graph.ConnectorFeedback.Border"))
			.Content()
			[		
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SImage)
					.Image(this, &FSocketDragDropOp::GetIcon)
				]

				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(STextBlock) 
					.Text( this, &FSocketDragDropOp::GetHoverText )
				]
			];
	}

	/** Passed into STextBlock so Slate can grab the current text for display */
	FText GetHoverText() const
	{
		return FText::Format(NSLOCTEXT("SocketDragDrop", "SocketNameFmt", "Socket {0}"), FText::FromName(SocketInfo.Socket->SocketName));
	}

	/** Passed into SImage so Slate can grab the current icon for display */
	const FSlateBrush* GetIcon( ) const
	{
		return CurrentIconBrush;
	}

	/** Sets the icon to be displayed */
	void SetIcon(const FSlateBrush* InIcon)
	{
		CurrentIconBrush = InIcon;
	}

	/** Accessor for the socket info */
	FSelectedSocketInfo& GetSocketInfo() { return SocketInfo; }

	/** Is this an alt-drag operation? */
	bool IsAltDrag() const { return ModifierKeysState.IsAltDown(); }

	/** Is this an shift-drag operation? */
	bool IsShiftDrag() const { return ModifierKeysState.IsShiftDown(); }
	
	/* Use this function to create a new one of me */
	static TSharedRef<FSocketDragDropOp> New( FSelectedSocketInfo InSocketInfo, FModifierKeysState InModifierKeysState )
	{
		check( InSocketInfo.Socket );
		TSharedRef<FSocketDragDropOp> Operation = MakeShareable(new FSocketDragDropOp);
		Operation->SocketInfo = InSocketInfo;
		Operation->ModifierKeysState = InModifierKeysState;
		Operation->SetIcon( FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error")) );
		Operation->Construct();
		return Operation;
	}

private:
	/** The icon to display before the text */
	const FSlateBrush* CurrentIconBrush = nullptr;

	/** The socket that we're dragging */
	FSelectedSocketInfo SocketInfo;

	/** What was the modifier state at the start of the drag? */
	FModifierKeysState ModifierKeysState;
};
